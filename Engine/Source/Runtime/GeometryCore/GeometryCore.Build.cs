// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryCore : ModuleRules
{
	public GeometryCore(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePathModuleNames.AddRange(
			[
				"AnimationCore",			// For the BoneWeights.h include
				"Engine",					// For GPUSkinPublicDefs.h
			]
		);

		PublicDependencyModuleNames.AddRange(
			[
				"Core"
			]
		);

		PrivateDependencyModuleNames.AddRange(
			[
				"AutoRTFM",
			]
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
