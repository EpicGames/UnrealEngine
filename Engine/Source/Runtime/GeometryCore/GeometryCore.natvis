<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

    <!-- TDynamicVector<typename Type, int32 BlockSize> visualizer -->
    <Type Name="UE::Geometry::TDynamicVector&lt;*,*&gt;">
        <DisplayString Condition="Blocks.ArrayNum &lt; 0 || Blocks.ArrayMax &lt; Blocks.ArrayNum || Blocks[0] == nullptr">Invalid</DisplayString>
        <DisplayString Condition="CurBlock == 0 &amp;&amp; CurBlockUsed == 0">Empty</DisplayString>
        <DisplayString>{{Num={CurBlock * $T2 + CurBlockUsed}, Capacity={Blocks.ArrayNum * $T2}}</DisplayString>
        <Expand>
            <Item Name="Bytes Allocated" Condition="!(Blocks.ArrayNum &lt; 0 || Blocks.ArrayMax &lt; Blocks.ArrayNum)">
                (size_t)(Blocks.ArrayMax * sizeof(void*) + Blocks.ArrayNum * sizeof($T1) * $T2)
            </Item>
            <Synthetic Name="Last Element"
                       Condition="!(Blocks.ArrayNum &lt; 0 || Blocks.ArrayMax &lt; Blocks.ArrayNum || Blocks[0] == nullptr) &amp;&amp; !(CurBlock == 0 &amp;&amp; CurBlockUsed == 0)">
                <DisplayString>Blocks[{CurBlock}]->Elements[{CurBlockUsed - 1}]={Blocks[CurBlock]->Elements[CurBlockUsed - 1]}</DisplayString>
            </Synthetic>
            <Item Name="Blocks" Condition="!(Blocks.ArrayNum &lt; 0 || Blocks.ArrayMax &lt; Blocks.ArrayNum)">
                Blocks,na
            </Item>
        </Expand>
    </Type>

    <!-- FRefCountVector visualizer -->
    <Type Name="UE::Geometry::FRefCountVector">
        <DisplayString Condition="RefCounts.Blocks.ArrayNum &lt; 0 || RefCounts.Blocks.ArrayMax &lt; RefCounts.Blocks.ArrayNum">
            Invalid
        </DisplayString>
        <DisplayString Condition="RefCounts.CurBlock == 0 &amp;&amp; RefCounts.CurBlockUsed == 0">Empty</DisplayString>
        <DisplayString>{{Used={UsedCount}, Free={FreeIndices.CurBlock * 512 + FreeIndices.CurBlockUsed}, MaxIndex={RefCounts.CurBlock * 512 + RefCounts.CurBlockUsed}}}
        </DisplayString>
        <Expand>
            <Item Name="RefCounts"
                  Condition="!(RefCounts.Blocks.ArrayNum &lt; 0 || RefCounts.Blocks.ArrayMax &lt; RefCounts.Blocks.ArrayNum)">
                RefCounts
            </Item>
            <Item Name="FreeIndices"
                  Condition="!(RefCounts.Blocks.ArrayNum &lt; 0 || RefCounts.Blocks.ArrayMax &lt; RefCounts.Blocks.ArrayNum)">
                FreeIndices
            </Item>
        </Expand>
    </Type>

</AutoVisualizer>
