// Copyright Epic Games, Inc. All Rights Reserved.

#include "Clustering/FaceNormalClustering.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "MathUtil.h"
#include "Math/Vector.h"
#include "Util/IndexPriorityQueue.h"
#include "VectorUtil.h"
#include "VertexConnectedComponents.h" // for FSizedDisjointSet

namespace UE::Local::FaceNormalClusteringHelpers
{

using namespace UE::Geometry;

// Common clustering functionality used by multiple functions below
void ComputeClustersHelper(FDynamicMesh3& Mesh, FSizedDisjointSet& OutPlaneGroups, const FaceNormalClustering::FClusterOptions& Options, TSet<int32>* IgnoreEdges)
{
	int32 NumT = Mesh.MaxTriangleID();

	struct FNormalArea
	{
		FVector3d Normal = FVector3d::Zero();
		double Area = 0;
	};

	TArray<FNormalArea> GroupNormalAreas;
	GroupNormalAreas.SetNumZeroed(NumT);
	FVector3d TriV[3];
	for (int32 TID = 0; TID < NumT; ++TID)
	{
		if (Mesh.IsTriangle(TID))
		{
			FNormalArea& NormalArea = GroupNormalAreas[TID];
			Mesh.GetTriVertices(TID, TriV[0], TriV[1], TriV[2]);
			NormalArea.Normal = VectorUtil::NormalArea(TriV[0], TriV[1], TriV[2], NormalArea.Area);
		}
	}

	FSizedDisjointSet PlaneGroups;
	OutPlaneGroups.Init(NumT, [&Mesh](int32 TID) { return Mesh.IsTriangle(TID); });
	int32 NumEdges = Mesh.MaxEdgeID();
	Mesh.GetEdge(0);
	FIndexPriorityQueue EdgeQueue(NumEdges);

	const double AreaThreshold = FMath::Max(FMathd::Epsilon, Options.SmallFaceAreaThreshold);
	auto GetMergeWeightForGroups = [&GroupNormalAreas, AreaThreshold](const FIndex2i& EdgeGroups) -> float
	{
		if (GroupNormalAreas[EdgeGroups.A].Area < AreaThreshold || GroupNormalAreas[EdgeGroups.B].Area < AreaThreshold)
		{
			return 0;
		}
		float NormalAlignment = 1 - float(GroupNormalAreas[EdgeGroups.A].Normal.Dot(GroupNormalAreas[EdgeGroups.B].Normal));
		return NormalAlignment;
	};
	auto GetMergeWeightForEdge = [&GetMergeWeightForGroups, &Mesh, &OutPlaneGroups](int32 EdgeID, FIndex2i& EdgeGroups) -> float
	{
		EdgeGroups = Mesh.GetEdgeT(EdgeID);
		EdgeGroups.A = OutPlaneGroups.Find(EdgeGroups.A);
		EdgeGroups.B = OutPlaneGroups.Find(EdgeGroups.B);
		if (EdgeGroups.A == EdgeGroups.B)
		{
			// return a value guaranteed to be above any merge tolerance, so we ignore edges connecting already-merged groups
			constexpr float CannotMergeValue = FMathf::MaxReal;
			return CannotMergeValue;
		}
		return GetMergeWeightForGroups(EdgeGroups);
	};

	const float Threshold = static_cast<float>(Options.NormalOneMinusCosTolerance);
	for (int32 EdgeID : Mesh.EdgeIndicesItr())
	{
		if (IgnoreEdges && IgnoreEdges->Contains(EdgeID))
		{
			continue;
		}
		FIndex2i EdgeT = Mesh.GetEdgeT(EdgeID);
		if (EdgeT.Contains(INDEX_NONE))
		{
			continue;
		}
		float Wt = GetMergeWeightForGroups(EdgeT);
		if (Wt < Threshold)
		{
			EdgeQueue.Insert(EdgeID, Wt);
		}
	}

	int32 NumRemainingPlanes = Mesh.TriangleCount();
	while (EdgeQueue.GetCount() > 0)
	{
		int32 EdgeID = EdgeQueue.Dequeue();
		FIndex2i EdgeGroups;

		// If we're considering the overall cluster normals, we need to re-evaluate the potential merge based on the current cluster normals
		if (Options.bApplyNormalToleranceToClusters)
		{
			float UpdatedWt = GetMergeWeightForEdge(EdgeID, EdgeGroups);

			// If updated weight is too large, we can skip it
			if (UpdatedWt >= Threshold)
			{
				continue;
			}

			// Test if the updated weight is still less than the next node's weight; if not, re-queue it
			if (EdgeQueue.GetCount() > 0)
			{
				float NextWt = EdgeQueue.GetFirstNodePriority();
				if (UpdatedWt > NextWt)
				{
					EdgeQueue.Insert(EdgeID, UpdatedWt);
					continue;
				}
			}
		}

		// We know UpdatedWt < Threshold, so merge the groups
		int32 Parent = OutPlaneGroups.Union(EdgeGroups.A, EdgeGroups.B);
		// Update the NormalArea of the new parent
		const FNormalArea& NA_A = GroupNormalAreas[EdgeGroups.A];
		const FNormalArea& NA_B = GroupNormalAreas[EdgeGroups.B];
		FNormalArea& NA_P = GroupNormalAreas[Parent];
		NA_P.Normal = NA_A.Area * NA_A.Normal + NA_B.Area * NA_B.Normal;
		NA_P.Normal.Normalize();
		NA_P.Area = NA_A.Area + NA_B.Area;
		NumRemainingPlanes--;
		if (NumRemainingPlanes < Options.TargetMinGroups)
		{
			break;
		}
	}
}


} // UE::Local::FaceNormalClusteringHelpers

namespace UE::Geometry::FaceNormalClustering
{


void ComputeMeshPolyGroupsFromClusters(FDynamicMesh3& Mesh, TArray<TArray<int32>>& OutPolyGroups, const FClusterOptions& Options, TSet<int32>* IgnoreEdges)
{
	FSizedDisjointSet PlaneGroups;
	UE::Local::FaceNormalClusteringHelpers::ComputeClustersHelper(Mesh, PlaneGroups, Options, IgnoreEdges);

	// Copy triangle IDs of plane groups into the OutPolyGroups arrays
	TArray<int32> CompactIdxToGroupID, GroupIDToCompactIdx;
	int32 NumFoundGroups = PlaneGroups.CompactedGroupIndexToGroupID(&CompactIdxToGroupID, &GroupIDToCompactIdx);

	OutPolyGroups.SetNum(NumFoundGroups);
	for (int32 GroupIndex = 0; GroupIndex < NumFoundGroups; ++GroupIndex)
	{
		int32 GroupID = CompactIdxToGroupID[GroupIndex];
		int32 Size = PlaneGroups.GetSize(GroupID);
		OutPolyGroups[GroupIndex].Reserve(Size);
	}
	for (int32 TID : Mesh.TriangleIndicesItr())
	{
		int32 GroupID = PlaneGroups.Find(TID);
		int32 CompactIdx = GroupIDToCompactIdx[GroupID];
		OutPolyGroups[CompactIdx].Add(TID);
	}
}

void ComputeClusterCornerVertices(FDynamicMesh3& Mesh, TArray<int32>& OutCornerVertices, const FClusterOptions& Options, TSet<int32>* IgnoreEdges)
{
	FSizedDisjointSet PlaneGroups;
	UE::Local::FaceNormalClusteringHelpers::ComputeClustersHelper(Mesh, PlaneGroups, Options, IgnoreEdges);

	for (int32 VID : Mesh.VertexIndicesItr())
	{
		int32 GIDs[3]{ -1,-1,-1 };
		int32 FoundGIDs = 0;
		Mesh.EnumerateVertexTriangles(VID,
			[&](int32 TID)
			{
				if (FoundGIDs > 2)
				{
					return;
				}
				int32 GID = PlaneGroups.Find(TID);
				for (int32 TestIdx = 0; TestIdx < FoundGIDs; ++TestIdx)
				{
					if (GIDs[TestIdx] == GID)
					{
						return;
					}
				}
				GIDs[FoundGIDs++] = GID;
			}
		);
		if (FoundGIDs > 2)
		{
			OutCornerVertices.Add(VID);
		}
	}
}


} // namespace UE::Geometry::FaceNormalClustering