// Copyright Epic Games, Inc. All Rights Reserved.

#include "Clustering/KMeans.h"


namespace UE::Geometry
{

void FClusterKMeans::GetClusters(TArray<TArray<int32>>& OutClusters)
{
	OutClusters.SetNum(ClusterSizes.Num());
	for (int32 ClusterIdx = 0; ClusterIdx < ClusterSizes.Num(); ++ClusterIdx)
	{
		OutClusters[ClusterIdx].Reset(ClusterSizes[ClusterIdx]);
	}
	if (OutClusters.IsEmpty())
	{
		return;
	}

	for (int32 PointIdx = 0; PointIdx < ClusterIDs.Num(); ++PointIdx)
	{
		OutClusters[ClusterIDs[PointIdx]].Add(PointIdx);
	}
}

}