// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/ConvexDecomposition3.h"
#include "CompGeom/ConvexHull3.h"
#include "CompGeom/Delaunay2.h"
#include "MeshQueries.h"
#include "VectorUtil.h"
#include "Spatial/PriorityOrderPoints.h"
#include "VertexConnectedComponents.h"
#include "Distance/DistPoint3Triangle3.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "Implicit/Morphology.h"

#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

static bool ComputeThickenedHull(double ThickenHullAfterFailureInLocalSpace, FConvexHull3d& HullCompute, const FDynamicMesh3& Mesh, TArray<FVector3d>& CutVertices, TArray<FVector3d>& AddedVertices, TFunctionRef<bool(int32)> UseMeshVertex = [](int32) {return true;})
{
	if (ThickenHullAfterFailureInLocalSpace <= 0)
	{
		return false;
	}

	const int32 OrigMaxID = Mesh.MaxVertexID();
	FMeshNormals Normals(&Mesh);
	bool bUseNormalsForOffset = HullCompute.GetDimension() >= 2;
	if (bUseNormalsForOffset)
	{
		Normals.ComputeVertexNormals();
	}
	// Compute offsets to use based on the dimension of the failed convex hull
	FVector3d OffsetBasis[3];
	FVector3d FallbackOffset = FVector3d(1, 1, 1) * FMathd::InvSqrt3;
	int32 NumOffsets = 0;
	if (HullCompute.GetDimension() == 0)
	{
		OffsetBasis[0] = FVector3d(1, 0, 0);
		OffsetBasis[1] = FVector3d(0, 1, 0);
		OffsetBasis[2] = FVector3d(0, 0, 1);
		NumOffsets = 3;
	}
	else if (HullCompute.GetDimension() == 1)
	{
		FLine3d Line = HullCompute.GetLine();
		VectorUtil::MakePerpVectors(Line.Direction, OffsetBasis[0], OffsetBasis[1]);
		NumOffsets = 2;
	}
	else if (HullCompute.GetDimension() == 2)
	{
		FPlane3d HullPlane = HullCompute.GetPlane();
		OffsetBasis[0] = HullPlane.Normal;
		FallbackOffset = HullPlane.Normal;
		NumOffsets = 1;
	}
	const double OffsetFactor = ThickenHullAfterFailureInLocalSpace;
	for (int32 VID = 0; VID < OrigMaxID; ++VID)
	{
		if (Mesh.IsVertex(VID) && UseMeshVertex(VID))
		{

			if (bUseNormalsForOffset)
			{
				FVector3d Offset = Normals[VID];
				if (Offset == FVector::ZeroVector)
				{
					Offset = FallbackOffset;
				}
				AddedVertices.Add(Mesh.GetVertex(VID) - Offset * OffsetFactor);
			}
			else
			{
				for (int32 OffsetIdx = 0; OffsetIdx < NumOffsets; ++OffsetIdx)
				{
					AddedVertices.Add(Mesh.GetVertex(VID) - OffsetBasis[OffsetIdx] * OffsetFactor);
				}
			}

		}
	}
	return HullCompute.Solve(Mesh.MaxVertexID() + CutVertices.Num() + AddedVertices.Num(),
		[&Mesh, &CutVertices, &AddedVertices, &UseMeshVertex](int32 Index)
		{
			const int32 MaxVID = Mesh.MaxVertexID();
			const int32 MaxCutVID = MaxVID + CutVertices.Num();
			return Index < MaxVID ? Mesh.GetVertex(Index) : Index < MaxCutVID ? CutVertices[Index - MaxVID] : AddedVertices[Index - MaxCutVID];
		},
		[&Mesh, &UseMeshVertex](int32 Index) { return Index < Mesh.MaxVertexID() ? UseMeshVertex(Index) : true; });

}

bool FConvexDecomposition3::ConvexPartVsSphereOverlap(const FConvexDecomposition3::FConvexPart& Part, FVector3d Center, double Radius, const FTransform* TransformIntoSphereSpace, double* OutDistanceSq)
{
	if (OutDistanceSq)
	{
		*OutDistanceSq = 0;
	}
	bool bMustTransformConvex = false;
	if (TransformIntoSphereSpace)
	{
		FVector ScaleVec = TransformIntoSphereSpace->GetScale3D();
		// Uniform scale: Can transform into sphere into part space
		if (ScaleVec.AllComponentsEqual())
		{
			if (ScaleVec.X == 0)
			{
				return false;
			}
			Radius = FMath::Abs(Radius / ScaleVec.X);
			Center = TransformIntoSphereSpace->InverseTransformPosition(Center);
		}
		else // Non-uniform scale: Must transform convex into sphere space
		{
			bMustTransformConvex = true;
		}
	}

	// Quick test vs FConvexPart's bounding box here for an early out
	if (!OutDistanceSq)
	{
		double BoundsOverlapDist = (Part.Bounds.DiagonalLength() * .5) + Radius;
		if (FVector3d::DistSquared(Part.Bounds.Center(), Center) > BoundsOverlapDist * BoundsOverlapDist)
		{
			return false;
		}
	}

	auto GetTriDistanceSq = [&Part, bMustTransformConvex, Center, TransformIntoSphereSpace](int32 PlaneIdx)
	{
		FIndex3i TriInds = Part.HullTriangles[PlaneIdx];
		FTriangle3d Tri(Part.InternalGeo.GetVertex(TriInds.A), Part.InternalGeo.GetVertex(TriInds.B), Part.InternalGeo.GetVertex(TriInds.C));
		if (bMustTransformConvex)
		{
			Tri.V[0] = TransformIntoSphereSpace->TransformPosition(Tri.V[0]);
			Tri.V[1] = TransformIntoSphereSpace->TransformPosition(Tri.V[1]);
			Tri.V[2] = TransformIntoSphereSpace->TransformPosition(Tri.V[2]);
		}
		FDistPoint3Triangle3d Dist(Center, Tri);
		return Dist.GetSquared();
	};

	double ClosestRadiusSq = Radius * Radius;
	double MaxPlaneDist = 0;
	bool bFoundDistSq = false;
	for (int32 PlaneIdx = 0; PlaneIdx < Part.HullPlanes.Num(); ++PlaneIdx)
	{
		FPlane3d Plane = Part.HullPlanes[PlaneIdx];
		if (Plane.Normal == FVector::ZeroVector)
		{
			// for degenerate tri, don't consider the plane distance but still test if we're close to the tri
			double TriDistSq = GetTriDistanceSq(PlaneIdx);
			if (TriDistSq < ClosestRadiusSq)
			{
				if (OutDistanceSq)
				{
					ClosestRadiusSq = TriDistSq;
					bFoundDistSq = true;
				}
				else
				{
					return true;
				}
			}

			continue;
		}
		if (bMustTransformConvex)
		{
			Plane.Transform(*TransformIntoSphereSpace); //-V522
		}
		double PlaneDist = Plane.DistanceTo(Center);
		MaxPlaneDist = FMath::Max(MaxPlaneDist, PlaneDist);
		if (PlaneDist > Radius) // can quick-reject based on plane distance
		{
			return false;
		}
		else if (Radius > 0 && PlaneDist > FMath::Max(0, MaxPlaneDist - FMathd::ZeroTolerance)) // use a heuristic to only test the most-likely culprit plane
		{
			double TriDistSq = GetTriDistanceSq(PlaneIdx);
			if (TriDistSq < ClosestRadiusSq)
			{
				if (OutDistanceSq)
				{
					ClosestRadiusSq = TriDistSq;
					bFoundDistSq = true;
				}
				else
				{
					return true;
				}
			}
		}
	}
	if (MaxPlaneDist > 0) // if we're outside the hull, but didn't accept or reject yet, do one more pass w/ the "most-likely culprit" heuristic flipped
	{
		double LocalMaxPlaneDist = 0;
		for (int32 PlaneIdx = 0; PlaneIdx < Part.HullPlanes.Num(); ++PlaneIdx)
		{
			FPlane3d Plane = Part.HullPlanes[PlaneIdx];
			if (Plane.Normal == FVector3d::ZeroVector)
			{
				continue;
			}
			
			if (bMustTransformConvex)
			{
				Plane.Transform(*TransformIntoSphereSpace);
			}
			double PlaneDist = Plane.DistanceTo(Center);
			LocalMaxPlaneDist = FMath::Max(LocalMaxPlaneDist, PlaneDist);
			// Note: We don't test for negative plane distances, etc, since the first pass, above, would have caught those
			if (PlaneDist <= FMath::Max(0, LocalMaxPlaneDist - FMathd::ZeroTolerance)) // cases the heuristic skipped in the first pass
			{
				double TriDistSq = GetTriDistanceSq(PlaneIdx);
				if (TriDistSq < ClosestRadiusSq)
				{
					if (OutDistanceSq)
					{
						ClosestRadiusSq = TriDistSq;
						bFoundDistSq = true;
					}
					else
					{
						return true;
					}
				}
			}
		}
	}
	if (OutDistanceSq && bFoundDistSq)
	{
		*OutDistanceSq = ClosestRadiusSq;
		return true;
	}
	checkSlow(!OutDistanceSq || *OutDistanceSq == 0); // If we reach here distance should still be zero (as set by default)
	return (MaxPlaneDist <= 0);
}

bool FSphereCovering::AddNegativeSpace(const TFastWindingTree<FDynamicMesh3>& Spatial, const FNegativeSpaceSampleSettings& SampleSettings, bool bHasFlippedTriangles)
{
	bool bAddedPoints = false;

	check(Spatial.IsBuilt());

	FAxisAlignedBox3d Bounds = Spatial.GetTree()->GetBoundingBox();

	double WindingSign = bHasFlippedTriangles ? -1 : 1;

	if (SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::Uniform)
	{
		// Expand the sampling region by ~ the radius of interest, to make sure there is room to fit negative space spheres on concavities near the borders
		Bounds.Expand(SampleSettings.MinRadius + SampleSettings.ReduceRadiusMargin);
		double MinCellSize = FMath::Max(SampleSettings.MinSpacing, Bounds.MaxDim() / SampleSettings.TargetNumSamples);
		if (MinCellSize == 0)
		{
			return false; // no space to sample
		}
		FVector3d Ranges = Bounds.Diagonal();
		FIndex3i Dims(FMath::Max(1, int(Ranges.X / MinCellSize)), FMath::Max(1, int(Ranges.Y / MinCellSize)), FMath::Max(1, int(Ranges.Z / MinCellSize)));
		double ReduceFactor = FMath::Min(1.0, FMath::Pow(double(SampleSettings.TargetNumSamples) / double(Dims.A * Dims.B * Dims.C), 1.0/3.0));
		Dims.A = FMath::Max(1, FMath::CeilToInt32(Dims.A * ReduceFactor));
		Dims.B = FMath::Max(1, FMath::CeilToInt32(Dims.B * ReduceFactor));
		Dims.C = FMath::Max(1, FMath::CeilToInt32(Dims.C * ReduceFactor));
		int32 MaxSamples = Dims.A * Dims.B * Dims.C;
		FVector3d Denoms(1.0 / double(Dims.A), 1.0 / double(Dims.B), 1.0 / double(Dims.C));
		FVector3d StepSizes = Ranges * Denoms;

		Position.Reserve(Position.Num() + MaxSamples);
		Radius.Reserve(Radius.Num() + MaxSamples);
		for (int32 X = 0; X < Dims.A; ++X)
		{
			double XPos = Bounds.Min.X + StepSizes.X * (.5 + X);
			for (int32 Y = 0; Y < Dims.B; ++Y)
			{
				double YPos = Bounds.Min.Y + StepSizes.Y * (.5 + Y);
				for (int32 Z = 0; Z < Dims.C; ++Z)
				{
					double ZPos = Bounds.Min.Z + StepSizes.Z * (.5 + Z);
					FVector3d Pos(XPos, YPos, ZPos);
					double Winding = Spatial.FastWindingNumber(Pos) * WindingSign;
					if (Winding > .5)
					{
						continue;
					}
					double NearDistSq;
					int NearTID = Spatial.GetTree()->FindNearestTriangle(Pos, NearDistSq);
					double NearDist = FMath::Sqrt(NearDistSq);
					if (SampleSettings.OptionalObstacleSDF)
					{
						double ObstacleSD = SampleSettings.ObstacleDistance(Pos);
						NearDist = FMath::Min(ObstacleSD, NearDist);
					}
					double R = FMath::Sqrt(NearDistSq) - SampleSettings.ReduceRadiusMargin;
					if (R < SampleSettings.MinRadius)
					{
						// point too close to the surfaces; skip it
						continue;
					}
					bAddedPoints = true;
					Position.Add(Pos);
					Radius.Add(R);
				}
			}
		}
	}
	else if (SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch || SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch)
	{
		const FDynamicMesh3* Mesh = Spatial.GetTree()->GetMesh();

		// Make a convex hull of the input surface as the domain to search
		FConvexHull3d MeshHull;
		MeshHull.Solve(Mesh->MaxVertexID(), [&](int32 VID) {return Mesh->GetVertex(VID);}, [&](int32 VID) {return Mesh->IsVertex(VID);});

		// Make a light wrapper to pass the convex hull to our AABB Tree and Fast Winding classes
		struct FHullMeshWrapper
		{
			const FDynamicMesh3* SourceMesh;
			const FConvexHull3d* Hull;
			inline int32 MaxVertexID() const
			{
				return SourceMesh->MaxVertexID();
			}
			inline bool IsVertex(int32 VID) const
			{
				return SourceMesh->IsVertex(VID);
			}
			inline FVector3d GetVertex(int32 VID) const
			{
				return SourceMesh->GetVertex(VID);
			}
			inline void GetTriVertices(int32 TID, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
			{
				FIndex3i Tri = GetTriangle(TID);
				V0 = GetVertex(Tri.A);
				V1 = GetVertex(Tri.B);
				V2 = GetVertex(Tri.C);
			}
			inline int32 GetChangeStamp() const
			{
				return 0;
			}
			inline int32 MaxTriangleID() const
			{
				return Hull->GetTriangles().Num();
			}
			inline int32 TriangleCount() const
			{
				return MaxTriangleID();
			}
			inline bool IsTriangle(int32 TID) const
			{
				return true;
			}
			inline FIndex3i GetTriangle(int32 TID) const
			{
				return Hull->GetTriangles()[TID];
			}
		};
		FHullMeshWrapper HullMeshWrap{ Mesh, &MeshHull };
		TMeshAABBTree3<FHullMeshWrapper> HullAABB(&HullMeshWrap, true);
		TFastWindingTree<FHullMeshWrapper> HullWinding(&HullAABB, true);

		// Compute the empty space inside the convex hull as (Convex Hull - Original Mesh)
		FAxisAlignedBox3d HullBox = HullAABB.GetBoundingBox();
		HullBox.Expand(SampleSettings.VoxelExpandBoundsFactor * SampleSettings.GetAppliedScaleFactor());
		FMarchingCubes MarchingCubes;
		const double TargetCubeSize = SampleSettings.ReduceRadiusMargin * SampleSettings.MarchingCubesGridScale * SampleSettings.MinCustomMarginScale;
		MarchingCubes.CubeSize = FMath::Clamp(TargetCubeSize, HullBox.MaxDim() / (double)SampleSettings.MaxVoxelsPerDim, HullBox.MinDim() * .5);
		MarchingCubes.Bounds = HullBox;
		MarchingCubes.RootMode = ERootfindingModes::Bisection;
		MarchingCubes.RootModeSteps = 5;
		MarchingCubes.IsoValue = 0;
		MarchingCubes.bParallelCompute = true;
		if (SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch)
		{
			MarchingCubes.Implicit = [&HullWinding, &Spatial, WindingSign, &SampleSettings](const FVector3d& Pt) -> double
			{
				// Volume is inside the hull and outside the input surface
				if ((HullWinding.FastWindingNumber(Pt) > .5) && (Spatial.FastWindingNumber(Pt) * WindingSign <= .5))
				{
					// Volume is at least ReduceRadiusMargin away from the input surface
					double NearDistSq = FMathd::MaxReal;
					int32 NearTri = Spatial.GetTree()->FindNearestTriangle(Pt, NearDistSq, SampleSettings.ReduceRadiusMargin);
					if (NearTri == INDEX_NONE)
					{
						if (SampleSettings.OptionalObstacleSDF)
						{
							double LocalSD = SampleSettings.ObstacleDistance(Pt);
							if (LocalSD < SampleSettings.ReduceRadiusMargin)
							{
								return -1.0;
							}
						}

						return 1.0;
					}
					return -1.0;
				}
				return -1.0;
			};
		}
		else // SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch
		{
			MarchingCubes.Bounds = HullAABB.GetBoundingBox();
			MarchingCubes.Bounds.Expand(SampleSettings.MinRadius + SampleSettings.ReduceRadiusMargin + UE_DOUBLE_KINDA_SMALL_NUMBER);
			// enforce max voxels per dim after bounds expand
			MarchingCubes.CubeSize = FMath::Max(MarchingCubes.CubeSize, MarchingCubes.Bounds.MaxDim() / (double)SampleSettings.MaxVoxelsPerDim);
			const double MinRadSq = SampleSettings.MinRadius * SampleSettings.MinRadius;
			MarchingCubes.Implicit = [&HullMeshWrap, &HullAABB, &Spatial, WindingSign, &SampleSettings, MinRadSq](const FVector3d& Pt) -> double
			{
				double NearDistSq = FMathd::MaxReal;
				int32 NearTri = Spatial.GetTree()->FindNearestTriangle(Pt, NearDistSq, SampleSettings.ReduceRadiusMargin + SampleSettings.MinRadius);
				if (NearTri != INDEX_NONE) // If we're closer than MinRadius + ReduceRadiusMargin to the surface, we can't place a MinRadius sphere here
				{
					// re-test the distance w/ scaled margin, if applicable
					if (SampleSettings.CustomRadiusMarginScale && SampleSettings.MinCustomMarginScale < 1.0)
					{
						double MarginScale = FMath::Clamp(SampleSettings.CustomRadiusMarginScale(*Spatial.GetTree()->GetMesh(), NearTri, Pt), SampleSettings.MinCustomMarginScale, 1.0);
						double ScaledRadiusWithMargin = SampleSettings.MinRadius + SampleSettings.ReduceRadiusMargin * MarginScale;
						if (NearDistSq < ScaledRadiusWithMargin * ScaledRadiusWithMargin)
						{
							return -1.0;
						}
					}
					else
					{
						return -1.0;
					}
				}
				if (!SampleSettings.bOnlyConnectedToHull && Spatial.FastWindingNumber(Pt) * WindingSign > .5) // If we're not requiring 'only connected to hull', then remove the internal spaces
				{
					return -1.0;
				}
				// Test that we're also inside the (offset) convex hull
				double HullDistSq = FMathd::MaxReal;
				int32 TID = HullAABB.FindNearestTriangle(Pt, HullDistSq);
				checkSlow(TID != -1);
				if (TID != -1)
				{
					FTriangle3d Tri;
					HullMeshWrap.GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
					FVector3d N = Tri.Normal();
					if (N.Dot(Pt - Tri.V[0]) > 0 && HullDistSq > MinRadSq)
					{
						return -1.0; // outside the convex hull
					}
				}
				// If there's an optional obstacle SDF, test that we're also not too close to that
				if (SampleSettings.OptionalObstacleSDF)
				{
					double LocalSD = SampleSettings.ObstacleDistance(Pt);
					if (LocalSD < SampleSettings.MinRadius)
					{
						return -1.0;
					}
				}
				return 1.0;
			};
		}

		// If we can ignore internal negative space, we can use a seeded/continuation marching cubes here, starting from the convex hull.
		// Since our negative space surface is away from the source mesh, we need to sample the hull away from its vertices as well;
		// we do so with a subdivision scheme that can generate a large number of seed points, but still far fewer than there are voxel cells.
		if (SampleSettings.bOnlyConnectedToHull)
		{
			TArray<FVector> Seeds;
			double MinSeedSpacing = FMath::Max(MarchingCubes.CubeSize, SampleSettings.ReduceRadiusMargin * SampleSettings.MinCustomMarginScale);
			double AddPtLenSq = 4 * MinSeedSpacing * MinSeedSpacing;
			double SubDivLenSq = 16 * MinSeedSpacing * MinSeedSpacing;

			// Make a manual stack of triangles to process --
			// adding midpoints on long edges and subdividing when the resulting triangles could have long enough edges
			TArray<FTriangle3d> ProcessTriStack;
			for (FIndex3i TriInds : MeshHull.GetTriangles())
			{
				FTriangle3d HullTri(
					Mesh->GetVertex(TriInds.A),
					Mesh->GetVertex(TriInds.B),
					Mesh->GetVertex(TriInds.C));
				ProcessTriStack.Reset();
				ProcessTriStack.Push(HullTri);

				FVector3d TriOffset = FVector3d::ZeroVector;
				if (SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch)
				{
					TriOffset = HullTri.Normal() * SampleSettings.MinRadius;
				}

				int32 SeedsAdded = 0;
				while (!ProcessTriStack.IsEmpty())
				{
					FTriangle3d Tri = ProcessTriStack.Pop();
					FVector3d EdgeLensSq(
						FVector::DistSquared(Tri.V[2], Tri.V[0]),
						FVector::DistSquared(Tri.V[0], Tri.V[1]),
						FVector::DistSquared(Tri.V[1], Tri.V[2]));
					int32 NumLongEdges = 0;
					FTriangle3d Mids;
					for (int32 SubIdx = 0, LastIdx = 2; SubIdx < 3; LastIdx = SubIdx++)
					{
						Mids.V[SubIdx] = (Tri.V[SubIdx] + Tri.V[LastIdx]) * .5;
						if (EdgeLensSq[SubIdx] > AddPtLenSq)
						{
							FVector3d ToAdd = Mids.V[SubIdx] + TriOffset;
							Seeds.Add(ToAdd);
						}
						NumLongEdges += (int32)(EdgeLensSq[SubIdx] > SubDivLenSq);
					}
					// Had any edge long enough to subdivide; go ahead and add all candidate tris
					if (NumLongEdges > 0)
					{
						ProcessTriStack.Add(Mids);
						ProcessTriStack.Emplace(Tri.V[0], Mids.V[0], Mids.V[1]);
						ProcessTriStack.Emplace(Tri.V[1], Mids.V[1], Mids.V[2]);
						ProcessTriStack.Emplace(Tri.V[2], Mids.V[2], Mids.V[0]);
					}
				}
			}

			MarchingCubes.GenerateContinuation(Seeds);
		}
		else
		{
			MarchingCubes.Generate();
		}
		FDynamicMesh3 NegativeSpaceMesh(&MarchingCubes);
		NegativeSpaceMesh.DiscardAttributes();

		// Make sure the mesh is compact to simplify downsampling below
		NegativeSpaceMesh.CompactInPlace();

		auto AddSample = [this, &Mesh, &Spatial, &SampleSettings, &bAddedPoints, WindingSign](FVector3d Pos, bool bTestCover = false, bool bRequireWalk = true)
		{
			if (bRequireWalk || !SampleSettings.bAllowSamplesInsideMesh)
			{
				double Winding = Spatial.FastWindingNumber(Pos) * WindingSign;
				if (Winding > .5)
				{
					return;
				}
			}

			double NearDistSq;
			int NearTID = Spatial.GetTree()->FindNearestTriangle(Pos, NearDistSq);
			double NearDist = FMath::Sqrt(NearDistSq);
			double UseRadiusMargin = SampleSettings.ReduceRadiusMargin;
			if (SampleSettings.CustomRadiusMarginScale && SampleSettings.MinCustomMarginScale < 1.0)
			{
				double MarginScale = FMath::Clamp(SampleSettings.CustomRadiusMarginScale(*Mesh, NearTID, Pos), SampleSettings.MinCustomMarginScale, 1.0);
				UseRadiusMargin = SampleSettings.ReduceRadiusMargin * MarginScale;
			}
			double R = NearDist - UseRadiusMargin;

			// Walk away from the closest point until we're far enough away to create a sample
			// (give up if we haven't found a valid sample after a few steps, or if we stepped inside the shape)
			if (bRequireWalk)
			{
				int32 Steps = 0;
				while (R < SampleSettings.MinRadius && Steps++ < 3)
				{
					bool bFoundValidSample = false;
					FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(*Mesh, NearTID, Pos);
					FVector3d Away = Pos - Query.ClosestTrianglePoint;
					if (!Away.Normalize())
					{
						return;
					}

					// Move away and re-test the sample
					Pos += Away * ((SampleSettings.MinRadius - R) * 1.1);
					if (Spatial.FastWindingNumber(Pos) * WindingSign <= .5)
					{
						NearTID = Spatial.GetTree()->FindNearestTriangle(Pos, NearDistSq);
						R = FMath::Sqrt(NearDistSq) - UseRadiusMargin;
						if (R >= SampleSettings.MinRadius)
						{
							break;
						}
					}
					else  // give up if we stepped into the volume
					{
						return;
					}
				}
			}

			if (!bRequireWalk || R >= SampleSettings.MinRadius)
			{
				if (bTestCover)
				{
					double SpacingThresholdSq = SampleSettings.MinSpacing * SampleSettings.MinSpacing;
					// TODO: Consider accelerating this coverage search w/ e.g. a sparse dynamic octree representation of the sphere covering
					for (int32 SphereIdx = 0; SphereIdx < Position.Num(); ++SphereIdx)
					{
						double ThreshSq = FMath::Max(SpacingThresholdSq, Radius[SphereIdx] * Radius[SphereIdx]);
						if (FVector3d::DistSquared(Position[SphereIdx], Pos) < ThreshSq)
						{
							return;
						}
					}
				}
				bAddedPoints = true;
				Position.Add(Pos);
				Radius.Add(R);
			}
		};

		// Compute an angle metric to prioritize samples on 'features' of the negative space mesh
		TArray<float> VertexAngleMetric; // favor points on sharper edges (near 'features')
		VertexAngleMetric.SetNumZeroed(NegativeSpaceMesh.MaxVertexID());
		TArray<FVector3d> TriNormals;
		TriNormals.SetNumZeroed(NegativeSpaceMesh.MaxTriangleID());
		for (int32 TID : NegativeSpaceMesh.TriangleIndicesItr())
		{
			TriNormals[TID] = NegativeSpaceMesh.GetTriNormal(TID);
		}
		TArray<FVector3d> VertexPositions;
		VertexPositions.SetNumZeroed(NegativeSpaceMesh.MaxVertexID());
		for (int32 VID : NegativeSpaceMesh.VertexIndicesItr())
		{
			VertexPositions[VID] = NegativeSpaceMesh.GetVertex(VID);
			// Note: Angle metric favors vertices on edges with larger dihedral angles
			float AngleMetric = 0;
			NegativeSpaceMesh.EnumerateVertexEdges(VID, [&](int32 EID)
				{
					FIndex2i EdgeT = NegativeSpaceMesh.GetEdgeT(EID);
					if (EdgeT.B != FDynamicMesh3::InvalidID)
					{
						AngleMetric = FMath::Max(AngleMetric, float(1 - TriNormals[EdgeT.A].Dot(TriNormals[EdgeT.B])));
					}
				});
			VertexAngleMetric[VID] = AngleMetric;
		}

		// If we want consistent results, sort the points -- since parallelism in marching cubes can add vertices in arbitrary order
		if (SampleSettings.bDeterministic)
		{
			check(NegativeSpaceMesh.IsCompactV());
			TArray<int32> Indices;
			Indices.SetNumUninitialized(VertexPositions.Num());
			for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
			{
				Indices[Idx] = Idx;
			}
			Indices.Sort([&](int32 A, int32 B)
				{
					FVector3d VA = VertexPositions[A];
					FVector3d VB = VertexPositions[B];
					if (VA.X < VB.X)
					{
						return true;
					}
					else if (VA.X == VB.X)
					{
						if (VA.Y < VB.Y)
						{
							return true;
						}
						else if (VA.Y == VB.Y)
						{
							return VA.Z < VB.Z;
						}
					}
					return false;
				});
			TArray<FVector> SortedPositions;
			TArray<float> SortedAngleMetric;
			SortedPositions.SetNumUninitialized(VertexPositions.Num());
			SortedAngleMetric.SetNumUninitialized(VertexPositions.Num());
			for (int32 Idx = 0; Idx < VertexPositions.Num(); ++Idx)
			{
				SortedPositions[Idx] = VertexPositions[Indices[Idx]];
				SortedAngleMetric[Idx] = VertexAngleMetric[Indices[Idx]];
			}
			Swap(VertexPositions, SortedPositions);
			Swap(VertexAngleMetric, SortedAngleMetric);
		}

		int32 NumSamples = FMath::Min(VertexPositions.Num(), SampleSettings.TargetNumSamples);
		FPriorityOrderPoints Ordering;
		Ordering.ComputeUniformSpaced(VertexPositions, VertexAngleMetric, SampleSettings.bRequireSearchSampleCoverage ? -1 : NumSamples);

		// The 'sample walk' is used to ensure samples are at least MinRadius + ReduceRadiusMargin away from the input surface. 
		// Note this isn't needed for the NavigableVoxelSearch method; its samples are already this distance from the surface by construction
		bool bRequireSampleWalk = SampleSettings.SampleMethod == FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch;
		for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
		{
			int32 VID = Ordering.Order[SampleIdx];
			AddSample(VertexPositions[VID], false /*test cover*/, bRequireSampleWalk);
		}

		if (SampleSettings.bRequireSearchSampleCoverage)
		{
			double SpacingThresholdSq = SampleSettings.MinSpacing * SampleSettings.MinSpacing;
			bool bTestCover = SpacingThresholdSq > 0;
			for (int32 SampleIdx = NumSamples; SampleIdx < Ordering.Order.Num(); ++SampleIdx)
			{
				int32 VID = Ordering.Order[SampleIdx];
				FVector3d Pos = VertexPositions[VID];
				AddSample(Pos, bTestCover, bRequireSampleWalk);
			}
		}
	}

	return bAddedPoints;
}

// Internal struct to run templated TriangleMeshType functions on the hull mesh
// Assumes the convex part is not compacted (i.e. is using the dynamic mesh vertices)
// Note this will include vertices that are not on the hull
struct FConvexPartHullMeshAdapter
{
	FConvexPartHullMeshAdapter(const FConvexDecomposition3::FConvexPart* ConvexPart) : ConvexPart(ConvexPart)
	{
	}

	const FConvexDecomposition3::FConvexPart* ConvexPart;

	bool IsTriangle(int32 Index) const
	{
		return Index >= 0 && Index < ConvexPart->HullTriangles.Num();
	}
	bool IsVertex(int32 VID) const
	{
		return ConvexPart->InternalGeo.IsVertex(VID);
	}
	int32 MaxTriangleID() const
	{
		return ConvexPart->HullTriangles.Num();
	}
	int32 MaxVertexID() const
	{
		return ConvexPart->InternalGeo.MaxVertexID();
	}
	int32 TriangleCount() const
	{
		return ConvexPart->HullTriangles.Num();
	}
	int32 VertexCount() const
	{
		return ConvexPart->InternalGeo.VertexCount();
	}
	uint64 GetChangeStamp() const
	{
		return 0;
	}
	FIndex3i GetTriangle(int32 Index) const
	{
		return ConvexPart->HullTriangles[Index];
	}
	FVector3d GetVertex(int32 VID) const
	{
		return ConvexPart->InternalGeo.GetVertex(VID);
	}

	inline void GetTriVertices(int TID, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		const FIndex3i& TriIndices = ConvexPart->HullTriangles[TID];
		V0 = GetVertex(TriIndices.A);
		V1 = GetVertex(TriIndices.B);
		V2 = GetVertex(TriIndices.C);
	}
};

// Helper functionality
namespace
{

// Compute volume by summing the volume of tetrahedra formed by the mesh triangles and a given reference point
// If we use a central point on the mesh (centroid/center of mass/etc) this can make the computation more robust,
// and can also make the result less arbitrary for meshes with holes
template <typename TriangleMeshType>
double GetVolumeUsingReferencePoint(const TriangleMeshType& Mesh, FVector3d RefPt, double DimScaleFactor = 1)
{
	double Volume = 0.0;
	for (int TriIdx = 0; TriIdx < Mesh.MaxTriangleID(); TriIdx++)
	{
		if (!Mesh.IsTriangle(TriIdx))
		{
			continue;
		}

		FVector3d V0, V1, V2;
		Mesh.GetTriVertices(TriIdx, V0, V1, V2);

		// (6x) volume of the tetrahedron formed by the triangles and the reference point
		FVector3d V1mRef = (V1 - RefPt) * DimScaleFactor;
		FVector3d V2mRef = (V2 - RefPt) * DimScaleFactor;
		FVector3d N = V2mRef.Cross(V1mRef);

		Volume += ((V0 - RefPt) * DimScaleFactor).Dot(N);
	}

	return Volume * (1.0 / 6.0);
}


// Partially-computed result of splitting a convex part by a plane -- just enough to evaluate what the error would be
// (with the caveat that it cannot see if there are multiple components to be broken apart)
struct FPartialCutResult
{
	TArray<FVector3d> CutVertices;
	TArray<FVector3d> OffsetVertices[2]; // vertices added to thicken an otherwise-degenerate convex hull; in most cases, not needed
	TArray<int32> CrossingEdgeIDs;
	TArray<double> SignDists;

	TArray<FIndex3i> HullTriangles[2];
	double HullVolumes[2]{ 0,0 };
	bool bSuccess = false;

	// Compute + return the error of the partial result
	// TODO: support multiple kinds of error
	double Score(const FConvexDecomposition3::FConvexPart& Convex)
	{
		double HullMinusGeoVolume = HullVolumes[0] + HullVolumes[1] - Convex.GeoVolume;
		return HullMinusGeoVolume;
	}

	void ApplyToGeo(TIndirectArray<FConvexDecomposition3::FConvexPart>& Decomposition, int32 OrigPartIdx, const FPlane3d& Plane, int32& OtherSideStartIdxOut, double PlaneTol, double ConnectedComponentTolerance, const FSphereCovering& NegativeSpace, bool bTreatAsSolid, bool bSplitDisconnectedComponents, double ThickenAfterHullFailureLocalSpace)
	{
		FConvexDecomposition3::FConvexPart& OrigPart = Decomposition[OrigPartIdx];
		bool bSourceGeometryVolumeUnreliable = OrigPart.bGeometryVolumeUnreliable;

		// Save out the negative space overlaps right away (as we will end up replacing this original part)
		TArray<int32> OrigOverlaps = MoveTemp(OrigPart.OverlapsNegativeSpace);

		int32 NewPartIdx = Decomposition.Add(new FConvexDecomposition3::FConvexPart);
		FConvexDecomposition3::FConvexPart& NewPart = Decomposition[NewPartIdx];
		int32 PartIndices[2]{ OrigPartIdx, NewPartIdx };

		TArray<int32> OnCutEdges;
		check(CrossingEdgeIDs.Num() == CutVertices.Num());

		TArray<int32> HullCrossingVertexMap;
		int32 HullCrossingVertexStart = OrigPart.InternalGeo.MaxVertexID();
		HullCrossingVertexMap.SetNumUninitialized(CrossingEdgeIDs.Num());
		// split all the saved crossing edges
		for (int32 Idx = 0; Idx < CrossingEdgeIDs.Num(); Idx++)
		{
			int32 EID = CrossingEdgeIDs[Idx];
			FDynamicMesh3::FEdge Edge = OrigPart.InternalGeo.GetEdge(EID);
			DynamicMeshInfo::FEdgeSplitInfo SplitInfo;
			double DistA = SignDists[Edge.Vert.A], DistB = SignDists[Edge.Vert.B];
			double Param = DistA / (DistA - DistB);
			EMeshResult SplitResult = OrigPart.InternalGeo.SplitEdge(EID, SplitInfo, Param);
			if (ensure(SplitResult == EMeshResult::Ok))
			{
				HullCrossingVertexMap[Idx] = SplitInfo.NewVertex;
			}
			else
			{
				// technically this could happen if we hit the vertex valence limit; that shouldn't really happen in practice though ...
				HullCrossingVertexMap[Idx] = -1;
			}
		}

		// remove all fully on-plane triangles
		if (bTreatAsSolid)
		{
			for (int32 TID = 0, MaxTID = OrigPart.InternalGeo.MaxTriangleID(); TID < MaxTID; TID++)
			{
				if (!OrigPart.InternalGeo.IsTriangle(TID))
				{
					continue;
				}
				FIndex3i Tri = OrigPart.InternalGeo.GetTriangle(TID);
				// Note any new vertex (ID beyond sign dist or un-set before) will be on the plane by construction (introduced by above edge split)
				// SignDists was zero-initialized, so we don't need to update it with the new vertices, and can just assume IDs beyond the previous max are on the plane
				if ((Tri.A >= SignDists.Num() || FMathd::Abs(SignDists[Tri.A]) <= PlaneTol) &&
					(Tri.B >= SignDists.Num() || FMathd::Abs(SignDists[Tri.B]) <= PlaneTol) &&
					(Tri.C >= SignDists.Num() || FMathd::Abs(SignDists[Tri.C]) <= PlaneTol))
				{
					EMeshResult Res = OrigPart.InternalGeo.RemoveTriangle(TID, true, false);
				}
			}
		}
		
		// Init the new part geometry w/ a full copy of the original
		NewPart.InternalGeo = OrigPart.InternalGeo;

		// Delete the other half from each part
		FDynamicMesh3* PartsMeshes[2] = { &OrigPart.InternalGeo, &NewPart.InternalGeo };
		for (int32 VID = 0, MaxVID = SignDists.Num(); VID < MaxVID; VID++)
		{
			double SignDist = SignDists[VID];
			if (SignDist > PlaneTol && PartsMeshes[0]->IsVertex(VID))
			{
				PartsMeshes[0]->RemoveVertex(VID, false);
			}
			else if (SignDist < -PlaneTol && PartsMeshes[1]->IsVertex(VID))
			{
				PartsMeshes[1]->RemoveVertex(VID, false);
			}
		}

		// for non-solid meshes, remove on-plane triangles based on the side they're facing
		if (!bTreatAsSolid)
		{
			for (int32 Side = 0; Side < 2; Side++)
			{
				for (int32 TID = 0, MaxTID = PartsMeshes[Side]->MaxTriangleID(); TID < MaxTID; TID++)
				{
					if (!PartsMeshes[Side]->IsTriangle(TID))
					{
						continue;
					}
					FIndex3i Tri = PartsMeshes[Side]->GetTriangle(TID);
					// Note any new vertex (ID beyond sign dist or un-set before) will be on the plane by construction (introduced by above edge split)
					// SignDists was zero-initialized, so we don't need to update it with the new vertices, and can just assume IDs beyond the previous max are on the plane
					if ((Tri.A >= SignDists.Num() || FMathd::Abs(SignDists[Tri.A]) <= PlaneTol) &&
						(Tri.B >= SignDists.Num() || FMathd::Abs(SignDists[Tri.B]) <= PlaneTol) &&
						(Tri.C >= SignDists.Num() || FMathd::Abs(SignDists[Tri.C]) <= PlaneTol))
					{
						double DotTriPlane = PartsMeshes[Side]->GetTriNormal(TID).Dot(Plane.Normal);
						if (DotTriPlane > 0 != (bool)Side)
						{
							PartsMeshes[Side]->RemoveTriangle(TID, true, false);
						}
					}
				}
			}
		}

		// Remove unreferenced vertices, which may be created by the above deletion (i.e., vertices on the plane that were only connected to now-deleted triangles)
		for (int32 Side = 0; Side < 2; Side++)
		{
			for (int32 VID = 0, MaxVID = PartsMeshes[Side]->MaxVertexID(); VID < MaxVID; VID++)
			{
				if (PartsMeshes[Side]->IsVertex(VID) && !PartsMeshes[Side]->IsReferencedVertex(VID))
				{
					PartsMeshes[Side]->RemoveVertex(VID, false);
				}
			}
		}

		// Fill in the surface on the cutting plane; return false if hole fill process was ambiguous / unreliable
		auto HoleFill = [this, &Plane, &PartsMeshes, &PlaneTol]() -> bool
		{
			bool bSuccessFill = true;

			FVector3d BasisU, BasisV;
			VectorUtil::MakePerpVectors(Plane.Normal, BasisU, BasisV);

			// Collect on-plane edges to do a hole-fill triangulation
			// TODO: detect case where the second hole fill can just be a flipped duplicate of the first
			TArray<int32> BoundaryEdges;
			for (int32 Side = 0; Side < 2; Side++)
			{
				BasisU = -BasisU;
				BoundaryEdges.Reset();
				for (int32 EID : PartsMeshes[Side]->BoundaryEdgeIndicesItr())
				{
					const FDynamicMesh3::FEdge Edge = PartsMeshes[Side]->GetEdge(EID);
					if ( // boundary edge is on plane
						(Edge.Vert.A >= SignDists.Num() || FMathd::Abs(SignDists[Edge.Vert.A]) <= PlaneTol) &&
						(Edge.Vert.B >= SignDists.Num() || FMathd::Abs(SignDists[Edge.Vert.B]) <= PlaneTol))
					{
						BoundaryEdges.Add(EID);
					}
				}


				if (BoundaryEdges.IsEmpty())
				{
					continue;
				}

				int32 FirstEID = BoundaryEdges[0];
				int32 FirstVID = PartsMeshes[Side]->GetEdge(FirstEID).Vert.A;
				FVector3d Orig = PartsMeshes[Side]->GetVertex(FirstVID);

				auto Proj2 = [&Orig, &BasisU, &BasisV](const FVector3d& P) -> FVector2d
				{
					FVector3d Diff = P - Orig;
					return FVector2d(Diff.Dot(BasisU), Diff.Dot(BasisV));
				};

				const int32 MaxVerts = BoundaryEdges.Num() * 2;
				TArray<FVector2d> Verts2; // 2D projections of the on-plane vertices
				Verts2.Reserve(MaxVerts);
				TArray<int32> V2ToVIDMap;
				V2ToVIDMap.Reserve(MaxVerts);
				TMap<int32, int32> VIDToV2Map; // map from mesh VID to 2d vertex index
				VIDToV2Map.Reserve(MaxVerts);
				TArray<FIndex2i> Edges2; // Edges indexing into the Verts2 array
				Edges2.Reserve(BoundaryEdges.Num());
				for (int32 EID : BoundaryEdges)
				{
					FIndex2i VIDs = PartsMeshes[Side]->GetOrientedBoundaryEdgeV(EID);
					FIndex2i Edge2;
					for (int32 SubIdx = 0; SubIdx < 2; SubIdx++)
					{
						int32 VID = VIDs[SubIdx];
						int32* SearchV2Idx = VIDToV2Map.Find(VID);
						int32 V2Idx;
						if (SearchV2Idx)
						{
							V2Idx = *SearchV2Idx;
						}
						else
						{
							FVector2d V2D = Proj2(PartsMeshes[Side]->GetVertex(VID));
							V2Idx = Verts2.Add(V2D);
							ensure(V2Idx == V2ToVIDMap.Add(VID));
							VIDToV2Map.Add(VIDs[SubIdx], V2Idx);
						}
						Edge2[SubIdx] = V2Idx;
					}
					Edges2.Add(Edge2);
				}

				FDelaunay2 Delaunay;
				Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
				TArray<FIndex3i> Triangles2; // Triangles indexing into the Verts2 array
				if (Delaunay.Triangulate(Verts2, Edges2))
				{
					bool bWellDefinedResult = Delaunay.GetFilledTriangles(Triangles2, Edges2, FDelaunay2::EFillMode::NegativeWinding);
					if (!bWellDefinedResult)
					{
						bSuccessFill = false;
					}
					// TODO: if result is not well defined, is there anything more robust we could do here?
					// Perhaps fill based on the winding number of the input mesh? (But more expensive, and we'd have to handle ~coplanar cases as well)
				}
				// TODO: Consider only attempting to append the hole fill if bSuccessFill is true
				if (!Triangles2.IsEmpty()) // Delaunay triangulation succeeded at generating some triangulation
				{
					for (const FIndex3i& Tri2 : Triangles2)
					{
						FIndex3i Tri3(
							V2ToVIDMap[Tri2.A],
							V2ToVIDMap[Tri2.B],
							V2ToVIDMap[Tri2.C]
						);
						int TID = PartsMeshes[Side]->AppendTriangle(Tri3/*, TODO: Pass a group ID to mark that this is a hole-fill triangle*/);
						if (TID < 0)
						{
							bSuccessFill = false;
						}
					}
				}
				else // Delaunay failed to generate a triangulation
				{
					bSuccessFill = false;

					// fallback triangulation: just fill everything with triangle fans
					// TODO: is this fallback method worth keeping?
					// It should make the volumes more stable for the subsequent geometry, but could prevent hulls from fitting as closely as possible
					// Leaving this here but disabled for now, to make it easier to test; likely we can remove it after some additional testing of triangulation failure cases
					if constexpr (false)
					{
						int32 FanCenterVID = PartsMeshes[Side]->AppendVertex(Orig);
						for (int32 EID : BoundaryEdges)
						{
							FIndex2i VIDs = PartsMeshes[Side]->GetOrientedBoundaryEdgeV(EID);
							if (!ensure(VIDs.A >= 0)) // this would happen if an edge is already not a boundary edge; TODO: can this actually happen here? 
							{
								continue;
							}
							PartsMeshes[Side]->AppendTriangle(FanCenterVID, VIDs.B, VIDs.A/*, TODO: Pass a group ID to mark that this is a hole-fill triangle*/);
						}

						// split off any complete loops as their own separate fan, and put each fan-center vertex at its one-ring's centroid
						TArray<int> TrianglesOut, ContiguousGroupLengths;
						TArray<bool> GroupIsLoop;
						DynamicMeshInfo::FVertexSplitInfo SplitInfo;
						bool bHasSplit = false;
						if (ensure(EMeshResult::Ok == PartsMeshes[Side]->GetVtxContiguousTriangles(FanCenterVID, TrianglesOut, ContiguousGroupLengths, GroupIsLoop)))
						{
							if (ContiguousGroupLengths.Num() > 1)
							{
								int32 NumLoops = 0;
								int32 NumSpans = 0;
								for (bool bIsLoop : GroupIsLoop)
								{
									NumLoops += int(bIsLoop);
									NumSpans += int(!bIsLoop);
								}

								if (NumLoops > 1 || (NumLoops == 1 && NumSpans > 0))
								{
									bHasSplit = true;
									
									for (int GroupIdx = 0, GroupStartIdx = 0; GroupIdx < ContiguousGroupLengths.Num(); GroupStartIdx += ContiguousGroupLengths[GroupIdx++])
									{
										if (GroupIsLoop[GroupIdx] && (NumSpans > 0 || GroupIdx > 0))
										{
											if (ensure(EMeshResult::Ok == PartsMeshes[Side]->SplitVertex(FanCenterVID, TArrayView<const int>(TrianglesOut.GetData() + GroupStartIdx, ContiguousGroupLengths[GroupIdx]), SplitInfo)))
											{
												FVector3d Centroid;
												PartsMeshes[Side]->GetVtxOneRingCentroid(SplitInfo.NewVertex, Centroid);
												PartsMeshes[Side]->SetVertex(SplitInfo.NewVertex, Centroid, false);
											}
										}
									}
								}
							}
						}
						FVector3d Centroid;
						PartsMeshes[Side]->GetVtxOneRingCentroid(FanCenterVID, Centroid);
						PartsMeshes[Side]->SetVertex(FanCenterVID, Centroid, false);
					}
				}
			}

			return bSuccessFill;
		};

		// TODO: We may be able to skip hole filling if we're using an error metric that is not volume-based
		bool bHoleFillResultUnreliable = !bTreatAsSolid || !HoleFill();

		auto ComputeHullsIfMultipleComponents = [&Decomposition, &ConnectedComponentTolerance, ThickenAfterHullFailureLocalSpace](int32 PartIdx)->int32
		{
			FConvexDecomposition3::FConvexPart& Part = Decomposition[PartIdx];
			FDynamicMesh3& Mesh = Part.InternalGeo;
			FVertexConnectedComponents VertexComponents;
			VertexComponents.Init(Mesh);
			VertexComponents.ConnectTriangles(Mesh);
			bool bMultipleGroups = VertexComponents.HasMultipleComponents(Mesh, 3);
			if (!bMultipleGroups)
			{
				return int32(Mesh.TriangleCount() > 0);
			}
			// multiple groups found; connect close vertices just to double check
			VertexComponents.ConnectCloseVertices(Mesh, ConnectedComponentTolerance, 3);

			// Additionally merge components if they overlap in space
			// Note this only looks at bounding box overlap currently; TODO: extend to test more possible plane of separation?
			VertexComponents.ConnectOverlappingComponents(Mesh, 3);

			// Get a mapping from component set IDs to contiguous indices from 0 to k-1 (for k components)
			TMap<int32, int32> ComponentToIdxMap = VertexComponents.MakeComponentMap(Mesh, 3);
			
			int32 NewComponents = ComponentToIdxMap.Num();
			if (NewComponents < 2)
			{
				return NewComponents;
			}


			const FDynamicMesh3& OrigMesh = Mesh; // Prepare to copy the vertices and triangles out of the old mesh and into the new parts
			TArray<TUniquePtr<FConvexDecomposition3::FConvexPart>> NewParts;
			for (int32 Idx = 0; Idx < NewComponents; Idx++)
			{
				NewParts.Add(MakeUnique<FConvexDecomposition3::FConvexPart>());
			}
			
			TMap<int32, int32> VertexMap; // Map vertices to their new VID in their new mesh (note: contains mappings into every component)
			for (int32 VID : OrigMesh.VertexIndicesItr())
			{
				if (!OrigMesh.IsReferencedVertex(VID))
				{
					continue;
				}
				int32 DisjointSetID = VertexComponents.GetComponent(VID);
				int32 NewPartIdx = ComponentToIdxMap[DisjointSetID];
				int32 NewVID = NewParts[NewPartIdx]->InternalGeo.AppendVertex(OrigMesh.GetVertex(VID));
				VertexMap.Add(VID, NewVID);
			}
			for (int32 TID : OrigMesh.TriangleIndicesItr())
			{
				FIndex3i Tri = OrigMesh.GetTriangle(TID);
				// TODO: if we use triangle groups, also get the group ID here and transfer it across
				int32 DisjointSetID = VertexComponents.GetComponent(Tri.A);
				int32 NewPartIdx = ComponentToIdxMap[DisjointSetID];
				FIndex3i NewTri(VertexMap[Tri.A], VertexMap[Tri.B], VertexMap[Tri.C]);
				NewParts[NewPartIdx]->InternalGeo.AppendTriangle(NewTri);
			}
			bool bHasFailedComponents = false;
			for (const TPair<int32, int32>& ToNewPartIdx : ComponentToIdxMap)
			{
				int32 NewPartIdx = ToNewPartIdx.Value;
				bHasFailedComponents = !NewParts[NewPartIdx]->ComputeHull(true, ThickenAfterHullFailureLocalSpace) || bHasFailedComponents;
				if (bHasFailedComponents)
				{
					break;
				}
				NewParts[NewPartIdx]->ComputeStats();
			}
			if (bHasFailedComponents)
			{
				// failed to build hull for some components; fall back to not splitting the components in this case
				return 1;
			}

			// Successfully built the connected components: transfer them into the decomposition
			Decomposition[PartIdx] = MoveTemp(*NewParts[0]);
			NewParts[0].Reset();
			for (int32 Idx = 1; Idx < NewParts.Num(); Idx++)
			{
				Decomposition.Add(NewParts[Idx].Release());
			}

			return NewComponents;
		};

		// Compact both side meshes and transfer hull vertices OR compute new hulls if there were multiple connected components
		int32 ComponentsOnSide0 = 1;
		OtherSideStartIdxOut = NewPartIdx;
		for (int32 Side = 0; Side < 2; Side++)
		{
			FConvexDecomposition3::FConvexPart* Part;
			Part = Side == 0 ? &OrigPart : &NewPart;

			Part->HullTriangles.Reset(HullTriangles[Side].Num());
			Part->HullTriangles.Reset();

			// If there are multiple components, we'll need to compute all new hulls for each of them (if enabled)
			if (bSplitDisconnectedComponents)
			{
				int32 NumComponents = ComputeHullsIfMultipleComponents(PartIndices[Side]);
				if (NumComponents > 1)
				{
					if (Side == 0) // keep the components from each side of the plane contiguous
					{
						ComponentsOnSide0 = NumComponents;
						int32 LastIdx = Decomposition.Num() - 1;
						Decomposition.Swap(NewPartIdx, LastIdx);
						NewPartIdx = LastIdx;
						OtherSideStartIdxOut = NewPartIdx;
					}
					continue;
				}
			}

			// There was only one component -- we can re-use the convex hull we computed, just need to fix the vertex indices
			FCompactMaps CompactMaps;
			PartsMeshes[Side]->CompactInPlace(&CompactMaps);
			TMap<int32, int32> AddedVertsMap;

			auto RemapIndex = [this, &CompactMaps, &HullCrossingVertexStart, &HullCrossingVertexMap, &PartsMeshes, &Side, &AddedVertsMap](int32 OrigHullIdx)
			{
				if (OrigHullIdx < HullCrossingVertexStart)
				{
					int32 Idx = CompactMaps.GetVertexMapping(OrigHullIdx);
					ensure(Idx > -1);
					return Idx;
				}
				else
				{
					const int32 CrossingVertexIdx = OrigHullIdx - HullCrossingVertexStart;
					if (CrossingVertexIdx >= CutVertices.Num()) // it's an offset vertex; will always need to be appended to the geo on first encounter
					{
						int32* AddedVertPtr = AddedVertsMap.Find(CrossingVertexIdx);
						if (!AddedVertPtr)
						{
							const int32 OffsetVertexIdx = CrossingVertexIdx - CutVertices.Num();
							int32 AppendedVertID = PartsMeshes[Side]->AppendVertex(OffsetVertices[Side][OffsetVertexIdx]);
							AddedVertsMap.Add(CrossingVertexIdx, AppendedVertID);
							return AppendedVertID;
						}
						else
						{
							return *AddedVertPtr;
						}
					}
					int32 NewIdx = HullCrossingVertexMap[CrossingVertexIdx];
					if (NewIdx == -1)
					{
						int32* AddedVertPtr = AddedVertsMap.Find(CrossingVertexIdx);
						if (!AddedVertPtr)
						{
							int32 AppendedVertID = PartsMeshes[Side]->AppendVertex(CutVertices[CrossingVertexIdx]);
							AddedVertsMap.Add(CrossingVertexIdx, AppendedVertID);
							return AppendedVertID;
						}
						else
						{
							return *AddedVertPtr;
						}
					}
					else
					{
						NewIdx = CompactMaps.GetVertexMapping(NewIdx);
					}
					checkSlow(NewIdx > -1);
					return NewIdx;
				}
			};
			for (FIndex3i& HullTri : HullTriangles[Side])
			{
				HullTri.A = RemapIndex(HullTri.A);
				HullTri.B = RemapIndex(HullTri.B);
				HullTri.C = RemapIndex(HullTri.C);
			}
			Part->HullTriangles = MoveTemp(HullTriangles[Side]);

			// Compute all the extra data the FConvexPart requires as well
			Part->ComputeHullPlanes();
			Part->GeoCenter = TMeshQueries<FDynamicMesh3>::GetMeshVerticesCentroid(Part->InternalGeo);
			Part->GeoVolume = GetVolumeUsingReferencePoint<FDynamicMesh3>(Part->InternalGeo, Part->GeoCenter);
			if (Part->GeoVolume < 0)
			{
				Part->bGeometryVolumeUnreliable = true;
				Part->GeoVolume = 0;
			}
			Part->HullVolume = HullVolumes[Side];

			Part->Bounds = Part->InternalGeo.GetBounds();
			Part->HullError = Part->HullVolume - Part->GeoVolume;
		}

		// Propagate down the flag that the geometry volume may be incorrect, if it was set on the source or if the hole fill had issues
		if (bSourceGeometryVolumeUnreliable || bHoleFillResultUnreliable)
		{
			OrigPart.bGeometryVolumeUnreliable = true;
			for (int32 PartIdx = NewPartIdx; PartIdx < Decomposition.Num(); PartIdx++)
			{
				Decomposition[PartIdx].bGeometryVolumeUnreliable = true;
			}
		}

		// Update lists of conflicting negative spaces (if negative space is available)
		if (NegativeSpace.Num() > 0 && !OrigOverlaps.IsEmpty())
		{
			auto AddIfOverlaps = [&NegativeSpace, &OrigOverlaps](FConvexDecomposition3::FConvexPart& Part)
			{
				// heuristics to multithread the convex part vs sphere overlap tests when there are enough spheres to be potentially worth it
				// TODO: consider also using some kind of spatial acceleration + revisit tuning
				if (OrigOverlaps.Num() > 200)
				{
					const int32 NumBatch = 7 + int32(OrigOverlaps.Num()/200);
					TArray<TArray<int32>> OverlapArr;
					OverlapArr.SetNum(NumBatch);
					const int32 PerBatch = 1+OrigOverlaps.Num() / NumBatch;
					ParallelFor(NumBatch, [&](int32 BatchIdx)
						{
							for (int32 Idx = PerBatch * BatchIdx, Num = FMath::Min(OrigOverlaps.Num(), PerBatch * (BatchIdx + 1)); Idx < Num; ++Idx)
							{
								int32 SphereIdx = OrigOverlaps[Idx];
								if (FConvexDecomposition3::ConvexPartVsSphereOverlap(Part, NegativeSpace.GetCenter(SphereIdx), NegativeSpace.GetRadius(SphereIdx)))
								{
									OverlapArr[BatchIdx].Add(SphereIdx);
								}
							}
						});
					int32 TotalSize = 0;
					for (int32 BatchIdx = 0; BatchIdx < NumBatch; ++BatchIdx)
					{
						TotalSize += OverlapArr[BatchIdx].Num();
					}
					Part.OverlapsNegativeSpace.Reserve(TotalSize);
					for (int32 BatchIdx = 0; BatchIdx < NumBatch; ++BatchIdx)
					{
						Part.OverlapsNegativeSpace.Append(OverlapArr[BatchIdx]);
					}
				}
				else
				{
					for (int32 SphereIdx : OrigOverlaps)
					{
						if (FConvexDecomposition3::ConvexPartVsSphereOverlap(Part, NegativeSpace.GetCenter(SphereIdx), NegativeSpace.GetRadius(SphereIdx)))
						{
							Part.OverlapsNegativeSpace.Add(SphereIdx);
						}
					}
				}
			};

			AddIfOverlaps(OrigPart);
			for (int32 PartIdx = NewPartIdx; PartIdx < Decomposition.Num(); ++PartIdx)
			{
				AddIfOverlaps(Decomposition[PartIdx]);
			}
		}
	}

	FPartialCutResult() {}

	FPartialCutResult(const FConvexDecomposition3::FConvexPart& Convex, const FPlane3d& Plane, double PlaneTol, bool bTreatAsSolid, double ThickenHullAfterFailureInLocalSpace)
	{
		SignDists.SetNumZeroed(Convex.InternalGeo.MaxVertexID());
		TArray<bool> OnHull; // indicates if vertex was on the original hull
		OnHull.SetNumZeroed(SignDists.Num());
		TArray<bool> ForHull[2]; // indicates if vertex should be used when making the new hull
		ForHull[0].SetNumZeroed(Convex.InternalGeo.MaxVertexID());
		ForHull[1].SetNumZeroed(Convex.InternalGeo.MaxVertexID());

		for (int32 VID : Convex.InternalGeo.VertexIndicesItr())
		{
			if (!Convex.InternalGeo.IsReferencedVertex(VID))
			{
				continue;
			}
			double SD = Plane.DistanceTo(Convex.InternalGeo.GetVertex(VID));
			SignDists[VID] = SD;
			if (SD < -PlaneTol)
			{
				ForHull[0][VID] = true;
			}
			else if (SD > PlaneTol)
			{
				ForHull[1][VID] = true;
			}
		}
		for (int32 EID : Convex.InternalGeo.EdgeIndicesItr())
		{
			FIndex2i EdgeVID = Convex.InternalGeo.GetEdgeV(EID);
			double DistA = SignDists[EdgeVID.A];
			double DistB = SignDists[EdgeVID.B];
			FInterval1d Interval = FInterval1d::MakeFromUnordered(DistA, DistB);
			
			bool OnSide[2]{ Interval.Min < -PlaneTol, Interval.Max > PlaneTol };
			if (OnSide[0] || OnSide[1])
			{
				if (OnSide[0] && OnSide[1]) // crossing case
				{
					CrossingEdgeIDs.Add(EID);
					double t = DistA / (DistA - DistB);
					FVector3d OnPlane = Lerp(
						Convex.InternalGeo.GetVertex(EdgeVID.A),
						Convex.InternalGeo.GetVertex(EdgeVID.B), t);
					CutVertices.Add(OnPlane); // crossing == always goes to both hulls
					// ForHull values already marked for the endpoints, by the per-vertex loop when signed distances were calculated
				}
				else // one-side case
				{
					int32 Side = OnSide[0] ? 0 : 1;
					ForHull[Side][EdgeVID.A] = true;
					ForHull[Side][EdgeVID.B] = true;
				}
			}
			// else edge is on the plane -- rely on the one-side case edges to protect these vertices in most cases
		}
		// for non-solid shapes, we cannot rely on non-planar segments to preserve on-plane geometry
		// so we also also mark on-plane triangle vertices based on the triangle normal
		if (!bTreatAsSolid)
		{
			for (int32 TID : Convex.InternalGeo.TriangleIndicesItr())
			{
				FIndex3i TriVID = Convex.InternalGeo.GetTriangle(TID);
				auto IsVertOnPlane = [this, PlaneTol](int32 TestVID) -> bool
					{
						double Dist = SignDists[TestVID];
						return Dist >= -PlaneTol && Dist <= PlaneTol;
					};
				bool bOnPlane = IsVertOnPlane(TriVID.A) && IsVertOnPlane(TriVID.B) && IsVertOnPlane(TriVID.C);
				if (bOnPlane)
				{
					FVector3d TriNormal = Convex.InternalGeo.GetTriNormal(TID);
					double TriDot = TriNormal.Dot(Plane.Normal);
					int32 KeepSide = int32(TriDot > 0);
					ForHull[KeepSide][TriVID.A] = true;
					ForHull[KeepSide][TriVID.B] = true;
					ForHull[KeepSide][TriVID.C] = true;
				}
			}
		}

		// compute the hulls, s.t. the hull triangle indices are referencing the original internal geo vertices w/ the cut vertices appended

		// crossing range is the range of signed distances spanned by original hull triangles that cross the plane
		// within that range, we need to also consider non-hull vertices when creating the new hulls
		if (!Convex.HullTriangles.IsEmpty())
		{
			FInterval1d CrossingRange(0, 0);
			for (const FIndex3i& HullTri : Convex.HullTriangles)
			{
				OnHull[HullTri.A] = true;
				OnHull[HullTri.B] = true;
				OnHull[HullTri.C] = true;
				FVector3d Signs(
					SignDists[HullTri.A],
					SignDists[HullTri.B],
					SignDists[HullTri.C]
				);
				FInterval1d TriRange(Signs.GetMin(), Signs.GetMax());
				if (TriRange.Contains(0))
				{
					CrossingRange.Contain(TriRange);
				}
			}
			// filter out the non-hull vertices that are not in the crossing range, since these are covered by the re-used hull points
			for (int32 VID : Convex.InternalGeo.VertexIndicesItr())
			{
				if (!OnHull[VID] && !CrossingRange.Contains(SignDists[VID]))
				{
					ForHull[0][VID] = false;
					ForHull[1][VID] = false;
				}
			}
		}

		for (int32 Side = 0; Side < 2; Side++)
		{
			FConvexHull3d HullCompute;
			bool bOK = HullCompute.Solve(Convex.InternalGeo.MaxVertexID() + CutVertices.Num(),
				[this, &Convex](int32 Index)
				{
					int32 MaxVID = Convex.InternalGeo.MaxVertexID();
					return Index < MaxVID ? Convex.InternalGeo.GetVertex(Index) : CutVertices[Index - MaxVID];
				},
				[this, &Convex, &ForHull, Side](int32 Index) { return Index < Convex.InternalGeo.MaxVertexID() ? ForHull[Side][Index] : true; });
			if (!bOK)
			{
				// optionally re-try w/ minimal offsets in degen directions
				bOK = ComputeThickenedHull(ThickenHullAfterFailureInLocalSpace, HullCompute, Convex.InternalGeo, CutVertices, OffsetVertices[Side], [&ForHull, Side](int32 VID) { return ForHull[Side][VID]; });

				if (!bOK)
				{
					// don't cut the hull on a plane that would lead to a degenerate hull on either side
					CutVertices.Empty();
					CrossingEdgeIDs.Empty();
					bSuccess = false;
					return;
				}
			}
			HullTriangles[Side] = HullCompute.MoveTriangles();

			// Custom hull volume calculation to account for the triangles indexing partly into InternalGeo, partly into CutVertices
			HullVolumes[Side] = 0;
			const int32 MaxVID = Convex.InternalGeo.MaxVertexID();
			const int32 MaxCutVID = MaxVID + CutVertices.Num();
			for (const FIndex3i& Tri : HullTriangles[Side])
			{
				FVector3d Verts[3];
				for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int32 VID = Tri[SubIdx];
					Verts[SubIdx] = VID < MaxVID ? Convex.InternalGeo.GetVertex(VID) : VID < MaxCutVID ? CutVertices[VID - MaxVID] : OffsetVertices[Side][VID - MaxCutVID];
				}

				// Get cross product of edges and (un-normalized) normal vector.
				FVector3d V1mRef = (Verts[1] - Convex.GeoCenter);
				FVector3d V2mRef = (Verts[2] - Convex.GeoCenter);
				FVector3d N = V2mRef.Cross(V1mRef);

				HullVolumes[Side] += ((Verts[0] - Convex.GeoCenter)).Dot(N);
			}
			HullVolumes[Side] *= (1.0 / 6.0);
		}

		bSuccess = true;
	}

};

} // end anonymous namespace for helper functionality

void FConvexDecomposition3::FConvexPart::Compact()
{
	if (bIsCompact)
	{
		return;
	}

	// remove all triangles, and all vertices that are not reference by the hull
	// and remap HullTriangles to the new compact indices
	TMap<int32, int32> VertexRemapIDs;
	FDynamicMesh3 VertexMesh;
	for (FIndex3i& Tri : HullTriangles)
	{
		for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
		{
			int32 OldVID = Tri[SubIdx];
			int32* NewVIDFound = VertexRemapIDs.Find(OldVID);
			int32 NewVID;
			if (NewVIDFound)
			{
				NewVID = *NewVIDFound;
			}
			else
			{
				NewVID = VertexMesh.AppendVertex(InternalGeo.GetVertex(OldVID));
				VertexRemapIDs.Add(OldVID, NewVID);
			}
			Tri[SubIdx] = NewVID;
		}
	}
	InternalGeo = MoveTemp(VertexMesh);
	bIsCompact = true;
}

void FConvexDecomposition3::InitializeFromMesh(const FDynamicMesh3& SourceMesh, bool bMergeEdges)
{
	FPreprocessMeshOptions Options;
	Options.bMergeEdges = bMergeEdges;
	InitializeFromMesh(SourceMesh, Options);
}
void FConvexDecomposition3::InitializeFromMesh(const FDynamicMesh3& SourceMesh, const FPreprocessMeshOptions& Options)
{
	Decomposition.Empty();
	FConvexPart* Convex = new FConvexPart(SourceMesh, Options, ResultTransform);
	if (Convex->IsFailed())
	{
		delete Convex;
		return;
	}
	Decomposition.Add(Convex);
}

FConvexDecomposition3::FConvexPart::FConvexPart(const FDynamicMesh3& SourceMesh, bool bMergeEdges, FTransformSRT3d& TransformOut)
{
	// Copy out the source mesh
	InternalGeo.Copy(SourceMesh, false, false, false, false);

	FConvexDecomposition3::FPreprocessMeshOptions Options;
	Options.bMergeEdges = bMergeEdges;
	InitializeFromInternalGeo(Options, TransformOut);
}

FConvexDecomposition3::FConvexPart::FConvexPart(const FDynamicMesh3& SourceMesh, const FConvexDecomposition3::FPreprocessMeshOptions& Options, FTransformSRT3d& TransformOut)
{
	// Copy out the source mesh
	InternalGeo.Copy(SourceMesh, false, false, false, false);

	InitializeFromInternalGeo(Options, TransformOut);
}

FConvexDecomposition3::FConvexPart::FConvexPart(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector3> Faces, const FConvexDecomposition3::FPreprocessMeshOptions& Options, FTransformSRT3d& TransformOut, int32 FaceVertexOffset)
{
	for (const FVector3f& V : Vertices)
	{
		InternalGeo.AppendVertex((FVector)V);
	}
	for (FIntVector F : Faces)
	{
		InternalGeo.AppendTriangle(FIndex3i(F.X + FaceVertexOffset, F.Y + FaceVertexOffset, F.Z + FaceVertexOffset));
	}

	InitializeFromInternalGeo(Options, TransformOut);
}

FConvexDecomposition3::FConvexPart::FConvexPart(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector3> Faces, bool bMergeEdges, FTransformSRT3d& TransformOut, int32 FaceVertexOffset)
{
	for (const FVector3f& V : Vertices)
	{
		InternalGeo.AppendVertex((FVector)V);
	}
	for (FIntVector F : Faces)
	{
		InternalGeo.AppendTriangle(FIndex3i(F.X + FaceVertexOffset, F.Y + FaceVertexOffset, F.Z + FaceVertexOffset));
	}

	FConvexDecomposition3::FPreprocessMeshOptions Options;
	Options.bMergeEdges = bMergeEdges;
	InitializeFromInternalGeo(Options, TransformOut);
}

void FConvexDecomposition3::FConvexPart::InitializeFromInternalGeo(const FConvexDecomposition3::FPreprocessMeshOptions& Preprocess, FTransformSRT3d& TransformOut)
{
	FAxisAlignedBox3d InitialBounds = InternalGeo.GetBounds();
	double InvScaleFactor = FMath::Clamp(InitialBounds.MaxDim(), KINDA_SMALL_NUMBER, 1e8);
	double ScaleFactor = 1.0 / InvScaleFactor;

	// Weld close edges so we can sample convex edges
	if (Preprocess.bMergeEdges)
	{
		FMergeCoincidentMeshEdges MergeEdges(&InternalGeo);
		// scale the tolerance by the bounds size, for consistency across input scales
		MergeEdges.MergeVertexTolerance = FMath::Max(MergeEdges.MergeVertexTolerance * ScaleFactor, FMathd::Epsilon);
		MergeEdges.Apply();
	}

	// Apply any custom preprocessing (e.g., simplification, mesh repair, etc)
	if (Preprocess.CustomPreprocess)
	{
		Preprocess.CustomPreprocess(InternalGeo, InitialBounds);
		
		// Re-compute bounds and scale factors, as the preprocess might have changed them
		InitialBounds = InternalGeo.GetBounds();
		InvScaleFactor = FMath::Clamp(InitialBounds.MaxDim(), KINDA_SMALL_NUMBER, 1e8);
		ScaleFactor = 1.0 / InvScaleFactor;
	}

	// Transform the mesh to a standard unit-cube-at-origin space, so thresholds have a consistent meaning
	FTransformSRT3d MeshTransform(-ScaleFactor * InitialBounds.Center());
	MeshTransform.SetScale(ScaleFactor * MeshTransform.GetScale());
	MeshTransforms::ApplyTransform(InternalGeo, MeshTransform);
	// Return the inverse transform by reference, so we can put the results back into the original space
	TransformOut = FTransformSRT3d(InitialBounds.Center());
	TransformOut.SetScale(FVector3d(InvScaleFactor, InvScaleFactor, InvScaleFactor));

	double ThickenInLocalSpace = Preprocess.ThickenInputAfterHullFailure * ScaleFactor;

	// Compute hull and standard measurements (volume, center, bounds, etc)
	ComputeHull(true /*bComputePlanes*/, ThickenInLocalSpace);
	ComputeStats();
}


void FConvexDecomposition3::InitializeFromIndexMesh(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector> Faces, bool bMergeEdges, int32 FaceVertexOffset)
{
	FConvexDecomposition3::FPreprocessMeshOptions Preprocess;
	Preprocess.bMergeEdges = bMergeEdges;
	InitializeFromIndexMesh(Vertices, Faces, Preprocess, FaceVertexOffset);
}


void FConvexDecomposition3::InitializeFromIndexMesh(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector> Faces, const FConvexDecomposition3::FPreprocessMeshOptions& Preprocess, int32 FaceVertexOffset)
{
	Decomposition.Empty();
	FConvexPart* Convex = new FConvexPart(Vertices, Faces, Preprocess, ResultTransform, FaceVertexOffset);
	if (Convex->IsFailed())
	{
		delete Convex;
		return;
	}
	Decomposition.Add(Convex);
}


bool FConvexDecomposition3::InitializeNegativeSpace(const FNegativeSpaceSampleSettings& Settings, TArrayView<const FVector3d> RequestedSamples)
{
	if (Decomposition.Num() != 1)
	{
		return false;
	}
	TMeshAABBTree3<FDynamicMesh3> InternalGeoAABBTree(&Decomposition[0].InternalGeo, true);
	TFastWindingTree<FDynamicMesh3> InternalGeoWinding(&InternalGeoAABBTree, true);

	FNegativeSpaceSampleSettings RescaledSettings = Settings;
	RescaledSettings.SetResultTransform(ResultTransform);
	NegativeSpace.Reset();
	NegativeSpace.AddNegativeSpace(InternalGeoWinding, RescaledSettings, false);
	for (const FVector3d& Req : RequestedSamples)
	{
		FVector3d Center = ResultTransform.InverseTransformPosition(Req);
		if (!RescaledSettings.bAllowSamplesInsideMesh)
		{
			double Winding = InternalGeoWinding.FastWindingNumber(Center);
			if (Winding > .5) // inside, should skip
			{
				continue;
			}
		}
		FVector3d NearSample = InternalGeoAABBTree.FindNearestPoint(Center);
		double Dist = FVector3d::Dist(Center, NearSample);
		double Rad = FMath::Max(0, Dist - RescaledSettings.ReduceRadiusMargin);
		NegativeSpace.AddSphere(Center, Rad);
	}
	InitNegativeSpaceConvexPartMapping();
	return true;
}

bool FConvexDecomposition3::AddCoveringToInitialNegativeSpace(const FSphereCovering& Covering, FMatrix CoveringToLocalSpace, double RadiusToLocalScale)
{
	if (Decomposition.Num() != 1)
	{
		return false;
	}
	int32 OrigNum = NegativeSpace.Num();
	RadiusToLocalScale /= FMath::Max(FMathd::ZeroTolerance, ResultTransform.GetScale3D().GetAbsMin());
	CoveringToLocalSpace *= ((FTransform)ResultTransform).ToInverseMatrixWithScale();
	for (int32 SphereIdx = 0, Num = Covering.Num(); SphereIdx < Num; ++SphereIdx)
	{
		FVector3d Center = CoveringToLocalSpace.TransformPosition(Covering.GetCenter(SphereIdx));
		double Radius = Covering.GetRadius(SphereIdx) * RadiusToLocalScale;
		NegativeSpace.AddSphere(Center, Radius);
	}
	InitNegativeSpaceConvexPartMapping(OrigNum);
	return true;
}

void FConvexDecomposition3::FixHullOverlapsInNegativeSpace(double NegativeSpaceTolerance, double NegativeSpaceMinRadius)
{
	if (!NegativeSpace.Num())
	{
		return;
	}

	// make sure tolerance and min radius are at least 0
	double UseTolerance = FMath::Max(0, NegativeSpaceTolerance);
	double UseMinRadius = FMath::Max(0, NegativeSpaceMinRadius);
	// fix overlaps
	for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); ++PartIdx)
	{
		for (int32 SphereIdx : Decomposition[PartIdx].OverlapsNegativeSpace)
		{
			double DistSq = 0;
			double Radius = NegativeSpace.GetRadius(SphereIdx);
			if (Radius > UseMinRadius && ConvexPartVsSphereOverlap(Decomposition[PartIdx], NegativeSpace.GetCenter(SphereIdx), Radius, nullptr, &DistSq))
			{
				double Dist = FMath::Sqrt(DistSq);
				check(Dist <= Radius);
				NegativeSpace.SetRadius(SphereIdx, Dist - UseTolerance);
			}
		}
		Decomposition[PartIdx].OverlapsNegativeSpace.Reset();
	}
	// delete all too-small spheres
	NegativeSpace.RemoveSmaller(UseMinRadius);
}

void FConvexDecomposition3::Compute(int32 NumOutputHulls, int32 NumAdditionalSplits, double ErrorTolerance, double MinThicknessTolerance, int32 MaxOutputHulls, bool bOnlySplitIfNegativeSpaceCovered)
{
	if (MaxOutputHulls > 0)
	{
		NumOutputHulls = FMath::Min(NumOutputHulls, MaxOutputHulls);
	}
	bool bUseNegativeSpace = NegativeSpace.Num() > 0;
	int32 TargetNumSplits = NumOutputHulls + NumAdditionalSplits;
	for (int32 SplitIdx = 0; SplitIdx < TargetNumSplits; SplitIdx++)
	{
		int32 NumNewParts = SplitWorst(bool(SplitIdx % 2), ErrorTolerance, bOnlySplitIfNegativeSpaceCovered);
		if (NumNewParts == 0)
		{
			break;
		}
	}

	if (bUseNegativeSpace)
	{
		// remove overlaps with negative space that the splitting didn't resolve by just shrinking/deleting the associated spheres
		// (b/c otherwise the overlapped parts will be incapable of merging with anything in the MergeBest step)
		FixHullOverlapsInNegativeSpace();
	}

	constexpr bool bAllowCompact = true;
	MergeBest(NumOutputHulls, ErrorTolerance, MinThicknessTolerance, bAllowCompact, false, MaxOutputHulls, nullptr, nullptr);
}



bool FConvexDecomposition3::FConvexPart::ComputeHull(bool bComputePlanes, double ThickenAfterHullFailureInLocalSpace)
{
	FConvexHull3d HullCompute;
	bool bOK = HullCompute.Solve(InternalGeo.MaxVertexID(),
		[this](int32 Index) { return InternalGeo.GetVertex(Index); },
		[this](int32 Index) { return InternalGeo.IsVertex(Index); });
	if (!bOK && ThickenAfterHullFailureInLocalSpace > 0)
	{
		TArray<FVector3d> CutVertices_Unused, ThickenVertices;
		if (ComputeThickenedHull(ThickenAfterHullFailureInLocalSpace, HullCompute, InternalGeo, CutVertices_Unused, ThickenVertices))
		{
			bOK = true;
			for (FVector3d V : ThickenVertices)
			{
				InternalGeo.AppendVertex(V);
			}
		}
		else
		{
			bFailed = true;
			return false;
		}
	}
	HullTriangles = HullCompute.MoveTriangles();
	if (bComputePlanes)
	{
		ComputeHullPlanes();
	}
	return true;
}

void FConvexDecomposition3::FConvexPart::ComputeHullPlanes()
{
	HullPlanes.SetNum(HullTriangles.Num());
	for (int32 TriIdx = 0; TriIdx < HullTriangles.Num(); TriIdx++)
	{
		const FIndex3i& Tri = HullTriangles[TriIdx];
		FVector3d VA = InternalGeo.GetVertex(Tri.A);
		double Area = 0;
		FVector3d Normal = VectorUtil::NormalArea(VA, InternalGeo.GetVertex(Tri.B), InternalGeo.GetVertex(Tri.C), Area);
		if (Area < FMathd::ZeroTolerance)
		{
			HullPlanes[TriIdx] = FPlane3d(FVector3d::ZeroVector, 0); // track degenerate plane with a zero-vector normal
		}
		else
		{
			HullPlanes[TriIdx] = FPlane3d(Normal, VA);
		}
	}
}

void FConvexDecomposition3::FConvexPart::ComputeStats()
{
	check(!bIsCompact); // cannot compute GeoVolume if the part is already compacted
	FConvexPartHullMeshAdapter HullAdapter(this);
	GeoCenter = TMeshQueries<FDynamicMesh3>::GetMeshVerticesCentroid(InternalGeo);
	GeoVolume = GetVolumeUsingReferencePoint<FDynamicMesh3>(InternalGeo, GeoCenter);
	HullVolume = GetVolumeUsingReferencePoint<FConvexPartHullMeshAdapter>(HullAdapter, GeoCenter);
	Bounds = InternalGeo.GetBounds();
	if (GeoVolume < 0)
	{
		bGeometryVolumeUnreliable = true;
		GeoVolume = 0;
	}
	HullError = HullVolume - GeoVolume;
}

int32 FConvexDecomposition3::SplitWorst(bool bCanSkipUnreliableGeoVolumes, double ErrorTolerance, bool bOnlySplitIfNegativeSpaceCovered, double MinSplitSizeInWorldSpace)
{
	int32 InitialNum = Decomposition.Num();
	int32 NumAttempts = InitialNum + 1;
	while (!SplitWorstHelper(bCanSkipUnreliableGeoVolumes, ErrorTolerance, bOnlySplitIfNegativeSpaceCovered, MinSplitSizeInWorldSpace)) // keep attempting to split until we succeed or run out of parts to try
	{
		if (!ensure(NumAttempts-- > 0)) // guard against infinite loop
		{
			break;
		}
	}
	return Decomposition.Num() - InitialNum;
}

bool FConvexDecomposition3::SplitWorstHelper(bool bCanSkipUnreliableGeoVolumes, double ErrorTolerance, bool bOnlySplitIfNegativeSpaceCovered, double MinSplitSizeInWorldSpace)
{
	if (Decomposition.Num() == 0)
	{
		return true;
	}

	double MinSplitSize = MinSplitSizeInWorldSpace <= 0 ? MinSplitSizeInWorldSpace : ConvertDistanceToleranceToLocalSpace(MinSplitSizeInWorldSpace);
	double ThickenAfterHullFailureLocalSpace = ConvertDistanceToleranceToLocalSpace(ThickenAfterHullFailure);

	double VolumeTolerance = ConvertDistanceToleranceToLocalVolumeTolerance(ErrorTolerance);

	double WorstError = -FMathd::MaxReal;
	bool bErrorAboveTolerance = false;
	int32 WorstIdx = INDEX_NONE;
	bool bCanSkipCurrent = true;
	bool bHasOverlapsNegative = false;
	// check if we have any negative-space-covered parts
	if (NegativeSpace.Num() > 0)
	{
		for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); PartIdx++)
		{
			if (Decomposition[PartIdx].bSplitFailed)
			{
				continue;
			}
			if (Decomposition[PartIdx].Bounds.MaxDim() < MinSplitSize)
			{
				continue;
			}
			if (!Decomposition[PartIdx].OverlapsNegativeSpace.IsEmpty())
			{
				bHasOverlapsNegative = true;
				break;
			}
		}
	}
	// stop early if there are no negative-space overlaps, and we're only splitting in those cases
	if (!bHasOverlapsNegative && bOnlySplitIfNegativeSpaceCovered)
	{
		return true;
	}
	for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); PartIdx++)
	{
		if (Decomposition[PartIdx].bSplitFailed)
		{
			continue;
		}
		if (Decomposition[PartIdx].Bounds.MaxDim() < MinSplitSize)
		{
			continue;
		}
		bool bOverlapsNegative = !Decomposition[PartIdx].OverlapsNegativeSpace.IsEmpty();
		// always favor the parts that overlap negative space as long as we have them
		if (bHasOverlapsNegative && !bOverlapsNegative)
		{
			continue;
		}
		if (Decomposition[PartIdx].HullError > VolumeTolerance)
		{
			bErrorAboveTolerance = true;
		}
		if (Decomposition[PartIdx].HullError > WorstError)
		{
			if (bCanSkipUnreliableGeoVolumes && !bCanSkipCurrent && Decomposition[PartIdx].bGeometryVolumeUnreliable)
			{
				continue;
			}
			WorstIdx = PartIdx;
			WorstError = Decomposition[PartIdx].HullError;
			bCanSkipCurrent = bCanSkipCurrent && Decomposition[PartIdx].bGeometryVolumeUnreliable;
		}
	}
	
	if (WorstIdx == INDEX_NONE)
	{
		return true;
	}

	// Stop splitting if we see no errors above tolerance and no negative-space overlaps
	// Note if bCanSkipUnreliableGeoVolumes==true, we may still split a below-tolerance part
	//  -- TODO: consider what to do in this case!
	//		It will just add one extra split currently as we alternate toggling bCanSkipUnreliableGeoVolumes off, but the behavior may be confusing.
	if (!bErrorAboveTolerance && !bHasOverlapsNegative)
	{
		return true;
	}

	FConvexPart& Part = Decomposition[WorstIdx];

	double OrigHullVolume = Decomposition[WorstIdx].HullVolume;

	TArray<FPlane3d> CandidatePlanes;

	// Always start with the major axis planes (as long as the axis bounds length is at least the min split size)
	FVector3d Center = Part.Bounds.Center();
	FVector3d Extents = Part.Bounds.Extents();
	int32 MaxBoundsDim = FMath::Max3Index(Extents.X, Extents.Y, Extents.Z);
	int32 MaxAxisAlignedIdx = 0;
	for (int32 AxisIdx = 0; AxisIdx < 3; ++AxisIdx)
	{
		bool bIsMaxDim = AxisIdx == MaxBoundsDim;
		if (bIsMaxDim || Extents[AxisIdx] >= MinSplitSize)
		{
			FVector3d AxisVector = FVector3d::ZeroVector;
			AxisVector[AxisIdx] = 1.0;
			int32 PlaneIdx = CandidatePlanes.Emplace(AxisVector, Center);
			if (bIsMaxDim)
			{
				MaxAxisAlignedIdx = PlaneIdx;
			}
		}
	}

	// make candidate planes out of convex and boundary edges

	// Structure of arrays for planes that come from convex (or open) edges
	constexpr int32 MaxPerEdge = 5;
	struct FConvexEdgeCandidates
	{
		TArray<FVector3d> Centers;
		TArray<float> Depths;
		TArray<float> Convexities; // will favor planes that come from edges that are deeper in the hull
		TArray<TArray<FPlane3d, TFixedAllocator<MaxPerEdge>>> PlaneArrays;

		void AddSingle(FConvexPart& ConvPart, const FVector3d& Center, const FVector3d& Normal, double Convexity = 0)
		{
			Centers.Add(Center);
			PlaneArrays.Emplace_GetRef().Add(FPlane3d(Normal, Center));
			Depths.Add((float)ConvPart.GetHullDepth(Center));
			Convexities.Add((float)Convexity);
		}

		void AddMultiple(FConvexPart& ConvPart, const FVector3d& Center, FVector3d Normals[MaxPerEdge], FVector3d Offsets[MaxPerEdge], int32 NumToAdd, double Convexity)
		{
			TArray<FPlane3d, TFixedAllocator<MaxPerEdge>> PlaneSet;
			for (int32 NormalIdx = 0; NormalIdx < NumToAdd; NormalIdx++)
			{
				if (Normals[NormalIdx].Normalize())
				{
					PlaneSet.Add(FPlane3d(Normals[NormalIdx], Center + Offsets[NormalIdx]));
				}
			}
			if (PlaneSet.IsEmpty())
			{
				return;
			}

			Centers.Add(Center);
			PlaneArrays.Add(PlaneSet);

			constexpr int32 DepthIdx = 0, ConvexityIdx = 1;
			Depths.Add((float)ConvPart.GetHullDepth(Center));
			Convexities.Add((float)Convexity);
		}
	};

	const double CosHalfAngleExtraSamplesThreshold = FMathd::Cos(FMathd::DegToRad * ConvexEdgeAngleMoreSamplesThreshold * .5);
	const double CosHalfAngleIsConvexThreshold = FMathd::Cos(FMathd::DegToRad * ConvexEdgeAngleThreshold * .5);

	FConvexEdgeCandidates ConvexEdgeCandidates;
	for (int EID : Part.InternalGeo.EdgeIndicesItr())
	{
		FDynamicMesh3::FEdge Edge = Part.InternalGeo.GetEdge(EID);
		FVector3d NormalA = Part.InternalGeo.GetTriNormal(Edge.Tri.A);

		constexpr bool bIncludeBoundaryEdges = false;
		if (Edge.Tri.B < 0) // Boundary edge; try adding a candidate plane on these just in case
		{
			if (!bIncludeBoundaryEdges)
			{
				continue;
			}
			FVector3d VA = Part.InternalGeo.GetVertex(Edge.Vert.A);
			FVector3d VB = Part.InternalGeo.GetVertex(Edge.Vert.B);
			FVector3d EdgeDir = VA - VB;
			FVector3d CutNormal = EdgeDir.Cross(NormalA);
			if (!CutNormal.Normalize())
			{
				continue;
			}
			ConvexEdgeCandidates.AddSingle(Part, (VA + VB) * .5, CutNormal);
		}
		else // Non-boundary edge; compute the angle on the edge
		{
			FVector3d NormalB = Part.InternalGeo.GetTriNormal(Edge.Tri.B);
			FVector3d AvgNormal = NormalA + NormalB;
			if (!AvgNormal.Normalize())
			{
				continue;
			}
			FIndex2i OppV = Part.InternalGeo.GetEdgeOpposingV(EID);
			FVector3d OppAPos = Part.InternalGeo.GetVertex(OppV.A);
			FVector3d VA = Part.InternalGeo.GetVertex(Edge.Vert.A);
			FVector3d VB = Part.InternalGeo.GetVertex(Edge.Vert.B);
			FVector3d EdgeDir = VA - VB;
			FVector3d OppAEdge = OppAPos - VA;
			if (!EdgeDir.Normalize())
			{
				continue;
			}
			// Direction of triangle A, perpendicular to the shared edge
			FVector3d OppADir = OppAEdge - OppAEdge.Dot(EdgeDir) * EdgeDir;
			if (!OppADir.Normalize())
			{
				continue;
			}

			double CosHalfAngle = AvgNormal.Dot(OppADir);
			if (CosHalfAngle < CosHalfAngleIsConvexThreshold)
			{
				// not convex enough
				continue;
			}

			// Direction of triangle B, perpendicular to the shared edge (by symmetry)
			FVector3d OppBDir = OppADir - 2 * (OppADir - OppADir.Dot(AvgNormal) * AvgNormal);

			FVector3d PlaneNormals[5];
			PlaneNormals[0] = EdgeDir.Cross(AvgNormal);
			PlaneNormals[1] = EdgeDir.Cross(OppADir);
			PlaneNormals[2] = EdgeDir.Cross(OppBDir);
			int32 NumPlanes = 3;
			if (CosHalfAngle < CosHalfAngleExtraSamplesThreshold) // Sample additional cutting planes if the spanned angle is large enough
			{
				NumPlanes = 5;
				PlaneNormals[3] = (PlaneNormals[0] + PlaneNormals[1]);
				PlaneNormals[4] = (PlaneNormals[0] + PlaneNormals[2]);
			}
			// Add small offsets to the on-triangle planes, so the source triangle is fully on one side of the plane
			FVector3d Offsets[5] {FVector3d::ZeroVector, FVector3d::ZeroVector, FVector3d::ZeroVector, FVector3d::ZeroVector, FVector3d::ZeroVector };
			Offsets[1] = NormalA * OnPlaneTolerance;
			Offsets[2] = NormalB * OnPlaneTolerance;
			ConvexEdgeCandidates.AddMultiple(Part, (VA + VB) * .5, PlaneNormals, Offsets, NumPlanes, CosHalfAngle);
		}
	}

	int32 InitialNumPlanes = CandidatePlanes.Num();
	FPriorityOrderPoints OrderConvexEdges;
	OrderConvexEdges.ComputeUniformSpaced(ConvexEdgeCandidates.Centers, ConvexEdgeCandidates.Depths, ConvexEdgeCandidates.Convexities, MaxConvexEdgePlanes);
	for (int32 OrderIdx = 0; OrderIdx < OrderConvexEdges.Order.Num() && CandidatePlanes.Num() < InitialNumPlanes + MaxConvexEdgePlanes; OrderIdx++)
	{
		int32 CvxEdgeCandIdx = OrderConvexEdges.Order[OrderIdx];
		CandidatePlanes.Append(ConvexEdgeCandidates.PlaneArrays[CvxEdgeCandIdx]);
	}

	// TODO: Consider doing parallel evaluation of candidate planes; however then we'd need to store the partial cut result for all of them
	double LowestError = FMathd::MaxReal;
	int32 BestPlaneIdx = -1;
	FPartialCutResult BestCutResult;
	// use solid cuts only if the setting is enabled and we haven't found problems w/ this geometry in previous cuts
	const bool bCutAsSolid = bTreatAsSolid && !Part.bGeometryVolumeUnreliable;

	for (int32 PlaneIdx = 0; PlaneIdx < CandidatePlanes.Num(); PlaneIdx++)
	{
		const FPlane3d& Plane = CandidatePlanes[PlaneIdx];
		FPartialCutResult PlaneResult(Part, Plane, OnPlaneTolerance, bCutAsSolid, ThickenAfterHullFailureLocalSpace);
		if (!PlaneResult.bSuccess)
		{
			continue;
		}
		double PlaneError = PlaneResult.Score(Part);
		if (MaxAxisAlignedIdx == PlaneIdx) // favor the larger axis (make predictable cuts when none of the planes are reducing the hull volume)
		{
			PlaneError *= CutLargestAxisErrorScale;
		}
		if (PlaneError < LowestError)
		{
			BestPlaneIdx = PlaneIdx;
			LowestError = PlaneError;
			BestCutResult = MoveTemp(PlaneResult);
		}
	}

	if (BestPlaneIdx == -1)
	{
		Decomposition[WorstIdx].bSplitFailed = true; // mark this part so we know not to try splitting it again
		return false; // return failure from the helper so split can be re-attempted
	}

	int32 NewPartsStartIdx = Decomposition.Num();
	int32 OtherSideStartIdx = -1;
	BestCutResult.ApplyToGeo(Decomposition, WorstIdx, CandidatePlanes[BestPlaneIdx], OtherSideStartIdx, OnPlaneTolerance, ConnectedComponentTolerance, NegativeSpace, bCutAsSolid, bSplitDisconnectedComponents, ThickenAfterHullFailureLocalSpace);

	UpdateProximitiesAfterSplit(WorstIdx, NewPartsStartIdx, CandidatePlanes[BestPlaneIdx], OtherSideStartIdx, OrigHullVolume);

	return true;
}

void FConvexDecomposition3::UpdateProximitiesAfterSplit(int32 SplitIdx, int32 NewIdxStart, FPlane3d CutPlane, int32 SecondSideIdxStart, double OrigHullVolume)
{
	auto IsMeshOnPlane = [this](int32 DecompIdx, const FPlane3d& Plane)
	{
		const FDynamicMesh3& Mesh = Decomposition[DecompIdx].InternalGeo;
		const TArray<FIndex3i>& HullTriangles = Decomposition[DecompIdx].HullTriangles;
		for (const FIndex3i& Tri : HullTriangles)
		{
			for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
			{
				FVector3d V = Mesh.GetVertex(Tri[SubIdx]);
				// Note by construction the geometry should not be crossing far through the plane; otherwise would need to look at spans / couldn't just check for small abs ...
				bool bOnPlane = FMathd::Abs(Plane.DistanceTo(V)) < ProximityTolerance;
				if (bOnPlane)
				{
					return true;
				}
			}
		}
		return false;
	};

	bool bNoExtraComponents = NewIdxStart == SecondSideIdxStart && (SecondSideIdxStart + 1) == Decomposition.Num();

	// First update the links that were connected to the previous SplitIdx

	TArray<int32> ToRemoveProximities;
	int32 StartNewProximitiesIdx = Proximities.Num();
	for (auto It = DecompositionToProximity.CreateConstKeyIterator(SplitIdx); It; ++It)
	{
		int32 ProxIdx = It.Value();
		checkSlow(Proximities[ProxIdx].Link.Contains(SplitIdx));
		// Note: intentionally not taking any const references to the FProximity data because the array may reallocate as it expands below
		FPlane3d Plane = Proximities[ProxIdx].Plane;
		FIndex2i Link = Proximities[ProxIdx].Link;
		int32 OtherDecoIdx = Link.A;
		if (OtherDecoIdx == SplitIdx)
		{
			OtherDecoIdx = Link.B;
		}

		// If the newly split mesh is still on the plane associated with this previous link, assume the link is still worth considering for merges
		FAxisAlignedBox3d Bounds = Decomposition[SplitIdx].Bounds;
		Bounds.Expand(ProximityTolerance);
		if (IsMeshOnPlane(SplitIdx, Plane) && Bounds.Intersects(Decomposition[OtherDecoIdx].Bounds))
		{
			// can re-use the old proximity; links are exactly the same
			Proximities[ProxIdx].ClearMergedVolume();
		}
		else
		{
			// If the cut mesh no longer contacts the plane, remove the proximity
			ToRemoveProximities.Add(ProxIdx);
		}

		// For all new decomposition elements, consider creating a new proximity link
		for (int32 NewDecoIdx = NewIdxStart; NewDecoIdx < Decomposition.Num(); NewDecoIdx++)
		{
			Bounds = Decomposition[NewDecoIdx].Bounds;
			Bounds.Expand(ProximityTolerance);
			if (IsMeshOnPlane(NewDecoIdx, Plane) && Bounds.Intersects(Decomposition[OtherDecoIdx].Bounds))
			{
				Proximities.Emplace(FIndex2i(NewDecoIdx, OtherDecoIdx), Plane, true);
			}
		}
	}

	// Connect up new edges in the Decomposition->Proximity map
	for (int32 ProxIdx = StartNewProximitiesIdx; ProxIdx < Proximities.Num(); ProxIdx++)
	{
		DecompositionToProximity.Add(Proximities[ProxIdx].Link.A, ProxIdx);
		DecompositionToProximity.Add(Proximities[ProxIdx].Link.B, ProxIdx);
	}

	// Delete removed edges (including updating the map as needed)
	// Note this must happen *after* connecting up new edges, or else the StartNewProximitiesIdx will be wrong
	DeleteProximity(MoveTemp(ToRemoveProximities), true);

	// Create new proximity links across the new plane
	bool bOrigOnPlane = IsMeshOnPlane(SplitIdx, CutPlane);
	TArray<bool> NewPartsOnPlane; // note: this array is only non-empty when connected components were split off
	NewPartsOnPlane.SetNum(SecondSideIdxStart - NewIdxStart);
	for (int32 Idx = NewIdxStart; Idx < SecondSideIdxStart; Idx++)
	{
		NewPartsOnPlane[Idx - NewIdxStart] = IsMeshOnPlane(Idx, CutPlane);
	}
	int32 BeforeNumProximities = Proximities.Num();
	auto TryLink = [this, &CutPlane](int32 DecoAIdx, int32 DecoBIdx)
	{
		FAxisAlignedBox3d Bounds = Decomposition[DecoAIdx].Bounds;
		Bounds.Expand(ProximityTolerance);
		if (Bounds.Intersects(Decomposition[DecoBIdx].Bounds))
		{
			int32 ProxIdx = Proximities.Emplace(FIndex2i(DecoAIdx, DecoBIdx), CutPlane, true);
			DecompositionToProximity.Add(DecoAIdx, ProxIdx);
			DecompositionToProximity.Add(DecoBIdx, ProxIdx);
		}
	};
	for (int32 DecoSideBIdx = SecondSideIdxStart; DecoSideBIdx < Decomposition.Num(); DecoSideBIdx++)
	{
		if (IsMeshOnPlane(DecoSideBIdx, CutPlane))
		{
			if (bOrigOnPlane)
			{
				TryLink(DecoSideBIdx, SplitIdx);
			}
			for (int32 Idx = NewIdxStart; Idx < SecondSideIdxStart; Idx++)
			{
				if (NewPartsOnPlane[Idx - NewIdxStart])
				{
					TryLink(DecoSideBIdx, Idx);
				}
			}
		}
	}
	// If there were no extra components, and a proximity was added, the merge cost is just the pre-split error
	if (bNoExtraComponents && Proximities.Num() == BeforeNumProximities + 1)
	{
		Proximities.Last().MergedVolume = OrigHullVolume;
	}
}

void FConvexDecomposition3::InitializeFromHulls(int32 NumHulls, TFunctionRef<double(int32)> HullVolumes, TFunctionRef<int32(int32)> HullNumVertices, TFunctionRef<FVector3d(int32, int32)> HullVertices, TArrayView<const TPair<int32, int32>> Proximity)
{
	Decomposition.Empty(NumHulls);
	for (int32 HullIdx = 0; HullIdx < NumHulls; ++HullIdx)
	{
		FConvexPart* Convex = new FConvexPart(true);
		Convex->HullSourceID = HullIdx;
		Convex->bGeometryVolumeUnreliable = false;
		Convex->bMustMerge = false;
		double Volume = HullVolumes(HullIdx);
		Convex->GeoVolume = Volume;
		Convex->HullVolume = Volume;
		Convex->GeoCenter = FVector3d(0, 0, 0);
		FAxisAlignedBox3d Bounds;
		int32 NumVertices = HullNumVertices(HullIdx);
		for (int32 Idx = 0; Idx < NumVertices; ++Idx)
		{
			FVector3d Vertex = HullVertices(HullIdx, Idx);
			Convex->GeoCenter += Vertex;
			Convex->InternalGeo.AppendVertex(Vertex);
			Bounds.Contain(Vertex);
		}
		if (ensure(NumVertices > 0))
		{
			Convex->GeoCenter /= NumVertices;
		}
		Convex->Bounds = Bounds;
		// Note: For merging, actual triangles + planes not required
		Decomposition.Add(Convex);
	}

	for (const TPair<int32, int32>& Link : Proximity)
	{
		int32 ProxIdx = Proximities.Emplace(FIndex2i(Link.Key, Link.Value), FPlane3d(), false /*bPlaneSeparates*/);
		DecompositionToProximity.Add(Link.Key, ProxIdx);
		DecompositionToProximity.Add(Link.Value, ProxIdx);
	}
}

void FConvexDecomposition3::InitializeProximityFromDecompositionBoundingBoxOverlaps(double ExpandByBoundsMinDimFactor, double ExpandByBoundsMaxDimFactor, double MinExpandAmount)
{
	Proximities.Empty();
	DecompositionToProximity.Empty();
	
	FAxisAlignedBox3d OverallBounds;
	double MaxDim = 0;
	TArray<FAxisAlignedBox3d> ExpandedBounds;
	ExpandedBounds.SetNum(Decomposition.Num());
	for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); ++PartIdx)
	{
		FAxisAlignedBox3d Bounds = Decomposition[PartIdx].Bounds;
		double ExpandAmount = FMath::Max3(MinExpandAmount, Bounds.MinDim() * ExpandByBoundsMinDimFactor, Bounds.MaxDim() * ExpandByBoundsMaxDimFactor);
		Bounds.Expand(ExpandAmount);
		OverallBounds.Contain(Bounds);
		MaxDim = FMath::Max(MaxDim, Bounds.MaxDim());
		ExpandedBounds[PartIdx] = Bounds;
	}
	FSparseDynamicOctree3 Octree;
	Octree.RootDimension = FMath::Max(MaxDim, OverallBounds.MaxDim() / 2.0);
	TArray<int32> Overlaps;
	for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); ++PartIdx)
	{
		FAxisAlignedBox3d QueryBounds = ExpandedBounds[PartIdx];
		
		Overlaps.Reset();
		Octree.RangeQuery(QueryBounds, Overlaps);
		for (int32 OverlapIdx : Overlaps)
		{
			// Filter range query results for actual overlaps with query bounds
			if (QueryBounds.Intersects(ExpandedBounds[OverlapIdx]))
			{
				int32 ProxIdx = Proximities.Emplace(FIndex2i(OverlapIdx, PartIdx), FPlane3d(), false /*bPlaneSeparates*/);
				DecompositionToProximity.Add(OverlapIdx, ProxIdx);
				DecompositionToProximity.Add(PartIdx, ProxIdx);
			}
		}

		Octree.InsertObject(PartIdx, ExpandedBounds[PartIdx]);
	}
}

int32 FConvexDecomposition3::MergeBest(int32 InTargetNumParts, double MaxErrorTolerance, double MinThicknessToleranceWorldSpace, bool bAllowCompact, bool bRequireHullTriangles, int32 MaxOutputHulls,
	const FSphereCovering* OptionalNegativeSpace, const FTransform* OptionalTransformIntoNegativeSpace)
{
	FMergeSettings Settings;
	Settings.TargetNumParts = InTargetNumParts;
	Settings.ErrorTolerance = MaxErrorTolerance;
	Settings.MinThicknessTolerance = MinThicknessToleranceWorldSpace;
	Settings.bAllowCompact = bAllowCompact;
	Settings.bRequireHullTriangles = bRequireHullTriangles;
	Settings.MaxOutputHulls = MaxOutputHulls;
	Settings.OptionalNegativeSpace = OptionalNegativeSpace;
	Settings.OptionalTransformIntoNegativeSpace = OptionalTransformIntoNegativeSpace;
	return MergeBest(Settings);
}

int32 FConvexDecomposition3::MergeBest(const FMergeSettings& Settings)
{
	// expand some settings out to variables for convenience
	int32 TargetNumParts = FMath::Max(1, Settings.TargetNumParts);
	double MaxErrorTolerance = Settings.ErrorTolerance;
	double MinThicknessToleranceWorldSpace = Settings.MinThicknessTolerance;
	bool bAllowCompact = Settings.bAllowCompact;
	bool bRequireHullTriangles = Settings.bRequireHullTriangles;
	int32 MaxOutputHulls = Settings.MaxOutputHulls;
	const FSphereCovering* OptionalNegativeSpace = Settings.OptionalNegativeSpace;
	const FTransform* OptionalTransformIntoNegativeSpace = Settings.OptionalTransformIntoNegativeSpace;

	const bool bHasValidMaxHulls = MaxOutputHulls > 0;
	if (bHasValidMaxHulls)
	{
		TargetNumParts = FMath::Min(TargetNumParts, MaxOutputHulls);
	}

	// Support having a max error tolerance
	double VolumeTolerance = ConvertDistanceToleranceToLocalVolumeTolerance(MaxErrorTolerance);
	double MinThicknessTolerance = ConvertDistanceToleranceToLocalSpace(MinThicknessToleranceWorldSpace);
	double ThickenAfterHullFailureLocalSpace = ConvertDistanceToleranceToLocalSpace(ThickenAfterHullFailure);

	int32 MergeNum = 0;

	
	// Map from proximity index to a pre-computed convex part that would result if the proximity-linked parts were merged
	TMap<int32, TUniquePtr<FConvexPart>> ProximityComputedParts;
	// threshold at which we will start to more aggressively evict cached computed parts from ProximityComputedParts
	constexpr int32 EvictComputedPartsThreshold = 10000;

	auto IsPartBelowSizeTolerance = [MinThicknessTolerance](const FConvexPart& Part) -> bool
	{ 
		if (MinThicknessTolerance <= 0) // tolerance is disabled
		{
			return false;
		}
		// Check the part vs the thickness tolerance
		// First check the already-computed AABB
		if (Part.Bounds.MinDim() < MinThicknessTolerance)
		{
			return true;
		}
		// Now check if it's a thin part that's just not aligned to a major axis
		
		// Use the extreme points on the AABB to judge whether it's a flat part
		// Note: TExtremePoints3 is likely ok for detecting very-thin parts, but could be off by ~2x easily. To more precisely measure thickness,
		// consider switching to CompGeom/DiTOrientedBox.h (could be important for a relatively large size tolerance)
		TExtremePoints3<double> ExtremePoints(Part.InternalGeo.MaxVertexID(), [&Part](int32 VID) {return Part.InternalGeo.GetVertex(VID);}, [&Part](int32 VID) {return Part.InternalGeo.IsVertex(VID);});
		if (UNLIKELY(ExtremePoints.Dimension < 3)) // Points were all coplanar (Shouldn't happen here because we wouldn't have a valid hull at all in this case)
		{
			return true;
		}
		// By convention, the last basis vector is in the direction most likely to be 'thin'
		FVector3d SmallDir = ExtremePoints.Basis[2];
		double TetHeight = FMathd::Max(
			FMathd::Abs(SmallDir.Dot(Part.InternalGeo.GetVertex(ExtremePoints.Extreme[0]) - Part.InternalGeo.GetVertex(ExtremePoints.Extreme[3]))),
			FMathd::Abs(SmallDir.Dot(Part.InternalGeo.GetVertex(ExtremePoints.Extreme[0]) - Part.InternalGeo.GetVertex(ExtremePoints.Extreme[2]))));
		if (TetHeight > MinThicknessTolerance)
		{
			return false;
		}
		if (TetHeight * 2 < MinThicknessTolerance)
		{
			return true;
		}
		FInterval1d SmallAxisInterval;
		for (FVector3d V : Part.InternalGeo.VerticesItr())
		{
			SmallAxisInterval.Contain(V.Dot(SmallDir));
		}
		return SmallAxisInterval.Extent() < MinThicknessTolerance;
	};

	int32 MustMergeCount = 0;
	bool bHaveCompactMustMergeParts = false;
	if (MinThicknessTolerance > 0)
	{
		for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); PartIdx++)
		{
			FConvexPart& Part = Decomposition[PartIdx];
			// Note if our thin parts are already compact, so we can skip processing that requires the underlying geometry
			if (Part.IsCompact())
			{
				bHaveCompactMustMergeParts = true;
			}
			Part.bMustMerge = IsPartBelowSizeTolerance(Part);
			MustMergeCount += Part.bMustMerge;
		}
	}
	// TODO: if MustMergeCount is >= NumParts-1, we could shortcut the process and just return a single convex hull

	// Try to further-split long/wide thin parts -- parts that are not thin in all directions, e.g. parts with max bounding box extent wider than MinThicknessTolerance
	if (MustMergeCount > 0 && !bHaveCompactMustMergeParts)
	{
		TArray<int32> PartsToConsider;
		for (int32 OrigPartIdx = 0, OrigNumDecomposition = Decomposition.Num(); OrigPartIdx < OrigNumDecomposition; OrigPartIdx++)
		{
			FConvexPart& OrigPart = Decomposition[OrigPartIdx];
			if (OrigPart.bMustMerge && OrigPart.Bounds.MaxDim() > MinThicknessTolerance)
			{
				TArray<FPlane3d> CutPlanes;
				// Collect neighboring planes that could split this part
				for (auto It = DecompositionToProximity.CreateConstKeyIterator(OrigPartIdx); It; ++It)
				{
					int32 OtherPartIdx = Proximities[It.Value()].Link.OtherElement(OrigPartIdx);
					for (auto OtherIt = DecompositionToProximity.CreateConstKeyIterator(OtherPartIdx); OtherIt; ++OtherIt)
					{
						const FProximity& Prox = Proximities[OtherIt.Value()];
						if (Prox.bPlaneSeparates && !Prox.Link.Contains(OrigPartIdx))
						{
							CutPlanes.Add(Prox.Plane);
						}
					}
				}

				// Attempt splits
				PartsToConsider.Reset();
				PartsToConsider.Add(OrigPartIdx);
				for (const FPlane3d& Plane : CutPlanes)
				{
					for (int32 Idx = 0, Num = PartsToConsider.Num(); Idx < Num; Idx++)
					{
						int32 PartIdx = PartsToConsider[Idx];
						FConvexPart& PartToSplit = Decomposition[PartIdx];
						if (PartToSplit.Bounds.MaxDim() < MinThicknessTolerance)
						{
							continue;
						}
						FPartialCutResult PlaneResult(PartToSplit, Plane, OnPlaneTolerance, bTreatAsSolid, ThickenAfterHullFailureLocalSpace);
						if (!PlaneResult.bSuccess)
						{
							continue;
						}

						double OrigHullVolume = PartToSplit.HullVolume;
						int32 NewPartsStartIdx = Decomposition.Num();
						int32 OtherSideStartIdx = -1;
						PlaneResult.ApplyToGeo(Decomposition, PartIdx, Plane, OtherSideStartIdx, OnPlaneTolerance, ConnectedComponentTolerance, NegativeSpace, bTreatAsSolid, bSplitDisconnectedComponents, ThickenAfterHullFailureLocalSpace);
						UpdateProximitiesAfterSplit(PartIdx, NewPartsStartIdx, Plane, OtherSideStartIdx, OrigHullVolume);
						PartToSplit.bMustMerge = true;
						for (int32 NewIdx = NewPartsStartIdx; NewIdx < Decomposition.Num(); NewIdx++)
						{
							PartsToConsider.Add(NewIdx);
							Decomposition[NewIdx].bMustMerge = true;
						}
					}
				}
			}
		}

		// recompute MustMergeCount to account for possible splits
		MustMergeCount = 0;
		for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); PartIdx++)
		{
			FConvexPart& Part = Decomposition[PartIdx];
			MustMergeCount += Part.bMustMerge;
		}
	}

	if (bAllowCompact)
	{
		Compact();
	}

	// initialize all parts with no merged hulls volume as having just their own hull volume (assuming no merges were yet performed on these parts)
	for (FConvexPart& Part : Decomposition)
	{
		if (Part.SumHullsVolume == -FMathd::MaxReal)
		{
			Part.SumHullsVolume = Part.HullVolume;
		}
	}

	// If we have a MergeCallback, maintain a mapping from current decomposition part indices back to original part indices
	TArray<int32> ToOriginalPartIdx;
	if (Settings.MergeCallback)
	{
		ToOriginalPartIdx.SetNumUninitialized(Decomposition.Num());
		for (int32 PartIdx = 0; PartIdx < ToOriginalPartIdx.Num(); ++PartIdx)
		{
			ToOriginalPartIdx[PartIdx] = PartIdx;
		}
	}


	auto CreateMergedPart = [this, &IsPartBelowSizeTolerance](FProximity& Prox) -> TUniquePtr<FConvexPart>
	{
		TUniquePtr<FConvexPart> NewPart = MakeUnique<FConvexPart>();
		NewPart->GeoCenter = FVector3d::ZeroVector;
		NewPart->SumHullsVolume = 0;
		for (int32 SubIdx = 0; SubIdx < 2; SubIdx++)
		{
			// TODO: If we support a non-volume-based error, will need to also append triangles (and do mapping across), and cannot compact the result
			const FConvexPart& OldPart = Decomposition[Prox.Link[SubIdx]];
			for (FVector3d V : OldPart.InternalGeo.VerticesItr())
			{
				NewPart->InternalGeo.AppendVertex(V);
			}
			NewPart->GeoCenter += OldPart.GeoCenter;
			NewPart->GeoVolume += OldPart.GeoVolume;
			NewPart->SumHullsVolume += OldPart.SumHullsVolume;
		}
		NewPart->GeoCenter *= .5;
		NewPart->ComputeHull();
		FConvexPartHullMeshAdapter HullAdapter(NewPart.Get());
		NewPart->HullVolume = GetVolumeUsingReferencePoint<FConvexPartHullMeshAdapter>(HullAdapter, NewPart->GeoCenter);
		NewPart->Bounds = NewPart->InternalGeo.GetBounds();
		NewPart->HullError = NewPart->HullVolume - NewPart->GeoVolume;
		NewPart->bMustMerge = // only re-evaluate part thickness if both source parts were below tolerance
			Decomposition[Prox.Link[0]].bMustMerge &&
			Decomposition[Prox.Link[1]].bMustMerge &&
			IsPartBelowSizeTolerance(*NewPart);
		NewPart->Compact();

		return NewPart;
	};

	for (; MustMergeCount > 0 || VolumeTolerance > 0 || Decomposition.Num() > TargetNumParts; MergeNum++)
	{
		double BestKnownCost = FMathd::MaxReal;
		bool bOnlyAllowMustMerges = (VolumeTolerance == 0 || !Settings.bErrorToleranceOverridesNumParts) && Decomposition.Num() <= TargetNumParts;
		int32 BestKnownIdx = -1;
		int32 ConsideredCount = 0;
		for (int32 ProxIdx = 0; ProxIdx < Proximities.Num(); ProxIdx++)
		{
			if (!Proximities[ProxIdx].bIsValidLink)
			{
				continue;
			}

			// Once we've considered a certain number of links and found a merge candidate, only consider links that are in the neighborhood of the current best link
			// This prevents excessive computation to find the absolute best merge, when we only really need a local best merge
			ConsideredCount++;
			if (RestrictMergeSearchToLocalAfterTestNumConnections > -1 && ConsideredCount > RestrictMergeSearchToLocalAfterTestNumConnections && BestKnownIdx != -1)
			{
				FIndex2i BestLink = Proximities[BestKnownIdx].Link;
				FIndex2i CurLink = Proximities[ProxIdx].Link;
				if (!BestLink.Contains(CurLink.A) && !BestLink.Contains(CurLink.B))
				{
					continue;
				}
			}

			bool bIncludesMustMergePart =
				Decomposition[Proximities[ProxIdx].Link.A].bMustMerge ||
				Decomposition[Proximities[ProxIdx].Link.B].bMustMerge;
			bool bOnlyMustMergeParts =
				Decomposition[Proximities[ProxIdx].Link.A].bMustMerge &&
				Decomposition[Proximities[ProxIdx].Link.B].bMustMerge;

			// Run the custom allow-merge function (if we have one)
			if (Settings.CustomAllowMergeParts)
			{
				if (!Settings.CustomAllowMergeParts(Decomposition[Proximities[ProxIdx].Link.A], Decomposition[Proximities[ProxIdx].Link.B]))
				{
					continue;
				}
			}

			double MergeCost;
			// attempt to remove parts from the proximity store if we're at the size limit
			bool bShouldRemoveParts = ProximityComputedParts.Num() > EvictComputedPartsThreshold;
			if (Proximities[ProxIdx].HasMergedVolume())
			{
				MergeCost = Proximities[ProxIdx].GetMergeCost(Decomposition);
			}
			else
			{
				// compute the actual cost by computing what the merge part would be
				TUniquePtr<FConvexPart>& NewPart = ProximityComputedParts.Add(ProxIdx, CreateMergedPart(Proximities[ProxIdx]));
				Proximities[ProxIdx].MergedVolume = NewPart->HullVolume;
				MergeCost = Proximities[ProxIdx].GetMergeCost(Decomposition);
			}

			bool bAboveMax = bHasValidMaxHulls && Decomposition.Num() > MaxOutputHulls;

			if (!bIncludesMustMergePart && ( // if there's no must-merge part, skip if ...
					(!bAboveMax && VolumeTolerance > 0 && MergeCost > VolumeTolerance) // it won't pass the volume tolerance, and we already have <= max hulls
						|| 
					bOnlyAllowMustMerges // or we're currently only considering must-merge parts (e.g., already at or below desired number of parts)
				))
			{
				// remove pre-computed part if they won't pass volume tolerance or we're at the part limit
				if (bShouldRemoveParts || (!bAboveMax && VolumeTolerance > 0 && MergeCost > VolumeTolerance))
				{
					ProximityComputedParts.Remove(ProxIdx);
				}
				continue;
			}

			// Apply a bias term to favor getting rid of 'must merge' parts first
			if (bIncludesMustMergePart && !bOnlyMustMergeParts)
			{
				MergeCost -= BiasToRemoveTooThinParts;
			}

			// if this would be the new best candidate, and we have negative space, check if the merge would cross protected negative space
			bool bHasNegativeSpace = NegativeSpace.Num() || (OptionalNegativeSpace && OptionalNegativeSpace->Num() > 0);
			if (bHasNegativeSpace && MergeCost < BestKnownCost)
			{
				TUniquePtr<FConvexPart>* FoundMergeHull = ProximityComputedParts.Find(ProxIdx);
				if (!FoundMergeHull)
				{
					FoundMergeHull = &ProximityComputedParts.Add(ProxIdx, CreateMergedPart(Proximities[ProxIdx]));
				}
				bool bCoversProtectedPt = false;
				auto TestVsNegativeSpace = [](const FConvexPart& Part, const FSphereCovering& SphereCovering, const FTransform* OptionalTransform = nullptr) -> bool
				{
					std::atomic_bool bOverlaps = false;
					ParallelFor(SphereCovering.Num(), [&](int32 SphereIdx)
						{
							if (bOverlaps.load(std::memory_order_relaxed))
							{
								return;
							}
							double Radius = SphereCovering.GetRadius(SphereIdx);
							FVector3d Center = SphereCovering.GetCenter(SphereIdx);
							if (ConvexPartVsSphereOverlap(Part, Center, Radius, OptionalTransform))
							{
								bOverlaps = true;
							}
						});
					return (bool)bOverlaps;
				};
				if (OptionalNegativeSpace)
				{
					if (TestVsNegativeSpace(**FoundMergeHull, *OptionalNegativeSpace, OptionalTransformIntoNegativeSpace))
					{
						bCoversProtectedPt = true;
					}
				}
				if (TestVsNegativeSpace(**FoundMergeHull, NegativeSpace))
				{
					bCoversProtectedPt = true;
				}
				if (bCoversProtectedPt)
				{
					// Mark link as invalid so we don't repeat this overlap test in later iterations
					Proximities[ProxIdx].bIsValidLink = false;
					// Never keep precomputed proximity for invalid links
					ProximityComputedParts.Remove(ProxIdx);
					continue;
				}
			}

			if (MergeCost < BestKnownCost)
			{
				BestKnownCost = MergeCost;
				BestKnownIdx = ProxIdx;
			}
			else if (bShouldRemoveParts) // if we are at the precomputed proximity part limit and it's not the best part, don't keep it
			{
				ProximityComputedParts.Remove(ProxIdx);
			}
		}


		// Perform the best merge and update proximity data

		if (BestKnownIdx == -1)
		{
			break;
		}

		// Get the proximity that we will merge -- note this proximity will be removed from the Proximities array
		FProximity Prox = Proximities[BestKnownIdx];
		int32 DecoToKeep = Prox.Link.A, DecoToRm = Prox.Link.B;
		if (DecoToKeep > DecoToRm)
		{
			Swap(DecoToKeep, DecoToRm); // keep the smaller index (this makes the book-keeping update below slightly easier)
		}

		MustMergeCount -= int32(Decomposition[DecoToKeep].bMustMerge) + int32(Decomposition[DecoToRm].bMustMerge);

		// Update the "to keep" index with the new, merged FConvexPart
		if (ProximityComputedParts.Contains(BestKnownIdx))
		{
			Decomposition[DecoToKeep] = MoveTemp(*ProximityComputedParts[BestKnownIdx]);
			ProximityComputedParts.Remove(BestKnownIdx);
		}
		else
		{
			TUniquePtr<FConvexPart> NewPart = CreateMergedPart(Prox);
			Decomposition[DecoToKeep] = MoveTemp(*NewPart);
		}

		MustMergeCount += int32(Decomposition[DecoToKeep].bMustMerge);

		// Do all the book-keeping to delete the other old FConvexPart and update all Proximity data

		// Clear all pre-computed merge info related to the newly-merged parts, saving out only what parts the merge should be linked to
		TSet<int32> NewLinks;
		for (int32 ProxIdx = 0; ProxIdx < Proximities.Num(); ProxIdx++)
		{
			FIndex2i Link = Proximities[ProxIdx].Link;
			if (Link.Contains(DecoToRm) || Link.Contains(DecoToKeep))
			{
				// Clear everything associated with the old link
				Proximities[ProxIdx].ClearMergedVolume();
				ProximityComputedParts.Remove(ProxIdx);
				NewLinks.Add(Link.A);
				NewLinks.Add(Link.B);
				DecompositionToProximity.RemoveSingle(Link.A, ProxIdx);
				DecompositionToProximity.RemoveSingle(Link.B, ProxIdx);

				// Swap-Remove the proximity links
				int32 LastProxIdx = Proximities.Num() - 1;
				Proximities.RemoveAtSwap(ProxIdx, EAllowShrinking::No);

				// Update the proximity that was swapped back to this position (if any)
				if (ProxIdx < LastProxIdx)
				{
					FIndex2i SwappedLink = Proximities[ProxIdx].Link;
					DecompositionToProximity.RemoveSingle(SwappedLink.A, LastProxIdx);
					DecompositionToProximity.RemoveSingle(SwappedLink.B, LastProxIdx);
					DecompositionToProximity.Add(SwappedLink.A, ProxIdx);
					DecompositionToProximity.Add(SwappedLink.B, ProxIdx);
					TUniquePtr<FConvexPart>* FoundPart = ProximityComputedParts.Find(LastProxIdx);
					if (FoundPart)
					{
						FConvexPart* Part = FoundPart->Release();
						ProximityComputedParts.Remove(LastProxIdx);
						ProximityComputedParts.Add(ProxIdx, TUniquePtr<FConvexPart>(Part)); // re-create with the new key
					}
					
					// Next iteration, we'll reconsider the just-updated element
					ProxIdx--;
				}
			}
		}
		NewLinks.Remove(DecoToKeep);
		NewLinks.Remove(DecoToRm);

		if (Settings.MergeCallback)
		{
			int32 OrigKeep = ToOriginalPartIdx[DecoToKeep];
			int32 OrigRm = ToOriginalPartIdx[DecoToRm];
			Settings.MergeCallback(OrigKeep, OrigRm);
		}

		// Remove the unused convex, updating references to the part that was swapped back
		int32 LastIdx = Decomposition.Num() - 1;
		if (DecoToRm != LastIdx)
		{
			if (Settings.MergeCallback)
			{
				ToOriginalPartIdx[DecoToRm] = ToOriginalPartIdx[LastIdx];
			}
			// Before swap-removing DecoToRm, update references that used to point to the last element of the array to now point to DecoToRm
			[this](int32 OrigIdx, int32 NewIdx)
			{
				TArray<int32> ProxIndices;
				DecompositionToProximity.MultiFind(OrigIdx, ProxIndices, false);
				DecompositionToProximity.Remove(OrigIdx);
				for (int32 ProxIdx : ProxIndices)
				{
					// any link to the new location should be removed before calling this Update function
					checkSlow(!Proximities[ProxIdx].Link.Contains(NewIdx));

					int32 SubIdx = Proximities[ProxIdx].Link.IndexOf(OrigIdx);
					Proximities[ProxIdx].Link[SubIdx] = NewIdx;
					DecompositionToProximity.Add(NewIdx, ProxIdx);
				}
			}(LastIdx, DecoToRm);
		}
		Decomposition.RemoveAtSwap(DecoToRm, EAllowShrinking::No);

		// Add new proximities for all new links to the merged part
		for (int32 ToLink : NewLinks)
		{
			if (ToLink == LastIdx)
			{
				ToLink = DecoToRm;
			}
			int32 ProxIdx = Proximities.Emplace(FIndex2i(DecoToKeep, ToLink), FPlane3d(), false /*bPlaneSeparates*/);
			DecompositionToProximity.Add(DecoToKeep, ProxIdx);
			DecompositionToProximity.Add(ToLink, ProxIdx);
		}
	}
	
	// Compute the triangulation for any hulls that are vertex-only (only possible if InitializeFromHulls was called before MergeBest)
	if (bRequireHullTriangles)
	{
		for (FConvexPart& ConvexPart : Decomposition)
		{
			if (ConvexPart.HullTriangles.IsEmpty())
			{
				ConvexPart.ComputeHull();
			}
		}
	}

	return MergeNum;
}

void FConvexDecomposition3::DeleteProximity(TArray<int32>&& ToRemoveProx, bool bDeleteMapReferences)
{
	for (int32 ToRemoveIdx = 0; ToRemoveIdx < ToRemoveProx.Num(); ++ToRemoveIdx)
	{
		int32 ProxIdx = ToRemoveProx[ToRemoveIdx];
		if (bDeleteMapReferences)
		{
			const FProximity& OldProx = Proximities[ProxIdx];
			DecompositionToProximity.RemoveSingle(OldProx.Link.A, ProxIdx);
			DecompositionToProximity.RemoveSingle(OldProx.Link.B, ProxIdx);
		}
		checkSlow(DecompositionToProximity.FindKey(ProxIdx) == nullptr); // verify there are no more references to this Proximity's index

		if (ProxIdx == Proximities.Num() - 1)
		{
			Proximities.RemoveAt(ProxIdx, EAllowShrinking::No);
			return;
		}
		else
		{
			// Remove by swapping the last element to the new slot, and removing the last element
			// Then update links for that swapped-in proximity
			int32 LastProxIdx = Proximities.Num() - 1;
			Proximities.RemoveAtSwap(ProxIdx, EAllowShrinking::No);
			const FProximity& NewProx = Proximities[ProxIdx];
			DecompositionToProximity.RemoveSingle(NewProx.Link.A, LastProxIdx);
			DecompositionToProximity.RemoveSingle(NewProx.Link.B, LastProxIdx);
			DecompositionToProximity.Add(NewProx.Link.A, ProxIdx);
			DecompositionToProximity.Add(NewProx.Link.B, ProxIdx);
			// check if the swapped-back index is one we planned to remove, and if so update the index
			for (int32 WillRemoveIdx = ToRemoveIdx + 1; WillRemoveIdx < ToRemoveProx.Num(); ++WillRemoveIdx)
			{
				if (LastProxIdx == ToRemoveProx[WillRemoveIdx])
				{
					ToRemoveProx[WillRemoveIdx] = ProxIdx;
					break;
				}
			}
		}
	}
}

void FConvexDecomposition3::InitNegativeSpaceConvexPartMapping(int32 StartSphereIndex)
{
	// Helper to check each sphere for overlap vs each convex part
	auto IteratePartsVsSpheres = [this, StartSphereIndex](TFunctionRef<void(FConvexPart& Part, int32 SphereIdx, bool bOverlapsPart)> ProcessPartVsSphere)
	{
		for (int32 PartIdx = 0; PartIdx < Decomposition.Num(); ++PartIdx)
		{
			FConvexPart& Part = Decomposition[PartIdx];
			if (StartSphereIndex == 0 && !ensure(Part.OverlapsNegativeSpace.IsEmpty())) // If calling on all spheres, expect this Init to only be called when there is no existing mapping
			{
				Part.OverlapsNegativeSpace.Reset();
			}
			else if (StartSphereIndex > 0 && !Part.OverlapsNegativeSpace.IsEmpty()) // If calling on subset of spheres, expect this Init to be called only a larger StartSphereIndex than previously mapped spheres
			{
				// Spot-check just the last sphere to catch out incorrect usage, since spheres are generally added in increasing order
				ensure(Part.OverlapsNegativeSpace.Last() < StartSphereIndex);
			}
			for (int32 SphereIdx = StartSphereIndex; SphereIdx < NegativeSpace.Num(); ++SphereIdx)
			{
				// Note: We assume the part is in the same coordinate space as the negative space spheres
				bool bOverlapsPart = ConvexPartVsSphereOverlap(Part, NegativeSpace.GetCenter(SphereIdx), NegativeSpace.GetRadius(SphereIdx));
				ProcessPartVsSphere(Part, SphereIdx, bOverlapsPart);
			}
		}
	};

	if (Decomposition.Num() == 1)
	{
		// Special handling of the single-part case, since any spheres that don't overlap that must be useless (future splits/merges will never be affected by them)
		IteratePartsVsSpheres([this](FConvexPart& Part, int32 SphereIdx, bool bOverlapsPart)
		{
			if (!bOverlapsPart)
			{
				// Set negative radius to indicate sphere is not useful
				NegativeSpace.SetRadius(SphereIdx, -FMathd::MaxReal);
			}
		});
		// Clear negative-radius / invalid spheres from the NegativeSpace
		NegativeSpace.RemoveSmaller(0);
		// Add all remaining spheres
		Decomposition[0].OverlapsNegativeSpace.SetNumUninitialized(NegativeSpace.Num());
		for (int32 Idx = 0; Idx < NegativeSpace.Num(); ++Idx)
		{
			Decomposition[0].OverlapsNegativeSpace[Idx] = Idx;
		}
	}
	else
	{
		// If there are multiple parts, just assign them to parts based on overlap
		IteratePartsVsSpheres([](FConvexPart& Part, int32 SphereIdx, bool bOverlapsPart)
		{
			if (bOverlapsPart)
			{
				Part.OverlapsNegativeSpace.Add(SphereIdx);
			}
		});
	}
}

} // end namespace UE::Geometry
} // end namespace UE