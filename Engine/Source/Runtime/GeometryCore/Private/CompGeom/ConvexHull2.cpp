// Copyright Epic Games, Inc. All Rights Reserved.

// Adaptation/Port of GTEngine's ConvexHull2 algorithm;
// ref: Engine\Plugins\Experimental\GeometryProcessing\Source\GeometryAlgorithms\Private\ThirdParty\GTEngine\Mathematics\GteConvexHull2.h

#include "CompGeom/ConvexHull2.h"

#include "CompGeom/ExactPredicates.h"
#include "Algo/Unique.h"
#include "Algo/Reverse.h"

namespace UE
{
namespace Geometry
{

namespace
{
	inline int32 IncModM(int32 Idx, int32 Mod)
	{
		int32 Inc = Idx + 1;
		return Inc < Mod ? Inc : 0;
	}
}

template<typename RealType>
bool TConvexHull2<RealType>::SolveSimplePolygon(int32 N, TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, bool bIsKnownCCW)
{
	Dimension = 0;
	NumUniquePoints = 0;

	if (N == 0)
	{
		Hull.Empty();
		return false;
	}
	Hull.Reset(N);


	// Find the (lowest) leftmost point (guaranteed to be on the hull)
	int32 LeftmostPtIdx = 0;
	TVector2<RealType> Leftmost = GetPointFunc(0);
	// If orientation is not already known to be CCW, we use area sign to decide orientation
	RealType OrientSign = 1;

	{
		auto AddEdgeArea2 = [&GetPointFunc](TVector2<RealType> V1, TVector2<RealType> V2)
		{
			return V1.X * V2.Y - V1.Y * V2.X;
		};
		RealType AreaSum2 = bIsKnownCCW ? 0 : AddEdgeArea2(GetPointFunc(N - 1), Leftmost);
		TVector2<RealType> LastPt = Leftmost; // LastPt only for computing AreaSum to check winding direction
		for (int32 Idx = 1; Idx < N; ++Idx)
		{
			TVector2<RealType> Pt = GetPointFunc(Idx);
			if (Pt.X < Leftmost.X || (Pt.X == Leftmost.X && Pt.Y < Leftmost.Y))
			{
				Leftmost = Pt;
				LeftmostPtIdx = Idx;
			}
			if (!bIsKnownCCW)
			{
				AreaSum2 += AddEdgeArea2(LastPt, Pt);
				LastPt = Pt;
			}
		}
		if (!bIsKnownCCW && AreaSum2 < 0)
		{
			OrientSign = -1;
		}
	}

	Hull.Add(LeftmostPtIdx);
	TVector2<RealType> PrevPt = Leftmost;
	int32 CurIdx = IncModM(LeftmostPtIdx, N);
	int32 EndIdx = LeftmostPtIdx;
	TVector2<RealType> CurPt = GetPointFunc(CurIdx);
	for (int32 NextIdx = IncModM(CurIdx, N); CurIdx != EndIdx; NextIdx = IncModM(NextIdx, N))
	{
		TVector2<RealType> NextPt = GetPointFunc(NextIdx);
		if (ExactPredicates::Orient2<RealType>(PrevPt, CurPt, NextPt)*OrientSign <= 0)
		{
			// Go backwards until we're out of points or we find a locally-convex point
			while (Hull.Num() > 1)
			{
				CurPt = PrevPt;
				CurIdx = Hull.Pop(EAllowShrinking::No);
				int32 PrevIdx = Hull.Last();
				PrevPt = GetPointFunc(PrevIdx);
				
				if (ExactPredicates::Orient2<RealType>(PrevPt, CurPt, NextPt)*OrientSign > 0)
				{
					Hull.Push(CurIdx);
					PrevPt = CurPt;
					break;
				}
			}
		}
		else // CurPt is locally convex with Prev,Next; go ahead and add it
		{
			Hull.Push(CurIdx);
			PrevPt = CurPt;
		}
		CurIdx = NextIdx;
		CurPt = NextPt;
	}
	if (OrientSign < 0)
	{
		Algo::Reverse(Hull);
	}
	bool bFoundValid = Hull.Num() >= 3;
	Dimension = bFoundValid ? 2 : 0;
	return bFoundValid;
}


template<class RealType>
bool TConvexHull2<RealType>::Solve(int32 NumPoints, TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc)
{
	Hull.Reset();

	Dimension = 0;
	NumUniquePoints = 0;

	// Sort the points.
	Hull.Reserve(NumPoints); // Reserve up to NumPoints assuming filter doesn't remove that many points in practice
	for (int32 Idx = 0; Idx < NumPoints; Idx++)
	{
		if (FilterFunc(Idx))
		{
			Hull.Add(Idx);
		}
	}
	Hull.Sort([&GetPointFunc](int32 A, int32 B)
		{
			TVector2<RealType> PA = GetPointFunc(A), PB = GetPointFunc(B);
			return (PA.X < PB.X) || (PA.X == PB.X && PA.Y < PB.Y);
		});

	Hull.SetNum(Algo::Unique(Hull, [&GetPointFunc](int32 A, int32 B)
		{
			return GetPointFunc(A) == GetPointFunc(B);
		}));
	NumUniquePoints = Hull.Num();

	if (Hull.Num() < 3)
	{
		Dimension = FMath::Max(0, Hull.Num() - 1);
		return false;
	}
	else
	{
		TVector2<RealType> FirstTwoPts[2]{ GetPointFunc(Hull[0]), GetPointFunc(Hull[1]) };
		bool bFoundSecondDim = false;
		for (int32 Idx = 2; Idx < Hull.Num(); Idx++)
		{
			TVector2<RealType> Pt = GetPointFunc(Hull[Idx]);
			if (ExactPredicates::Orient2<RealType>(FirstTwoPts[0], FirstTwoPts[1], Pt) != 0)
			{
				bFoundSecondDim = true;
				break;
			}
		}
		if (!bFoundSecondDim)
		{
			Dimension = 1;
			return false;
		}
	}

	// points are not all collinear; proceed with finding hull
	Dimension = 2;

	// Use a divide-and-conquer algorithm.  The merge step computes the
	// convex hull of two convex polygons.
	TArray<int32> Merged;
	Merged.SetNum(NumUniquePoints);
	// Note these indices will be changed by the recursive GetHull call
	int32 IdxFirst = 0, IdxLast = NumUniquePoints - 1;
	GetHull(GetPointFunc, Merged, IdxFirst, IdxLast);
	Hull.SetNum(IdxLast - IdxFirst + 1);

	return true;
}

template<class RealType>
void TConvexHull2<RealType>::GetHull(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32& IdxFirst, int32& IdxLast)
{
	int32 NumVertices = IdxLast - IdxFirst + 1;
	if (NumVertices > 1)
	{
		// Compute the middle index of input range.
		int32 Mid = (IdxFirst + IdxLast) / 2;

		// Compute the hull of subsets (mid-i0+1 >= i1-mid).
		int32 j0 = IdxFirst, j1 = Mid, j2 = Mid + 1, j3 = IdxLast;
		GetHull(GetPointFunc, Merged, j0, j1);
		GetHull(GetPointFunc, Merged, j2, j3);

		// Merge the convex hulls into a single convex hull.
		Merge(GetPointFunc, Merged, j0, j1, j2, j3, IdxFirst, IdxLast);
	}
	// else: The convex hull is a single point.
}

template<class RealType>
void TConvexHull2<RealType>::Merge(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32 j0, int32 j1, int32 j2, int32 j3, int32& i0, int32& i1)
{
	// Subhull0 is to the left of subhull1 because of the initial sorting of
	// the points by x-components.  We need to find two mutually visible
	// points, one on the left subhull and one on the right subhull.
	int32 size0 = j1 - j0 + 1;
	int32 size1 = j3 - j2 + 1;

	int32 i;
	TVector2<RealType> p;

	// Find the right-most point of the left subhull.
	TVector2<RealType> pmax0 = GetPointFunc(Hull[j0]);
	int32 imax0 = j0;
	for (i = j0 + 1; i <= j1; ++i)
	{
		p = GetPointFunc(Hull[i]);
		if (pmax0.X < p.X || (pmax0.X == p.X && pmax0.Y < p.Y)) // lexicographic pmax0 < p
		{
			pmax0 = p;
			imax0 = i;
		}
	}

	// Find the left-most point of the right subhull.
	TVector2<RealType> pmin1 = GetPointFunc(Hull[j2]);
	int32 imin1 = j2;
	for (i = j2 + 1; i <= j3; ++i)
	{
		p = GetPointFunc(Hull[i]);
		if (p.X < pmin1.X || (p.X == pmin1.X && p.Y < pmin1.Y)) // lexicographic p < pmin1
		{
			pmin1 = p;
			imin1 = i;
		}
	}

	// Get the lower tangent to hulls (LL = lower-left, LR = lower-right).
	int32 iLL = imax0, iLR = imin1;
	GetTangent(GetPointFunc, Merged, j0, j1, j2, j3, iLL, iLR);

	// Get the upper tangent to hulls (UL = upper-left, UR = upper-right).
	int32 iUL = imax0, iUR = imin1;
	GetTangent(GetPointFunc, Merged, j2, j3, j0, j1, iUR, iUL);

	// Construct the counterclockwise-ordered merged-hull vertices.
	int32 k;
	int32 numMerged = 0;

	i = iUL;
	for (k = 0; k < size0; ++k)
	{
		Merged[numMerged++] = Hull[i];
		if (i == iLL)
		{
			break;
		}
		i = (i < j1 ? i + 1 : j0);
	}
	checkfSlow(k < size0, TEXT("Unexpected condition."));

	i = iLR;
	for (k = 0; k < size1; ++k)
	{
		Merged[numMerged++] = Hull[i];
		if (i == iUR)
		{
			break;
		}
		i = (i < j3 ? i + 1 : j2);
	}
	checkfSlow(k < size1, TEXT("Unexpected condition."));

	int32 next = j0;
	for (k = 0; k < numMerged; ++k)
	{
		Hull[next] = Merged[k];
		++next;
	}

	i0 = j0;
	i1 = next - 1;
}

// All possible point orderings for GetTangent (note that duplicate points have been filtered already, so those cases are not included)
enum class EPointOrdering
{
	Positive,
	Negative,
	CollinearLeft,
	CollinearRight,
	CollinearContain
};

template<typename RealType>
EPointOrdering PointOnLine(const TVector2<RealType>& P, const TVector2<RealType>& L0, const TVector2<RealType>& L1)
{
	RealType OnLine = ExactPredicates::Orient2(P, L0, L1);
	if (OnLine > 0)
	{
		return EPointOrdering::Positive;
	}
	else if (OnLine < 0)
	{
		return EPointOrdering::Negative;
	}

	// Since points are exactly collinear, and not duplicate, should be ok to just compare on one axis
	int UseDim = 0;
	// if the points have same value on first axis, use the second
	if (P[0] == L0[0])
	{
		UseDim = 1;
	}

	bool bPv0 = P[UseDim] > L0[UseDim];
	bool bPv1 = P[UseDim] > L1[UseDim];
	if (bPv0 != bPv1)
	{
		return EPointOrdering::CollinearContain;
	}
	bool b1v0 = L1[UseDim] > L0[UseDim];
	if (b1v0 == bPv0)
	{
		return EPointOrdering::CollinearRight;
	}
	else
	{
		return EPointOrdering::CollinearLeft;
	}
}

template<class RealType>
void TConvexHull2<RealType>::GetTangent(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32 j0, int32 j1, int32 j2, int32 j3, int32& i0, int32& i1)
{
	// In theory the loop terminates in a finite number of steps, but the
	// upper bound for the loop variable is used to trap problems caused by
	// floating point roundoff errors that might lead to an infinite loop.

	int32 size0 = j1 - j0 + 1;
	int32 size1 = j3 - j2 + 1;
	int32 const imax = size0 + size1;
	int32 i, iLm1, iRp1;
	TVector2<RealType> L0, L1, R0, R1;

	for (i = 0; i < imax; i++)
	{
		// Get the endpoints of the potential tangent.
		L1 = GetPointFunc(Hull[i0]);
		R0 = GetPointFunc(Hull[i1]);

		// Walk along the left hull to find the point of tangency.
		if (size0 > 1)
		{
			iLm1 = (i0 > j0 ? i0 - 1 : j1);
			L0 = GetPointFunc(Hull[iLm1]);
			EPointOrdering Order = PointOnLine<RealType>(R0, L0, L1);
			if (Order == EPointOrdering::Negative
				|| Order == EPointOrdering::CollinearRight)
			{
				i0 = iLm1;
				continue;
			}
		}

		// Walk along right hull to find the point of tangency.
		if (size1 > 1)
		{
			iRp1 = (i1 < j3 ? i1 + 1 : j2);
			R1 = GetPointFunc(Hull[iRp1]);
			EPointOrdering Order = PointOnLine<RealType>(L1, R0, R1);
			if (Order == EPointOrdering::Negative
				|| Order == EPointOrdering::CollinearLeft)
			{
				i1 = iRp1;
				continue;
			}
		}

		// The tangent segment has been found.
		break;
	}

	// Detect an "infinite loop" caused by floating point round-off errors; should never happen b/c we use exact predicates
	checkfSlow(i < imax, TEXT("Unexpected condition."));

}


template class TConvexHull2<float>;
template class TConvexHull2<double>;

} // end namespace UE::Geometry
} // end namespace UE