// Copyright Epic Games, Inc. All Rights Reserved.

// Adaptation/Port of GTEngine's ConvexHull3 algorithm;
// ref: Engine\Plugins\Runtime\GeometryProcessing\Source\GeometryAlgorithms\Private\ThirdParty\GTEngine\Mathematics\GteConvexHull3.h
// ExtremePoints::Init adapted from GteVector3.h's IntrinsicsVector3
// ref: Engine\Plugins\Runtime\GeometryProcessing\Source\GeometryAlgorithms\Private\ThirdParty\GTEngine\Mathematics\GteVector3.h

#include "CompGeom/ConvexHull3.h"
#include "CompGeom/ExactPredicates.h"
#include "VertexConnectedComponents.h" // for FSizedDisjointSet

#include "Algo/RemoveIf.h"
#include "Async/ParallelFor.h"
#include "Containers/SparseSet.h"

namespace UE
{
namespace Geometry
{


template<typename RealType>
void TExtremePoints3<RealType>::Init(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc, RealType Epsilon)
{
	TVector<RealType> FirstPoint;
	int FirstPtIdx = -1;
	for (int FirstPtIdxTest = 0; FirstPtIdxTest < NumPoints; FirstPtIdxTest++)
	{
		if (FilterFunc(FirstPtIdxTest))
		{
			FirstPoint = GetPointFunc(FirstPtIdxTest);
			FirstPtIdx = FirstPtIdxTest;
			break;
		}
	}
	if (FirstPtIdx == -1)
	{
		// no points passed filter
		Dimension = 0;
		return;
	}

	TVector<RealType> Min = GetPointFunc(FirstPtIdx), Max = GetPointFunc(FirstPtIdx);
	FIndex3i IndexMin(FirstPtIdx, FirstPtIdx, FirstPtIdx), IndexMax(FirstPtIdx, FirstPtIdx, FirstPtIdx);
	for (int Idx = FirstPtIdx + 1; Idx < NumPoints; Idx++)
	{
		if (!FilterFunc(Idx))
		{
			continue;
		}
		for (int Dim = 0; Dim < 3; Dim++)
		{
			RealType Val = GetPointFunc(Idx)[Dim];
			if (Val < Min[Dim])
			{
				Min[Dim] = Val;
				IndexMin[Dim] = Idx;
			}
			else if (Val > Max[Dim])
			{
				Max[Dim] = Val;
				IndexMax[Dim] = Idx;
			}
		}
	}

	RealType MaxRange = Max[0] - Min[0];
	int MaxRangeDim = 0;
	for (int Dim = 1; Dim < 3; Dim++)
	{
		RealType Range = Max[Dim] - Min[Dim];
		if (Range > MaxRange)
		{
			MaxRange = Range;
			MaxRangeDim = Dim;
		}
	}
	Extreme[0] = IndexMin[MaxRangeDim];
	Extreme[1] = IndexMax[MaxRangeDim];

	// all points within box of Epsilon extent; Dimension must be 0
	if (MaxRange <= Epsilon)
	{
		Dimension = 0;
		Extreme[3] = Extreme[2] = Extreme[1] = Extreme[0];
		return;
	}

	Origin = GetPointFunc(Extreme[0]);
	Basis[0] = GetPointFunc(Extreme[1]) - Origin;
	Normalize(Basis[0]);

	// find point furthest from the line formed by the first two extreme points
	{
		TLine3<RealType> Basis0Line(Origin, Basis[0]);
		RealType MaxDistSq = 0;
		for (int Idx = FirstPtIdx; Idx < NumPoints; Idx++)
		{
			if (!FilterFunc(Idx))
			{
				continue;
			}
			RealType DistSq = Basis0Line.DistanceSquared(GetPointFunc(Idx));
			if (DistSq > MaxDistSq)
			{
				MaxDistSq = DistSq;
				Extreme[2] = Idx;
			}
		}

		// Nearly collinear points
		if (TMathUtil<RealType>::Sqrt(MaxDistSq) <= Epsilon * MaxRange)
		{
			Dimension = 1;
			Extreme[3] = Extreme[2] = Extreme[1];
			return;
		}
	}


	Basis[1] = GetPointFunc(Extreme[2]) - Origin;
	// project Basis[1] to be orthogonal to Basis[0]
	Basis[1] -= (Basis[0].Dot(Basis[1])) * Basis[0];
	if (!Basis[1].Normalize(Epsilon)) // points too collinear to form a valid basis
	{
		Dimension = 1;
		Extreme[3] = Extreme[2] = Extreme[1];
		return;
	}
	Basis[2] = Basis[0].Cross(Basis[1]);
	if (!Basis[2].Normalize(Epsilon)) // points too collinear to form a valid basis
	{
		Dimension = 1;
		Extreme[3] = Extreme[2] = Extreme[1];
		return;
	}


	{
		TPlane3<RealType> Plane(Basis[2], Origin);
		RealType MaxDist = 0, MaxSign = 0;
		for (int Idx = FirstPtIdx; Idx < NumPoints; Idx++)
		{
			if (!FilterFunc(Idx))
			{
				continue;
			}
			RealType DistSigned = (RealType)Plane.DistanceTo(GetPointFunc(Idx));
			RealType Dist = TMathUtil<RealType>::Abs(DistSigned);
			if (Dist > MaxDist)
			{
				MaxDist = Dist;
				MaxSign = TMathUtil<RealType>::Sign(DistSigned);
				Extreme[3] = Idx;
			}
		}

		// Nearly coplanar points
		if (MaxDist <= Epsilon * MaxRange)
		{
			Dimension = 2;
			Extreme[3] = Extreme[2];
			return;
		}

		// make sure the tetrahedron is CW-oriented
		if (MaxSign > 0)
		{
			Swap(Extreme[3], Extreme[2]);
		}
	}

	Dimension = 3;
}


template<typename RealType>
struct FHullConnectivity
{

	struct FVisiblePoints
	{
		TArray<int32> Indices;
		int32 MaxIdx = -1; // Note: an index into the source point array, not into the above Indices array
		// Max value of the Orient3D predicate, indicating the farthest point outside; values less than zero are inside the hull so not considered
		double MaxValue = -FMathd::MaxReal;
		FPlane3d Plane;

		void SetPlane(const TVector<RealType> TriPts[3])
		{
			Plane = FPlane3d(FVector3d(TriPts[0]), FVector3d(TriPts[1]), FVector3d(TriPts[2]));
		}

		double GetPlaneDistance(const TVector<RealType>& Pt)
		{
			return Plane.DistanceTo((FVector3d)Pt);
		}

		void AddPt(int32 Idx, const TVector<RealType>& Pt)
		{
			double Value = GetPlaneDistance(Pt);
			if (Value > MaxValue)
			{
				MaxValue = Value;
				MaxIdx = Idx;
			}
			Indices.Add(Idx);
		}

		void AddPtByValue(int32 Idx, double Value)
		{
			if (Value > MaxValue)
			{
				MaxValue = Value;
				MaxIdx = Idx;
			}
			Indices.Add(Idx);
		}

		// Remove a point from the visible point set; if it was the tracked MaxValue point, find a new MaxValue point
		void RemovePt(int32 SourcePointIdx, TFunctionRef<TVector<RealType>(int32)> GetPointFunc)
		{
			if (MaxIdx == SourcePointIdx)
			{
				MaxValue = -FMathd::MaxReal;
				MaxIdx = -1;
				for (int32 SubIdx = 0; SubIdx < Indices.Num(); ++SubIdx)
				{
					int32 PointIdx = Indices[SubIdx];
					if (PointIdx == SourcePointIdx)
					{
						Indices.RemoveAtSwap(SubIdx, EAllowShrinking::No);
						SubIdx--;
					}
					else
					{
						double Value = GetPlaneDistance(GetPointFunc(PointIdx));
						if (Value > MaxValue)
						{
							MaxValue = Value;
							MaxIdx = PointIdx;
						}
					}
				}
			}
			else
			{
				Indices.RemoveSingleSwap(SourcePointIdx, EAllowShrinking::No);
			}
		}

		int32 Num()
		{
			return Indices.Num();
		}

		int32 MaxPt()
		{
			return MaxIdx;
		}

		void Reset()
		{
			Indices.Reset();
			MaxIdx = -1;
			MaxValue = -FMathd::MaxReal;
		}
	};

	TArray<FIndex3i> TriNeighbors;
	TArray<FVisiblePoints> VisiblePoints;

	// Prefering a sparse set here to retain identical cook output before/after the compact set switch.
	TSparseSet<int32> TrisWithPoints;

	// If positive, this threshold additionally filters which points are considered 'visible' as only points at least this far from the plane
	double VisibleDistanceThreshold = -FMathd::MaxReal;

	/**
	 * Fully build neighbor connectivity; only on the initial tet, 
	 * as after that we can incrementally update connectivity w/ mesh changes
	 */
	void BuildNeighbors(const TArray<FIndex3i>& Triangles)
	{
		TMap<FIndex2i, int32> EdgeToTri;
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			const FIndex3i& Tri = Triangles[TriIdx];
			for (int32 LastIdx = 2, Idx = 0; Idx < 3; LastIdx = Idx++)
			{
				EdgeToTri.Add(FIndex2i(Tri[LastIdx], Tri[Idx]), TriIdx);
			}
		}
		TriNeighbors.SetNum(Triangles.Num());
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			const FIndex3i& Tri = Triangles[TriIdx];
			for (int32 LastIdx = 2, Idx = 0; Idx < 3; LastIdx = Idx++)
			{
				FIndex2i BackEdge(Tri[Idx], Tri[LastIdx]);
				int32* Val = EdgeToTri.Find(BackEdge);
				TriNeighbors[TriIdx][LastIdx] = Val ? *Val : -1;
			}
		}
	}

	/**
	 * @return true if Pt is on the 'positive' side of the triangle
	 */
	bool IsVisible(const TVector<RealType> TriPts[3], const TVector<RealType>& Pt)
	{
		double PredicateValue = ExactPredicates::Orient3<RealType>(TriPts[0], TriPts[1], TriPts[2], Pt);
		return PredicateValue > 0;
	}

	/**
	 * Set triangle positions to prep for IsVisible calls
	 */
	void SetTriPts(const FIndex3i& Tri, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,  TVector<RealType> TriPtsOut[3])
	{
		TriPtsOut[0] = GetPointFunc(Tri[0]);
		TriPtsOut[1] = GetPointFunc(Tri[1]);
		TriPtsOut[2] = GetPointFunc(Tri[2]);
	}

	/**
	 * Bucket all points based on which triangles can see them
	 */
	void InitVisibility(const TArray<FIndex3i>& Triangles, int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc)
	{
		VisiblePoints.SetNum(Triangles.Num());

		// Track which points are already assigned so never assign one point to multiple tris
		TArray<bool> PointAssigned;
		PointAssigned.SetNumZeroed(NumPoints);
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			FIndex3i Tri = Triangles[TriIdx];
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				PointAssigned[Tri[SubIdx]] = true;
			}
		}

		TVector<RealType> TriPts[3];
		TVector<RealType> Pt;
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			SetTriPts(Triangles[TriIdx], GetPointFunc, TriPts);
			VisiblePoints[TriIdx].SetPlane(TriPts);

			for (int32 PtIdx = 0; PtIdx < NumPoints; PtIdx++)
			{
				if (!FilterFunc(PtIdx) || PointAssigned[PtIdx] == true)
				{
					continue;
				}
				Pt = GetPointFunc(PtIdx);
				double Distance = VisiblePoints[TriIdx].GetPlaneDistance(Pt);
				bool bDistanceOk = VisibleDistanceThreshold < 0 || Distance > VisibleDistanceThreshold;
				if (bDistanceOk && IsVisible(TriPts, Pt))
				{
					if (!VisiblePoints[TriIdx].Num())
					{
						TrisWithPoints.Add(TriIdx);
					}
					VisiblePoints[TriIdx].AddPtByValue(PtIdx, Distance);
					PointAssigned[PtIdx] = true;
				}
			}
		}
	}

	/**
	 * Update any neighbor "back pointers" to TriIdx to instead point to NewIdx
	 * Used during triangle deletion when we swap a triangle to a new index
	 */
	void UpdateNeighbors(int32 TriIdx, int32 NewIdx)
	{
		FIndex3i Neighbors = TriNeighbors[TriIdx];
		for (int32 Idx = 0; Idx < 3; Idx++)
		{
			int32 NbrIdx = Neighbors[Idx];
			if (NbrIdx > -1 && NbrIdx < TriNeighbors.Num())
			{
				FIndex3i& NbrNbrs = TriNeighbors[NbrIdx];
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					if (NbrNbrs[SubIdx] == TriIdx)
					{
						NbrNbrs[SubIdx] = NewIdx;
						break; // should only be one back-pointer per neighbor (or sometimes zero if it was already partially unhooked)
					}
				}
			}
		}
	}

	/**
	 * Rewrite one neighbor connection (and NOT the backwards connection)
	 *
	 * @param Triangles All triangles
	 * @param TriIdx The triangle to update
	 * @param EdgeFirstVertex The first vertex of the edge that should be connected to the new neighbor
	 * @param NewNbrIdx The new neighbor to connect to
	 */
	void UpdateNeighbor(TArray<FIndex3i>& Triangles, int32 TriIdx, int32 EdgeFirstVertex, int32 NewNbrIdx)
	{
		const FIndex3i& Triangle = Triangles[TriIdx];
		for (int32 Idx = 0; Idx < 3; Idx++)
		{
			if (Triangle[Idx] == EdgeFirstVertex)
			{
				TriNeighbors[TriIdx][Idx] = NewNbrIdx;
				return;
			}
		}
		check(false); // EdgeFirstVertex wasn't found in triangle; invalid UpdateNeighbor call
	}

	/**
	 * Debugging aid to validate the neighbor connectivity
	 */
	bool ValidateConnectivity(TArray<FIndex3i>& Triangles, TSet<int32> Skip = TSet<int32>())
	{
		TMap<FIndex2i, int32> EdgeToTri;
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			if (Skip.Contains(TriIdx))
			{
				continue;
			}
			const FIndex3i& Tri = Triangles[TriIdx];
			for (int32 LastIdx = 2, Idx = 0; Idx < 3; LastIdx = Idx++)
			{
				EdgeToTri.Add(FIndex2i(Tri[LastIdx], Tri[Idx]), TriIdx);
			}
		}
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			if (Skip.Contains(TriIdx))
			{
				continue;
			}
			const FIndex3i& Tri = Triangles[TriIdx];
			const FIndex3i& Nbrs = TriNeighbors[TriIdx];
			for (int32 LastIdx = 2, Idx = 0; Idx < 3; LastIdx = Idx++)
			{
				FIndex2i BackEdge = FIndex2i(Tri[Idx], Tri[LastIdx]);
				int32* FoundNbr = EdgeToTri.Find(BackEdge);
				if (!FoundNbr || Nbrs[LastIdx] != *FoundNbr)
				{
					ensure(false);
					return false;
				}
			}
		}

		return true;
	}

	/** 
	 * Remove a triangle and update connectivity accordingly.  Removes by swapping, and updates references as needed.
	 * Note: removes a whole set at a time, because indices change during deletion
	 *
	 * @param Triangles All triangles
	 * @param ToDelete All triangles to be deleted
	 */
	void DeleteTriangles(TArray<FIndex3i>& Triangles, const TSet<int32>& ToDelete)
	{
		TSet<int32> Moved; // indices of deleted tris that non-delete triangles have been moved into, must not be deleted
		for (int32 TriIdx : ToDelete)
		{
			checkSlow(!Moved.Contains(TriIdx));

			// skip triangles we already greedy-deleted while looking for the LastIdx in a previous iter
			if (TriIdx >= Triangles.Num())
			{
				continue;
			}
			checkSlow(Triangles.Num() == TriNeighbors.Num());
			checkSlow(Triangles.Num() == VisiblePoints.Num());

			// remove back-refs
			UpdateNeighbors(TriIdx, -1);

			int32 LastIdx = Triangles.Num() - 1;
			// greedily remove any triangles from the end rather than swapping them back
			while (TriIdx < LastIdx && (ToDelete.Contains(LastIdx) && !Moved.Contains(LastIdx)))
			{
				TrisWithPoints.Remove(LastIdx);
				UpdateNeighbors(LastIdx, -1);
				LastIdx--;
			}

			// found a non-delete tri at end to swap in
			if (TriIdx < LastIdx)
			{
				Moved.Add(TriIdx); // track so we don't later "greedy delete" the now non-delete tri
				UpdateNeighbors(LastIdx, TriIdx);
				TriNeighbors[TriIdx] = TriNeighbors[LastIdx];
				Triangles[TriIdx] = Triangles[LastIdx];
				VisiblePoints[TriIdx] = MoveTemp(VisiblePoints[LastIdx]);
				if (TrisWithPoints.Remove(LastIdx))
				{
					TrisWithPoints.Add(TriIdx);
				}
			}
			else // this was the last tri, just let it pop off
			{
				TrisWithPoints.Remove(TriIdx);
			}
			Triangles.SetNum(LastIdx, EAllowShrinking::No);
			TriNeighbors.SetNum(LastIdx, EAllowShrinking::No);
			VisiblePoints.SetNum(LastIdx, EAllowShrinking::No);
		}
	}

	/**
	 * @param bChooseBestPoint	Whether to choose the point with the highest value, rather than the first found point
	 * @return A triangle index and visible-from-that-triangle point index that can be added to the hull next
	 */
	FIndex2i ChooseVisiblePoint(bool bChooseBestPoint)
	{
		FIndex2i FoundTriPointPair(-1, -1);
		if (bChooseBestPoint)
		{
			// Note: If MaxHullVertices is large enough, it could make sense to use a priority queue to efficiently track the triangles with farthest points.
			// Though if MaxHullVertices is small, this simple linear pass will be faster.
			// TODO: Try using FIndexPriorityQueue to track the faces with the best visible points, for the large vertex count case.
			// (Note the priority queue would need the max triangle index; this should be (2*FMath::Min(NumPoints, MaxHullVertices)-4))
			double BestValue = -1;
			for (int32 TriIdx : TrisWithPoints)
			{
				checkSlow(VisiblePoints[TriIdx].Num() > 0);

				if (VisiblePoints[TriIdx].MaxValue > BestValue)
				{
					FoundTriPointPair[0] = TriIdx;
					FoundTriPointPair[1] = VisiblePoints[TriIdx].MaxPt();
					BestValue = VisiblePoints[TriIdx].MaxValue;
				}
			}
		}
		else
		{
			if (const auto TriIdxItr = TrisWithPoints.CreateConstIterator())
			{
				int32 TriIdx = *TriIdxItr;
				checkSlow(VisiblePoints[TriIdx].Num() > 0);

				FoundTriPointPair[0] = TriIdx;
				// choose the "max point" -- the point with the largest volume when it makes a tetrahedron w/ the triangle
				FoundTriPointPair[1] = VisiblePoints[TriIdx].MaxPt();
			}
		}
		return FoundTriPointPair;
	}

	/**
	 * @param TriPointPair		A tri point pair, as returned from ChooseVisiblePoint, that should be removed from consideration
	 */
	void RemoveVisiblePoint(FIndex2i TriPointPair, TFunctionRef<TVector<RealType>(int32)> GetPointFunc)
	{
		VisiblePoints[TriPointPair.A].RemovePt(TriPointPair.B, GetPointFunc);
		if (VisiblePoints[TriPointPair.A].Indices.IsEmpty())
		{
			TrisWithPoints.Remove(TriPointPair.A);
		}
	}

	/**
	 * Helper struct carrying enough information to describe triangles that will be added when we UpdateHullWithNewPoint
	 */
	struct FNewTriangle
	{
		FNewTriangle()
		{}
		FNewTriangle(int32 V0, int32 V1, int32 ConnectedTri) : EdgeVertices(V0, V1), ConnectedTri(ConnectedTri)
		{}

		// The edge the new point will connect to
		FIndex2i EdgeVertices;
		// The neighboring triangle across that edge
		int32 ConnectedTri;
	};

	inline void ClearVisiblePointsData(int32 TriIdx, TArray<int32>& NewlyUnclaimed)
	{
		if (VisiblePoints[TriIdx].Num() > 0)
		{
			TrisWithPoints.Remove(TriIdx);
			NewlyUnclaimed.Append(VisiblePoints[TriIdx].Indices);
			VisiblePoints[TriIdx].Reset();
		}
	}

	/**
	 * Recursive traversal strategy that starts from a 'visible' triangle and walks to the connected set of all visible triangles, s.t. it visits the boundary in order
	 * See explanation here: http://algolist.ru/maths/geom/convhull/qhull3d.php
	 *
	 * @param Triangles All triangles
	 * @param GetPointFunc Point access function
	 * @param Pt Current 'eye point'; triangles will be deleted if this point is on the 'positive' side of the triangle
	 * @param NewlyUnclaimed (Output) As we mark triangles for deletion, we move the visible points stored on the triangles to this giant list of points that need to be re-assigned (or culled) wrt the new triangles
	 * @param ToDelete (Output) Set of all triangles that we will delete
	 * @param ToAdd (Output) Info required to add and connect up new triangles to the eye point
	 * @param TriIdx Index of triangle to currently process
	 * @param CrossedEdgeFirstVertex The first vertex of the edge that was 'crossed over' to traverse to this TriIdx, or -1 for the initial call
	 * @return false if triangle is beyond horizon (so we don't need to search through it / remove it), true otherwise
	 */
	bool HorizonHelper(const TArray<FIndex3i>& Triangles, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, const TVector<RealType>& Pt, TArray<int32>& NewlyUnclaimed, TSet<int32>& ToDelete, TArray<FNewTriangle>& ToAdd, int32 TriIdx, int32 CrossedEdgeFirstVertex, bool bCouldSkipPoint = false)
	{
		// if it's not the first triangle, crossed edge should be set and we should check if the triangle is visible / actually needs to be replaced
		if (CrossedEdgeFirstVertex != -1)
		{
			TVector<RealType> TriPts[3];
			SetTriPts(Triangles[TriIdx], GetPointFunc, TriPts);
			if (!IsVisible(TriPts, Pt))
			{
				return false;
			}
		}

		// track the triangle as needing deletion; for simplicity wait until after traversal to actually delete
		ToDelete.Add(TriIdx); // TODO: could we delete as we go rather than keep this set?  seems tricky (also, could not do so if 'bCouldSkipPoint == true')
		
		// if we know we can't ultimately skip adding this point, clean out the visible points right away; otherwise this must be done after
		if (!bCouldSkipPoint)
		{
			ClearVisiblePointsData(TriIdx, NewlyUnclaimed);
		}

		FIndex3i Tri = Triangles[TriIdx];
		int32 FirstOff = 0, OffMax = 3; // Cross all three edges for first triangle
		if (CrossedEdgeFirstVertex > -1) // Cross the two edges we didn't 'come from' for all other triangles
		{
			FirstOff = Tri.IndexOf(CrossedEdgeFirstVertex) + 1; // index of the crossed edge wrt this triangle
			OffMax = FirstOff + 3;
		}
		for (int32 EdgeIdxOff = FirstOff; EdgeIdxOff < OffMax; EdgeIdxOff++)
		{
			int32 AcrossTri = TriNeighbors[TriIdx][EdgeIdxOff % 3];
			if (ToDelete.Contains(AcrossTri))
			{
				continue;
			}
			// second vertex of the edge (should be first seen when looking for the edge on opposite side)
			int32 EdgeSecondVertex = Tri[(EdgeIdxOff + 1) % 3];
			if (!HorizonHelper(Triangles, GetPointFunc, Pt, NewlyUnclaimed, ToDelete, ToAdd, AcrossTri, EdgeSecondVertex))
			{
				int32 EdgeFirstVertex = Tri[EdgeIdxOff % 3];
				ToAdd.Add(FNewTriangle(EdgeFirstVertex, EdgeSecondVertex, AcrossTri));
			}
		}

		return true;
	}

	bool UpdateHullWithNewPoint(TArray<FIndex3i>& Triangles, int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, int32 StartTriIdx, int32 PtIdx, RealType DegenerateEdgeToleranceSq = (RealType)0)
	{
		// Note: Commented-out ValidateConnectivity calls are very slow, but useful for debugging if the algorithm produces an invalid result
		//ValidateConnectivity(Triangles);

		TVector<RealType> Pt = GetPointFunc(PtIdx);

		// Call the recursive helper to find all the tris we need to delete + the 'horizon' of edges that will form new triangles
		TArray<int32> NewlyUnclaimed;
		TSet<int32> ToDelete;
		TArray<FNewTriangle> ToAdd;
		bool bCouldSkipPoint = DegenerateEdgeToleranceSq > (RealType)0;
		HorizonHelper(Triangles, GetPointFunc, Pt, NewlyUnclaimed, ToDelete, ToAdd, StartTriIdx, -1, bCouldSkipPoint);

		// Optionally skip points if they are almost on top of an existing point on the current hull, as specified by the Degenerate Edge Tolerance (squared)
		if (bCouldSkipPoint)
		{
			for (const FNewTriangle& NewTri : ToAdd)
			{
				RealType DistSq = TVector<RealType>::DistSquared(Pt, GetPointFunc(NewTri.EdgeVertices.A));
				if (DistSq < DegenerateEdgeToleranceSq)
				{
					return false;
				}
			}

			// Once we know the point will not be skipped, clear out the visible point data for all the tris that will be deleted
			for (int32 TriIdx : ToDelete)
			{
				ClearVisiblePointsData(TriIdx, NewlyUnclaimed);
			}
		}

		// Connect up all the horizon triangles
		int32 NewTriStart = Triangles.Num();
		int32 NumAdd = ToAdd.Num();
		TVector<RealType> TriPts[3];


		for (int32 AddIdx = 0; AddIdx < NumAdd; AddIdx++)
		{
			const FNewTriangle& TriData = ToAdd[AddIdx];
			int32 NewTriIdx = Triangles.Add(FIndex3i(PtIdx, TriData.EdgeVertices.A, TriData.EdgeVertices.B));
			int32 PrevTriIdx = NewTriStart + ((AddIdx - 1 + NumAdd) % NumAdd);
			int32 NextTriIdx = NewTriStart + ((AddIdx + 1) % NumAdd);

			int32 AcrossTriIdx = TriData.ConnectedTri;
			TriNeighbors.Add(FIndex3i(PrevTriIdx, AcrossTriIdx, NextTriIdx)); // TODO: is this the right order? need to validate!
			UpdateNeighbor(Triangles, AcrossTriIdx, TriData.EdgeVertices.B, NewTriIdx);
			FVisiblePoints& Visible = VisiblePoints.Emplace_GetRef();
			SetTriPts(Triangles[NewTriIdx], GetPointFunc, TriPts);
			Visible.SetPlane(TriPts);
			// claim any claim-able points
			if (VisibleDistanceThreshold > 0) // If using VisibleDistanceThreshold, compute PlaneDist first so we can use it to threshold
			{
				for (int32 UnclaimedIdx = 0; UnclaimedIdx < NewlyUnclaimed.Num(); UnclaimedIdx++)
				{
					int32 UnPtIdx = NewlyUnclaimed[UnclaimedIdx];
					TVector<RealType> UnPt = GetPointFunc(UnPtIdx);
					double PlaneDist = Visible.GetPlaneDistance(UnPt);
					if (PlaneDist > VisibleDistanceThreshold && IsVisible(TriPts, UnPt))
					{
						Visible.AddPtByValue(UnPtIdx, PlaneDist);
						NewlyUnclaimed.RemoveAtSwap(UnclaimedIdx, EAllowShrinking::No);
						UnclaimedIdx--;
						continue;
					}
				}
			}
			else // otherwise skip the comparison and only compute PlaneDist when adding the point
			{
				for (int32 UnclaimedIdx = 0; UnclaimedIdx < NewlyUnclaimed.Num(); UnclaimedIdx++)
				{
					int32 UnPtIdx = NewlyUnclaimed[UnclaimedIdx];
					TVector<RealType> UnPt = GetPointFunc(UnPtIdx);
					if (IsVisible(TriPts, UnPt))
					{
						Visible.AddPt(UnPtIdx, UnPt);
						NewlyUnclaimed.RemoveAtSwap(UnclaimedIdx, EAllowShrinking::No);
						UnclaimedIdx--;
						continue;
					}
				}
			}

			if (Visible.Num() > 0)
			{
				TrisWithPoints.Add(NewTriIdx);
			}
		}

		//ValidateConnectivity(Triangles, ToDelete);


		// do the deletions *last* so we don't invalidate tri indices in ToAdd
		DeleteTriangles(Triangles, ToDelete);

		//ValidateConnectivity(Triangles);

		return true;
	}
};


template<class RealType>
double TConvexHull3<RealType>::ComputeVolume(const TArrayView<const TVector<RealType>> Points)
{
	TConvexHull3<RealType> Hull;
	bool bSuccess = Hull.Solve(Points.Num(), [&Points](int32 Idx) { return Points[Idx]; });
	if (!bSuccess)
	{
		return 0.0;
	}
	const TArray<FIndex3i>& Tris = Hull.GetTriangles();
	double Volume = 0.0;
	for (FIndex3i Tri : Tris)
	{
		FVector3d V0 = (FVector3d)Points[Tri.A], V1 = (FVector3d)Points[Tri.B], V2 = (FVector3d)Points[Tri.C];
		FVector3d V1mV0 = V1 - V0;
		FVector3d V2mV0 = V2 - V0;
		FVector3d N = V2mV0.Cross(V1mV0);
		double tmp0 = V0.X + V1.X;
		double f1x = tmp0 + V2.X;
		Volume += N.X * f1x;
	}

	return Volume / 6.0;
}


template<class RealType>
bool TConvexHull3<RealType>::Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc)
{
	Hull.Reset();
	NumHullPoints = 0;

	TExtremePoints3<RealType> InitialTet(NumPoints, GetPointFunc, FilterFunc);
	Dimension = InitialTet.Dimension;
	if (Dimension < 3)
	{
		if (Dimension == 1)
		{
			Line = TLine3<RealType>(InitialTet.Origin, InitialTet.Basis[0]);
		}
		else if (Dimension == 2)
		{
			Plane = TPlane3<RealType>(InitialTet.Basis[2], InitialTet.Origin);
		}
		return false;
	}

	// safety check; seems possible the InitialTet chosen points were actually coplanar, because it was constructed w/ inexact math
	if (ExactPredicates::Orient3<RealType>(GetPointFunc(InitialTet.Extreme[0]), GetPointFunc(InitialTet.Extreme[1]), GetPointFunc(InitialTet.Extreme[2]), GetPointFunc(InitialTet.Extreme[3])) == 0)
	{
		Plane = TPlane3<RealType>(InitialTet.Basis[2], InitialTet.Origin);
		Dimension = 2;
		return false;
	}

	// Add triangles from InitialTet
	Hull.Add(FIndex3i(InitialTet.Extreme[1], InitialTet.Extreme[2], InitialTet.Extreme[3]));
	Hull.Add(FIndex3i(InitialTet.Extreme[0], InitialTet.Extreme[3], InitialTet.Extreme[2]));
	Hull.Add(FIndex3i(InitialTet.Extreme[0], InitialTet.Extreme[1], InitialTet.Extreme[3]));
	Hull.Add(FIndex3i(InitialTet.Extreme[0], InitialTet.Extreme[2], InitialTet.Extreme[1]));

	NumHullPoints = 4;
	bool bHasPointBudget = SimplificationSettings.MaxHullVertices > 0;
	int32 UseMaxHullVertices = SimplificationSettings.MaxHullVertices;
	if (!bHasPointBudget)
	{
		UseMaxHullVertices = NumPoints;
	}

	RealType DegenerateEdgeToleranceSq = SimplificationSettings.DegenerateEdgeTolerance * SimplificationSettings.DegenerateEdgeTolerance;

	FHullConnectivity<RealType> Connectivity;
	double UseSkipAtHullDistance = SimplificationSettings.SkipAtHullDistanceAbsolute;
	if (SimplificationSettings.SkipAtHullDistanceAsFraction > 0)
	{
		double Extent = (GetPointFunc(InitialTet.Extreme[0]) - GetPointFunc(InitialTet.Extreme[1])).Length();
		UseSkipAtHullDistance = FMathd::Max(UseSkipAtHullDistance, Extent * SimplificationSettings.SkipAtHullDistanceAsFraction);
	}
	Connectivity.VisibleDistanceThreshold = UseSkipAtHullDistance;
	Connectivity.BuildNeighbors(Hull);
	Connectivity.InitVisibility(Hull, NumPoints, GetPointFunc, FilterFunc);
	while (NumHullPoints < UseMaxHullVertices)
	{
		if (Progress && (NumHullPoints % 100) == 0 && Progress->Cancelled())
		{
			return false;
		}
		FIndex2i Visible = Connectivity.ChooseVisiblePoint(bHasPointBudget);
		if (Visible.A == -1)
		{
			break;
		}
		NumHullPoints++;
		bool bAdded = Connectivity.UpdateHullWithNewPoint(Hull, NumPoints, GetPointFunc, Visible.A, Visible.B, DegenerateEdgeToleranceSq);
		if (!bAdded)
		{
			Connectivity.RemoveVisiblePoint(Visible, GetPointFunc);
		}
	}

	if (bSaveTriangleNeighbors)
	{
		HullNeighbors = MoveTemp(Connectivity.TriNeighbors);
	}

	return true;
}

template<class RealType>
void TConvexHull3<RealType>::GetFaces(TFunctionRef<void(TArray<int32>&, TVector<RealType>)> PolygonFunc, TFunctionRef<TVector<RealType>(int32)> GetPointFunc) const
{
	if (!ensureMsgf(bSaveTriangleNeighbors && HullNeighbors.Num() == Hull.Num(), TEXT("To extract faces, set bSaveTriangleNeighbors = true before calling Solve()")))
	{
		return;
	}

	TArray<int32> CurFaceVertIDs; // Used for face vertex IDs
	TArray<int32> ToProcess; // Used as a stack for hull indices

	// IDs to indicate which triangles can be grouped into the same convex polygon
	TArray<int32> GroupIDs;
	GroupIDs.Init(-1, Hull.Num());

	int32 NextGroupID = 0;

	// Greedily flood-fill to find coplanar triangles in each group
	for (int32 TriIdx = 0; TriIdx < Hull.Num(); ++TriIdx)
	{
		if (GroupIDs[TriIdx] >= 0)
		{
			continue;
		}

		int32 CurGroupID = NextGroupID++;
		GroupIDs[TriIdx] = CurGroupID;

		ToProcess.Reset();

		const FIndex3i& Tri = Hull[TriIdx];
		const FIndex3i& NbrInds = HullNeighbors[TriIdx];
		TVector<RealType> TriPosns[3]{ GetPointFunc(Tri.A), GetPointFunc(Tri.B), GetPointFunc(Tri.C) };
		TVector<RealType> FaceNormal = VectorUtil::Normal(TriPosns[0], TriPosns[1], TriPosns[2]);
		bool bValidNormal = FaceNormal.SquaredLength() > 0;
		ToProcess.Add(NbrInds[0]);
		ToProcess.Add(NbrInds[1]);
		ToProcess.Add(NbrInds[2]);
		while (ToProcess.Num() > 0)
		{
			int32 NbrTriIdx = ToProcess.Pop(EAllowShrinking::No);
			// skip the triangle if it's already assigned a group or we've already visited it in this pass
			if (GroupIDs[NbrTriIdx] >= 0 || GroupIDs[NbrTriIdx] < -CurGroupID - 1)
			{
				continue;
			}

			// region grow to the triangle if its vertices are all coplanar w/ the source triangle
			const FIndex3i& NbrTri = Hull[NbrTriIdx];
			bool bFoundOnlyNewCoplanar = false;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				int32 V = NbrTri[SubIdx];
				// only bother testing the vertex if it's new (wasn't on the source triangles)
				if (V != Tri.A && V != Tri.B && V != Tri.C)
				{
					TVector<RealType> OtherPos = GetPointFunc(V);
					RealType O3DVal = ExactPredicates::Orient3<RealType>(TriPosns[0], TriPosns[1], TriPosns[2], OtherPos);
					if (O3DVal == 0)
					{
						// found a coplanar vertex; consider growing to this tri
						bFoundOnlyNewCoplanar = true;
					}
					else
					{
						// found a non-coplanar vertex; stop searching and do not grow to this tri
						bFoundOnlyNewCoplanar = false;
						break;
					}
				}
			}
			if (bFoundOnlyNewCoplanar)
			{
				GroupIDs[NbrTriIdx] = CurGroupID;

				// if we failed to compute a stable normal from the source triangle, compute one now
				if (!bValidNormal)
				{
					FaceNormal = VectorUtil::Normal(GetPointFunc(NbrTri.A), GetPointFunc(NbrTri.B), GetPointFunc(NbrTri.C));
					bValidNormal = FaceNormal.SquaredLength() > 0;
				}

				// grow to unlabelled neighbors
				const FIndex3i& NbrNbrInds = HullNeighbors[NbrTriIdx];
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					int32 NbrNbrTriIdx = NbrNbrInds[SubIdx];
					// consider the triangle for region growing if it's not assigned a group and we haven't visited it yet this pass
					if (GroupIDs[NbrNbrTriIdx] < 0 && GroupIDs[NbrNbrTriIdx] >= -CurGroupID - 1)
					{
						ToProcess.Add(NbrNbrTriIdx);
					}
				}
			}
			else
			{
				// Flag the unset triangles as already-visited, to avoid repeat tests
				GroupIDs[NbrTriIdx] = -CurGroupID - 2;
			}
		}

		CurFaceVertIDs.Reset();
		WalkBorder(Hull, HullNeighbors, [&](int32 Idx) { return GroupIDs[Idx] == CurGroupID; }, TriIdx, CurFaceVertIDs);
		PolygonFunc(CurFaceVertIDs, FaceNormal);
	}
}

template<class RealType>
void TConvexHull3<RealType>::GetSimplifiedFaces(TFunctionRef<void(TArray<int32>&, TVector<RealType>)> PolygonFunc, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,
	RealType FaceAngleTolerance, RealType PlaneDistanceTolerance) const
{
	if (!ensureMsgf(bSaveTriangleNeighbors && HullNeighbors.Num() == Hull.Num(), TEXT("To extract faces, set bSaveTriangleNeighbors = true before calling Solve()")))
	{
		return;
	}

	TArray<FPolygonFace> Polygons;
	TArray<TVector<RealType>> Normals;
	GetSimplifiedFaces(Polygons, GetPointFunc, FaceAngleTolerance, PlaneDistanceTolerance, &Normals);
	TArray<int32> PolygonVertices;
	for (int32 Idx = 0; Idx < Polygons.Num(); ++Idx)
	{
		const FPolygonFace& Face = Polygons[Idx];
		PolygonVertices.Reset(Face.Num());
		PolygonVertices.Append(Face);
		PolygonFunc(PolygonVertices, Normals[Idx]);
	}
}

template<class RealType>
void TConvexHull3<RealType>::GetSimplifiedFaces(TArray<FPolygonFace>& OutPolygons, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,
	RealType FaceAngleToleranceInDegrees, RealType PlaneDistanceTolerance, TArray<TVector<RealType>>* OutPolygonNormals) const
{
	if (!ensureMsgf(bSaveTriangleNeighbors && HullNeighbors.Num() == Hull.Num(), TEXT("To extract faces, set bSaveTriangleNeighbors = true before calling Solve()")))
	{
		return;
	}

	OutPolygons.Reset();
	if (OutPolygonNormals)
	{
		OutPolygonNormals->Reset();
	}

	FaceAngleToleranceInDegrees = FMath::Min(FaceAngleToleranceInDegrees, 60); // Angle Tolerance should be a small value; if a very large angle tolerance is allowed, it could create a degenerate (flat or zero vertex) hull
	const double FaceMergeThreshold = 1 - FMath::Cos(FMath::DegreesToRadians(FaceAngleToleranceInDegrees));

	int32 NumTris = Hull.Num();
	FSizedDisjointSet PlaneGroups;
	PlaneGroups.Init(NumTris);
	TArray<FVector3d> PlaneNormals; PlaneNormals.SetNumUninitialized(NumTris);
	TArray<double> PlaneAreas; PlaneAreas.SetNumUninitialized(NumTris);
	TArray<FVector3d> PlaneOrigins; PlaneOrigins.SetNumUninitialized(NumTris);
	// Map of plane group ID -> vertex indices, only for faces that have been merged at least once (otherwise, we can use the triangle indices)
	using FPlaneVertArray = TArray<int32, TInlineAllocator<16>>;
	TMap<int32, FPlaneVertArray> PlaneGroupToVerticesMap;
	int32 MaxVertexNum = 0;
	for (int32 TriIdx = 0; TriIdx < NumTris; ++TriIdx)
	{
		const FIndex3i& Tri = Hull[TriIdx];
		MaxVertexNum = FMath::Max(MaxVertexNum, 1 + FMath::Max(Tri.A, FMath::Max(Tri.B, Tri.C)));
		FVector3d TriPosns[3]{ (FVector3d)GetPointFunc(Tri.A), (FVector3d)GetPointFunc(Tri.B), (FVector3d)GetPointFunc(Tri.C) };
		double Area = 0;
		FVector3d FaceNormal = VectorUtil::NormalArea(TriPosns[0], TriPosns[1], TriPosns[2], Area);
		PlaneNormals[TriIdx] = FaceNormal;
		PlaneAreas[TriIdx] = Area;
		PlaneOrigins[TriIdx] = (TriPosns[0] + TriPosns[1] + TriPosns[2]) * (1.0/3.0);
	}

	struct FEdgeWeight
	{
		FEdgeWeight() = default;
		FEdgeWeight(FIndex2i TriPair, FIndex2i VertPair, double Weight) : TriPair(TriPair), VertPair(VertPair), Weight(Weight) {}
		FIndex2i TriPair;
		FIndex2i VertPair;
		double Weight;
	};

	constexpr double AreaThreshold = UE_DOUBLE_SMALL_NUMBER;
	auto GetMergeWeight = [&PlaneGroups, &PlaneAreas, &PlaneNormals, &GetPointFunc, AreaThreshold](FIndex2i PlanePair, FIndex2i VertexPair, bool bHasGroups) -> double
	{
		if (bHasGroups)
		{
			PlanePair[0] = PlaneGroups.Find(PlanePair[0]);
			PlanePair[1] = PlaneGroups.Find(PlanePair[1]);
			if (PlanePair[0] == PlanePair[1])
			{
				// return a value higher than the max possible angle threshold if the plane groups are already merged
				constexpr float CannotMergeValue = 4;
				return CannotMergeValue;
			}
		}
		// Planes with small area can be merged into any neighbor, because we don't trust their plane normal
		int32 TooSmallAreas = int32(PlaneAreas[PlanePair[0]] < AreaThreshold) + int32(PlaneAreas[PlanePair[1]] < AreaThreshold);
		if (TooSmallAreas == 1)
		{
			// favor merging too-small-area faces to the neighboring valid-area face with the longest shared edge
			FVector3d V0 = (FVector3d)GetPointFunc(VertexPair.A);
			FVector3d V1 = (FVector3d)GetPointFunc(VertexPair.B);
			double EdgeLenSq = FVector3d::DistSquared(V0, V1);
			return -EdgeLenSq;
		}
		else if (TooSmallAreas == 2)
		{
			// if both areas are too small, still allow a low-cost merge, but favor the above single-small-area merges
			return 0;
		}
		double NormalAlignment = 1 - PlaneNormals[PlanePair[0]].Dot(PlaneNormals[PlanePair[1]]);
		return NormalAlignment;
	};

	TArray<FEdgeWeight> EdgeWeights;
	for (int32 TriIdx = 0; TriIdx < NumTris; ++TriIdx)
	{
		const FIndex3i& NbrInds = HullNeighbors[TriIdx];
		const FIndex3i& Tri = Hull[TriIdx];
		for (int32 PrevIdx = 2, SubIdx = 0; SubIdx < 3; PrevIdx = SubIdx++)
		{
			if (TriIdx < NbrInds[PrevIdx])
			{
				FIndex2i FacePair(TriIdx, NbrInds[PrevIdx]);
				FIndex2i VertPair(PrevIdx, SubIdx);
				double Weight = GetMergeWeight(FacePair, VertPair, false);
				if (Weight < FaceMergeThreshold)
				{
					EdgeWeights.Emplace(FacePair, VertPair, Weight);
				}
			}
		}
	}
	
	// Consider merging across edges with lower edge weights first
	EdgeWeights.Sort([&](const FEdgeWeight& A, const FEdgeWeight& B)
		{
			return A.Weight < B.Weight;
		});
	for (const FEdgeWeight& EdgeWeight : EdgeWeights)
	{
		// re-evaluate the edge weight with the current planes
		double MergeWeight = GetMergeWeight(EdgeWeight.TriPair, EdgeWeight.VertPair, true);
		if (MergeWeight < FaceMergeThreshold)
		{
			int32 Groups[2]{ PlaneGroups.Find(EdgeWeight.TriPair.A), PlaneGroups.Find(EdgeWeight.TriPair.B) };
			int32 GroupSizes[2]{ PlaneGroups.GetSize(Groups[0]), PlaneGroups.GetSize(Groups[1]) };
			double Areas[2]{ PlaneAreas[Groups[0]], PlaneAreas[Groups[1]] };
			double AreaSum = Areas[0] + Areas[1];
			FVector3d Centroid;
			if (AreaSum > UE_DOUBLE_KINDA_SMALL_NUMBER)
			{
				Centroid = (PlaneOrigins[Groups[0]] * (Areas[0]/AreaSum) + PlaneOrigins[Groups[1]] * (Areas[1]/AreaSum));
			}
			else
			{
				Centroid = (PlaneOrigins[Groups[0]] + PlaneOrigins[Groups[1]]) * .5;
			}

			FVector3d Normal = PlaneNormals[Groups[0]] * Areas[0] + PlaneNormals[Groups[1]] * Areas[1];
			Normal.Normalize();

			// Test that the points on each plane are close enough to the merged plane
			bool bPointsCloseEnoughToPlane = true;

			auto TestPoint = [&bPointsCloseEnoughToPlane, &GetPointFunc, Centroid, Normal, PlaneDistanceTolerance](int32 PointIdx) -> bool
			{
				FVector3d PointPos = (FVector3d)GetPointFunc(PointIdx);
				if (FMath::Abs((PointPos - Centroid).Dot(Normal)) > PlaneDistanceTolerance)
				{
					bPointsCloseEnoughToPlane = false;
					return false;
				}
				return true;
			};

			for (int32 GroupSubIdx = 0; GroupSubIdx < 2 && bPointsCloseEnoughToPlane; ++GroupSubIdx)
			{
				int32 GroupIdx = Groups[GroupSubIdx];
				if (GroupSizes[GroupSubIdx] == 1)
				{
					FIndex3i Tri = Hull[GroupIdx];
					for (int32 Idx = 0; Idx < 3; ++Idx)
					{
						if (!TestPoint(Tri[Idx]))
						{
							break;
						}
					}
				}
				else
				{
					for (int32 PointIdx : PlaneGroupToVerticesMap[GroupIdx])
					{
						if (!TestPoint(PointIdx))
						{
							break;
						}
					}
				}
			}
			if (!bPointsCloseEnoughToPlane)
			{
				continue;
			}

			PlaneGroups.Union(Groups[0], Groups[1]);
			int32 NewParent = PlaneGroups.Find(Groups[0]);
			int32 OldGroupIdx = Groups[0] == NewParent ? 1 : 0;
			int32 OldGroup = Groups[OldGroupIdx];
			PlaneNormals[NewParent] = Normal;
			PlaneOrigins[NewParent] = Centroid;
			PlaneAreas[NewParent] = Areas[0] + Areas[1];

			// Update the PlaneGroupToVerticesMap structures, removing the old group if needed, creating the new group if needed, and adding the appropriate vertices
			if (GroupSizes[1 - OldGroupIdx] <= 1)
			{
				FIndex3i Tri = Hull[Groups[1 - OldGroupIdx]];
				FPlaneVertArray& NewGroupVerts = PlaneGroupToVerticesMap.Emplace(NewParent);
				NewGroupVerts.Add(Tri.A);
				NewGroupVerts.Add(Tri.B);
				NewGroupVerts.Add(Tri.C);
			}
			if (GroupSizes[OldGroupIdx] > 1)
			{
				const FPlaneVertArray OldGroupVerts = PlaneGroupToVerticesMap.FindAndRemoveChecked(OldGroup);
				FPlaneVertArray& NewGroupVerts = PlaneGroupToVerticesMap[NewParent];
				NewGroupVerts.Reserve(NewGroupVerts.Num() + OldGroupVerts.Num() - 2);
				for (int32 VIdx : OldGroupVerts)
				{
					if (!EdgeWeight.VertPair.Contains(VIdx))
					{
						NewGroupVerts.Add(VIdx);
					}
				}
			}
			else
			{
				checkSlow(!PlaneGroupToVerticesMap.Contains(OldGroup));
				FPlaneVertArray& NewGroupVerts = PlaneGroupToVerticesMap[NewParent];
				FIndex3i Tri = Hull[Groups[OldGroupIdx]];
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					int32 VIdx = Tri[SubIdx];
					if (!EdgeWeight.VertPair.Contains(VIdx))
					{
						NewGroupVerts.Add(VIdx);
					}
				}
			}
		}
	}

	// Track which vertices are attached to at least 3 groups
	TArray<FIndex3i> VertPlaneGroups;
	VertPlaneGroups.Init(FIndex3i::Invalid(), MaxVertexNum);
	for (int32 TriIdx = 0; TriIdx < Hull.Num(); ++TriIdx)
	{
		int32 Group = PlaneGroups.Find(TriIdx);
		FIndex3i Tri = Hull[TriIdx];
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			int32 VIdx = Tri[SubIdx];
			for (int32 GroupNbrIdx = 0; GroupNbrIdx < 3; ++GroupNbrIdx)
			{
				if (VertPlaneGroups[VIdx][GroupNbrIdx] == -1)
				{
					VertPlaneGroups[VIdx][GroupNbrIdx] = Group;
					break;
				}
				else if (VertPlaneGroups[VIdx][GroupNbrIdx] == Group)
				{
					break;
				}
			}
		}
	}

	auto VertexTouchesAtLeastThreeGroups = [&VertPlaneGroups](int32 VertIdx)
	{
		return VertPlaneGroups[VertIdx][2] != -1;
	};

	TArray<int32> CurFaceVertIDs;
	bool bHasDeletedFaces = false;
	// Track the mapping from polygons back to groups, only if not using OutPolygonNormals, to recover the polygon normals for convexity tests
	TArray<int32> PolygonToGroup;
	for (int32 TriIdx = 0; TriIdx < Hull.Num(); ++TriIdx)
	{
		int32 GroupIdx = PlaneGroups.Find(TriIdx);
		if (GroupIdx != TriIdx)
		{
			continue;
		}
		int32 GroupSize = PlaneGroups.GetSize(TriIdx);
		if (GroupSize == 1)
		{
			FIndex3i Tri = Hull[TriIdx];
			if (VertexTouchesAtLeastThreeGroups(Tri.A) &&
				VertexTouchesAtLeastThreeGroups(Tri.B) &&
				VertexTouchesAtLeastThreeGroups(Tri.C))
			{
				FPolygonFace& Face = OutPolygons.Emplace_GetRef();
				Face.Add(Tri.A);
				Face.Add(Tri.B);
				Face.Add(Tri.C);
				if (OutPolygonNormals)
				{
					OutPolygonNormals->Add((TVector<RealType>)PlaneNormals[GroupIdx]);
				}
				else
				{
					PolygonToGroup.Add(GroupIdx);
				}
			}
			else
			{
				bHasDeletedFaces = true;
			}

			continue;
		}

		CurFaceVertIDs.Reset();
		WalkBorder(Hull, HullNeighbors, [&](int32 Idx) { return PlaneGroups.Find(Idx) == GroupIdx; }, TriIdx, CurFaceVertIDs);
		int32 NumKept = Algo::StableRemoveIf(CurFaceVertIDs, [&](int32 VertIdx) { return !VertexTouchesAtLeastThreeGroups(VertIdx); });

		if (NumKept < 3)
		{
			bHasDeletedFaces = true;
			continue;
		}

		CurFaceVertIDs.SetNum(NumKept, EAllowShrinking::No);

		FPolygonFace& Face = OutPolygons.Emplace_GetRef();
		Face.Append(CurFaceVertIDs);
		if (OutPolygonNormals)
		{
			OutPolygonNormals->Add((TVector<RealType>)PlaneNormals[GroupIdx]);
		}
		else
		{
			PolygonToGroup.Add(GroupIdx);
		}
	}

	// If we deleted faces in the initial pass, keep doing passes to identify vertices that touch fewer than 3 polygons & deleting them + deleting polygons w/ < 3 remaining vertices,
	// until we either stop deleting polygons or we have too few polygons to form a solid volume
	while (bHasDeletedFaces && OutPolygons.Num() > 3)
	{
		bHasDeletedFaces = false; // reset for the next pass

		// recompute vert plane groups using the output polygons
		VertPlaneGroups.Init(FIndex3i::Invalid(), MaxVertexNum);
		for (int32 PolyIdx = 0; PolyIdx < OutPolygons.Num(); ++PolyIdx)
		{
			FPolygonFace& Face = OutPolygons[PolyIdx];
			for (int32 SubIdx = 0; SubIdx < Face.Num(); ++SubIdx)
			{
				int32 VIdx = Face[SubIdx];
				for (int32 GroupNbrIdx = 0; GroupNbrIdx < 3; ++GroupNbrIdx)
				{
					if (VertPlaneGroups[VIdx][GroupNbrIdx] == -1)
					{
						VertPlaneGroups[VIdx][GroupNbrIdx] = PolyIdx;
						break;
					}
					else if (VertPlaneGroups[VIdx][GroupNbrIdx] == PolyIdx)
					{
						break;
					}
				}
			}
		}


		for (int32 PolyIdx = 0; PolyIdx < OutPolygons.Num(); ++PolyIdx)
		{
			FPolygonFace& Face = OutPolygons[PolyIdx];
			int32 NumKept = Algo::StableRemoveIf(Face, [&](int32 VertIdx) { return !VertexTouchesAtLeastThreeGroups(VertIdx); });
			if (NumKept != Face.Num())
			{
				if (NumKept > 2)
				{
					Face.SetNum(NumKept);
				}
				else
				{
					OutPolygons.RemoveAtSwap(PolyIdx, EAllowShrinking::No);
					if (OutPolygonNormals)
					{
						OutPolygonNormals->RemoveAtSwap(PolyIdx, EAllowShrinking::No);
					}
					else
					{
						PolygonToGroup.RemoveAtSwap(PolyIdx, EAllowShrinking::No);
					}
					bHasDeletedFaces = true;
					PolyIdx--;
				}
			}
		}
	}

	// Validate convex-enough winding vs the plane normal, and trigger a fallback if this fails
	bool bFacesAreConvex = true;
	for (int32 PolyIdx = 0; PolyIdx < OutPolygons.Num() && bFacesAreConvex; ++PolyIdx)
	{

		FVector3d PlaneNormal;
		if (OutPolygonNormals)
		{
			PlaneNormal = (FVector3d)(*OutPolygonNormals)[PolyIdx];
		}
		else
		{
			PlaneNormal = PlaneNormals[PolygonToGroup[PolyIdx]];
		}

		// Iterate over each polygon face corner ABC, testing direction the AB edge turns vs the BC edge (from the PoV of the plane normal)
		const FPolygonFace& Polygon = OutPolygons[PolyIdx];
		int32 PolygonVertNum = Polygon.Num();
		int32 VertIdxA = PolygonVertNum - 2, VertIdxB = PolygonVertNum - 1, VertIdxC = 0;
		FVector3d PtA = (FVector3d)GetPointFunc(Polygon[VertIdxA]);
		FVector3d PtB = (FVector3d)GetPointFunc(Polygon[VertIdxB]);
		FVector3d EdgeAB = PtB - PtA;
		bool bABIsNormalized = EdgeAB.Normalize();
		for (; VertIdxC < PolygonVertNum && bFacesAreConvex; VertIdxA = VertIdxB, VertIdxB = VertIdxC++)
		{
			FVector3d PtC = (FVector3d)GetPointFunc(Polygon[VertIdxC]);
			FVector3d EdgeBC = PtC - PtB;
			bool bBCIsNormalized = EdgeBC.Normalize();
			if (bABIsNormalized && bBCIsNormalized)
			{
				FVector3d EdgeABPerp = EdgeAB.Cross(PlaneNormal);
				if (EdgeBC.Dot(EdgeABPerp) < -UE_DOUBLE_KINDA_SMALL_NUMBER)
				{
					bFacesAreConvex = false;
					break;
				}
			}

			PtA = PtB;
			PtB = PtC;
			EdgeAB = EdgeBC;
			bABIsNormalized = bBCIsNormalized;
		}
	}


	// If the algorithm has failed to create convex faces, or failed to create at least a tetrahedron, e.g. due to unfortunate face merges, fall back to a more exact hull
	// Note: This should be a rare case, and it still uses the reduced vertex set of the 'vertex touches at least 3 groups' criteria,
	// so should still give a simpler hull than calling GetFaces() directly would have.
	if (!bFacesAreConvex || OutPolygons.Num() < 4)
	{
		OutPolygons.Reset();
		if (OutPolygonNormals)
		{
			OutPolygonNormals->Reset();
		}
		TConvexHull3<RealType> FallbackHull;
		FallbackHull.bSaveTriangleNeighbors = true;
		// Note: We do not need the FallbackHull to use the original simplification settings, because the filter will guarantee we only use a subset of on-hull vertices in this pass

		bool bFallbackSolveSuccess = FallbackHull.Solve(MaxVertexNum, GetPointFunc, VertexTouchesAtLeastThreeGroups);
		if (bFallbackSolveSuccess)
		{
			FallbackHull.GetFaces([&](TArray<int32>& FaceIndices, TVector<RealType> Normal)
			{
				FPolygonFace& Face = OutPolygons.Emplace_GetRef();
				Face.Append(FaceIndices);
				if (OutPolygonNormals)
				{
					OutPolygonNormals->Add(Normal);
				}
			}, GetPointFunc);
		}
		else
		{
			// If we failed to solve for a new hull using just the group-corner vertices, then use the initial hull faces without simplification
			// This could happen if the faces of the convex hull were all too small in area, so all faces were merged, leaving no vertices for the FallbackHull to find
			GetFaces([&](TArray<int32>& FaceIndices, TVector<RealType> Normal)
			{
				FPolygonFace& Face = OutPolygons.Emplace_GetRef();
				Face.Append(FaceIndices);
				if (OutPolygonNormals)
				{
					OutPolygonNormals->Add(Normal);
				}
			}, GetPointFunc);
		}

	}
}

template<typename RealType>
void TConvexHull3<RealType>::WalkBorder(const TArray<FIndex3i>& Triangles, const TArray<FIndex3i>& TriangleNeighbors, TFunctionRef<bool(int32)> InGroupFunc, int32 StartIdx, TArray<int32>& OutBorderVertexIndices)
{
	check(Triangles.Num() == TriangleNeighbors.Num());

	struct FVisit
	{
		int32 Tri;
		int8 Edge;
	};

	auto CrossEdge = [&Triangles, &TriangleNeighbors](int32 Tri, int32 EdgeNum) -> FVisit
	{
		int32 NbrTri = TriangleNeighbors[Tri][EdgeNum];
		int32 SecondVert = Triangles[Tri][EdgeNum + 1 < 3 ? EdgeNum + 1 : 0];
		int8 NbrEdge = (int8)Triangles[NbrTri].IndexOf(SecondVert);

		return FVisit{ NbrTri, NbrEdge };
	};

	TArray<bool> Visited;
	Visited.SetNumZeroed(Triangles.Num());
	TArray<FVisit> VisitStack;
	VisitStack.Reserve(Triangles.Num() / 3);
	Visited[StartIdx] = true;
	VisitStack.Add(CrossEdge(StartIdx, 2));
	VisitStack.Add(CrossEdge(StartIdx, 1));
	VisitStack.Add(CrossEdge(StartIdx, 0));

	while (VisitStack.Num())
	{
		FVisit Visit = VisitStack.Pop(EAllowShrinking::No);
		if (!InGroupFunc(Visit.Tri))
		{
			OutBorderVertexIndices.Add(Triangles[Visit.Tri][Visit.Edge]);
			continue;
		}
		if (Visited[Visit.Tri])
		{
			continue;
		}
		Visited[Visit.Tri] = true;
		VisitStack.Add(CrossEdge(Visit.Tri, (Visit.Edge + 2) % 3));
		VisitStack.Add(CrossEdge(Visit.Tri, (Visit.Edge + 1) % 3));
	}
}

template struct TExtremePoints3<float>;
template struct TExtremePoints3<double>;
template class TConvexHull3<float>;
template class TConvexHull3<double>;

} // end namespace UE::Geometry
} // end namespace UE