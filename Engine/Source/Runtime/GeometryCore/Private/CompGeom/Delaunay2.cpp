// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/Delaunay2.h"

#include "CompGeom/ExactPredicates.h"
#include "Spatial/ZOrderCurvePoints.h"
#include "MathUtil.h"

#include "Algo/RemoveIf.h"

#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

// Simple triangle connectivity structure designed for Delaunay triangulation specifically; may not support e.g. non-manifold meshes
// To support Delaunay triangulation algorithms, the structure supports having a single 'ghost vertex' connected to the boundary of the triangulation
// Currently this is a very simple edge TMap + an optional vertex->edge cache; it may be faster if switched to something that is not so TMap-based
struct FDelaunay2Connectivity
{
	static constexpr int32 GhostIndex = -1;
	static constexpr int32 InvalidIndex = -2;

	FDelaunay2Connectivity(bool bTrackDuplicateVertices) : bTrackDuplicateVertices(bTrackDuplicateVertices)
	{}

	void Empty(int32 ExpectedMaxVertices = 0)
	{
		bHasDuplicates = false;
		EdgeToVert.Empty(ExpectedMaxVertices * 2 * 3);
		DisableVertexAdjacency();
	}

	// Add the triangles from another mesh directly to this one
	void Append(const FDelaunay2Connectivity& ToAdd)
	{
		EdgeToVert.Reserve(EdgeToVert.Num() + ToAdd.EdgeToVert.Num());
		for (const TPair<FIndex2i, int32>& EdgeV : ToAdd.EdgeToVert)
		{
			EdgeToVert.Add(EdgeV.Key, EdgeV.Value);
			if (bUseAdjCache)
			{
				UpdateAdjCache(EdgeV.Key);
			}
		}
	}

	// just turns on faster vertex->edge lookups, at cost of increased storage and bookkeeping
	void EnableVertexAdjacency(int32 NumVertices)
	{
		// if cache was off or the wrong size, create it fresh
		if (!bUseAdjCache || VertexAdjCache.Num() < VertexIDToAdjIndex(NumVertices))
		{
			bUseAdjCache = true;
			VertexAdjCache.Init(-1, VertexIDToAdjIndex(NumVertices));
			for (const TPair<FIndex2i, int32>& EV : EdgeToVert)
			{
				UpdateAdjCache(EV.Key);
			}
		}
	}

	const TArray<int32>& GetVertexAdjacencyCache() const
	{
		return VertexAdjCache;
	}

	TArray<int32> MakeVertexAdjacency(int32 NumVertices) const
	{
		TArray<int32> VertexAdj;
		VertexAdj.Init(-1, VertexIDToAdjIndex(NumVertices));
		for (const TPair<FIndex2i, int32>& EV : EdgeToVert)
		{
			FIndex2i AdjEdge(VertexIDToAdjIndex(EV.Key.A), VertexIDToAdjIndex(EV.Key.B));
			VertexAdj[AdjEdge.A] = AdjEdge.B;
		}
		return VertexAdj;
	}

	void DisableVertexAdjacency()
	{
		bUseAdjCache = false;
		VertexAdjCache.Empty();
	}

	bool HasVertexAdjacency() const
	{
		return bUseAdjCache;
	}

	bool HasCompleteVertexAdjacency(int32 NumVertices) const
	{
		return bUseAdjCache && VertexAdjCache.Num() == VertexIDToAdjIndex(NumVertices);
	}

	bool HasEdge(const FIndex2i& Edge) const
	{
		return EdgeToVert.Contains(Edge);
	}

	int32 NumTriangles() const
	{
		return EdgeToVert.Num() / 3;
	}

	int32 NumHalfEdges() const
	{
		return EdgeToVert.Num();
	}

	TArray<FIndex3i> GetTriangles() const
	{
		TArray<FIndex3i> Triangles;
		Triangles.Reserve(EdgeToVert.Num() / 3);
		for (const TPair<FIndex2i, int32>& EV : EdgeToVert)
		{
			if (!IsGhost(EV.Key, EV.Value) && EV.Value < EV.Key.A && EV.Value < EV.Key.B)
			{
				Triangles.Emplace(EV.Key.A, EV.Key.B, EV.Value);
			}
		}
		return Triangles;
	}

	void GetTrianglesAndAdjacency(TArray<FIndex3i>& Triangles, TArray<FIndex3i>& Adjacency) const
	{
		Triangles.Reset(EdgeToVert.Num() / 3);
		for (const TPair<FIndex2i, int32>& EV : EdgeToVert)
		{
			if (!IsGhost(EV.Key, EV.Value) && EV.Value < EV.Key.A && EV.Value < EV.Key.B)
			{
				if (!IsGhost(EV.Key, EV.Value) && EV.Value < EV.Key.A && EV.Value < EV.Key.B)
				{
					Triangles.Emplace(EV.Key.A, EV.Key.B, EV.Value);
				}
			}
		}

		// Because the EdgeToVert map doesn't know anything about our triangle indices,
		// it's easiest to build the Adjacency data from scratch with a new Map
		Adjacency.Init(FIndex3i::Invalid(), Triangles.Num());
		TMap<FIndex2i, int32> EdgeToTri;
		for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
		{
			const FIndex3i& Tri = Triangles[TriIdx];
			for (int32 Sub0 = 2, Sub1 = 0; Sub1 < 3; Sub0 = Sub1++)
			{
				FIndex2i RevEdge(Tri[Sub1], Tri[Sub0]);
				int32* NbrTriIdx = EdgeToTri.Find(RevEdge);
				if (NbrTriIdx)
				{
					const FIndex3i& AdjTri = Triangles[*NbrTriIdx];
					int32 AdjSub = AdjTri.IndexOf(RevEdge.A);
					Adjacency[*NbrTriIdx][AdjSub] = TriIdx;
					Adjacency[TriIdx][Sub0] = *NbrTriIdx;

					EdgeToTri.Remove(RevEdge); // only need the match once
				}
				else
				{
					EdgeToTri.Add(FIndex2i(RevEdge.B, RevEdge.A), TriIdx);
				}
			}
		}
	}

	// return the triangle indices rotated s.t. the smallest vertex is first (and the winding is unchanged)
	inline FIndex3i AsUniqueTriangle(const FIndex2i& Edge, int32 Vertex) const
	{
		if (Edge.A < Edge.B)
		{
			if (Vertex < Edge.A)
			{
				return FIndex3i(Vertex, Edge.A, Edge.B);
			}
			else
			{
				return FIndex3i(Edge.A, Edge.B, Vertex);
			}
		}
		else
		{
			if (Vertex < Edge.B)
			{
				return FIndex3i(Vertex, Edge.A, Edge.B);
			}
			else
			{
				return FIndex3i(Edge.B, Vertex, Edge.A);
			}
		}
	}

	bool GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> Edges, FDelaunay2::EFillMode FillMode) const
	{
		bool bWellDefinedResult = true;

		TMap<FIndex2i, int> EdgeDelta;
		EdgeDelta.Reserve(Edges.Num());
		for (const FIndex2i& Edge : Edges)
		{
			FIndex2i Unoriented = Edge;
			if (bTrackDuplicateVertices)
			{
				FixDuplicatesOnEdge(Unoriented);
			}
			int32 Sign = 1;
			if (Unoriented.A > Unoriented.B)
			{
				Unoriented.Swap();
				if (FillMode != FDelaunay2::EFillMode::Solid)
				{
					Sign = -1;
				}
			}
			int32& Delta = EdgeDelta.FindOrAdd(Unoriented, 0);
			Delta += Sign;
		}


		TMap<FIndex3i, int32> Windings;
		struct FWalk
		{
			FIndex2i Edge;
			int32 Winding;
		};
		TArray<FWalk> ToWalk;

		auto GetWindingChange = [&EdgeDelta](FIndex2i Edge)
		{
			int32 Winding = 0;
			int32 Sign = -1;
			if (Edge.A > Edge.B)
			{
				Sign = 1;
				Swap(Edge.A, Edge.B);
			}
			int32* Change = EdgeDelta.Find(Edge);
			if (!Change)
			{
				return 0;
			}
			else
			{
				return Sign * (*Change);
			}
		};
		auto WalkEdge = [FillMode, &EdgeDelta, &ToWalk, &GetWindingChange](const FIndex2i& Edge, int32 BeforeWinding)
		{
			if (FillMode == FDelaunay2::EFillMode::Solid)
			{
				FIndex2i Unoriented = Edge;
				if (Unoriented.A > Unoriented.B)
				{
					Swap(Unoriented.A, Unoriented.B);
				}
				if (!EdgeDelta.Contains(Unoriented)) // in solid fill, only check if the edge is present
				{
					ToWalk.Add({ FIndex2i(Edge.B, Edge.A), 1 });
				}
			}
			else
			{
				int32 Winding = BeforeWinding + GetWindingChange(Edge);
				ToWalk.Add({ FIndex2i(Edge.B, Edge.A), Winding });
			}
		};

		// First walk all the ghost triangles, through their non-ghost edge
		int32 GhostTriCount = 0;
		for (const TPair<FIndex2i, int32>& EdgeV : EdgeToVert)
		{
			if (EdgeV.Value == GhostIndex)
			{
				WalkEdge(EdgeV.Key, 0);
				GhostTriCount++;
			}
		}

		int32 PickIdx = 0;
		while (ToWalk.Num())
		{
			// Traverse the walk list in something similar to a breadth-first traversal
			// Non-closed shapes in the input edges will give different results depending on visit order
			//  and depth-first traversal would be more sensitive to starting triangle in that case
			PickIdx = (PickIdx + 1) % ToWalk.Num();
			const FWalk Walk = ToWalk[PickIdx];
			ToWalk.RemoveAtSwap(PickIdx, EAllowShrinking::No);
			int32 Vert = EdgeToVert[Walk.Edge];
			FIndex3i UniqueTri = AsUniqueTriangle(Walk.Edge, Vert);
			if (UniqueTri.A < 0) // it's a ghost
			{
				continue;
			}
			int32* Winding = Windings.Find(UniqueTri);
			if (Winding)
			{
				bWellDefinedResult = bWellDefinedResult && (*Winding == Walk.Winding);
				continue;
			}
			Windings.Add(UniqueTri, Walk.Winding);
			WalkEdge(FIndex2i(Walk.Edge.B, Vert), Walk.Winding);
			WalkEdge(FIndex2i(Vert, Walk.Edge.A), Walk.Winding);
		}

		TrianglesOut.Reset();
		if (FillMode == FDelaunay2::EFillMode::Solid)
		{
			TrianglesOut.Reserve(EdgeToVert.Num() / 3 - GhostTriCount - Windings.Num());
			// everything we *didn't* walk over is included
			EnumerateTriangles([&Windings, &TrianglesOut](const FIndex2i& Edge, int32 Vertex) -> bool
				{
					FIndex3i Triangle(Vertex, Edge.A, Edge.B);
					if (!Windings.Contains(Triangle))
					{
						TrianglesOut.Add(Triangle);
					}
					return true;
				}, true /*bSkipGhosts*/);
		}
		else
		{
			TrianglesOut.Reserve(Windings.Num());
			for (const TPair<FIndex3i, int32>& TriWinding : Windings)
			{
				if ((FillMode == FDelaunay2::EFillMode::NegativeWinding	&& TriWinding.Value < 0) ||
					(FillMode == FDelaunay2::EFillMode::NonZeroWinding	&& TriWinding.Value != 0) ||
					(FillMode == FDelaunay2::EFillMode::PositiveWinding && TriWinding.Value > 0) ||
					(FillMode == FDelaunay2::EFillMode::OddWinding		&& (TriWinding.Value % 2) != 0))
				{
					TrianglesOut.Add(TriWinding.Key);
				}
			}
		}
		return bWellDefinedResult;
	}

	template<typename RealType>
	bool GetFilledTrianglesGeneralizedWinding(TArray<FIndex3i>& TrianglesOut, TArrayView<const TVector2<RealType>> Vertices, TArrayView<const FIndex2i> BoundaryEdges, FDelaunay2::EFillMode FillMode) const
	{
		if (FillMode == FDelaunay2::EFillMode::Solid)
		{
			UE_LOGF(LogGeometry, Warning, "Generalized Winding-based fill does not support the Solid fill mode -- instead using the exact fill path.");
			GetFilledTriangles(TrianglesOut, BoundaryEdges, FillMode);
			return false;
		}
		
		TrianglesOut = GetTriangles();
		TArray<bool> FillTri;
		FillTri.SetNumUninitialized(TrianglesOut.Num());
		ParallelFor(TrianglesOut.Num(), [&](int32 TriIdx)
			{
				const FIndex3i Tri = TrianglesOut[TriIdx];
				const TVector2<RealType> Centroid = (Vertices[Tri.A] + Vertices[Tri.B] + Vertices[Tri.C]) / (RealType)3;
				RealType ApproxWinding = (RealType)0;
				for (const FIndex2i& Edge : BoundaryEdges)
				{
					TVector2<RealType> A = Vertices[Edge.A] - Centroid;
					TVector2<RealType> B = Vertices[Edge.B] - Centroid;
					ApproxWinding += TMathUtil<RealType>::Atan2(A.X * B.Y - A.Y * B.X, A.X * B.X + A.Y * B.Y);
				}
				const int32 Winding = FMath::RoundToInt32(ApproxWinding * TMathUtil<RealType>::InvTwoPi);
				FillTri[TriIdx] = (
					(FillMode == FDelaunay2::EFillMode::NegativeWinding && Winding < 0) ||
					(FillMode == FDelaunay2::EFillMode::NonZeroWinding && Winding != 0) ||
					(FillMode == FDelaunay2::EFillMode::PositiveWinding && Winding > 0) ||
					(FillMode == FDelaunay2::EFillMode::OddWinding && (Winding % 2) != 0)
				);
			});

		for (int32 TriIdx = 0; TriIdx < TrianglesOut.Num(); ++TriIdx)
		{
			if (!FillTri[TriIdx])
			{
				FillTri.RemoveAtSwap(TriIdx, EAllowShrinking::No);
				TrianglesOut.RemoveAtSwap(TriIdx, EAllowShrinking::No);
				TriIdx--; // re-consider the index w/ the newly swapped element
			}
		}
		return true;
	}

	bool GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> BoundaryEdges, TArrayView<const FIndex2i> HoleEdges) const
	{
		TSet<FIndex2i> BoundarySet, HoleSet;
		auto AddToSet = [this](TArrayView<const FIndex2i> Edges, TSet<FIndex2i>& Set)
		{
			for (FIndex2i Edge : Edges)
			{
				if (bTrackDuplicateVertices)
				{
					FixDuplicatesOnEdge(Edge);
				}
				Edge.Sort();
				Set.Add(Edge);
			}
		};
		AddToSet(BoundaryEdges, BoundarySet);
		AddToSet(HoleEdges, HoleSet);

		TMap<FIndex3i, bool> InBoundary;
		struct FWalk
		{
			FIndex2i Edge;
		};
		TArray<FWalk> ToWalkOuter;
		TArray<FWalk> ToWalkInner;

		auto WalkOuter = [&ToWalkOuter, &ToWalkInner, &HoleEdges, &BoundaryEdges](const FIndex2i& Edge)
		{
			FIndex2i Unoriented = Edge;
			Unoriented.Sort();
			if (HoleEdges.Contains(Unoriented))
			{
				return;
			}
			FIndex2i ReverseEdge(Edge.B, Edge.A);
			if (BoundaryEdges.Contains(Unoriented))
			{
				ToWalkInner.Add({ ReverseEdge });
			}
			else
			{
				ToWalkOuter.Add({ ReverseEdge });
			}
		};

		auto WalkInner = [&ToWalkInner, &HoleEdges](const FIndex2i& Edge)
		{
			FIndex2i Unoriented = Edge;
			Unoriented.Sort();
			if (HoleEdges.Contains(Unoriented))
			{
				return;
			}
			FIndex2i ReverseEdge(Edge.B, Edge.A);
			ToWalkInner.Add({ ReverseEdge });
		};

		// First walk all the ghost triangles, through their non-ghost edge
		int32 GhostTriCount = 0;
		for (const TPair<FIndex2i, int32>& EdgeV : EdgeToVert)
		{
			if (EdgeV.Value == GhostIndex)
			{
				WalkOuter(EdgeV.Key);
				GhostTriCount++;
			}
		}

		// Next walk up to the outer boundary
		while (!ToWalkOuter.IsEmpty())
		{
			const FWalk Walk = ToWalkOuter.Pop(EAllowShrinking::No);
			int32 Vert = EdgeToVert[Walk.Edge];
			FIndex3i UniqueTri = AsUniqueTriangle(Walk.Edge, Vert);
			if (UniqueTri.A < 0) // it's a ghost
			{
				continue;
			}
			bool* InBoundaryPtr = InBoundary.Find(UniqueTri);
			if (InBoundaryPtr)
			{
				continue;
			}
			InBoundary.Add(UniqueTri, false);
			WalkOuter(FIndex2i(Walk.Edge.B, Vert));
			WalkOuter(FIndex2i(Vert, Walk.Edge.A));
		}

		// Finally walk up to the inner hole boundary
		int32 SolidTriCount = 0;
		while (!ToWalkInner.IsEmpty())
		{
			const FWalk Walk = ToWalkInner.Pop(EAllowShrinking::No);
			int32 Vert = EdgeToVert[Walk.Edge];
			FIndex3i UniqueTri = AsUniqueTriangle(Walk.Edge, Vert);
			if (UniqueTri.A < 0) // it's a ghost
			{
				continue;
			}
			bool* InBoundaryPtr = InBoundary.Find(UniqueTri);
			if (InBoundaryPtr)
			{
				continue;
			}
			InBoundary.Add(UniqueTri, true);
			WalkInner(FIndex2i(Walk.Edge.B, Vert));
			WalkInner(FIndex2i(Vert, Walk.Edge.A));
		}

		TrianglesOut.Reset();
		
		TrianglesOut.Reserve(InBoundary.Num());
		EnumerateTriangles([&InBoundary, &TrianglesOut](const FIndex2i& Edge, int32 Vertex) -> bool
			{
				FIndex3i Triangle(Vertex, Edge.A, Edge.B);
				bool* InBoundaryPtr = InBoundary.Find(Triangle);
				if (InBoundaryPtr && *InBoundaryPtr)
				{
					TrianglesOut.Add(Triangle);
				}
				return true;
			}, true /*bSkipGhosts*/);
		
		return true;
	}

	static bool IsGhost(const FIndex2i& Edge, int32 Vertex)
	{
		return Edge.A == GhostIndex || Edge.B == GhostIndex || Vertex == GhostIndex;
	}

	static bool IsGhost(const FIndex2i& Edge)
	{
		return Edge.A == GhostIndex || Edge.B == GhostIndex;
	}

	void AddTriangle(const FIndex3i& Tri)
	{
		checkSlow(Tri.A != Tri.B && Tri.A != Tri.C && Tri.B != Tri.C);
		EdgeToVert.Add(FIndex2i(Tri.A, Tri.B), Tri.C);
		EdgeToVert.Add(FIndex2i(Tri.B, Tri.C), Tri.A);
		EdgeToVert.Add(FIndex2i(Tri.C, Tri.A), Tri.B);
		if (bUseAdjCache)
		{
			FIndex3i TriAdjI(VertexIDToAdjIndex(Tri.A), VertexIDToAdjIndex(Tri.B), VertexIDToAdjIndex(Tri.C));
			VertexAdjCache[TriAdjI.A] = TriAdjI.B;
			VertexAdjCache[TriAdjI.B] = TriAdjI.C;
			VertexAdjCache[TriAdjI.C] = TriAdjI.A;
		}
	}

	// Create a first initial triangle that is surround by ghost triangles
	void InitWithGhosts(const FIndex3i& Tri)
	{
		AddTriangle(Tri);
		AddTriangle(FIndex3i(Tri.B, Tri.A, GhostIndex));
		AddTriangle(FIndex3i(Tri.C, Tri.B, GhostIndex));
		AddTriangle(FIndex3i(Tri.A, Tri.C, GhostIndex));
	}

	void DeleteTriangle(const FIndex3i& Tri)
	{
		EdgeToVert.Remove(FIndex2i(Tri.A, Tri.B));
		EdgeToVert.Remove(FIndex2i(Tri.B, Tri.C));
		EdgeToVert.Remove(FIndex2i(Tri.C, Tri.A));
		if (bUseAdjCache)
		{
			// clear any adj info if it was set in the cache
			FIndex3i TriAdjI(VertexIDToAdjIndex(Tri.A), VertexIDToAdjIndex(Tri.B), VertexIDToAdjIndex(Tri.C));
			const int32 InvalidAdj = VertexIDToAdjIndex(InvalidIndex);
			if (VertexAdjCache[TriAdjI.A] == TriAdjI.B)
			{
				VertexAdjCache[TriAdjI.A] = InvalidAdj;
			}
			if (VertexAdjCache[TriAdjI.B] == TriAdjI.C)
			{
				VertexAdjCache[TriAdjI.B] = InvalidAdj;
			}
			if (VertexAdjCache[TriAdjI.C] == TriAdjI.A)
			{
				VertexAdjCache[TriAdjI.C] = InvalidAdj;
			}
		}
	}

	int32 GetVertex(const FIndex2i& Edge) const
	{
		const int32* V = EdgeToVert.Find(Edge);
		if (!V)
		{
			return InvalidIndex;
		}
		return *V;
	}

	void MarkDuplicateVertex(int32 Orig, int32 Duplicate)
	{
		checkSlow(bTrackDuplicateVertices);
		DuplicateVertices.Add(Duplicate, Orig);
	}

	bool HasDuplicateTracking() const
	{
		return bTrackDuplicateVertices;
	}

	// called during triangulation when a duplicate vertex is found
	void SetHasDuplicates()
	{
		bHasDuplicates = true;
	}

	// @return whether duplicate vertices were detected during triangulation
	bool HasDuplicates() const
	{
		return bHasDuplicates;
	}

	void FixDuplicatesOnEdge(FIndex2i& Edge) const
	{
		const int32* OrigA = DuplicateVertices.Find(Edge.A);
		if (OrigA)
		{
			Edge.A = *OrigA;
		}
		const int32* OrigB = DuplicateVertices.Find(Edge.B);
		if (OrigB)
		{
			Edge.B = *OrigB;
		}
	}

	int32 RemapIfDuplicate(int32 Index) const
	{
		const int32* Remap = DuplicateVertices.Find(Index);
		return Remap ? *Remap : Index;
	}

	// Get any edge BC opposite vertex A, such that triangle ABC is in the mesh (or return InvalidIndex edge if no such edge is present)
	// Before calling this frequently, consider calling EnableVertexAdjacency()
	// Not thread-safe if cache is enabled, because it may try to update the cache
	FIndex2i GetEdge(int32 Vertex)
	{
		if (bUseAdjCache)
		{
			int32 AdjVertex = GetCachedAdjVertex(VertexAdjCache, Vertex);
			if (AdjVertex != InvalidIndex)
			{
				int32 LastVertex = GetVertex(FIndex2i(Vertex, AdjVertex));
				return FIndex2i(AdjVertex, LastVertex);
			}
		}
		for (const TPair<FIndex2i, int32>& EV : EdgeToVert)
		{
			if (bUseAdjCache)
			{
				UpdateAdjCache(EV.Key);
			}
			if (Vertex == EV.Key.A)
			{
				return FIndex2i(EV.Key.B, EV.Value);
			}
		}
		return FIndex2i(InvalidIndex, InvalidIndex);
	}

	// Same as GetEdge but only reads from a vertex adjacency cache; returns InvalidIndex if not in cache
	// Thread-safe because it will never try to update the cache.
	FIndex2i GetEdgeFromCache(const TArray<int32>& Cache, int32 Vertex) const
	{
		int32 AdjVertex = GetCachedAdjVertex(Cache, Vertex);
		if (AdjVertex == InvalidIndex)
		{
			return FIndex2i(InvalidIndex, InvalidIndex);
		}
		int32 LastVertex = GetVertex(FIndex2i(Vertex, AdjVertex));
		return FIndex2i(AdjVertex, LastVertex);
	}
	
	// Call a function on every oriented edge (+ next vertex) on the mesh
	// (note the number of edges visited will be 3x the number of triangles)
	// VisitFunctionType is expected to take (FIndex2i Edge, int32 Vertex) and return bool
	// Returning false from VisitFn will end the enumeration early
	template<typename VisitFunctionType>
	void EnumerateOrientedEdges(VisitFunctionType VisitFn) const
	{
		for (const TPair<FIndex2i, int32>& EdgeVert : EdgeToVert)
		{
			if (!VisitFn(EdgeVert.Key, EdgeVert.Value))
			{
				break;
			}
		}
	}

	// Similar to EnumerateOrientedEdges but only visits each triangle once, instead of 3x
	// Calls VisitFn(FIndex2i Edge, int32 Vertex) with Vertex always smaller than the Edge.A and Edge.B
	// Returning false from VisitFn will end the enumeration early
	// @param bSkipGhosts If true, skip ghost triangles (triangles connected to the ghost vertex)
	template<typename VisitFunctionType>
	void EnumerateTriangles(VisitFunctionType VisitFn, bool bSkipGhosts = false) const
	{
		for (const TPair<FIndex2i, int32>& EdgeVert : EdgeToVert)
		{
			// to visit triangles only once, only visit when the vertex ID is smaller than the edge IDs
			// since the vertex ID is the smallest ID, it is also the only one we need to check vs the GhostIndex (if we're skipping ghosts)
			if (EdgeVert.Key.A < EdgeVert.Value || EdgeVert.Key.B < EdgeVert.Value || (bSkipGhosts && EdgeVert.Value == GhostIndex))
			{
				continue;
			}
			if (!VisitFn(EdgeVert.Key, EdgeVert.Value))
			{
				break;
			}
		}
	}

protected:
	TMap<FIndex2i, int32> EdgeToVert;
	
	// Optional cache of a single vertex in the 1-ring of each vertex
	// Makes GetEdge() constant time (as long as the cache hits) instead of O(#Edges), at the cost of additional storage and bookkeeping.
	TArray<int32> VertexAdjCache;
	bool bUseAdjCache = false;

	bool bHasDuplicates = false;
	bool bTrackDuplicateVertices = false;
	TMap<int32, int32> DuplicateVertices;

	static inline int32 VertexIDToAdjIndex(int32 VertexID)
	{
		// Offset by 1 so that GhostIndex enters slot 0
		return VertexID + 1;
	}
	static inline int32 AdjIndexToVertexID(int32 AdjIndex)
	{
		return AdjIndex - 1;
	}
	inline int32 GetCachedAdjVertex(const TArray<int32>& Cache, int32 VertexID) const
	{
		int32 AdjIndex = VertexIDToAdjIndex(VertexID);
		return AdjIndexToVertexID(Cache[AdjIndex]);
	}
	inline void UpdateAdjCache(FIndex2i Edge)
	{
		FIndex2i AdjEdge(VertexIDToAdjIndex(Edge.A), VertexIDToAdjIndex(Edge.B));
		VertexAdjCache[AdjEdge.A] = AdjEdge.B;
	}
};

namespace DelaunayInternal
{
	// Helper to create a permutation array
	TArray<int32> GetShuffledOrder(FRandomStream& Random, int32 Num, int32 StartIn = -1, int32 EndIn = -1)
	{
		TArray<int32> Order;
		Order.Reserve(Num);
		for (int32 OrderIdx = 0; OrderIdx < Num; OrderIdx++)
		{
			Order.Add(OrderIdx);
		}
		int32 Start = StartIn >= 0 ? StartIn : 0;
		int32 End = EndIn >= 0 ? EndIn : Num - 1;
		for (int32 OrderIdx = EndIn; OrderIdx > Start; OrderIdx--)
		{
			int32 SwapIdx = Start + Random.RandHelper(OrderIdx - 1 - Start);
			Swap(Order[SwapIdx], Order[OrderIdx]);
		}
		return Order;
	}

	// @return true if Vertex is inside the circumcircle of Tri
	// For ghost triangles, this is defined as being on the one solid edge of the triangle or inside that edge's half-space
	template<typename RealType>
	bool InTriCircle(TArrayView<const TVector2<RealType>> Vertices, FIndex3i Tri, int32 Vertex)
	{
		int32 GhostSub = Tri.IndexOf(FDelaunay2Connectivity::GhostIndex);
		if (GhostSub == -1)
		{
			return ExactPredicates::InCircle2<RealType>(Vertices[Tri.A], Vertices[Tri.B], Vertices[Tri.C], Vertices[Vertex]) > 0;
		}
		FIndex3i GhostFirst = Tri.GetCycled(FDelaunay2Connectivity::GhostIndex);
		RealType Pred = ExactPredicates::Orient2<RealType>(Vertices[GhostFirst.B], Vertices[GhostFirst.C], Vertices[Vertex]);
		if (Pred > 0)
		{
			return true;
		}
		if (Pred < 0)
		{
			return false;
		}
		// Pred == 0 case: need to check if Vertex is *on* the edge
		if (Vertices[GhostFirst.B].X != Vertices[GhostFirst.C].X)
		{
			TInterval1<RealType> XRange = TInterval1<RealType>::MakeFromUnordered(Vertices[GhostFirst.B].X, Vertices[GhostFirst.C].X);
			return XRange.Contains(Vertices[Vertex].X);
		}
		else
		{
			TInterval1<RealType> YRange = TInterval1<RealType>::MakeFromUnordered(Vertices[GhostFirst.B].Y, Vertices[GhostFirst.C].Y);
			return YRange.Contains(Vertices[Vertex].Y);
		}
	}

	// @return triangle containing Vertex
	template<typename RealType>
	FIndex3i WalkToContainingTri(FRandomStream& Random, FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, FIndex3i StartTri, int32 Vertex, bool bAssumeDelaunay, int32& IsDuplicateOfOut)
	{
		IsDuplicateOfOut = FDelaunay2Connectivity::InvalidIndex;

		constexpr int32 GhostV = FDelaunay2Connectivity::GhostIndex; // shorter name

		auto ChooseCross = [&Random, bAssumeDelaunay, &Vertices, Vertex, GhostV](const FIndex3i& Tri, bool bSkipFirst) -> int32
		{
			auto CrossesEdge = [&Vertices, Vertex, GhostV](int32 A, int32 B, bool bOnGhostTri) -> bool
			{
				if (!bOnGhostTri || (A != GhostV && B != GhostV))
				{
					RealType Orient = ExactPredicates::Orient2<RealType>(Vertices[A], Vertices[B], Vertices[Vertex]);
					// Note: could refine to quickly say we're on the triangle in the ghost + orient==0 case, if we're exactly on the edge, but this only saves walking one edge
					if (Orient < 0 || (bOnGhostTri && Orient == 0))
					{
						return true;
					}
				}
				return false;
			};

			int32 Choose[2]{ -1, -1 };
			int32 Chosen = 0;
			bool bIsGhost = Tri.Contains(GhostV);
			int32 NextSub[3]{ 1, 2, 0 };
			for (int32 EdgeSub = (int32)bSkipFirst; EdgeSub < 3; EdgeSub++)
			{
				if (CrossesEdge(Tri[EdgeSub], Tri[NextSub[EdgeSub]], bIsGhost))
				{
					// On a Delaunay mesh we can always walk across the first edge that has the target vertex on the other side of it
					if (bAssumeDelaunay)
					{
						return EdgeSub;
					}
					// If the mesh is not Delaunay, randomly choose between edges that have the target vertex on the other side; this avoids a possible infinite cycle
					else
					{
						Choose[Chosen++] = EdgeSub;
					}
				}
			}
			if (Chosen == 0)
			{
				return -1; // we're on this tri
			}
			else if (Chosen == 1)
			{
				return Choose[0];
			}
			else // (Chosen == 2)
			{
				return Choose[Random.RandHelper(2)];
			}
		};
		
		FIndex3i WalkTri = StartTri;
		int32 Cross = ChooseCross(WalkTri, false);
		int32 NumSteps = 0;
		int32 NextIdx[3]{ 1,2,0 };
		while (Cross != -1)
		{
			// if !bAssumeDelaunay, the random edge walk could choose poorly enough for any amount of steps to occur, but it should not happen in practice ...
			// if this ensure() triggers it is more likely that some other problem has caused an infinite loop
			if (!ensure(NumSteps++ < Connectivity.NumTriangles() * 100))
			{
				return FIndex3i(FDelaunay2Connectivity::InvalidIndex, FDelaunay2Connectivity::InvalidIndex, FDelaunay2Connectivity::InvalidIndex);
			}

			FIndex2i OppEdge = FIndex2i(WalkTri[NextIdx[Cross]], WalkTri[Cross]);
			int32 OppVert = Connectivity.GetVertex(OppEdge);
			checkSlow(OppVert != FDelaunay2Connectivity::InvalidIndex);
			WalkTri = FIndex3i(OppEdge.A, OppEdge.B, OppVert);
			Cross = ChooseCross(WalkTri, true);
		}

		if (WalkTri.A >= 0 && Vertices[Vertex] == Vertices[WalkTri.A])
		{
			IsDuplicateOfOut = WalkTri.A;
		}
		else if (WalkTri.B >= 0 && Vertices[Vertex] == Vertices[WalkTri.B])
		{
			IsDuplicateOfOut = WalkTri.B;
		}
		else if (WalkTri.C >= 0 && Vertices[Vertex] == Vertices[WalkTri.C])
		{
			IsDuplicateOfOut = WalkTri.C;
		}

		return WalkTri;
	}

	// Insert Vertex into the triangulation; it must already be on the OnTri triangle
	// Uses Bowyer-Watson algorithm:
	//  1. Delete all the connected triangles whose circumcircles contain the vertex
	//  2. Make a fan of triangles from the new vertex out to the border of the deletions
	// @return one of the inserted triangles containing Vertex
	template<typename RealType>
	FIndex3i Insert(FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, FIndex3i OnTri, int32 ToInsertV)
	{
		// depth first search + deletion of triangles whose circles contain vertex
		TArray<FIndex2i> ToConsider;
		auto AddIfValid = [&ToConsider, &Connectivity](FIndex2i Edge)
		{
			if (Connectivity.HasEdge(Edge))
			{
				ToConsider.Add(Edge);
			}
		};

		auto DeleteTri = [&AddIfValid, &Connectivity](FIndex3i Tri)
		{
			Connectivity.DeleteTriangle(Tri);

			AddIfValid(FIndex2i(Tri.B, Tri.A));
			AddIfValid(FIndex2i(Tri.C, Tri.B));
			AddIfValid(FIndex2i(Tri.A, Tri.C));
		};
		DeleteTri(OnTri);

		TArray<FIndex2i> Border;
		while (!ToConsider.IsEmpty())
		{
			FIndex2i Edge = ToConsider.Pop(EAllowShrinking::No);
			int32 TriV = Connectivity.GetVertex(Edge);
			if (TriV != FDelaunay2Connectivity::InvalidIndex) // tri still existed (wasn't deleted by earlier traversal)
			{
				FIndex3i ConsiderTri = FIndex3i(Edge.A, Edge.B, TriV);
				if (InTriCircle(Vertices, ConsiderTri, ToInsertV))
				{
					DeleteTri(ConsiderTri);
				}
				else
				{
					Border.Add(Edge);
				}
			}
		}
		
		for (FIndex2i BorderEdge : Border)
		{
			Connectivity.AddTriangle(FIndex3i(BorderEdge.B, BorderEdge.A, ToInsertV));
		}

		return Border.Num() > 0 ? FIndex3i(Border[0].B, Border[0].A, ToInsertV) : FIndex3i::Invalid();
	}

	template<typename RealType>
	TArray<TArray<TVector2<RealType>>> GetVoronoiCells(const FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, bool bIncludeBoundary, TAxisAlignedBox2<RealType> BoundsClip, RealType ExpandBounds)
	{
		// get or create vertex adjacency data; we will use it heavily here
		TArray<int32> ComputedVertexAdj;
		const TArray<int32>* UseVertexAdj;
		int32 NumVertices = Vertices.Num();
		if (Connectivity.HasCompleteVertexAdjacency(NumVertices))
		{
			UseVertexAdj = &Connectivity.GetVertexAdjacencyCache();
		}
		else
		{
			ComputedVertexAdj = Connectivity.MakeVertexAdjacency(NumVertices);
			UseVertexAdj = &ComputedVertexAdj;
		}

		bool bDoClip = !BoundsClip.IsEmpty();
		if (bDoClip)
		{
			BoundsClip.Expand(ExpandBounds);
		}

		TAxisAlignedBox2<RealType> Bounds;

		if (bIncludeBoundary)
		{
			for (const TVector2<RealType>& Vert : Vertices)
			{
				Bounds.Contain(Vert);
			}

			// Note: We're recomputing each circumcenter 4 times (once here, and once per triangle vertex),
			// but they're cheap to compute, so still probably not worth caching
			Connectivity.EnumerateTriangles([&](const FIndex2i& Edge, int32 Vertex)
			{
				Bounds.Contain(VectorUtil::Circumcenter(Vertices[Vertex], Vertices[Edge.A], Vertices[Edge.B]));
				return true;
			}, true);
			Bounds.Expand(FMath::Max((RealType)UE_SMALL_NUMBER, ExpandBounds));
			Bounds.Contain(BoundsClip);
		}

		// Helper finds ray-boundary intersection, and an index indicating which side of the boundary was crossed
		auto GetBoundaryCrossing = [&Vertices, &Bounds](int32 A, int32 B, int32& OutCrossIdx)
		{
			// Cast the ray from the midpoint of a Delaunay edge, in the perpendicular direction
			TVector2<RealType> Mid = (Vertices[A] + Vertices[B]) * .5;
			TVector2<RealType> Edge = (Vertices[B] - Vertices[A]);
			TVector2<RealType> EdgePerp(-Edge.Y, Edge.X);
			RealType T = TMathUtilConstants<RealType>::MaxReal;
			OutCrossIdx = -1;
			if (EdgePerp.X < 0)
			{
				T = (Bounds.Min.X - Mid.X) / EdgePerp.X;
				OutCrossIdx = 0;
			}
			else if (EdgePerp.X > 0)
			{
				T = (Bounds.Max.X - Mid.X) / EdgePerp.X;
				OutCrossIdx = 2;
			}
			if (EdgePerp.Y < 0)
			{
				RealType TCand = (Bounds.Min.Y - Mid.Y) / EdgePerp.Y;
				if (TCand < T)
				{
					T = TCand;
					OutCrossIdx = 1;
				}
			}
			else if (EdgePerp.Y > 0)
			{
				RealType TCand = (Bounds.Max.Y - Mid.Y) / EdgePerp.Y;
				if (TCand < T)
				{
					T = TCand;
					OutCrossIdx = 3;
				}
			}
			// Delaunay triangulation will not include duplicate points in the triangulation, so OutCrossIdx should never stay -1 (corresponds to EdgePerp of 0,0)
			checkSlow(OutCrossIdx != -1);
			return Mid + T * EdgePerp;
		};

		const TVector2<RealType> Corners[4]{Bounds.GetCorner(0), Bounds.GetCorner(1), Bounds.GetCorner(2), Bounds.GetCorner(3)};

		TArray<TArray<TVector2<RealType>>> Cells;
		Cells.SetNum(Vertices.Num());
		ParallelFor(Vertices.Num(), [&](int32 VertIdx)
		{
			TArray<TVector2<RealType>>& Polygon = Cells[VertIdx];
			FIndex2i CacheEdge = Connectivity.GetEdgeFromCache(*UseVertexAdj, VertIdx);
			if (CacheEdge.A == FDelaunay2Connectivity::InvalidIndex)
			{
				return;
			}
			if (!bIncludeBoundary)
			{
				if (Connectivity.HasEdge(FIndex2i(VertIdx, FDelaunay2Connectivity::GhostIndex)))
				{
					return;
				}
			}
			else
			{
				// Make sure that our first triangle is not a ghost triangle, to simplify subsequent processing
				while (CacheEdge.Contains(FDelaunay2Connectivity::GhostIndex))
				{
					int32 NextV = Connectivity.GetVertex(FIndex2i(VertIdx, CacheEdge.B));
					CacheEdge = FIndex2i(CacheEdge.B, NextV);
				}
			}

			FIndex2i WalkEdge = FIndex2i(VertIdx, CacheEdge.A);
			int32 NextVert = CacheEdge.B;
			int32 InitialBoundaryCrossIdx = -1;
			do
			{
				FIndex3i Tri = Connectivity.AsUniqueTriangle(WalkEdge, NextVert);
				if (Tri.A == -1) // passing through a ghost tri
				{
					if (NextVert == -1)
					{
						// Entering the infinite part of the cell: Add where it exits the bounds, and note the side of the bounding box
						TVector2<RealType> CrossPt = GetBoundaryCrossing(VertIdx, WalkEdge.B, InitialBoundaryCrossIdx);
						Polygon.Add(CrossPt);
					}
					else // WalkEdge.B == -1
					{
						// Exiting the infinite part of the cell: Add the corners between the exiting and entering bounding box sides
						// and then add the crossing vertex
						int32 EndBoundaryCrossIdx;
						TVector2<RealType> CrossPt = GetBoundaryCrossing(NextVert, VertIdx, EndBoundaryCrossIdx);
						// Note: both CrossIdx values should always be > -1 but double-check anyway
						if (InitialBoundaryCrossIdx > -1 && EndBoundaryCrossIdx > -1)
						{
							for (int32 BdryIdx = InitialBoundaryCrossIdx; BdryIdx != EndBoundaryCrossIdx; BdryIdx = (BdryIdx + 1) % 4)
							{
								Polygon.Add(Corners[BdryIdx]);
							}
						}
						Polygon.Add(CrossPt);
					}
				}
				else // a regular tri; the cell vertex is at the circumcenter
				{
					TVector2<RealType> CenterPt = VectorUtil::Circumcenter(Vertices[Tri[0]], Vertices[Tri[1]], Vertices[Tri[2]]);
					Polygon.Add(CenterPt);
				}

				WalkEdge = FIndex2i(VertIdx, NextVert);
				NextVert = Connectivity.GetVertex(WalkEdge);
			} while (NextVert != CacheEdge.B);

			if (bDoClip)
			{
				CurveUtil::ClipConvexToBounds<RealType, TVector2<RealType>>(Polygon, BoundsClip.Min, BoundsClip.Max);
			}
		}); // end ParallelFor

		return Cells;
	}


	// Handle special case of voronoi diagram for collinear input, where Delaunay triangulation cannot be computed
	template<typename RealType>
	TArray<TArray<TVector2<RealType>>> GetVoronoiCells_Collinear(TArrayView<const TVector2<RealType>> Vertices, bool bIncludeBoundary, TAxisAlignedBox2<RealType> BoundsClip, RealType ExpandBounds)
	{
		TArray<TArray<TVector2<RealType>>> Result;

		// In a voronoi diagram of collinear input points, all cells will touch the boundary
		if (!bIncludeBoundary)
		{
			return Result;
		}

		if (!ensure(!Vertices.IsEmpty()))
		{
			return Result;
		}

		TVector2<RealType> V0 = Vertices[0];
		int32 FarIdx = INDEX_NONE;
		RealType FarDistSq = (RealType)0;
		for (int32 Idx = 1; Idx < Vertices.Num(); ++Idx)
		{ 
			RealType DSq = TVector2<RealType>::DistSquared(Vertices[Idx], V0);
			if (DSq > FarDistSq)
			{
				FarIdx = Idx;
				FarDistSq = DSq;
			}
		}


		TAxisAlignedBox2<RealType> Bounds;
		for (const TVector2<RealType>& Vert : Vertices)
		{
			Bounds.Contain(Vert);
		}
		Bounds.Expand(FMath::Max((RealType)UE_SMALL_NUMBER, ExpandBounds));
		if (!BoundsClip.IsEmpty())
		{
			BoundsClip.Expand(ExpandBounds);
			Bounds.Contain(BoundsClip);
		}
		else
		{
			BoundsClip = Bounds;
		}

		Result.SetNum(Vertices.Num());

		// All points had zero squared distance from the first point; treat this as a single-point 'diagram' and just generate a single cell of the entire bounds
		if (FarIdx == INDEX_NONE)
		{
			TArray<TVector2<RealType>>& BoundsPoly = Result[0];
			BoundsPoly.Add(Bounds.GetCorner(0));
			BoundsPoly.Add(Bounds.GetCorner(1));
			BoundsPoly.Add(Bounds.GetCorner(2));
			BoundsPoly.Add(Bounds.GetCorner(3));
		}
		else
		{
			// Safe distance that will always move us outside the bounding box from a point inside the bounding box
			RealType ToOutside = Bounds.DiagonalLength();

			TVector2<RealType> Dir = Vertices[FarIdx] - V0;
			Dir.Normalize();
			TVector2<RealType> PerpToOutside = ToOutside * PerpCW(Dir);
			TArray<int32> VertexOrder;
			VertexOrder.SetNumUninitialized(Vertices.Num());
			for (int32 Idx = 0; Idx < Vertices.Num(); ++Idx)
			{
				VertexOrder[Idx] = Idx;
			}
			VertexOrder.Sort([Dir, &Vertices](const int32& A, const int32& B)
			{
				return Vertices[A].Dot(Dir) < Vertices[B].Dot(Dir);
			});
			int32 PrevIdx = INDEX_NONE, NextIdx = INDEX_NONE;
			for (int32 Idx = 0; Idx < VertexOrder.Num(); PrevIdx = Idx, Idx = NextIdx)
			{
				for (NextIdx = Idx + 1; NextIdx < VertexOrder.Num(); ++NextIdx)
				{
					if (Vertices[VertexOrder[Idx]] != Vertices[VertexOrder[NextIdx]])
					{
						break;
					}
				}
				TVector2<RealType> CurV = Vertices[VertexOrder[Idx]];
				TVector2<RealType> PrevMid, NextMid;
				if (PrevIdx != INDEX_NONE)
				{
					PrevMid = (Vertices[VertexOrder[PrevIdx]] + CurV) * (RealType).5;
				}
				else
				{
					PrevMid = CurV - Dir * ToOutside;
				}
				if (NextIdx < VertexOrder.Num())
				{
					NextMid = (Vertices[VertexOrder[NextIdx]] + CurV) * (RealType).5;
				}
				else
				{
					NextMid = CurV + Dir * ToOutside;
				}

				TArray<TVector2<RealType>>& Polygon = Result[VertexOrder[Idx]];
				Polygon.Add(PrevMid - PerpToOutside);
				Polygon.Add(PrevMid + PerpToOutside);
				Polygon.Add(NextMid + PerpToOutside);
				Polygon.Add(NextMid - PerpToOutside);
				CurveUtil::ClipConvexToBounds<RealType, TVector2<RealType>>(Polygon, BoundsClip.Min, BoundsClip.Max);
			}
		}

		return Result;
	}

	template<typename RealType>
	bool IsDelaunay(FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, TArrayView<const FIndex2i> SkipEdgesIn)
	{
		TSet<FIndex2i> SkipSet;
		SkipSet.Reserve(SkipEdgesIn.Num());
		for (FIndex2i Edge : SkipEdgesIn)
		{
			if (Connectivity.HasDuplicateTracking())
			{
				Connectivity.FixDuplicatesOnEdge(Edge);
			}
			if (Edge.A > Edge.B)
			{
				Swap(Edge.A, Edge.B);
			}
			SkipSet.Add(Edge);
		}

		bool bFoundNonDelaunay = false;
		Connectivity.EnumerateOrientedEdges([&Connectivity, &Vertices, &SkipSet, &bFoundNonDelaunay](const FIndex2i& Edge, int32 Vert) -> bool
		{
			if (SkipSet.Num())
			{
				FIndex2i Unoriented = Edge;
				if (Unoriented.A > Unoriented.B)
				{
					Swap(Unoriented.A, Unoriented.B);
				}
				if (SkipSet.Contains(Unoriented))
				{
					return true;
				}
			}
			if (Connectivity.IsGhost(Edge, Vert))
			{
				return true;
			}
			const FIndex2i Pair(Edge.B, Edge.A);
			if (Pair.Contains(Connectivity.GhostIndex))
			{
				return true;
			}
			const int32 PairV = Connectivity.GetVertex(Pair);
			if (PairV < 0) // skip if ghost or missing
			{
				return true;
			}
			const RealType InCircleRes = ExactPredicates::InCircle2<RealType>(Vertices[Edge.A], Vertices[Edge.B], Vertices[Vert], Vertices[PairV]);
			if (InCircleRes > 0)
			{
				bFoundNonDelaunay = true;
				return false;
			}
			return true;
		});
		return !bFoundNonDelaunay;
	}

	template<typename RealType>
	bool GetFirstCrossingEdge(FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, const FIndex2i& EdgeToConnect, FIndex2i& CrossingEdgeOut)
	{
		if (Connectivity.HasEdge(EdgeToConnect))
		{
			return false; // nothing to dig
		}

		FIndex2i StartWalk = Connectivity.GetEdge(EdgeToConnect.A);
		if (StartWalk.A == FDelaunay2Connectivity::InvalidIndex)
		{
			return false; // edge starts at a vertex that is not in the triangulation (e.g., could have been a duplicate that was rejected)
		}

		TVector2<RealType> VA = Vertices[EdgeToConnect.A];
		TVector2<RealType> VB = Vertices[EdgeToConnect.B];

		auto IsCrossingEdgeOnA = [&VA, &VB, &Vertices](const FIndex2i& Edge, RealType OrientA, RealType& OrientBOut) -> bool
		{
			if (Edge.B >= 0)
			{
				OrientBOut = ExactPredicates::Orient2<RealType>(VA, VB, Vertices[Edge.B]);
			}

			if (FDelaunay2Connectivity::IsGhost(Edge))
			{
				return false;
			}
			int32 SignA = (int32)FMath::Sign(OrientA);
			if (SignA >= 0)
			{
				return false;
			}
			int32 SignB = (int32)FMath::Sign(OrientBOut);
			// A properly oriented edge crossing the AB segment, on a tri that includes A, must go from the negative side to the positive side of AB
			// (positive to negative would be behind the AB edge, and a zero would either be behind or would prevent the edge from being inserted)
			return SignB == 1;
		};

		FIndex2i WalkEdge = StartWalk;
		RealType OrientA = 0;
		if (WalkEdge.A != FDelaunay2Connectivity::GhostIndex)
		{
			OrientA = ExactPredicates::Orient2<RealType>(VA, VB, Vertices[WalkEdge.A]);
		}
		RealType OrientB; // computed by the crossing edge test
		bool bIsCrossing = IsCrossingEdgeOnA(WalkEdge, OrientA, OrientB);
		int32 EdgesWalked = 0;
		while (!bIsCrossing)
		{
			checkSlow(EdgesWalked++ < Connectivity.NumHalfEdges());
			int32 NextVertex = Connectivity.GetVertex(FIndex2i(EdgeToConnect.A, WalkEdge.B));
			check(NextVertex != Connectivity.InvalidIndex); // should not be a hole in the mesh at this stage!  if there were, need to stop this loop and then walk the opposite direction
			WalkEdge = FIndex2i(WalkEdge.B, NextVertex);
			if (WalkEdge == StartWalk) // full cycle with no crossing found, cannot insert the edge (this can happen if the edge is blocked by an exactly-on-edge vertex)
			{
				return false;
			}
			OrientA = OrientB;
			bIsCrossing = IsCrossingEdgeOnA(WalkEdge, OrientA, OrientB);
		}
		CrossingEdgeOut = WalkEdge;
		return true;
	}

	// @return vertex we need to fill to.  If EdgeToConnect.A, no fill needed; if EdgeToConnect.B, a normal re-triangulation needed; if other, digging failed in the middle and we need to fill partially
	template<typename RealType>
	int32 DigCavity(FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, const FIndex2i& EdgeToConnect, TArray<int32>& CavityLOut, TArray<int32>& CavityROut, bool& bDigSuccess)
	{
		bDigSuccess = false;

		CavityLOut.Reset();
		CavityROut.Reset();

		FIndex2i FirstCross;
		bool bFoundCross = GetFirstCrossingEdge<RealType>(Connectivity, Vertices, EdgeToConnect, FirstCross);
		if (!bFoundCross)
		{
			bDigSuccess = true; // no-dig-needed case counts as a success
			return EdgeToConnect.A;
		}

		// Delete the first triangle in the cavity
		Connectivity.DeleteTriangle(FIndex3i(EdgeToConnect.A, FirstCross.A, FirstCross.B));

		CavityROut.Add(EdgeToConnect.A);
		CavityLOut.Add(EdgeToConnect.A);
		CavityROut.Add(FirstCross.A);
		CavityLOut.Add(FirstCross.B);

		TVector2<RealType> VA = Vertices[EdgeToConnect.A];
		TVector2<RealType> VB = Vertices[EdgeToConnect.B];
		
		// by convention WalkCross is always crossing from left to right
		int32 PrevVertex = EdgeToConnect.A;
		FIndex2i WalkCross(FirstCross.B, FirstCross.A);
		while (true) // Note: Can't loop infinitely because it is deleting triangles as it walks
		{
			int32 NextV = Connectivity.GetVertex(WalkCross);
			if (NextV == FDelaunay2Connectivity::InvalidIndex)
			{
				ensure(false); // walking off the triangulation would mean the triangulation is unrecoverably broken
				return EdgeToConnect.A;
			}
			Connectivity.DeleteTriangle(FIndex3i(WalkCross.A, WalkCross.B, NextV)); // immediately delete where we walk

			if (NextV == EdgeToConnect.B)
			{
				CavityROut.Add(EdgeToConnect.B);
				CavityLOut.Add(EdgeToConnect.B);
				bDigSuccess = true;
				return EdgeToConnect.B;
			}

			FIndex2i NextCross;

			RealType OrientNextV = ExactPredicates::Orient2<RealType>(VA, VB, Vertices[NextV]);
			if (OrientNextV == 0)
			{
				// can't reach target edge due to intersecting this vertex; just stop here
				CavityROut.Add(NextV);
				CavityLOut.Add(NextV);
				return NextV;
			}
			else if (OrientNextV < 0)
			{
				PrevVertex = WalkCross.B;
				NextCross = FIndex2i(WalkCross.A, NextV); // facing the next triangle
				CavityROut.Add(NextV);
			}
			else
			{
				PrevVertex = WalkCross.A;
				NextCross = FIndex2i(NextV, WalkCross.B); // facing the next triangle
				CavityLOut.Add(NextV);
			}
			WalkCross = NextCross;
		}

		check(false); // can't reach here
		return EdgeToConnect.A;
	}

	// Note: previously this implemented the cavity CDT algorithm from "Delaunay Mesh Generation" page 76, 77, but that algorithm appears incorrect; see
	//  "Fast Segment Insertion and Incremental Construction of Constrained Delaunay Triangulations" by Shewchuk and Brown, for a more complicated,
	//  corrected version of that algorithm.
	// For now FillCavity implements a simpler algorithm from Anglada (1997), which the Shewchuk and Brown paper reports as faster in practice for cavities less
	// than ~30-85 vertices, which should be the vast majority of cases.  They suggest implementing both algorithms and switching based on the input size;
	// so that is something to consider if FillCavity becomes a bottleneck in the future.
	template<typename RealType>
	void FillCavity(FRandomStream& Random, FDelaunay2Connectivity& Connectivity, TArrayView<const TVector2<RealType>> Vertices, const FIndex2i& Edge, const TArray<int32>& Cavity)
	{
		check(Cavity.Num() > 2 && Edge.B == Cavity[0] && Edge.A == Cavity.Last());
		int32 CavityNum = Cavity.Num();


		TArray<FIndex2i> Ranges;
		Ranges.Reserve(Cavity.Num() - 2);
		Ranges.Emplace(0, Cavity.Num() - 1);
		while (!Ranges.IsEmpty())
		{

			FIndex2i Range = Ranges.Pop(EAllowShrinking::No);
			int32 Mid = Range.A + 1;
			if (Range.A + 2 == Range.B)
			{
				Connectivity.AddTriangle(FIndex3i(Cavity[Range.A], Cavity[Mid], Cavity[Range.B]));
				continue;
			}
			for (int32 Cand = Mid + 1; Cand < Range.B; Cand++)
			{
				if (0 < ExactPredicates::InCircle2<RealType>(Vertices[Cavity[Range.A]], Vertices[Cavity[Mid]], Vertices[Cavity[Range.B]], Vertices[Cavity[Cand]]))
				{
					Mid = Cand;
				}
			}
			Connectivity.AddTriangle(FIndex3i(Cavity[Range.A], Cavity[Mid], Cavity[Range.B]));
			if (Mid - Range.A > 1)
			{
				Ranges.Emplace(Range.A, Mid);
			}
			if (Range.B - Mid > 1)
			{
				Ranges.Emplace(Mid, Range.B);
			}
		}
	}

	template<typename RealType>
	bool ConstrainEdges(FRandomStream& Random, FDelaunay2Connectivity& Connectivity,
		TArrayView<const TVector2<RealType>> Vertices, TArrayView<const FIndex2i> Edges, bool bKeepFastEdgeAdjacencyData)
	{
		constexpr int32 NeedFasterEdgeLookupThreshold = 4; // TODO: do some profiling to determine what this threshold should be
		if (bKeepFastEdgeAdjacencyData || Edges.Num() > NeedFasterEdgeLookupThreshold)
		{
			Connectivity.EnableVertexAdjacency(Vertices.Num());
		}

		bool bSuccess = true;

		TArray<int32> CavityVerts[2];

		// Random insertion order to improve expected performance
		TArray<int32> EdgeOrder = GetShuffledOrder(Random, Edges.Num());
		for (int32 OrderIdx = 0; OrderIdx < Edges.Num(); OrderIdx++)
		{
			int32 EdgeIdx = EdgeOrder[OrderIdx];
			FIndex2i Edge = Edges[EdgeIdx];
			if (Connectivity.HasDuplicateTracking())
			{
				Connectivity.FixDuplicatesOnEdge(Edge);
			}
			bool bDigSucceeded = false;
			int32 DigTo = DigCavity<RealType>(Connectivity, Vertices, Edge, CavityVerts[0], CavityVerts[1], bDigSucceeded);
			if (DigTo != Edge.A)
			{
				FIndex2i DugEdge(Edge.A, DigTo); // fill the cavity we dug out (which may end at a different vertex than target, if there was a colinear vertex first)
				FIndex2i RevEdge(DugEdge.B, DugEdge.A);
				Algo::Reverse(CavityVerts[0]);
				FillCavity<RealType>(Random, Connectivity, Vertices, DugEdge, CavityVerts[0]);
				FillCavity<RealType>(Random, Connectivity, Vertices, RevEdge, CavityVerts[1]);
			}
		}

		if (!bKeepFastEdgeAdjacencyData)
		{
			Connectivity.DisableVertexAdjacency();
		}

		return bSuccess;
	}
	
	template<typename RealType>
	void Update(FRandomStream& Random, FDelaunay2Connectivity& Connectivity,
		TArrayView<const TVector2<RealType>> Vertices, TArray<int32> VertexOrder, FIndex3i StartSearchTri, bool bAssumeDelaunay)
	{
		FIndex3i SearchTri = StartSearchTri;
		for (int32 OrderIdx = 0; OrderIdx < VertexOrder.Num(); OrderIdx++)
		{
			int32 Vertex = VertexOrder[OrderIdx];
			int32 DuplicateOf = -1;
			FIndex3i ContainingTri = WalkToContainingTri<RealType>(Random, Connectivity, Vertices, SearchTri, Vertex, bAssumeDelaunay, DuplicateOf);
			if (DuplicateOf >= 0)
			{
				Connectivity.SetHasDuplicates();
				if (Connectivity.HasDuplicateTracking())
				{
					Connectivity.MarkDuplicateVertex(DuplicateOf, Vertex);
				}
				continue;
			}
			if (ContainingTri[0] == FDelaunay2Connectivity::InvalidIndex)
			{
				continue;
			}
			SearchTri = Insert<RealType>(Connectivity, Vertices, ContainingTri, Vertex);
			checkSlow(SearchTri.A != FDelaunay2Connectivity::InvalidIndex);
		}
	}

	template<typename RealType>
	bool Triangulate(FRandomStream& Random, FDelaunay2Connectivity& Connectivity,
		TArrayView<const TVector2<RealType>> Vertices, TArrayView<const FIndex2i> Edges, bool bKeepFastEdgeAdjacencyData, FDelaunay2::EResult& OutResult)
	{
		OutResult = FDelaunay2::EResult::Unknown;

		Connectivity.Empty(Vertices.Num());

		if (Vertices.Num() < 3)
		{
			OutResult = Vertices.IsEmpty() ? FDelaunay2::EResult::EmptyInput : FDelaunay2::EResult::Collinear;
			return false;
		}

		FBRIOPoints InsertOrder;
		InsertOrder.Compute(Vertices);
		TArray<int32>& Order = InsertOrder.Order;

		int32 BootstrapIndices[3]{ Order.Num() - 1, -1, -1 };
		TVector2<RealType> Pts[3];
		Pts[0] = Vertices[Order[BootstrapIndices[0]]];
		RealType BootstrapOrient = 0;
		for (int32 SecondIdx = Order.Num() - 2; SecondIdx >= 0; SecondIdx--)
		{
			if (Pts[0] != Vertices[Order[SecondIdx]])
			{
				Pts[1] = Vertices[Order[SecondIdx]];
				BootstrapIndices[1] = SecondIdx;
				break;
			}
		}
		if (BootstrapIndices[1] == -1) // all points were identical; nothing to triagulate
		{
			OutResult = FDelaunay2::EResult::Collinear;
			return false;
		}

		for (int32 ThirdIdx = BootstrapIndices[1] - 1; ThirdIdx >= 0; ThirdIdx--)
		{
			RealType Orient = ExactPredicates::Orient2<RealType>(Pts[0], Pts[1], Vertices[Order[ThirdIdx]]);
			if (Orient != 0)
			{
				Pts[2] = Vertices[Order[ThirdIdx]];
				BootstrapIndices[2] = ThirdIdx;
				BootstrapOrient = Orient;
				break;
			}
		}
		if (BootstrapIndices[2] == -1) // all points were colinear; nothing to triangulate
		{
			OutResult = FDelaunay2::EResult::Collinear;
			return false;
		}

		// Make the first triangle from the bootstrap points and remove the bootstrap points from the insertion ordering
		FIndex3i FirstTri(Order[BootstrapIndices[0]], Order[BootstrapIndices[1]], Order[BootstrapIndices[2]]);
		if (BootstrapOrient < 0)
		{
			Swap(FirstTri.B, FirstTri.C);
		}
		Connectivity.InitWithGhosts(FirstTri);
		Order.RemoveAt(BootstrapIndices[0]);
		Order.RemoveAt(BootstrapIndices[1]);
		Order.RemoveAt(BootstrapIndices[2]);

		constexpr bool bAssumeDelaunay = true; // initial construction, before constraint edges, so safe to assume Delaunay
		Update(Random, Connectivity, Vertices, Order, FirstTri, bAssumeDelaunay);

		bool bSuccess = ConstrainEdges(Random, Connectivity, Vertices, Edges, bKeepFastEdgeAdjacencyData);
		if (bSuccess)
		{
			// Note: The ConstrainEdges implementation does not currently detect failure reasons, 
			// so we rely on the ValidateEdgesResult method to set the result on edge failures
			OutResult = FDelaunay2::EResult::Success; 
		}
		return bSuccess;
	}
} // namespace DelaunayInternal

bool FDelaunay2::Triangulate(TArrayView<const FVector2d> Vertices, TArrayView<const FIndex2i> Edges)
{
	Connectivity = MakePimpl<FDelaunay2Connectivity>(bAutomaticallyFixEdgesToDuplicateVertices);

	bIsConstrained = Edges.Num() > 0;

	bool bSuccess = DelaunayInternal::Triangulate<double>(RandomStream, *Connectivity, Vertices, Edges, bKeepFastEdgeAdjacencyData, Result);
	return bSuccess && ValidateEdgesResult(Edges);
}

bool FDelaunay2::Triangulate(TArrayView<const FVector2f> Vertices, TArrayView<const FIndex2i> Edges)
{
	Connectivity = MakePimpl<FDelaunay2Connectivity>(bAutomaticallyFixEdgesToDuplicateVertices);

	bIsConstrained = Edges.Num() > 0;

	bool bSuccess = DelaunayInternal::Triangulate<float>(RandomStream, *Connectivity, Vertices, Edges, bKeepFastEdgeAdjacencyData, Result);
	return bSuccess && ValidateEdgesResult(Edges);
}

bool FDelaunay2::ConstrainEdges(TArrayView<const FVector2d> Vertices, TArrayView<const FIndex2i> Edges)
{
	bIsConstrained = bIsConstrained || Edges.Num() > 0;
	// Note: automatic edge-to-duplicate fixing will not work if duplicate tracking was not enabled
	checkSlow(!bAutomaticallyFixEdgesToDuplicateVertices || Connectivity->HasDuplicateTracking());

	bool bSuccess = DelaunayInternal::ConstrainEdges<double>(RandomStream, *Connectivity, Vertices, Edges, bKeepFastEdgeAdjacencyData);
	return bSuccess && ValidateEdgesResult(Edges);
}

bool FDelaunay2::ConstrainEdges(TArrayView<const FVector2f> Vertices, TArrayView<const FIndex2i> Edges)
{
	bIsConstrained = bIsConstrained || Edges.Num() > 0;
	// Note: automatic edge-to-duplicate fixing will not work if duplicate tracking was not enabled
	checkSlow(!bAutomaticallyFixEdgesToDuplicateVertices || Connectivity->HasDuplicateTracking());

	bool bSuccess = DelaunayInternal::ConstrainEdges<float>(RandomStream, *Connectivity, Vertices, Edges, bKeepFastEdgeAdjacencyData);
	return bSuccess && ValidateEdgesResult(Edges);
}

bool FDelaunay2::Update(TArrayView<const TVector2<double>> Vertices, int32 FirstNewIdx)
{
	// Check if a triangulation already exists
	if (Connectivity->NumTriangles() == 0)
	{
		return false;
	}

	// Construct BRIO for new vertices
	FBRIOPoints InsertOrder;
	InsertOrder.Compute(Vertices.RightChop(FirstNewIdx));
	TArray<int32>& Order = InsertOrder.Order;
	for (int32& VertexIndex : Order)
	{
		VertexIndex += FirstNewIdx;
	}

	// Get any existing triangle as a bootstrap one
	FIndex3i SearchTri;
	Connectivity->EnumerateTriangles([&SearchTri](const FIndex2i& Edge, int32 Vertex) -> bool
	{
		SearchTri.A = Vertex;
		SearchTri.B = Edge.A;
		SearchTri.C = Edge.B;
		return false;
	});

	bool bAssumeDelaunay = !bIsConstrained; // this may have previously constrained edges in the original triangulation so we cannot assume Delaunay
	DelaunayInternal::Update<double>(RandomStream, *Connectivity, Vertices, Order, SearchTri, bAssumeDelaunay);
	return true;
}

TArray<FIndex3i> FDelaunay2::GetTriangles() const
{
	if (Connectivity.IsValid())
	{
		return Connectivity->GetTriangles();
	}
	return TArray<FIndex3i>();
}

void FDelaunay2::GetTrianglesAndAdjacency(TArray<FIndex3i>& Triangles, TArray<FIndex3i>& Adjacency) const
{
	if (Connectivity.IsValid())
	{
		Connectivity->GetTrianglesAndAdjacency(Triangles, Adjacency);
	}
}

bool FDelaunay2::GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> Edges, EFillMode FillMode) const
{
	if (!ensure(Connectivity.IsValid()))
	{
		return false;
	}
	return Connectivity->GetFilledTriangles(TrianglesOut, Edges, FillMode);
}

bool FDelaunay2::GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> BoundaryEdges, TArrayView<const FIndex2i> HoleEdges) const
{
	if (!ensure(Connectivity.IsValid()))
	{
		return false;
	}
	return Connectivity->GetFilledTriangles(TrianglesOut, BoundaryEdges, HoleEdges);
}

bool FDelaunay2::GetFilledTrianglesGeneralizedWinding(TArray<FIndex3i>& TrianglesOut, TArrayView<const TVector2<double>> Vertices, TArrayView<const FIndex2i> Edges, EFillMode FillMode) const
{
	if (!ensure(Connectivity.IsValid()))
	{
		return false;
	}
	return Connectivity->GetFilledTrianglesGeneralizedWinding<double>(TrianglesOut, Vertices, Edges, FillMode);
}

bool FDelaunay2::GetFilledTrianglesGeneralizedWinding(TArray<FIndex3i>& TrianglesOut, TArrayView<const TVector2<float>> Vertices, TArrayView<const FIndex2i> Edges, EFillMode FillMode) const
{
	if (!ensure(Connectivity.IsValid()))
	{
		return false;
	}
	return Connectivity->GetFilledTrianglesGeneralizedWinding<float>(TrianglesOut, Vertices, Edges, FillMode);
}

bool FDelaunay2::IsDelaunay(TArrayView<const FVector2f> Vertices, TArrayView<const FIndex2i> SkipEdges) const
{
	if (ensure(Connectivity.IsValid()))
	{
		return DelaunayInternal::IsDelaunay<float>(*Connectivity, Vertices, SkipEdges);
	}
	return false; // if no triangulation was performed, function should be been called
}

bool FDelaunay2::IsDelaunay(TArrayView<const FVector2d> Vertices, TArrayView<const FIndex2i> SkipEdges) const
{
	if (ensure(Connectivity.IsValid()))
	{
		return DelaunayInternal::IsDelaunay<double>(*Connectivity, Vertices, SkipEdges);
	}
	return false; // if no triangulation was performed, function should not be called
}


TArray<TArray<FVector2d>> FDelaunay2::GetVoronoiCells(TArrayView<const FVector2d> Vertices, bool bIncludeBoundary, FAxisAlignedBox2d Bounds, double ExpandBounds) const
{
	if (Result == FDelaunay2::EResult::Collinear)
	{
		return DelaunayInternal::GetVoronoiCells_Collinear<double>(Vertices, bIncludeBoundary, Bounds, ExpandBounds);
	}
	else if (ensureMsgf(Connectivity.IsValid() && !bIsConstrained, TEXT("Voronoi diagram computation requires an unconstrained Delaunay triangulation to be already computed")))
	{
		return DelaunayInternal::GetVoronoiCells<double>(*Connectivity, Vertices, bIncludeBoundary, Bounds, ExpandBounds);
	}
	return TArray<TArray<FVector2d>>();
}

TArray<TArray<FVector2f>> FDelaunay2::GetVoronoiCells(TArrayView<const FVector2f> Vertices, bool bIncludeBoundary, FAxisAlignedBox2f Bounds, float ExpandBounds) const
{
	if (Result == FDelaunay2::EResult::Collinear)
	{
		return DelaunayInternal::GetVoronoiCells_Collinear<float>(Vertices, bIncludeBoundary, Bounds, ExpandBounds);
	}
	else if (ensureMsgf(Connectivity.IsValid() && !bIsConstrained, TEXT("Voronoi diagram computation requires an unconstrained Delaunay triangulation to be already computed")))
	{
		return DelaunayInternal::GetVoronoiCells<float>(*Connectivity, Vertices, bIncludeBoundary, Bounds, ExpandBounds);
	}
	return TArray<TArray<FVector2f>>();
}


bool FDelaunay2::HasEdges(TArrayView<const FIndex2i> Edges) const
{
	if (ensure(Connectivity.IsValid()))
	{
		for (FIndex2i Edge : Edges)
		{
			if (bAutomaticallyFixEdgesToDuplicateVertices)
			{
				Connectivity->FixDuplicatesOnEdge(Edge);
			}
			if (!Connectivity->HasEdge(Edge))
			{
				return false;
			}
		}
		return true;
	}
	return false; // if no triangulation was performed, function should not be called
}

void FDelaunay2::FixDuplicatesOnEdge(FIndex2i& Edge)
{
	checkSlow(Connectivity.IsValid() && Connectivity->HasDuplicateTracking());
	Connectivity->FixDuplicatesOnEdge(Edge);
}

bool FDelaunay2::HasEdge(const FIndex2i& Edge, bool bRemapDuplicates)
{
	checkSlow(Connectivity.IsValid());
	if (bRemapDuplicates)
	{
		checkSlow(Connectivity->HasDuplicateTracking());
		FIndex2i RemapEdge = Edge;
		Connectivity->FixDuplicatesOnEdge(RemapEdge);
		return Connectivity->HasEdge(RemapEdge);
	}
	return Connectivity->HasEdge(Edge);
}

bool FDelaunay2::HasDuplicates() const
{
	return Connectivity->HasDuplicates();
}

int32 FDelaunay2::RemapIfDuplicate(int32 Index) const
{
	checkSlow(Connectivity->HasDuplicateTracking());
	return Connectivity->RemapIfDuplicate(Index);
}

} // end namespace UE::Geometry
} // end namespace UE