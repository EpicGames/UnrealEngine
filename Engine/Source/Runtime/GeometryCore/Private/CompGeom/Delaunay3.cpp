// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/Delaunay3.h"

#include "CompGeom/ExactPredicates.h"
#include "CompGeom/ConvexHull3.h"
#include "Spatial/ZOrderCurvePoints.h"
#include "TetUtil.h"

#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

// Simple tetrahedra connectivity structure designed for Delaunay triangulation specifically. Has a single 'ghost vertex' connected to the boundary.
// Currently this is a very simple triangle->opposite vertex TMap; it can be made faster if switched to something that is not so TMap-based.
struct FDelaunay3Connectivity
{
	static constexpr int32 GhostIndex = -1;
	static constexpr int32 InvalidIndex = -2;

	FDelaunay3Connectivity()
	{}

	void Empty(int32 ExpectedMaxVertices = 0)
	{
		TriToVert.Empty(ExpectedMaxVertices * 30);
	}

	// To support looking up triangles in the connectivity hash map, we rotate the triangles to a "canonical" form
	// @return Tri with its indices rotated so the smallest index is first
	static FIndex3i ToCanon(const FIndex3i& Tri)
	{
		if (Tri.A < Tri.B)
		{
			if (Tri.A < Tri.C) // A smallest
			{
				return Tri;
			}
			else // C smallest
			{
				return FIndex3i(Tri.C, Tri.A, Tri.B);
			}
		}
		else
		{
			if (Tri.B < Tri.C) // B smallest
			{
				return FIndex3i(Tri.B, Tri.C, Tri.A);
			}
			else // C smallest
			{
				return FIndex3i(Tri.C, Tri.A, Tri.B);
			}
		}
	}

	static bool IsCanon(const FIndex3i& Tri)
	{
		return Tri.A < Tri.B && Tri.A < Tri.C;
	}

	// Check if triangle is present in the mesh, assuming it is in canonical order
	bool HasCanonTri(const FIndex3i& CanonTri) const
	{
		return TriToVert.Contains(CanonTri);
	}

	int32 NumTets() const
	{
		return TriToVert.Num() / 4;
	}

	int32 NumHalfTris() const
	{
		return TriToVert.Num();
	}

	TArray<FIndex4i> GetTets() const
	{
		TArray<FIndex4i> Tets;
		Tets.Reserve(TriToVert.Num() / 3);
		for (const TPair<FIndex3i, int32>& EV : TriToVert)
		{
			if (!IsGhost(EV.Key, EV.Value) && EV.Value < EV.Key.A && EV.Value < EV.Key.B && EV.Value < EV.Key.C)
			{
				Tets.Emplace(EV.Key.A, EV.Key.B, EV.Key.C, EV.Value);
			}
		}
		return Tets;
	}

	static bool IsGhost(const FIndex3i& Tri, int32 Vertex)
	{
		return Vertex == GhostIndex || IsGhost(Tri);
	}

	static bool IsGhost(const FIndex3i& Tri)
	{
		return Tri.A == GhostIndex || Tri.B == GhostIndex || Tri.C == GhostIndex;
	}

	void AddTet(const FIndex4i& Tet)
	{
		checkSlow(Tet.A != Tet.B && Tet.A != Tet.C && Tet.B != Tet.C && Tet.A != Tet.D && Tet.B != Tet.D && Tet.C != Tet.D);
		TriToVert.Add(ToCanon(FIndex3i(Tet.A, Tet.B, Tet.C)), Tet.D);
		TriToVert.Add(ToCanon(FIndex3i(Tet.A, Tet.D, Tet.B)), Tet.C);
		TriToVert.Add(ToCanon(FIndex3i(Tet.A, Tet.C, Tet.D)), Tet.B);
		TriToVert.Add(ToCanon(FIndex3i(Tet.B, Tet.D, Tet.C)), Tet.A);
	}

	// Create a first initial tet that is surround by ghost tets
	void InitWithGhosts(const FIndex4i& Tet)
	{
		AddTet(Tet);
		AddTet(FIndex4i(Tet.C, Tet.B, Tet.A, GhostIndex));
		AddTet(FIndex4i(Tet.B, Tet.D, Tet.A, GhostIndex));
		AddTet(FIndex4i(Tet.D, Tet.C, Tet.A, GhostIndex));
		AddTet(FIndex4i(Tet.C, Tet.D, Tet.B, GhostIndex));
	}

	void DeleteTet(const FIndex4i& Tet)
	{
		TriToVert.Remove(ToCanon(FIndex3i(Tet.A, Tet.B, Tet.C)));
		TriToVert.Remove(ToCanon(FIndex3i(Tet.A, Tet.D, Tet.B)));
		TriToVert.Remove(ToCanon(FIndex3i(Tet.A, Tet.C, Tet.D)));
		TriToVert.Remove(ToCanon(FIndex3i(Tet.B, Tet.D, Tet.C)));
	}

	// Get vertex from a canonical-form tri (lowest-index first)
	int32 GetVertexFromCanon(const FIndex3i& CanonTri) const
	{
		const int32* V = TriToVert.Find(CanonTri);
		if (!V)
		{
			return InvalidIndex;
		}
		return *V;
	}

	// Get vertex from a tri (lowest-index first)
	int32 GetVertex(const FIndex3i& Tri) const
	{
		const int32* V = TriToVert.Find(ToCanon(Tri));
		if (!V)
		{
			return InvalidIndex;
		}
		return *V;
	}

	void MarkDuplicateVertex(int32 Orig, int32 Duplicate)
	{
		checkSlow(bTrackDuplicateVertices);
		DuplicateVertices.Add(Duplicate, Orig);
	}

	bool HasDuplicateTracking() const
	{
		return bTrackDuplicateVertices;
	}
	
	// Call a function on every oriented tri (+ associated vertex) on the mesh
	// (note the number of tris visited will be 4x the number of tets)
	// VisitFunctionType is expected to take (FIndex3i Tri, int32 Vertex) and return bool
	// Returning false from VisitFn will end the enumeration early
	template<typename VisitFunctionType>
	void EnumerateOrientedTriangles(VisitFunctionType VisitFn) const
	{
		for (const TPair<FIndex3i, int32>& TriVert : TriToVert)
		{
			if (!VisitFn(TriVert.Key, TriVert.Value))
			{
				break;
			}
		}
	}

	// Similar to EnumerateOrientedTriangles but only visits each tet once, instead of 4x
	// Calls VisitFn(FIndex3i Tri, int32 Vertex) with Vertex index always smaller than the Tri indices
	// Returning false from VisitFn will end the enumeration early
	// @param bSkipGhosts If true, skip ghost triangles (triangles connected to the ghost vertex)
	template<typename VisitFunctionType>
	void EnumerateTrianglePerTet(VisitFunctionType VisitFn, bool bSkipGhosts = false) const
	{
		for (const TPair<FIndex3i, int32>& TriVert : TriToVert)
		{
			// to visit triangles only once, only visit when the vertex ID is smaller than the edge IDs
			// since the vertex ID is the smallest ID, it is also the only one we need to check vs the GhostIndex (if we're skipping ghosts)
			if (TriVert.Key.A < TriVert.Value || TriVert.Key.B < TriVert.Value || TriVert.Key.C < TriVert.Value || (bSkipGhosts && TriVert.Value == GhostIndex))
			{
				continue;
			}
			if (!VisitFn(TriVert.Key, TriVert.Value))
			{
				break;
			}
		}
	}

protected:
	TMap<FIndex3i, int32> TriToVert;

	bool bTrackDuplicateVertices = false;
	TMap<int32, int32> DuplicateVertices;
};

namespace Delaunay3Internal
{
	FIndex3i GetTriOpposite(const FIndex4i& Tet, int SubIdx)
	{
		// Return the face opposite the ith vertex of the tetrahedron
		switch (SubIdx)
		{
		case 0:
			return FIndex3i(Tet.B, Tet.D, Tet.C);
		case 1:
			return FIndex3i(Tet.A, Tet.C, Tet.D);
		case 2:
			return FIndex3i(Tet.A, Tet.D, Tet.B);
		case 3:
			return FIndex3i(Tet.A, Tet.B, Tet.C);
		default:
			check(false);
		}
		return FIndex3i(FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex);
	}

	FIndex4i ToTet(const FIndex3i& Tri, int32 Vert)
	{
		return FIndex4i(Tri.A, Tri.B, Tri.C, Vert);
	}

	// Fill an array with the properly-oriented triangle faces of the tet, starting with Tri itself
	void GetTetFaces(const FIndex4i& Tet, FIndex3i OutFaces[4])
	{
		return TetUtil::GetTetFaces(Tet, OutFaces);
	}

	// Fill an array with four possible orientations of the given tet, each starting with a different triangle face
	void GetTetOrientations(const FIndex4i& Tet, FIndex4i OutFaces[4])
	{
		OutFaces[0] = Tet;
		OutFaces[1] = FIndex4i(Tet.A, Tet.D, Tet.B, Tet.C);
		OutFaces[2] = FIndex4i(Tet.A, Tet.C, Tet.D, Tet.B);
		OutFaces[3] = FIndex4i(Tet.B, Tet.D, Tet.C, Tet.A);
	}

	// Fill an array with the properly-oriented triangle faces of the tet (Tri.A, Tri.B, Tri.C, Vert), starting with Tri itself
	void GetTetFaces(const FIndex3i& Tri, int32 Vert, FIndex3i OutFaces[4])
	{
		GetTetFaces(ToTet(Tri, Vert), OutFaces);
	}

	// @return Tri with its winding reversed
	FIndex3i ReverseTri(const FIndex3i& Tri)
	{
		// Note: Ordered s.t. the triangle remains 'canonical' (lowest index first) if it was before
		return FIndex3i(Tri.A, Tri.C, Tri.B);
	}

	template<typename RealType>
	bool IsDelaunay(FDelaunay3Connectivity& Connectivity, TArrayView<const TVector<RealType>> Vertices)
	{
		bool bFoundNonDelaunay = false;
		Connectivity.EnumerateOrientedTriangles([&Connectivity, &Vertices, &bFoundNonDelaunay](const FIndex3i& Tri, int32 Vert) -> bool
			{
				if (Connectivity.IsGhost(Tri, Vert))
				{
					return true;
				}
				const RealType OrientRes = ExactPredicates::Orient3<RealType>(Vertices[Tri.A], Vertices[Tri.B], Vertices[Tri.C], Vertices[Vert]);
				if (OrientRes <= 0)
				{
					bFoundNonDelaunay = true; // tet must not be inverted or flat
					return false;
				}
				const FIndex3i FlipTri = ReverseTri(Tri);
				check(Connectivity.IsCanon(FlipTri)); // Tris should always be in canonical form when enumerated
				const int32 PairV = Connectivity.GetVertex(FlipTri);
				if (PairV < 0) // opposite tet's opposite vert is a ghost or missing
				{ // this is only ok if the pair is a ghost; the connectivity should not have any missing tets
					return (PairV == FDelaunay3Connectivity::GhostIndex);
				}
				const RealType InSphereRes = ExactPredicates::InSphere3<RealType>(Vertices[Tri.A], Vertices[Tri.B], Vertices[Tri.C], Vertices[Vert], Vertices[PairV]);
				if (InSphereRes > 0)
				{
					bFoundNonDelaunay = true; // opposite tet's opposite vertex must not be in this tet's circumsphere
					return false;
				}
				return true;
			});
		return !bFoundNonDelaunay;
	}

	// @return true if Vertex is inside the circumsphere of Tet
	// For ghost tets, this is defined as being coplanar & in the open circumcircle of the one solid (non-ghost) triangle, or inside that triangle's half-space
	template<typename RealType>
	bool InTetSphere(TArrayView<const TVector<RealType>> Vertices, const FIndex4i& Tet, int32 Vertex)
	{
		int32 GhostSub = Tet.IndexOf(FDelaunay3Connectivity::GhostIndex);
		if (GhostSub == -1)
		{
			return ExactPredicates::InSphere3<RealType>(Vertices[Tet.A], Vertices[Tet.B], Vertices[Tet.C], Vertices[Tet.D], Vertices[Vertex]) > 0;
		}
		FIndex3i SolidTri = GetTriOpposite(Tet, GhostSub);
		checkSlow(SolidTri.IndexOf(FDelaunay3Connectivity::GhostIndex) == -1);
		RealType Pred = ExactPredicates::Orient3<RealType>(Vertices[SolidTri.A], Vertices[SolidTri.B], Vertices[SolidTri.C], Vertices[Vertex]);
		if (Pred > 0)
		{
			return true;
		}
		if (Pred < 0)
		{
			return false;
		}

		// Pred == 0 case: need to check if Vertex is in the open circumcircle of the solid triangle, or equivalently inside
		// the circumsphere an arbitrary non-flat tet made of the solid triangle + any additional (non-coplanar) point
		TVector<RealType> NotCoplanar = Vertices[SolidTri.A] + VectorUtil::NormalDirection(Vertices[SolidTri.A], Vertices[SolidTri.B], Vertices[SolidTri.C]);
		RealType TetOrient = ExactPredicates::Orient3<RealType>(Vertices[SolidTri.A], Vertices[SolidTri.B], Vertices[SolidTri.C], NotCoplanar);
		// Loop to make sure the NotCoplanar point is definitely not coplanar
		// (typically the initial normal-direction offset will have already achieved this, so the loop will not be entered)
		for (int32 Idx = 0; TetOrient == 0 && Idx < 3; ++Idx)
		{
			NotCoplanar[Idx] += (RealType)10;
			TetOrient = ExactPredicates::Orient3<RealType>(Vertices[SolidTri.A], Vertices[SolidTri.B], Vertices[SolidTri.C], NotCoplanar);
		}
		// Note if the tet is inverted, we need to flip the sign of the InSphere test.
		TetOrient = FMath::Sign(TetOrient);
		ensure(TetOrient != 0);
		RealType InSolidTriCircumcircle = TetOrient * ExactPredicates::InSphere3<RealType>(Vertices[SolidTri.A], Vertices[SolidTri.B], Vertices[SolidTri.C], NotCoplanar, Vertices[Vertex]);
		return InSolidTriCircumcircle > 0;
	}

	// @return tet containing Vertex
	template<typename RealType, bool bAssumeDelaunay = true>
	FIndex4i WalkToContainingTet(FRandomStream& Random, FDelaunay3Connectivity& Connectivity, TArrayView<const TVector<RealType>> Vertices, FIndex4i StartTet, int32 Vertex, int32& IsDuplicateOfOut)
	{
		IsDuplicateOfOut = FDelaunay3Connectivity::InvalidIndex;

		constexpr FIndex3i NoTri(FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex);
		constexpr int32 GhostV = FDelaunay3Connectivity::GhostIndex; // shorter name

		auto ChooseCross = [&Random, &Vertices, Vertex, NoTri, GhostV](const FIndex3i& FromTri, int32 TetV, bool bSkipFirst) -> FIndex3i
		{
			auto CrossesTri = [&Vertices, Vertex, GhostV](const FIndex3i& Tri, bool bOnGhostTet) -> bool
			{
				// only consider if it's not a ghost tet or we're on the one solid triangle of a ghost
				if (!bOnGhostTet || (Tri.A != GhostV && Tri.B != GhostV && Tri.C != GhostV))
				{
					RealType Orient = ExactPredicates::Orient3<RealType>(Vertices[Tri.A], Vertices[Tri.B], Vertices[Tri.C], Vertices[Vertex]);
					// Note in the ghost case, if we're on the plane, it's easier to just walk into the mesh rather than try to check the in-circumcenter condition
					if (Orient < 0 || (bOnGhostTet && Orient == 0))
					{
						// the vertex we're searching for could be on the other side of this tri
						return true;
					}
				}
				return false;
			};

			FIndex3i Consider[4];
			GetTetFaces(FromTri, TetV, Consider);

			int32 Choose[3]{ -1, -1, -1 };
			int32 Chosen = 0;
			bool bIsGhost = TetV == GhostV || FromTri.Contains(GhostV);
			for (int32 SubIdx = (int32)bSkipFirst; SubIdx < 4; ++SubIdx)
			{
				if (CrossesTri(Consider[SubIdx], bIsGhost))
				{
					// On a Delaunay mesh we can always walk across the first edge that has the target vertex on the other side of it
					if constexpr (bAssumeDelaunay)
					{
						return Consider[SubIdx];
					}
					// If the mesh is not Delaunay, randomly choose between edges that have the target vertex on the other side; this avoids a possible infinite cycle
					else
					{
						Choose[Chosen++] = SubIdx;
					}
				}
			}
			if constexpr (bAssumeDelaunay)
			{
				return NoTri;
			}
			else if (Chosen == 0)
			{
				return NoTri; // we're on this tri
			}
			else if (Chosen == 1)
			{
				return Consider[Choose[0]];
			}
			else // (Chosen > 1, pick randomly from the options)
			{
				return Consider[Choose[Random.RandHelper(Chosen)]];
			}
		};
		
		FIndex4i WalkTet = StartTet;
		FIndex3i CrossTri = ChooseCross(FIndex3i(WalkTet.A, WalkTet.B, WalkTet.C), WalkTet.D, false);
		int32 NumSteps = 0;
		int32 NextIdx[3]{ 1,2,0 };
		while (CrossTri.A != FDelaunay3Connectivity::InvalidIndex) // while we're not at the containing tet already
		{
			// if !bAssumeDelaunay, the random edge walk could choose poorly enough for any amount of steps to occur, but it should not happen in practice ...
			// if this ensure() triggers it is more likely that some other problem has caused an infinite loop
			if (!ensure(NumSteps++ < Connectivity.NumTets() * 100))
			{
				return FIndex4i(FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex, FDelaunay3Connectivity::InvalidIndex);
			}

			FIndex3i OppTri = Connectivity.ToCanon(ReverseTri(CrossTri));
			int32 OppVert = Connectivity.GetVertex(OppTri);
			WalkTet = ToTet(OppTri, OppVert);
			checkSlow(OppVert != FDelaunay3Connectivity::InvalidIndex);
			CrossTri = ChooseCross(OppTri, OppVert, true);
		}

		if (WalkTet.A >= 0 && Vertices[Vertex] == Vertices[WalkTet.A])
		{
			IsDuplicateOfOut = WalkTet.A;
		}
		else if (WalkTet.B >= 0 && Vertices[Vertex] == Vertices[WalkTet.B])
		{
			IsDuplicateOfOut = WalkTet.B;
		}
		else if (WalkTet.C >= 0 && Vertices[Vertex] == Vertices[WalkTet.C])
		{
			IsDuplicateOfOut = WalkTet.C;
		}
		else if (WalkTet.D >= 0 && Vertices[Vertex] == Vertices[WalkTet.D])
		{
			IsDuplicateOfOut = WalkTet.D;
		}

		return WalkTet;
	}

	// Insert Vertex into the tet mesh; it must already be on the current (OnTet) tetrahedron
	// Uses Bowyer-Watson algorithm:
	//  1. Delete all the connected tets whose circumspheres contain the vertex
	//  2. Make tetrahedra connecting the new vertex to the boundary of deleted tetrahedra
	// @return one of the inserted tets containing Vertex
	template<typename RealType>
	FIndex4i Insert(FDelaunay3Connectivity& Connectivity, TArrayView<const TVector<RealType>> Vertices, FIndex4i OnTet, int32 ToInsertV)
	{
		// depth first search + deletion of triangles whose circles contain vertex
		TArray<FIndex3i> ToConsider;
		auto AddIfValid = [&ToConsider, &Connectivity](const FIndex3i& Consider)
		{
			checkSlow(Connectivity.IsCanon(Consider));
			if (Connectivity.HasCanonTri(Consider))
			{
				ToConsider.Add(Consider);
			}
		};

		auto DeleteTet = [&AddIfValid, &Connectivity](FIndex4i Tet)
		{
			Connectivity.DeleteTet(Tet);

			FIndex4i OutTets[4];
			GetTetOrientations(Tet, OutTets);
			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				FIndex3i ReverseTri(OutTets[Idx].A, OutTets[Idx].C, OutTets[Idx].B);
				int32 Vert = OutTets[Idx].D;

				AddIfValid(FDelaunay3Connectivity::ToCanon(ReverseTri));
			}
		};
		DeleteTet(OnTet);

		TArray<FIndex3i> Border;
		while (!ToConsider.IsEmpty())
		{
			FIndex3i Consider = ToConsider.Pop(EAllowShrinking::No);
			int32 TetV = Connectivity.GetVertex(Consider);
			if (TetV != FDelaunay3Connectivity::InvalidIndex) // tet still exists (wasn't deleted by earlier traversal)
			{
				FIndex4i ConsiderTet = ToTet(Consider, TetV);
				if (InTetSphere(Vertices, ConsiderTet, ToInsertV))
				{
					DeleteTet(ConsiderTet);
				}
				else
				{
					Border.Add(ReverseTri(Consider));
				}
			}
		}
		
		for (FIndex3i BorderTri : Border)
		{
			Connectivity.AddTet(ToTet(BorderTri, ToInsertV));
		}

		return Border.Num() > 0 ? ToTet(Border[0], ToInsertV) : FIndex4i::Invalid();
	}




	
	template<typename RealType>
	bool Triangulate(FRandomStream& Random, FDelaunay3Connectivity& Connectivity, TArrayView<const TVector<RealType>> Vertices)
	{
		Connectivity.Empty(Vertices.Num());

		if (Vertices.Num() < 4)
		{
			return false;
		}

		FBRIOPoints InsertOrder;
		InsertOrder.Compute(Vertices);
		TArray<int32>& Order = InsertOrder.Order;

		TExtremePoints3<RealType> InitialTetPoints(Vertices.Num(), [&Vertices, &Order](int32 Idx)
			{
				return Vertices[Order[Idx]];
			});
		if (InitialTetPoints.Dimension < 3)
		{
			return false;
		}

		// Make the first triangle from the bootstrap points and remove the bootstrap points from the insertion ordering
		FIndex4i FirstTet(Order[InitialTetPoints.Extreme[0]], Order[InitialTetPoints.Extreme[1]], Order[InitialTetPoints.Extreme[2]], Order[InitialTetPoints.Extreme[3]]);
		Connectivity.InitWithGhosts(FirstTet);
		TArray<int32> ExtremeOrdered{ InitialTetPoints.Extreme[0], InitialTetPoints.Extreme[1], InitialTetPoints.Extreme[2], InitialTetPoints.Extreme[3] };
		ExtremeOrdered.Sort();
		Order.RemoveAt(ExtremeOrdered[3]);
		Order.RemoveAt(ExtremeOrdered[2]);
		Order.RemoveAt(ExtremeOrdered[1]);
		Order.RemoveAt(ExtremeOrdered[0]);

		FIndex4i SearchTet = FirstTet;
		for (int32 OrderIdx = 0; OrderIdx < Order.Num(); OrderIdx++)
		{
			int32 Vertex = Order[OrderIdx];
			int32 DuplicateOf = -1;
			FIndex4i ContainingTet = WalkToContainingTet<RealType>(Random, Connectivity, Vertices, SearchTet, Vertex, DuplicateOf);
			if (DuplicateOf >= 0 || ContainingTet[0] == FDelaunay3Connectivity::InvalidIndex)
			{
				if (DuplicateOf >= 0 && Connectivity.HasDuplicateTracking())
				{
					Connectivity.MarkDuplicateVertex(DuplicateOf, Vertex);
				}
				continue;
			}
			SearchTet = Insert<RealType>(Connectivity, Vertices, ContainingTet, Vertex);
			checkSlow(SearchTet.A != FDelaunay3Connectivity::InvalidIndex);
		}

		return true;
	}
}

bool FDelaunay3::Triangulate(TArrayView<const FVector3d> Vertices)
{
	Connectivity = MakePimpl<FDelaunay3Connectivity>();

	bool bSuccess = Delaunay3Internal::Triangulate<double>(RandomStream, *Connectivity, Vertices);
	return bSuccess;
}

bool FDelaunay3::Triangulate(TArrayView<const FVector3f> Vertices)
{
	Connectivity = MakePimpl<FDelaunay3Connectivity>();

	bool bSuccess = Delaunay3Internal::Triangulate<float>(RandomStream, *Connectivity, Vertices);
	return bSuccess;
}

TArray<FIndex4i> FDelaunay3::GetTetrahedraAsFIndex4i(bool bReverseOrientation) const
{
	TArray<FIndex4i> ToRet;
	if (Connectivity.IsValid())
	{
		Connectivity->EnumerateTrianglePerTet(
			[&ToRet, bReverseOrientation](FIndex3i Tri, int32 Vert)
			{
				if (bReverseOrientation)
				{
					ToRet.Emplace(Tri.A, Tri.B, Tri.C, Vert);
				}
				else
				{
					// Note: UE expects negative-sign orientation, so B,C swapped is 'not reversed'
					ToRet.Emplace(Tri.A, Tri.C, Tri.B, Vert);
				}
				return true;
			}, true
		);
	}
	return ToRet;
}

TArray<FIntVector4> FDelaunay3::GetTetrahedra(bool bReverseOrientation) const
{
	TArray<FIntVector4> ToRet;
	if (Connectivity.IsValid())
	{
		Connectivity->EnumerateTrianglePerTet(
			[&ToRet, bReverseOrientation](FIndex3i Tri, int32 Vert)
			{
				if (bReverseOrientation)
				{
					ToRet.Emplace(Tri.A, Tri.B, Tri.C, Vert);
				}
				else
				{
					// Note: UE expects negative-sign orientation, so B,C swapped is 'not reversed'
					ToRet.Emplace(Tri.A, Tri.C, Tri.B, Vert);
				}
				return true;
			}, true
		);
	}
	return ToRet;
}

bool FDelaunay3::IsDelaunay(TArrayView<const FVector3f> Vertices) const
{
	if (ensure(Connectivity.IsValid()))
	{
		return Delaunay3Internal::IsDelaunay<float>(*Connectivity, Vertices);
	}
	return false; // if no triangulation was performed, function should be been called
}

bool FDelaunay3::IsDelaunay(TArrayView<const FVector3d> Vertices) const
{
	if (ensure(Connectivity.IsValid()))
	{
		return Delaunay3Internal::IsDelaunay<double>(*Connectivity, Vertices);
	}
	return false; // if no triangulation was performed, function should not be called
}


} // end namespace UE::Geometry
} // end namespace UE