// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/DiTOrientedBox.h"
#include "Containers/Array.h"
#include "Math/Vector.h"
#include "BoxTypes.h"
#include "VectorTypes.h"
#include "IndexTypes.h"
#include "MathUtil.h"

namespace 
{ 
	// Different directions used by DiTO in computing interval bounds.   
	// Note these aren't normalized, but within each DiTO flavor the vectors have a consistent length so projections 
	// can be compared.  In the original source  DiTO 14 and 26 were questionable because the vectors were not of uniform length.
	template <typename RealType>
	TArray<UE::Math::TVector<RealType>>  DiTODirections(const UE::Geometry::EDiTO DiTO)
	{
		typedef UE::Math::TVector<RealType>   VectorType;
		typedef RealType    T;

		const double a =  0.5 * (FMath::Sqrt(5.) - 1.);
		switch (DiTO)
		{
			case UE::Geometry::EDiTO::DiTO_12:
			{
				// length Sqrt(5) * a  vectors
				return TArray<VectorType>({ VectorType(T(0),  T(1),  T(a)),
										    VectorType(T(0),  T(1), -T(a)),
											VectorType(T(1),  T(a),  T(0)),
											VectorType(T(1), -T(a),  T(0)),
											VectorType(T(a),  T(0),  T(1)),
											VectorType(T(a),  T(0), -T(1)) });
			}
			break;

			case  UE::Geometry::EDiTO::DiTO_14:
			{
				const double b = FMath::Sqrt(3.);
				// length Sqrt(3) vectors
				// note: in game engine gems vol 2.  b =  1. 
				return TArray<VectorType>({ VectorType(T(b), T(0), T(0)),
											VectorType(T(0), T(b), T(0)),
											VectorType(T(0), T(0), T(b)),
											VectorType(T(1), T(1), T(1)),
											VectorType(T(1), T(1),-T(1)),
											VectorType(T(1),-T(1), T(1)),
											VectorType(T(1),-T(1),-T(1)) });
			}
			break;

			case UE::Geometry::EDiTO::DiTO_20:
			{
				// length Sqrt(3) vectors
				return TArray<VectorType>({ VectorType(T(0),          T(a),  T(1. + a)),
											VectorType(T(0),          T(a), -T(1. + a)),
											VectorType(T(a),     T(1. + a),       T(0)),
											VectorType(T(a),    -T(1. + a),       T(0)),
											VectorType(T(1. + a),     T(0),       T(a)),
											VectorType(T(1. + a),     T(0),      -T(a)),
											VectorType(T(1),          T(1),       T(1)),
											VectorType(T(1),          T(1),      -T(1)),
											VectorType(T(1),         -T(1),       T(1)),
											VectorType(T(1),         -T(1),      -T(1)) });
			}
			break;

			case UE::Geometry::EDiTO::DiTO_26:
			{
				// length Sqrt(3) vectors.
				// note: in game engine gems vol 2.  b = c = 1. 
				const double b = FMath::Sqrt(3.);
				const double c = FMath::Sqrt(1.5);
				return TArray<VectorType>({ VectorType(T(b),  T(0),  T(0)),
											VectorType(T(0),  T(b),  T(0)),
											VectorType(T(0),  T(0),  T(b)),
											VectorType(T(1),  T(1),  T(1)),
											VectorType(T(1),  T(1), -T(1)),
											VectorType(T(1), -T(1),  T(1)),
											VectorType(T(1), -T(1), -T(1)),
											VectorType(T(c),  T(c),  T(0)),
											VectorType(T(c), -T(c),  T(0)),
											VectorType(T(c),  T(0),  T(c)),
											VectorType(T(c),  T(0), -T(c)),
											VectorType(T(0),  T(c),  T(c)),
											VectorType(T(0),  T(c), -T(c)) });
			}
			break;

			default:
			{
				checkSlow(0);
				return TArray<VectorType>();
			}
		}
	}

	// Expand an oriented box to contain given points.  
	template <typename RealType, typename GetPointFuncType>
	void ExpandBoxToContain(UE::Geometry::TOrientedBox3<RealType>& Box, const GetPointFuncType& GetPointFunc, int32 NumPoints)
	{
		typedef UE::Math::TVector<RealType> VectorType;

		RealType MinBounds[3] = { -Box.Extents[0], -Box.Extents[1], -Box.Extents[2] };
		RealType MaxBounds[3] = { Box.Extents[0],  Box.Extents[1],  Box.Extents[2] };

		for (int32 VID = 0; VID < NumPoints; ++VID)
		{
			const VectorType Point = GetPointFunc(VID);
			const VectorType InFramePoint = Box.Frame.ToFramePoint(Point);
			for (int32 i = 0; i < 3; ++i)
			{
				MinBounds[i] = TMathUtil<RealType>::Min(MinBounds[i], InFramePoint[i]);
				MaxBounds[i] = TMathUtil<RealType>::Max(MaxBounds[i], InFramePoint[i]);
			}
		}

		VectorType InFrameOffset;
		for (int32 i = 0; i < 3; ++i)
		{
			InFrameOffset[i] = (RealType)0.5 * (MaxBounds[i] + MinBounds[i]);

			// this would be the fastest way to update extents, but we have to do the slower second pass below instead.
			// if the way TOrientedBox3::Contains() is computed changes then this can be revisited.
			// Box.Extents[i]   = (RealType)0.5 * (MaxBounds[i] - MinBounds[i]); 
		}
		const VectorType UpdatedOrigin = Box.Frame.FromFramePoint(InFrameOffset);
		Box.Frame.Origin = UpdatedOrigin;

		// This second pass is done to ensure round-off errors match those in the test TOrientedBox3::Contains()
		for (int32 VID = 0; VID < NumPoints; ++VID)
		{
			const VectorType Point = GetPointFunc(VID);
			const VectorType InFramePoint = Box.Frame.ToFramePoint(Point);
			for (int32 i = 0; i < 3; ++i)
			{
				Box.Extents[i] = TMathUtil<RealType>::Max(Box.Extents[i], TMathUtil<RealType>::Abs(InFramePoint[i]));
			}
		}
	}

	template <typename RealType, typename GetPointFuncType>
	void ExpandBoxToContain(UE::Geometry::TAxisAlignedBox3<RealType>& Box, const GetPointFuncType& GetPointFunc, int32 NumPoints)
	{
		typedef UE::Math::TVector<RealType> VectorType;
		for (int32 VID = 0; VID < NumPoints; ++VID)
		{
			const VectorType Point = GetPointFunc(VID);
			Box.Contain(Point);
		}
	}

	// implementation of the DiTO algorithm
	template <typename RealType>
	UE::Geometry::TOrientedBox3<RealType> ComputeDiTOImpl(const TArray<UE::Math::TVector<RealType>>& SampleDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc)
	{
		typedef UE::Math::TVector<RealType> VectorType;
		
		const int32 NumSampleDir    = SampleDirections.Num();
		const int32 NumSamplePoints = 2 * NumSampleDir;

		
		// construct AABB for reference.
		const UE::Geometry::TAxisAlignedBox3<RealType> ReferenceAABBox = [&]
																		 {
																			 UE::Geometry::TAxisAlignedBox3<RealType> AABB;
																			 ExpandBoxToContain(AABB, GetPointFunc, NumPoints);
																			 return AABB;
																		 }();
		// if no sample directions were provided, just return the AABB
		if (NumSampleDir == 0)
		{
			return UE::Geometry::TOrientedBox3<RealType>(ReferenceAABBox);
		}

		struct FIntervalBounds
		{
			RealType Upper = -TMathUtilConstants<RealType>::MaxReal;
			RealType Lower =  TMathUtilConstants<RealType>::MaxReal;
			int32 UpperID = -1;
			int32 LowerID = -1;
		};

		// identify the extreme projected distance in each sample direction.
		const TArray<FIntervalBounds> DirectionalBounds = [&] 
														{
															TArray<FIntervalBounds> BoundsArray;
															BoundsArray.Reserve(NumSampleDir);
															for (int32 i = 0; i < NumSampleDir; ++i)
															{
																BoundsArray.AddDefaulted();
															}

															for (int32 VID = 0; VID < NumPoints; ++VID)
															{
																const VectorType Point = GetPointFunc(VID);
																for (int32 dir = 0; dir < NumSampleDir; ++dir)
																{
																	FIntervalBounds& Bounds = BoundsArray[dir];

																	const VectorType& Direction      = SampleDirections[dir];
																	const RealType ProjectedDistance = Direction.Dot(Point);
																	if (ProjectedDistance < Bounds.Lower)
																	{
																		Bounds.Lower   = ProjectedDistance;
																		Bounds.LowerID = VID;
																	}
																	if (ProjectedDistance > Bounds.Upper)
																	{
																		Bounds.Upper   = ProjectedDistance;
																		Bounds.UpperID = VID;
																	}

																}
															}

															return MoveTemp(BoundsArray);
														}();
		
	
		// create a vertex buffer that corresponds to the points with the extreme projected distances
		const TArray<VectorType> SamplePoints = [&]
												{
													TArray<VectorType> VertexBuffer;
													VertexBuffer.SetNumUninitialized(NumSamplePoints);
													for (int32 dir = 0; dir < NumSampleDir; ++dir)
													{
														const FIntervalBounds& Bounds = DirectionalBounds[dir];
														VertexBuffer[2 * dir]         = GetPointFunc(Bounds.LowerID);
														VertexBuffer[2 * dir + 1]     = GetPointFunc(Bounds.UpperID);
													}
													return MoveTemp(VertexBuffer);
												}();
		
		// identify first direction with largest interval
		int32 MajorDir = -1;
		RealType MajorDirExtent = -TMathUtilConstants<RealType>::MaxReal;
		for (int32 dir = 0; dir < NumSampleDir; ++dir)
		{
			const FIntervalBounds& Bounds = DirectionalBounds[dir];
			const RealType BoundsExtent   = Bounds.Upper - Bounds.Lower;
			if (BoundsExtent > MajorDirExtent)
			{
				MajorDirExtent = BoundsExtent;
				MajorDir       = dir;
			}
		}

		// possible early out 
		// if largest projected interval is very small: just use AABB
		if (MajorDirExtent < TMathUtilConstants<RealType>::ZeroTolerance)
		{
			return UE::Geometry::TOrientedBox3<RealType>(ReferenceAABBox);
		}

		// create base triangle with an edge connecting the two points with the most extreme projection (e.g MajorDir)
		// note, the third vertex will be filled in later as the sample point furthest from the infinite line containing this first edge
		UE::Geometry::FIndex3i BaseTri(2 * MajorDir, 2 * MajorDir + 1, -1);

		const VectorType e0 = SamplePoints[BaseTri[1]] - SamplePoints[BaseTri[0]];
		VectorType en0 = e0; en0.Normalize();

		// identify furthest sample point from the line connecting BaseTri[0] and BaseTri[1] (i.e. points with maximal projected separation).
		RealType MaxOrthogonalDistSqr = -TMathUtilConstants<RealType>::MaxReal;
		for (int32 sid = 0; sid < NumSamplePoints; ++sid)
		{
			const VectorType VecToSample         = SamplePoints[sid] - SamplePoints[BaseTri[0]];
			const VectorType OrthogonalComponent = VecToSample - en0.Dot(VecToSample) * en0;
			const RealType OrthogonalDistSqr     = OrthogonalComponent.SizeSquared();

			if (OrthogonalDistSqr > MaxOrthogonalDistSqr)
			{
				BaseTri[2]           = sid;
				MaxOrthogonalDistSqr = OrthogonalDistSqr;
			}
		}

		// base triangle normal
		const VectorType e1           = SamplePoints[BaseTri[2]] - SamplePoints[BaseTri[1]];
		VectorType BaseTriNormal      = en0.Cross(e1);
		const bool bSuccesfullNormal  = BaseTriNormal.Normalize(TMathUtilConstants<RealType>::ZeroTolerance);

		// possible early out 
		// if all the points are very close the major direction, just align a box around this direction
		// note: the construction of Frame will pick to other axis directions for us..
		if (!bSuccesfullNormal || MaxOrthogonalDistSqr < TMathUtilConstants<RealType>::ZeroTolerance)
		{
			const VectorType InitialOrigin(0, 0, 0);
			const VectorType SetZ(en0);
			const UE::Geometry::TFrame3<RealType> E0DirFrame(InitialOrigin, SetZ);
			UE::Geometry::TOrientedBox3<RealType> OrientedBox(E0DirFrame, VectorType(-TMathUtilConstants<RealType>::MaxReal));

			// expand the best oriented box with all the points 
			ExpandBoxToContain(OrientedBox, GetPointFunc, NumPoints);
			return OrientedBox;
		}

		// to complete the di-tet, find the extreme points relative to the base triangle
		const FIntervalBounds AboveBelowPoints = [&] 
												{
													FIntervalBounds Bounds;
													for (int32 sid = 0; sid < NumSamplePoints; ++sid)
													{
														const VectorType VecToSample   = SamplePoints[sid] - SamplePoints[2 * MajorDir];
														const RealType DistFromBaseTri = VecToSample.Dot(BaseTriNormal);
												
														if (DistFromBaseTri > Bounds.Upper)
														{
															Bounds.Upper   = DistFromBaseTri;
															Bounds.UpperID = sid;
														}
														if (DistFromBaseTri < Bounds.Lower)
														{
															Bounds.Lower   = DistFromBaseTri;
															Bounds.LowerID = sid;
														}
													}
													return Bounds;
												}();
		
		// index buffer for the DiTet constructed from base triangle and the furthest verts above/below the base tri
		const TArray<UE::Geometry::FIndex3i> DiTet = [&] 
													{
														TArray<UE::Geometry::FIndex3i> Mesh;
														Mesh.Reserve(7);
														Mesh.Add(BaseTri);

														// IDs relative to sample point array of the verts used in making the Di-Tet
														const int32 a = BaseTri[0];
														const int32 b = BaseTri[1];
														const int32 c = BaseTri[2];
														const int32 d = AboveBelowPoints.UpperID;
														const int32 e = AboveBelowPoints.LowerID;
														if (AboveBelowPoints.Upper > TMathUtilConstants<RealType>::ZeroTolerance)
														{

															Mesh.Emplace(a, b, d);
															Mesh.Emplace(b, c, d);
															Mesh.Emplace(c, a, d);
														}
														if (AboveBelowPoints.Lower < -TMathUtilConstants<RealType>::ZeroTolerance)
														{
															Mesh.Emplace(b, a, e);
															Mesh.Emplace(c, b, e);
															Mesh.Emplace(a, c, e);
														}
														return MoveTemp(Mesh);
													}();
		
		// for each edge and face, find the oriented bounding box that contains all the sample points
		// and identify the one with the smallest surface area.
		RealType MinSurfaceArea = TMathUtilConstants<RealType>::MaxReal;
		UE::Geometry::TOrientedBox3<RealType> BestOrientedBox;
		for (const UE::Geometry::FIndex3i& Tri : DiTet)
		{
			const VectorType edge[3] = { SamplePoints[Tri.B] - SamplePoints[Tri.A],
									     SamplePoints[Tri.C] - SamplePoints[Tri.B],
										 SamplePoints[Tri.A] - SamplePoints[Tri.C] };


			// note these return zero length vectors if normalize fails.
			const VectorType en[3] = { UE::Geometry::Normalized(edge[0]), UE::Geometry::Normalized(edge[1]), UE::Geometry::Normalized(edge[2]) };

			VectorType FaceNormal = en[0].Cross(-en[2]); 
			const bool bHasValidNormal  = FaceNormal.Normalize();

			if (bHasValidNormal)
			{
				for (int32 i = 0; i < 3; ++i)
				{
					const UE::Geometry::TFrame3<RealType> Frame(VectorType(0), en[i], FaceNormal, en[i].Cross(FaceNormal));
					// create OrientedBox and expand to hold all the sample points
					UE::Geometry::TOrientedBox3<RealType> OBox(Frame, VectorType(-TMathUtilConstants<RealType>::MaxReal));
					ExpandBoxToContain(OBox, [&SamplePoints](int32 j){return SamplePoints[j];}, NumSamplePoints);

					const RealType SurfaceArea = OBox.SurfaceArea();
					if (SurfaceArea < MinSurfaceArea)
					{
						MinSurfaceArea = SurfaceArea;
						BestOrientedBox = OBox;
					}
				}
			}
		}

		// expand the best oriented box with all the points
		ExpandBoxToContain(BestOrientedBox, GetPointFunc, NumPoints);

		// return the smaller of the OrientedBox and the AABB
		const RealType OBoxSurfaceArea = BestOrientedBox.SurfaceArea();
		const RealType AABBSurfaceArea = ReferenceAABBox.SurfaceArea();
		if (AABBSurfaceArea < OBoxSurfaceArea)
		{
			return UE::Geometry::TOrientedBox3<RealType>(ReferenceAABBox);
		}
		else
		{
			return BestOrientedBox;
		}
	}
}

template <typename RealType>
UE::Geometry::TOrientedBox3<RealType> UE::Geometry::ComputeOrientedBBox(const UE::Geometry::EDiTO DiTOType, const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc)
{
	typedef UE::Math::TVector<RealType> VectorType;
	
	const TArray<VectorType> SampleDirections = DiTODirections<RealType>(DiTOType);
	return ComputeDiTOImpl(SampleDirections, NumPoints, GetPointFunc);
}

template <typename RealType>
UE::Geometry::TOrientedBox3<RealType> UE::Geometry::ComputeOrientedBBox(const TArray<UE::Math::TVector<RealType>>& SampleDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc)
{
	return ComputeDiTOImpl(SampleDirections, NumPoints, GetPointFunc);
}

// explicit instantiations
namespace UE
{
	namespace Geometry
	{
		template TOrientedBox3<float> GEOMETRYCORE_API ComputeOrientedBBox<float>(const UE::Geometry::EDiTO DiTOType, const int32 NumPoints, TFunctionRef<UE::Math::TVector<float>(int32)> GetPointFunc);
		template TOrientedBox3<double> GEOMETRYCORE_API ComputeOrientedBBox<double>(const UE::Geometry::EDiTO DiTOType, const int32 NumPoints, TFunctionRef<UE::Math::TVector<double>(int32)> GetPointFunc);
		
		template TOrientedBox3<float> GEOMETRYCORE_API ComputeOrientedBBox<float>(const TArray<UE::Math::TVector<float>>& SampleDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<float>(int32)> GetPointFunc);
		template TOrientedBox3<double> GEOMETRYCORE_API ComputeOrientedBBox<double>(const TArray<UE::Math::TVector<double>>& SampleDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<double>(int32)> GetPointFunc);
	
	}
}