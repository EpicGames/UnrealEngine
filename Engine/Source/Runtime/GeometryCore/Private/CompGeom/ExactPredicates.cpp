// Copyright Epic Games, Inc. All Rights Reserved.

// Code to call exact predicates on vectors

#include "CompGeom/ExactPredicates.h"
#include "ThirdParty/ShewchukPredicatesInterface.h"


namespace UE
{
namespace Geometry
{
namespace ExactPredicates
{

void GlobalInit()
{
	ShewchukExactPredicates::exactinit();
	ShewchukExactPredicatesFloat::exactinit();
}

double Orient2DInexact(const double* pa, const double* pb, const double* pc)
{
	return ShewchukExactPredicates::orient2dfast(pa, pb, pc);
}

double Orient2D(const double* pa, const double* pb, const double* pc)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::orient2d(pa, pb, pc);
}

double Orient2DOrigin(double ax, double ay, double bx, double by)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::orient2d_origin(ax, ay, bx, by);
}

double Orient3DInexact(const double* PA, const double* PB, const double* PC, const double* PD)
{
	return ShewchukExactPredicates::orient3dfast(PA, PB, PC, PD);
}

double Orient3D(const double* PA, const double* PB, const double* PC, const double* PD)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::orient3d(PA, PB, PC, PD);
}

double Facing3D(const double* PA, const double* PB, const double* PC, const double* Direction)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::facing3d(PA, PB, PC, Direction);
}

double Facing2D(const double* PA, const double* PB, const double* Direction)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::facing2d(PA, PB, Direction);
}

double InCircleInexact(const double* PA, const double* PB, const double* PC, const double* PD)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::incirclefast(PA, PB, PC, PD);
}

double InCircle(const double* PA, const double* PB, const double* PC, const double* PD)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::incircle(PA, PB, PC, PD);
}

double InSphereInexact(const double* PA, const double* PB, const double* PC, const double* PD, const double* PE)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::inspherefast(PA, PB, PC, PD, PE);
}
double InSphere(const double* PA, const double* PB, const double* PC, const double* PD, const double* PE)
{
	checkSlow(ShewchukExactPredicates::IsExactPredicateDataInitialized());
	return ShewchukExactPredicates::insphere(PA, PB, PC, PD, PE);
}

float Orient2DInexact(const float* pa, const float* pb, const float* pc)
{
	return ShewchukExactPredicatesFloat::orient2dfast(pa, pb, pc);
}

float Orient2D(const float* pa, const float* pb, const float* pc)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::orient2d(pa, pb, pc);
}

float Orient2DOrigin(float ax, float ay, float bx, float by)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::orient2d_origin(ax, ay, bx, by);
}

float Orient3DInexact(const float* PA, const float* PB, const float* PC, const float* PD)
{
	return ShewchukExactPredicatesFloat::orient3dfast(PA, PB, PC, PD);
}

float Orient3D(const float* PA, const float* PB, const float* PC, const float* PD)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::orient3d(PA, PB, PC, PD);
}

float Facing3D(const float* PA, const float* PB, const float* PC, const float* Direction)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::facing3d(PA, PB, PC, Direction);
}

float Facing2D(const float* PA, const float* PB, const float* Direction)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::facing2d(PA, PB, Direction);
}

float InCircleInexact(const float* PA, const float* PB, const float* PC, const float* PD)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::incirclefast(PA, PB, PC, PD);
}

float InCircle(const float* PA, const float* PB, const float* PC, const float* PD)
{
	checkSlow(ShewchukExactPredicatesFloat::IsExactPredicateDataInitialized());
	return ShewchukExactPredicatesFloat::incircle(PA, PB, PC, PD);
}

}}} // namespace UE::Geometry::ExactPredicates