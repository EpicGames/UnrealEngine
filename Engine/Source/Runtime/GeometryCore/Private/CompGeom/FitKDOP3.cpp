// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/FitKDOP3.h"
#include "BoxTypes.h"
#include "Curve/CurveUtil.h"
#include "VectorTypes.h"
#include "VectorUtil.h"
#include "Templates/Function.h"

#include "CompGeom/ConvexHull3.h"

namespace UE::Private::FitKDop3Helpers
{
	using namespace UE::Geometry;

	template<typename RealType>
	bool FitKDopImpl3(TArrayView<const UE::Math::TVector<RealType>> PlaneDirections,
		const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc,
		TArray<UE::Math::TVector<RealType>>& OutVertices, TArray<UE::Math::TPlane<RealType>>* OptionalOutPlanes, RealType Epsilon, RealType VertexSnapDistance, RealType Inflate)
	{
		using namespace UE::Math;

		OutVertices.Reset();

		if (PlaneDirections.Num() < 4)
		{
			return false;
		}

		const int32 NumPlanes = PlaneDirections.Num();
		TAxisAlignedBox3<RealType> Bounds;

		TArray<RealType> PlaneDists;
		PlaneDists.Init(-TMathUtil<RealType>::MaxReal, NumPlanes);
		for (int32 PointIdx = 0; PointIdx < NumPoints; ++PointIdx)
		{
			TVector<RealType> Point = GetPointFunc(PointIdx);
			Bounds.Contain(Point);
			for (int32 DirIdx = 0; DirIdx < NumPlanes; ++DirIdx)
			{
				RealType Dist = Point.Dot(PlaneDirections[DirIdx]);
				PlaneDists[DirIdx] = FMath::Max(PlaneDists[DirIdx], Dist);
			}
		}

		for (RealType& Dist : PlaneDists)
		{
			Dist += Inflate;
		}

		TArray<TPlane<RealType>> LocalPlanes;
		TArray<TPlane<RealType>>* UsePlanes = OptionalOutPlanes ? OptionalOutPlanes : &LocalPlanes;
		UsePlanes->SetNumUninitialized(PlaneDists.Num());
		for (int32 DirIdx = 0; DirIdx < NumPlanes; ++DirIdx)
		{
			(*UsePlanes)[DirIdx] = TPlane<RealType>(PlaneDirections[DirIdx], PlaneDists[DirIdx]);
			if (!ensureMsgf((*UsePlanes)[DirIdx].Normalize(), TEXT("k-DOP fitting attempted with invalid (near-zero) search direction(s)")))
			{
				return false;
			}
		}

		TVector<RealType> Center = Bounds.Center();
		RealType BoundingSize = Bounds.DiagonalLength();
		RealType VertexSnapDistanceSq = VertexSnapDistance * VertexSnapDistance;
		TArray<TVector<RealType>> PlanePoly, ClippedPlanePoly; // Note: Re-used per clip plane
		for (int32 WorkPlaneIdx = 0; WorkPlaneIdx < NumPlanes; ++WorkPlaneIdx)
		{
			const TPlane<RealType>& WorkPlane = (*UsePlanes)[WorkPlaneIdx];
			TVector<RealType> AxisX, AxisY;
			VectorUtil::MakePerpVectors(WorkPlane.GetNormal(), AxisX, AxisY);

			TVector<RealType> CenterOnPlane = TVector<RealType>::PointPlaneProject(Center, WorkPlane);

			// create an initial polygon bounding the possible hull face
			PlanePoly.Reset();
			PlanePoly.Add(CenterOnPlane + AxisX * BoundingSize + AxisY * BoundingSize);
			PlanePoly.Add(CenterOnPlane - AxisX * BoundingSize + AxisY * BoundingSize);
			PlanePoly.Add(CenterOnPlane - AxisX * BoundingSize - AxisY * BoundingSize);
			PlanePoly.Add(CenterOnPlane + AxisX * BoundingSize - AxisY * BoundingSize);

			// clip the polygon vs all other planes
			for (int32 CutPlaneIdx = 0; CutPlaneIdx < NumPlanes; ++CutPlaneIdx)
			{
				if (CutPlaneIdx == WorkPlaneIdx)
				{
					continue;
				}

				const TPlane<RealType>& CutPlane = (*UsePlanes)[CutPlaneIdx];
				if (CurveUtil::ClipConvexToPlane<RealType, TVector<RealType>, true>(PlanePoly, CutPlane, ClippedPlanePoly))
				{
					Swap(PlanePoly, ClippedPlanePoly);
					ClippedPlanePoly.Reset();
				}
				if (PlanePoly.Num() < 3)
				{
					break;
				}
			}

			for (int32 PolyIdx = 0; PolyIdx < PlanePoly.Num(); ++PolyIdx)
			{
				bool bSnapped = false;
				for (int32 OutIdx = 0; !bSnapped && OutIdx < OutVertices.Num(); ++OutIdx)
				{
					bSnapped = TVector<RealType>::DistSquared(PlanePoly[PolyIdx], OutVertices[OutIdx]) < VertexSnapDistanceSq;
				}
				if (!bSnapped)
				{
					OutVertices.Add(PlanePoly[PolyIdx]);
				}
			}
		}

		if (OutVertices.Num() < 4)
		{
			return false;
		}

		TExtremePoints3<RealType> ExtremePoints(OutVertices.Num(), [&OutVertices](int32 Idx) {return OutVertices[Idx];}, [](int32) {return true;}, Epsilon);
		if (ExtremePoints.Dimension < 3)
		{
			return false;
		}

		return true;
	}
} // namespace UE::Private::FitKDop3Helpers

namespace UE::Geometry
{
	template<typename RealType>
	bool FitKDOPVertices3(
		TArrayView<const UE::Math::TVector<RealType>> PlaneDirections,
		const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc,
		TArray<UE::Math::TVector<RealType>>& OutVertices, TArray<UE::Math::TPlane<RealType>>* OptionalOutPlanes, RealType Epsilon, RealType VertexSnapDistance, RealType Inflate)
	{
		return UE::Private::FitKDop3Helpers::FitKDopImpl3(PlaneDirections,
			NumPoints, GetPointFunc, FilterFunc,
			OutVertices, OptionalOutPlanes,
			Epsilon, VertexSnapDistance, Inflate);
	}


	// explicit instantiations
	template bool GEOMETRYCORE_API FitKDOPVertices3<float>(
		TArrayView<const UE::Math::TVector<float>> PlaneDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<float>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc,
		TArray<UE::Math::TVector<float>>& OutVertices, TArray<UE::Math::TPlane<float>>* OptionalOutPlanes, float Epsilon, float VertexSnapDistance, float Inflate);
	template bool GEOMETRYCORE_API FitKDOPVertices3<double>(
		TArrayView<const UE::Math::TVector<double>> PlaneDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<double>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc,
		TArray<UE::Math::TVector<double>>& OutVertices, TArray<UE::Math::TPlane<double>>* OptionalOutPlanes, double Epsilon, double VertexSnapDistance, double Inflate);
	

} /// namespace UE::Geometry
