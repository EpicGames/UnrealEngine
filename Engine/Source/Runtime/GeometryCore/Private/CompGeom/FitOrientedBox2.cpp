// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/FitOrientedBox2.h"
#include "Containers/Array.h"
#include "Math/Vector.h"
#include "OrientedBoxTypes.h"
#include "VectorTypes.h"
#include "IndexTypes.h"
#include "MathUtil.h"
#include "CompGeom/ConvexHull2.h"
#include "CompGeom/ExactPredicates.h"

namespace UE
{
namespace Geometry
{

template<typename RealType>
TOrientedBox2<RealType> FitOrientedBox2ConvexHull(int32 NumPts, TFunctionRef<TVector2<RealType>(int32)> GetHullPt, TFunctionRef<RealType(RealType, RealType)> FitFn)
{
	// fallback to using the AABB in failure cases
	auto ComputeFallback = [NumPts, &GetHullPt]
	{
		TAxisAlignedBox2<RealType> AABB;
		for (int32 PtIdx = 0; PtIdx < NumPts; ++PtIdx)
		{
			AABB.Contain(GetHullPt(PtIdx));
		}
		return TOrientedBox2<RealType>(AABB);
	};

	if (NumPts < 3) // input has too few points to be a convex hull
	{
		return ComputeFallback();
	}

	// Extreme Indices ordered as ymin, xmax, ymax, xmin
	FIndex4i InitialExtreme(0, 0, 0, 0);

	// Function to walk to the farthest point in a given direction
	// For first pass, don't need looping or step count
	auto FindExtremePtInitial = [NumPts, &GetHullPt](TVector2<RealType> BasisVec, int Start, RealType& OutExtremeVal)
	{
		OutExtremeVal = BasisVec.Dot(GetHullPt(Start));
		int32 ExtremeIdx = Start;
		for (int32 Idx = Start + 1; Idx < NumPts; ++Idx)
		{
			RealType Val = BasisVec.Dot(GetHullPt(Idx));
			if (Val < OutExtremeVal)
			{
				return ExtremeIdx;
			}
			OutExtremeVal = Val;
			ExtremeIdx = Idx;
		}

		// For first pass, we should only need to test the first point again
		RealType LastVal = BasisVec.Dot(GetHullPt(0));
		if (LastVal < OutExtremeVal)
		{
			return ExtremeIdx;
		}
		OutExtremeVal = LastVal;
		return 0;
	};

	// Function to walk to the farthest point in a given direction, and report the point + number of steps taken to get there
	auto FindExtremePt = [NumPts, &GetHullPt](TVector2<RealType> BasisVec, int Start, RealType& OutExtremeVal, int32& OutSteps)
	{
		OutExtremeVal = BasisVec.Dot(GetHullPt(Start));
		OutSteps = 0;
		int32 ExtremeIdx = Start;
		for (int32 Idx = Start + 1; Idx < NumPts; ++Idx)
		{
			RealType Val = BasisVec.Dot(GetHullPt(Idx));
			if (Val < OutExtremeVal)
			{
				return ExtremeIdx;
			}
			OutExtremeVal = Val;
			ExtremeIdx = Idx;
			++OutSteps;
		}
		// same loop as above, but covering [0, Start)
		for (int32 Idx = 0; Idx < Start; ++Idx)
		{
			RealType Val = BasisVec.Dot(GetHullPt(Idx));
			if (Val < OutExtremeVal)
			{
				return ExtremeIdx;
			}
			OutExtremeVal = Val;
			ExtremeIdx = Idx;
			++OutSteps;
		}

		return ExtremeIdx;
	};

	auto GetEdge = [&GetHullPt, NumPts](int32 Idx)
	{
		const TVector2<RealType>& A = GetHullPt(Idx);
		int32 NextIdx = Idx + 1;
		const TVector2<RealType>& B = GetHullPt(NextIdx < NumPts ? NextIdx : 0);
		return B - A;
	};

	// Aligned with the first edge of the hull, find the 'extreme' points
	// in each direction, and use that to create an initial bounding box
	TVector2<RealType> ExtremesMax, ExtremesMin;
	InitialExtreme.A = 0;
	TVector2<RealType> Base = GetEdge(0);
	if (!ensure(Base.Normalize(0)))
	{
		return ComputeFallback(); // duplicate points -> not a valid hull, return the fallback bounds
	}
	ExtremesMin.Y = PerpCW(Base).Dot(GetHullPt(0)); // Extreme down point is the start point
	InitialExtreme.B = FindExtremePtInitial(Base, 0, ExtremesMax.X); // Extreme right point (relative to initial edge)
	InitialExtreme.C = FindExtremePtInitial(-PerpCW(Base), InitialExtreme.B, ExtremesMax.Y); // Extreme up point
	InitialExtreme.D = FindExtremePtInitial(-Base, InitialExtreme.C, ExtremesMin.X); // Extreme left point
	TAxisAlignedBox2<RealType> Range(-ExtremesMin, ExtremesMax);

	RealType BestScore = FitFn(Range.Width(), Range.Height());
	TVector2<RealType> BestBase = Base;
	TAxisAlignedBox2<RealType> BestRange = Range;

	// Now consider each edge in sequence, update the 'extreme' points in each direction to compute a new bounding box
	// Note this is a variant of rotating calipers algorithm, but rather than advance by the smallest rotation at each step, it instead
	// always advances the bottom edge one step, and advances the extreme points forward from their saved indices as needed
	FIndex4i LastOffsets = InitialExtreme;
	FIndex4i LastExtreme = InitialExtreme;
	for (int32 Idx = 1; Idx < NumPts; ++Idx)
	{
		Base = GetEdge(Idx);
		if (!ensure(Base.Normalize(0)))
		{
			return ComputeFallback(); // duplicate points -> not a valid hull, return the fallback bounds
		}

		TVector2<RealType> CurDir = PerpCW(Base);
		// first extreme point is the starting point
		ExtremesMin.Y = CurDir.Dot(GetHullPt(Idx));
		int32 CurSteps = 0;
		int32 CurIdx = Idx;
		RealType* RangeValues[3]{ &ExtremesMax.X, &ExtremesMax.Y, &ExtremesMin.X };
		for (int32 Turn = 0; Turn < 3; ++Turn)
		{
			CurDir = -PerpCW(CurDir);
			int32 LastOffset = LastOffsets[Turn + 1] - 1;
			if (LastOffset > CurSteps) // jump to the saved extreme point if we've not already passed it
			{
				CurSteps = LastOffset;
				CurIdx = LastExtreme[Turn + 1];
			}
			int32 TakenSteps = 0;
			CurIdx = LastExtreme[Turn + 1] = FindExtremePt(CurDir, CurIdx, *RangeValues[Turn], TakenSteps);
			CurSteps += TakenSteps;
			LastOffsets[Turn + 1] = CurSteps;
		}
		// Use the distances travelled in each direction to evaluate the bounding box for this edge
		Range = TAxisAlignedBox2<RealType>(-ExtremesMin, ExtremesMax);
		RealType Score = FitFn(Range.Width(), Range.Height());
		if (Score < BestScore)
		{
			BestScore = Score;
			BestBase = Base;
			BestRange = Range;
		}
	}

	TVector2<RealType> RangeCenter = BestRange.Center();
	TVector2<RealType> Center = RangeCenter.X * BestBase - RangeCenter.Y * PerpCW(BestBase);
	TOrientedBox2<RealType> ToRet(Center, BestBase, BestRange.Extents());
	return ToRet;
}

// Compute the best fit box for a simple polygon.
template <typename RealType>
TOrientedBox2<RealType> FitOrientedBox2SimplePolygon(TArrayView<const TVector2<RealType>> Polygon, TFunctionRef<RealType(RealType, RealType)> FitFn)
{
	int32 Num = Polygon.Num();
	if (!ensure(Num >= 3)) // fall back to an AABB if input has too few points to be a simple polygon
	{
		return TOrientedBox2<RealType>(TAxisAlignedBox2<RealType>(Polygon));
	}

	TConvexHull2<RealType> Hull;
	if (!Hull.SolveSimplePolygon(Num, [&Polygon](int32 Idx)
		{
			return Polygon[Idx];
		}, false))
	{
		// fall back to an AABB to gracefully fail when hull cannot be found
		return TOrientedBox2<RealType>(TAxisAlignedBox2<RealType>(Polygon));
	}

	const TArray<int32>& Indices = Hull.GetPolygonIndices();
	return FitOrientedBox2ConvexHull<RealType>(Indices.Num(), [&Indices, &Polygon](int32 Idx)
		{
			return Polygon[Indices[Idx]];
		}, FitFn);
}

template <typename RealType>
TOrientedBox2<RealType> FitOrientedBox2Points(TArrayView<const TVector2<RealType>> Points, TFunctionRef<RealType(RealType, RealType)> FitFn)
{
	if (Points.Num() < 2)
	{
		return TOrientedBox2<RealType>(TAxisAlignedBox2<RealType>(Points));
	}

	TConvexHull2<RealType> Hull;
	bool bHas2DHull = Hull.Solve(Points);
	if (!bHas2DHull)
	{
		if (Hull.GetDimension() == 0)
		{
			return TOrientedBox2<RealType>(Points[0], TVector2<RealType>(1, 0), TVector2<RealType>::ZeroVector);
		}
		else // Dimensions == 1
		{
			TLine2<RealType> Line = Hull.GetLine(Points);
			
			TInterval1<RealType> XInterval;
			for (const TVector2<RealType>& Pt : Points)
			{
				XInterval.Contain(Line.Project(Pt));
			}
			TVector2<RealType> Center = Line.PointAt(XInterval.Center());
			TVector2<RealType> Extents(XInterval.Extent(), TMathUtil<RealType>::ZeroTolerance);
			TOrientedBox2<RealType> LineBox(Center, Line.Direction, Extents);
			return LineBox;
		}
	}

	const TArray<int32>& Indices = Hull.GetPolygonIndices();
	return FitOrientedBox2ConvexHull<RealType>(Indices.Num(), [&Indices, &Points](int32 Idx)
		{
			return Points[Indices[Idx]];
		}, FitFn);
}

// explicit instantiations
template TOrientedBox2<float> GEOMETRYCORE_API FitOrientedBox2Points<float>(TArrayView<const TVector2<float>> Points, TFunctionRef<float(float, float)> FitFn);
template TOrientedBox2<double> GEOMETRYCORE_API FitOrientedBox2Points<double>(TArrayView<const TVector2<double>> Points, TFunctionRef<double(double, double)> FitFn);
template TOrientedBox2<float> GEOMETRYCORE_API FitOrientedBox2SimplePolygon<float>(TArrayView<const TVector2<float>> Points, TFunctionRef<float(float, float)> FitFn);
template TOrientedBox2<double> GEOMETRYCORE_API FitOrientedBox2SimplePolygon<double>(TArrayView<const TVector2<double>> Points, TFunctionRef<double(double, double)> FitFn);
template TOrientedBox2<float> GEOMETRYCORE_API FitOrientedBox2ConvexHull<float>(int32 NumPts, TFunctionRef<TVector2<float>(int32)> GetHullPt, TFunctionRef<float(float, float)> FitFn);
template TOrientedBox2<double> GEOMETRYCORE_API FitOrientedBox2ConvexHull<double>(int32 NumPts, TFunctionRef<TVector2<double>(int32)> GetHullPt, TFunctionRef<double(double, double)> FitFn);
	
}
}