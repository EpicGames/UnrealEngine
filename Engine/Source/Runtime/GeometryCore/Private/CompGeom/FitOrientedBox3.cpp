// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/FitOrientedBox3.h"
#include "Containers/Array.h"
#include "Math/Vector.h"
#include "OrientedBoxTypes.h"
#include "VectorTypes.h"
#include "IndexTypes.h"
#include "MathUtil.h"
#include "Spatial/PointHashGrid2.h"
#include "CompGeom/FitOrientedBox2.h"
#include "CompGeom/ConvexHull3.h"
#include "CompGeom/ExactPredicates.h"
#include "Async/ParallelTransformReduce.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{

namespace
{
	// Extract the "horizon" of the convex hull w/ Facing matching the Start triangle
	// Note that this is equivalent to the Horizon algorithm in ConvexHull3.h, but translated to have an explicit stack rather than recursion
	bool GetHorizon(const TArray<FIndex3i>& Triangles, const TArray<FIndex3i>& TriangleNeighbors, const TArray<bool>& Facing, int32 Start, TArray<int32>& OutHorizon)
	{
		check(Triangles.Num() == TriangleNeighbors.Num());
		check(Triangles.Num() == Facing.Num());

		struct FVisit
		{
			int32 Tri;
			int8 Edge;
		};

		auto CrossEdge = [&Triangles, &TriangleNeighbors](int32 Tri, int32 EdgeNum) -> FVisit
		{
			int32 NbrTri = TriangleNeighbors[Tri][EdgeNum];
			int32 SecondVert = Triangles[Tri][EdgeNum + 1 < 3 ? EdgeNum + 1 : 0];
			int8 NbrEdge = (int8)Triangles[NbrTri].IndexOf(SecondVert);

			return FVisit{ NbrTri, NbrEdge };
		};

		TArray<bool> Visited;
		Visited.SetNumZeroed(Triangles.Num());
		TArray<FVisit> VisitStack;
		VisitStack.Reserve(Triangles.Num() / 3);
		Visited[Start] = true;
		bool StartFacing = Facing[Start];
		VisitStack.Add(CrossEdge(Start, 2));
		VisitStack.Add(CrossEdge(Start, 1));
		VisitStack.Add(CrossEdge(Start, 0));

		while (VisitStack.Num())
		{
			FVisit Visit = VisitStack.Pop(EAllowShrinking::No);
			if (Facing[Visit.Tri] != StartFacing)
			{
				OutHorizon.Add(Triangles[Visit.Tri][Visit.Edge]);
				continue;
			}
			if (Visited[Visit.Tri])
			{
				continue;
			}
			Visited[Visit.Tri] = true;
			VisitStack.Add(CrossEdge(Visit.Tri, (Visit.Edge + 2) % 3));
			VisitStack.Add(CrossEdge(Visit.Tri, (Visit.Edge + 1) % 3));
		}

		return true;
	}

	// Helper to compute an oriented box from the axis-aligned box, as a fallback
	template <typename RealType>
	TOrientedBox3<RealType> GetAxisAligned(int32 NumPts, TFunctionRef<TVector<RealType>(int32)> GetPtFn, TFunctionRef<bool(int32)> FilterFn)
	{
		TAxisAlignedBox3<RealType> AABB;
		for (int32 Idx = 0; Idx < NumPts; ++Idx)
		{
			if (FilterFn(Idx))
			{
				AABB.Contain(GetPtFn(Idx));
			}
		}
		return TOrientedBox3<RealType>(AABB);
	}
}

template <typename RealType>
TOrientedBox3<RealType> OptimizeOrientedBox3Points(const TOrientedBox3<RealType>& InitialBox, int32 NumIterations, int32 NumPoints,
	TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> Filter, EBox3FitCriteria FitMethod, FProgressCancel* ProgressCancel)
{
	TArray<TVector2<RealType>> Projection;
	Projection.Reserve(NumPoints);
	TOrientedBox3<RealType> BestBox = InitialBox;
	RealType BestScore = (FitMethod == EBox3FitCriteria::Volume) ? BestBox.Volume() : BestBox.SurfaceArea();
	int32 BestBoxIter = -1; // iteration that BestBox was updated
	for (int32 Iter = 0; Iter < NumIterations; ++Iter)
	{
		if (ProgressCancel && ProgressCancel->Cancelled())
		{
			return BestBox;
		}

		if (Iter > BestBoxIter + 3)
		{
			return BestBox;
		}
		int32 Axis = Iter % 3;
		TVector<RealType> FixedAxis = BestBox.GetAxis(Axis);
		int32 BasisXIdx = (Axis + 1) % 3;
		int32 BasisYIdx = (Axis + 2) % 3;
		TVector<RealType> BasisX = BestBox.GetAxis(BasisXIdx);
		TVector<RealType> BasisY = BestBox.GetAxis(BasisYIdx);

		Projection.Reset();
		for (int32 PtIdx = 0; PtIdx < NumPoints; ++PtIdx)
		{
			if (Filter(PtIdx))
			{
				TVector<RealType> Pt = GetPointFunc(PtIdx);
				Projection.Emplace(Pt.Dot(BasisX), Pt.Dot(BasisY));
			}
		}
		RealType Depth = BestBox.Extents[Axis] * 2;
		using TFitFn = TFunction<RealType(RealType, RealType)>;
		TFitFn Score2 = FitMethod == EBox3FitCriteria::Volume ?
			TFitFn([](RealType Width, RealType Height) -> RealType
				{
					// for volume, we can just use area when finding the best 2D box
					return Width * Height;
				}) :
			TFitFn([Depth](RealType Width, RealType Height) -> RealType
				{
					// for surface area, we also need to use the depth in the 2D box search
					return Width * Height + (Width + Height) * Depth;
				});
		TOrientedBox2<RealType> ProjBox = FitOrientedBox2Points<RealType>(Projection, Score2);

		// rotate the solution so we don't arbitrarily swap or flip the axes
		if (FMath::Abs(ProjBox.UnitAxisX.X) < FMath::Abs(ProjBox.UnitAxisX.Y))
		{
			ProjBox.UnitAxisX = PerpCW(ProjBox.UnitAxisX);
			Swap(ProjBox.Extents.X, ProjBox.Extents.Y);
		}
		if (ProjBox.UnitAxisX.X < 0)
		{
			ProjBox.UnitAxisX = -ProjBox.UnitAxisX;
		}

		TVector<RealType> Extents = BestBox.Extents;
		Extents[BasisXIdx] = ProjBox.Extents.X;
		Extents[BasisYIdx] = ProjBox.Extents.Y;
		TAxisAlignedBox3<RealType> SizeBox(-Extents, Extents);
		RealType Score = FitMethod == EBox3FitCriteria::Volume ? SizeBox.Volume() : SizeBox.SurfaceArea();
		if (Score < BestScore)
		{
			BestScore = Score;
			BestBoxIter = Iter;
			RealType CenterOnFixedAxis = FixedAxis.Dot(BestBox.Center());
			// Convert result to an oriented box
			TVector2<RealType> ProjC = ProjBox.Center();
			TVector<RealType> Center = FixedAxis * CenterOnFixedAxis + BasisX * ProjC.X + BasisY * ProjC.Y;
			TVector<RealType> Axes[3];
			Axes[Axis] = FixedAxis;
			TVector2<RealType> ProjAxisX = ProjBox.AxisX(), ProjAxisY = ProjBox.AxisY();
			Axes[BasisXIdx] = ProjAxisX.X * BasisX + ProjAxisX.Y * BasisY;
			Axes[BasisYIdx] = ProjAxisY.X * BasisX + ProjAxisY.Y * BasisY;
			TFrame3<RealType> Frame(Center, Axes[0], Axes[1], Axes[2]);
			BestBox = TOrientedBox3<RealType>(Frame, Extents);
		}
	}

	return BestBox;
}

template <typename RealType>
TOrientedBox3<RealType> FitOrientedBox3Points(int32 NumPts, TFunctionRef<TVector<RealType>(int32)> GetPtFn, TFunctionRef<bool(int32)> FilterFn, EBox3FitCriteria FitMethod, RealType SameNormalTolerance, FProgressCancel* ProgressCancel)
{
	// For failure cases, we fall back to an axis-aligned box instead
	auto GetFallbackBox = [NumPts, &GetPtFn, &FilterFn]() -> TOrientedBox3<RealType>
	{
		return GetAxisAligned(NumPts, GetPtFn, FilterFn);
	};

	TConvexHull3<RealType> Hull;
	Hull.bSaveTriangleNeighbors = true;
	bool bHasHull = Hull.Solve(NumPts, GetPtFn, FilterFn);
	if (!bHasHull)
	{
		// Convex Hull failure typically means the input didn't span all 3 dimensions
		// We fit in the reduced space
		if (Hull.GetDimension() == 0)
		{
			return GetFallbackBox();
		}
		else if (Hull.GetDimension() == 1)
		{
			TLine3<RealType> Line = Hull.GetLine();
			TVector<RealType> Dir = Line.Direction;
			TInterval1<RealType> ZInterval;
			for (int32 Idx = 0; Idx < NumPts; ++Idx)
			{
				if (FilterFn(Idx))
				{
					ZInterval.Contain(Line.Project(GetPtFn(Idx)));
				}
			}
			TVector<RealType> Center = Line.PointAt(ZInterval.Center());
			TFrame3<RealType> Frame(Center, Dir);
			TVector<RealType> Extents(TMathUtil<RealType>::ZeroTolerance, TMathUtil<RealType>::ZeroTolerance, ZInterval.Extent());
			return TOrientedBox3<RealType>(Frame, Extents);
		}
		else if (Hull.GetDimension() == 2)
		{
			TPlane3<RealType> Plane = Hull.GetPlane();
			TVector<RealType> PtOnPlane = Plane.Normal * Plane.Constant;
			TVector<RealType> BasisX, BasisY;
			VectorUtil::MakePerpVectors(Plane.Normal, BasisX, BasisY);
			// TODO: Should actually use FitOrientedBox2Points here
			TAxisAlignedBox2<RealType> FlatBounds;
			for (int32 Idx = 0; Idx < NumPts; ++Idx)
			{
				if (FilterFn(Idx))
				{
					TVector<RealType> FromOrigin = GetPtFn(Idx) - PtOnPlane;
					FlatBounds.Contain(TVector2<RealType>(BasisX.Dot(FromOrigin), BasisY.Dot(FromOrigin)));
				}
			}
			TVector2<RealType> FlatCenter = FlatBounds.Center();
			TVector<RealType> Center = PtOnPlane + BasisX * FlatCenter.X + BasisY * FlatCenter.Y;
			TFrame3<RealType> Frame(Center, BasisX, BasisY, Plane.Normal);
			TVector<RealType> Extents(FlatBounds.Extents().X, FlatBounds.Extents().Y, TMathUtil<RealType>::ZeroTolerance);
			TOrientedBox3<RealType> MinBox(Frame, Extents);
			if (FitMethod == EBox3FitCriteria::SurfaceArea)
			{
				// The AABB can beat the point-aligned AABB for surface area; if so, we return that instead
				// TODO: revisit whether this is needed if we use FitOrientedBox2Points to optimize the planar fit
				TOrientedBox3<RealType> FallbackBox = GetFallbackBox();
				if (FallbackBox.SurfaceArea() < MinBox.SurfaceArea())
				{
					return FallbackBox;
				}
			}
			return MinBox;
		}
		// failed w/ Dimension of 3 -- fallback to AABB, unclear what has happened
		return GetFallbackBox();
	}

	const TArray<FIndex3i>& Tris = Hull.GetTriangles();
	const TArray<FIndex3i>& TriNbrs = Hull.GetTriangleNeighbors();
	RealType BestScore = -1;
	TOrientedBox3<RealType> BestBox;

	// Cache the vertex positions that are on the hull,
	// to be used to find extremal points for each sampled direction, below
	TSet<int32> UsedVertSet;
	for (const FIndex3i& Tri : Tris)
	{
		UsedVertSet.Add(Tri.A);
		UsedVertSet.Add(Tri.B);
		UsedVertSet.Add(Tri.C);
	}
	TArray<TVector<RealType>> UsedVerts;
	UsedVerts.Reserve(UsedVertSet.Num());
	for (int32 ID : UsedVertSet)
	{
		UsedVerts.Add(GetPtFn(ID));
	}
	UsedVertSet.Empty();
	
	TArray<TVector<RealType>> Normals;
	Normals.Reserve(Tris.Num() + 1);
	Normals.Emplace(RealType(0), RealType(0), RealType(1)); // Always insert a major axis normal, to fit at least as well as the AABB
	if (SameNormalTolerance > 0)
	{
		// filter normals for duplicates
		TPointHashGrid2<int32, RealType> NormalsHash(SameNormalTolerance * 4, -1);
		NormalsHash.InsertPointUnsafe(0, TVector2<RealType>::ZeroVector); // Insert the major axis normal

		for (int32 TriIdx = 0; TriIdx < Tris.Num(); TriIdx++)
		{
			FIndex3i T = Tris[TriIdx];
			TVector<RealType> N = VectorUtil::Normal(GetPtFn(T.A), GetPtFn(T.B), GetPtFn(T.C));
			if (N.Z < 0) // always put the normal in the positive +Z hemisphere (opposite normals have the same horizon)
			{
				N = -N;
			}
			TVector2<RealType> HashN(N.X, N.Y);
			TPair<int32, RealType> FoundNear = NormalsHash.FindNearestInRadius(HashN, SameNormalTolerance, [&Normals, &N](const int32& NormalIdx)
				{
					// Note 3D distance is strictly bigger than the 2D distance in the projected space used for hashing
					// so we may consider normals that are outside the target radius, but shouldn't miss any normals due to the projection
					return TVector<RealType>::DistSquared(Normals[NormalIdx], N);
				});
			if (FoundNear.Key == -1)
			{
				int32 Idx = Normals.Add(N);
				NormalsHash.InsertPointUnsafe(Idx, HashN);
			}
			// Note there's a chance to miss matches near Z=0, if the 'flip to positive hemisphere' is down to floating point error
			// In theory we could try to account for this, but for now it seems fine to just allow the extra sample of that direction
		}
	}
	else
	{
		for (int32 TriIdx = 0; TriIdx < Tris.Num(); TriIdx++)
		{
			FIndex3i T = Tris[TriIdx];
			TVector<RealType> N = VectorUtil::Normal(GetPtFn(T.A), GetPtFn(T.B), GetPtFn(T.C));
			Normals.Add(N);
		}
	}

	struct FResult
	{
		RealType Score;
		TOrientedBox3<RealType> Box;
	};
	FResult DefaultInit{ TMathUtil<RealType>::MaxReal, TOrientedBox3<RealType>() };
	bool bSingleThread = Tris.Num() < 1000; // TODO: test to find the threshold at which threading is helpful
	int64 TasksToUse = bSingleThread ? 1 : 64;

	if (ProgressCancel && ProgressCancel->Cancelled())
	{
		return TOrientedBox3<RealType>();
	}

	auto FitBoxForDirection = [&GetPtFn, &Normals, &Tris, &TriNbrs, &UsedVerts, &FitMethod, bSingleThread, &ProgressCancel](int32 NormalIdx) -> FResult
	{
		if (ProgressCancel && ProgressCancel->Cancelled())
		{
			return FResult { TMathUtil<RealType>::MaxReal, TOrientedBox3<RealType>() };
		}

		TVector<RealType> N = Normals[NormalIdx];
		TArray<bool> Facing;
		Facing.SetNumUninitialized(Tris.Num());
		ParallelFor(Tris.Num(), [&Tris, &GetPtFn, &Facing, N](int32 Idx)
			{
				FIndex3i T = Tris[Idx];
				Facing[Idx] = ExactPredicates::Facing3<RealType>(GetPtFn(T.A), GetPtFn(T.B), GetPtFn(T.C), N) > 0;
			}, bSingleThread);

		// Compute the range of the hull in the normal direction, as the depth of the bounding box
		TInterval1<RealType> Interval;
		for (TVector<RealType> Pt : UsedVerts)
		{
			Interval.Contain(Pt.Dot(N));
		}
		RealType Depth = (Interval.Max - Interval.Min);

		// Walk the hull mesh to extract the boundary of its projection
		// Note because we use an exact predicate to determine facing, any starting tri should be fine, so we always start from tri 0
		// (Otherwise, it would be safer to start from a tri that is aligned with the projection direction)
		TArray<int32> Horizon;
		GetHorizon(Tris, TriNbrs, Facing, 0, Horizon);

		// Project the hull boundary to a 2D hull
		TArray<TVector2<RealType>> ProjBoundary;
		ProjBoundary.Reserve(Horizon.Num());
		TVector<RealType> BasisX, BasisY;
		VectorUtil::MakePerpVectors(N, BasisX, BasisY);
		for (int32 HorizIdx : Horizon)
		{
			TVector<RealType> Pt = GetPtFn(HorizIdx);
			ProjBoundary.Emplace(BasisX.Dot(Pt), BasisY.Dot(Pt));
		}
		
		using TFitFn = TFunction<RealType(RealType, RealType)>;
		TFitFn Score2 = FitMethod == EBox3FitCriteria::Volume ? 
			TFitFn([](RealType Width, RealType Height) -> RealType
			{
				// for volume, we can just use area when finding the best 2D box
				return Width * Height;
			}) :
			TFitFn([Depth](RealType Width, RealType Height) -> RealType
			{
				// for surface area, we also need to use the depth in the 2D box search
				return Width * Height + (Width + Height) * Depth;
			});

		// Fit a rectangle to the projected hull.  (Treat it as a simple polygon in case projection made it not quite convex.)
		TOrientedBox2<RealType> ProjBox = FitOrientedBox2SimplePolygon<RealType>(ProjBoundary, Score2);
		TVector<RealType> Extents(ProjBox.Extents.X, ProjBox.Extents.Y, Interval.Extent());
		TAxisAlignedBox3<RealType> SizeBox(-Extents, Extents);
		RealType Score = FitMethod == EBox3FitCriteria::Volume ? SizeBox.Volume() : SizeBox.SurfaceArea();

		// Convert result to an oriented box
		TVector2<RealType> ProjC = ProjBox.Center();
		TVector<RealType> Center = N * Interval.Center() + BasisX * ProjC.X + BasisY * ProjC.Y;
		TVector2<RealType> ProjAxisX = ProjBox.AxisX(), ProjAxisY = ProjBox.AxisY();
		TVector<RealType> AlignX = ProjAxisX.X * BasisX + ProjAxisX.Y * BasisY;
		TVector<RealType> AlignY = ProjAxisY.X * BasisX + ProjAxisY.Y * BasisY;
		TFrame3<RealType> Frame(Center, AlignX, AlignY, N);

		return FResult{ Score,TOrientedBox3<RealType>(Frame, Extents) };
	};

	// Reduce for the below TransformReduce -- return the best-scoring result
	auto TakeMinBox = [](FResult A, FResult B) -> FResult
	{
		if (A.Score < B.Score)
		{
			return A;
		}
		return B;
	};

	if (ProgressCancel && ProgressCancel->Cancelled())
	{
		return TOrientedBox3<RealType>();
	}

	FResult BestResult = ParallelTransformReduce<int32, FResult, TFunctionRef<FResult(int32)>, TFunctionRef<FResult(FResult, FResult)>>
		(Normals.Num(), DefaultInit, FitBoxForDirection, TakeMinBox, TasksToUse);

	return BestResult.Box;
}

// explicit instantiations
template TOrientedBox3<float> GEOMETRYCORE_API FitOrientedBox3Points<float>(int32 NumPts, TFunctionRef<TVector<float>(int32)> GetPtFn, 
	TFunctionRef<bool(int32)> FilterFn, EBox3FitCriteria FitMethod, float SameNormalTolerance, FProgressCancel* ProgressCancel);
template TOrientedBox3<double> GEOMETRYCORE_API FitOrientedBox3Points<double>(int32 NumPts, TFunctionRef<TVector<double>(int32)> GetPtFn,
	TFunctionRef<bool(int32)> FilterFn, EBox3FitCriteria FitMethod, double SameNormalTolerance, FProgressCancel* ProgressCancel);

template TOrientedBox3<float> GEOMETRYCORE_API OptimizeOrientedBox3Points(const TOrientedBox3<float>& InitialBox, int32 NumIterations, int32 NumPoints,
	TFunctionRef<TVector<float>(int32)> GetPointFunc, TFunctionRef<bool(int32)> Filter, EBox3FitCriteria FitMethod, FProgressCancel* ProgressCancel);
template TOrientedBox3<double> GEOMETRYCORE_API OptimizeOrientedBox3Points(const TOrientedBox3<double>& InitialBox, int32 NumIterations, int32 NumPoints,
	TFunctionRef<TVector<double>(int32)> GetPointFunc, TFunctionRef<bool(int32)> Filter, EBox3FitCriteria FitMethod, FProgressCancel* ProgressCancel);


}
}
