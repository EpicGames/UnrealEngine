// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/PolygonTriangulation.h"
#include "TriangleTypes.h"

using namespace UE::Geometry;

// explicit instantiations
namespace PolygonTriangulation
{
	template GEOMETRYCORE_API void TriangulateSimplePolygon<float>(const TArray<TVector2<float>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill);
	template GEOMETRYCORE_API void TriangulateSimplePolygon<double>(const TArray<TVector2<double>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill);

	template GEOMETRYCORE_API float ComputePolygonPlane<float>(const TArray<TVector<float>>& VertexPositions, TVector<float>& NormalOut, TVector<float>& PlanePointOut);
	template GEOMETRYCORE_API double ComputePolygonPlane<double>(const TArray<TVector<double>>& VertexPositions, TVector<double>& NormalOut, TVector<double>& PlanePointOut);

	template GEOMETRYCORE_API void TriangulateSimplePolygon<float>(const TArray<TVector<float>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill);
	template GEOMETRYCORE_API void TriangulateSimplePolygon<double>(const TArray<TVector<double>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill);
}




//
// Triangulate using ear clipping
// This is based on the 3D triangulation code from MeshDescription.cpp, simplified for 2D polygons
// 
template<typename T>
void PolygonTriangulation::TriangulateSimplePolygon(const TArray<TVector2<T>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill)
{
	struct Local
	{
		static inline bool IsTriangleFlipped(T OrientationSign, const TVector2<T>& VertexPositionA, const TVector2<T>& VertexPositionB, const TVector2<T>& VertexPositionC)
		{
			T TriSignedArea = TTriangle2<T>::SignedArea(VertexPositionA, VertexPositionB, VertexPositionC);
			return TriSignedArea * OrientationSign < 0;
		}
	};
	
	
	OutTriangles.Reset();

	// Polygon must have at least three vertices/edges, or result is empty
	int32 PolygonVertexCount = VertexPositions.Num();
	if (PolygonVertexCount < 3)
	{
		return;
	}


	// compute signed area of polygon
	double PolySignedArea2 = 0;
	for (int i = 0; i < PolygonVertexCount; ++i)
	{
		const TVector2<T>& v1 = VertexPositions[i];
		const TVector2<T>& v2 = VertexPositions[(i + 1) % PolygonVertexCount];
		PolySignedArea2 += v1.X*v2.Y - v1.Y*v2.X;
	}

	bool bIsClockwise = PolySignedArea2 < 0;
	T OrientationSign = (bIsClockwise) ? -T(1) : T(1);




	// If perimeter has 3 vertices, just copy content of perimeter out 
	if (PolygonVertexCount == 3)
	{
		OutTriangles.Add(bOrientAsHoleFill ? FIndex3i(0, 2, 1) : FIndex3i(0, 1, 2));
		return;
	}

	// Make a simple linked list array of the previous and next vertex numbers, for each vertex number
	// in the polygon.  This will just save us having to iterate later on.
	TArray<int32> PrevVertexNumbers, NextVertexNumbers;

	PrevVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);
	NextVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);

	for (int32 VertexNumber = 0; VertexNumber < PolygonVertexCount; ++VertexNumber)
	{
		PrevVertexNumbers[VertexNumber] = VertexNumber - 1;
		NextVertexNumbers[VertexNumber] = VertexNumber + 1;
	}
	PrevVertexNumbers[0] = PolygonVertexCount - 1;
	NextVertexNumbers[PolygonVertexCount - 1] = 0;


	int32 EarVertexNumber = 0;
	int32 EarTestCount = 0;
	for (int32 RemainingVertexCount = PolygonVertexCount; RemainingVertexCount >= 3; )
	{
		bool bIsEar = true;

		// If we're down to only a triangle, just treat it as an ear.  Also, if we've tried every possible candidate
		// vertex looking for an ear, go ahead and just treat the current vertex as an ear.  This can happen when 
		// vertices are collinear or other degenerate cases.
		if (RemainingVertexCount > 3 && EarTestCount < RemainingVertexCount)
		{
			const TVector2<T>& PrevVertexPosition = VertexPositions[PrevVertexNumbers[EarVertexNumber]];
			const TVector2<T>& EarVertexPosition = VertexPositions[EarVertexNumber];
			const TVector2<T>& NextVertexPosition = VertexPositions[NextVertexNumbers[EarVertexNumber]];

			// Figure out whether the potential ear triangle is facing the same direction as the polygon
			// itself.  If it's facing the opposite direction, then we're dealing with a concave triangle
			// and we'll skip it for now.
			if (!Local::IsTriangleFlipped(
				OrientationSign, PrevVertexPosition, EarVertexPosition, NextVertexPosition))
			{
				int32 TestVertexNumber = NextVertexNumbers[NextVertexNumbers[EarVertexNumber]];

				do
				{
					// Test every other remaining vertex to make sure that it doesn't lie inside our potential ear
					// triangle.  If we find a vertex that's inside the triangle, then it cannot actually be an ear.
					const TVector2<T>& TestVertexPosition = VertexPositions[TestVertexNumber];
					if (TTriangle2<T>::IsInside(PrevVertexPosition, EarVertexPosition, NextVertexPosition, TestVertexPosition))
					{
						bIsEar = false;
						break;
					}

					TestVertexNumber = NextVertexNumbers[TestVertexNumber];
				} while (TestVertexNumber != PrevVertexNumbers[EarVertexNumber]);
			}
			else
			{
				bIsEar = false;
			}
		}

		if (bIsEar)
		{
			// OK, we found an ear!  Let's save this triangle in our output buffer.
			{
				int32 A = PrevVertexNumbers[EarVertexNumber]
					, B = EarVertexNumber
					, C = NextVertexNumbers[EarVertexNumber];
				OutTriangles.Add(bOrientAsHoleFill ? FIndex3i(A, C, B) : FIndex3i(A, B, C));
			}

			// Update our linked list.  We're effectively cutting off the ear by pointing the ear vertex's neighbors to
			// point at their next sequential neighbor, and reducing the remaining vertex count by one.
			{
				NextVertexNumbers[PrevVertexNumbers[EarVertexNumber]] = NextVertexNumbers[EarVertexNumber];
				PrevVertexNumbers[NextVertexNumbers[EarVertexNumber]] = PrevVertexNumbers[EarVertexNumber];
				--RemainingVertexCount;
			}

			// Move on to the previous vertex in the list, now that this vertex was cut
			EarVertexNumber = PrevVertexNumbers[EarVertexNumber];

			EarTestCount = 0;
		}
		else
		{
			// The vertex is not the ear vertex, because it formed a triangle that either had a normal which pointed in the opposite direction
			// of the polygon, or at least one of the other polygon vertices was found to be inside the triangle.  Move on to the next vertex.
			EarVertexNumber = NextVertexNumbers[EarVertexNumber];

			// Keep track of how many ear vertices we've tested, so that if we exhaust all remaining vertices, we can
			// fall back to clipping the triangle and adding it to our mesh anyway.  This is important for degenerate cases.
			++EarTestCount;
		}
	}

	check(OutTriangles.Num() > 0);
}





template<typename T>
T PolygonTriangulation::ComputePolygonPlane(const TArray<TVector<T>>& VertexPositions, TVector<T>& PlaneNormalOut, TVector<T>& PlanePointOut)
{
	// NOTE: This polygon plane computation code is partially based on the implementation of "Newell's method" from Real-Time
	//       Collision Detection by Christer Ericson, published by Morgan Kaufmann Publishers, (c) 2005 Elsevier Inc

	PlaneNormalOut = TVector<T>::Zero();
	PlanePointOut = TVector<T>::Zero();

	// Use 'Newell's Method' to compute a robust 'best fit' plane from the vertices of this polygon
	const int32 NumVertices = VertexPositions.Num();
	for (int32 VertexNumberI = NumVertices - 1, VertexNumberJ = 0; VertexNumberJ < NumVertices; VertexNumberI = VertexNumberJ++)
	{
		const TVector<T>& PositionI = VertexPositions[VertexNumberI];
		const TVector<T>& PositionJ = VertexPositions[VertexNumberJ];
		PlanePointOut += PositionJ;
		PlaneNormalOut.X += (PositionJ.Y - PositionI.Y) * (PositionI.Z + PositionJ.Z);
		PlaneNormalOut.Y += (PositionJ.Z - PositionI.Z) * (PositionI.X + PositionJ.X);
		PlaneNormalOut.Z += (PositionJ.X - PositionI.X) * (PositionI.Y + PositionJ.Y);
	}

	PlanePointOut /= (T)NumVertices;
	return static_cast<T>(0.5) * Normalize(PlaneNormalOut);
}






/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
static bool VectorsOnSameSide(const FVector& Vec, const FVector& A, const FVector& B, const float SameSideDotProductEpsilon)
{
	const FVector CrossA = FVector::CrossProduct(Vec, A);
	const FVector CrossB = FVector::CrossProduct(Vec, B);
	double DotWithEpsilon = (double)SameSideDotProductEpsilon + FVector::DotProduct(CrossA, CrossB);
	return !(DotWithEpsilon < 0.0);
}


/** Util to see if P lies within triangle created by A, B and C. */
static bool PointInTriangle(const FVector& A, const FVector& B, const FVector& C, const FVector& P, const float InsideTriangleDotProductEpsilon)
{
	// Cross product indicates which 'side' of the vector the point is on
	// If its on the same side as the remaining vert for all edges, then its inside.	
	return (VectorsOnSameSide(B - A, P - A, C - A, InsideTriangleDotProductEpsilon) &&
		VectorsOnSameSide(C - B, P - B, A - B, InsideTriangleDotProductEpsilon) &&
		VectorsOnSameSide(A - C, P - C, B - C, InsideTriangleDotProductEpsilon));
}


//
// Triangulate using ear clipping
// This is based on the 3D triangulation code from MeshDescription.cpp, simplified for 2D polygons
// 
template<typename T>
void PolygonTriangulation::TriangulateSimplePolygon(const TArray<TVector<T>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill)
{
	struct Local3
	{
		static bool IsTriangleFlipped(const TVector<T>& ReferenceNormal, const TVector<T>& VertexPositionA, const TVector<T>& VertexPositionB, const TVector<T>& VertexPositionC)
		{
			TVector<T> TriangleNormal = VectorUtil::Normal(VertexPositionA, VertexPositionB, VertexPositionC);
			return TriangleNormal.Dot(ReferenceNormal) <= (T)0;
		}

		static bool VectorsOnSameSide(const TVector<T>& Vec, const TVector<T>& A, const TVector<T>& B, T SameSideDotProductEpsilon)
		{
			TVector<T> CrossA = Vec.Cross(A);
			TVector<T> CrossB = Vec.Cross(B);
			T DotWithEpsilon = SameSideDotProductEpsilon + CrossA.Dot(CrossB);
			return DotWithEpsilon >= 0; // !FMath::IsNegativeFloat(DotWithEpsilon);
		}

		static bool PointInTriangle(const TVector<T>& A, const TVector<T>& B, const TVector<T>& C, const TVector<T>& P, const T InsideTriangleDotProductEpsilon)
		{
			return (VectorsOnSameSide(B - A, P - A, C - A, InsideTriangleDotProductEpsilon) &&
				VectorsOnSameSide(C - B, P - B, A - B, InsideTriangleDotProductEpsilon) &&
				VectorsOnSameSide(A - C, P - C, B - C, InsideTriangleDotProductEpsilon));
		}
	};

	OutTriangles.Reset();

	// Polygon must have at least three vertices/edges, or result is empty
	int32 PolygonVertexCount = VertexPositions.Num();
	if (PolygonVertexCount < 3)
	{
		return;
	}

	// If perimeter has 3 vertices, just copy content of perimeter out 
	if (PolygonVertexCount == 3)
	{
		OutTriangles.Add(bOrientAsHoleFill ? FIndex3i(0, 2, 1) : FIndex3i(0, 1, 2));
		return;
	}

	// First figure out the polygon normal.  We need this to determine which triangles are convex, so that
	// we can figure out which ears to clip
	TVector<T> PolygonNormal, PolygonCentroid;
	ComputePolygonPlane(VertexPositions, PolygonNormal, PolygonCentroid);

	// Make a simple linked list array of the previous and next vertex numbers, for each vertex number
	// in the polygon.  This will just save us having to iterate later on.
	TArray<int32> PrevVertexNumbers, NextVertexNumbers;

	PrevVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);
	NextVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);

	for (int32 VertexNumber = 0; VertexNumber < PolygonVertexCount; ++VertexNumber)
	{
		PrevVertexNumbers[VertexNumber] = VertexNumber - 1;
		NextVertexNumbers[VertexNumber] = VertexNumber + 1;
	}
	PrevVertexNumbers[0] = PolygonVertexCount - 1;
	NextVertexNumbers[PolygonVertexCount - 1] = 0;


	int32 EarVertexNumber = 0;
	int32 EarTestCount = 0;
	for (int32 RemainingVertexCount = PolygonVertexCount; RemainingVertexCount >= 3; )
	{
		bool bIsEar = true;

		// If we're down to only a triangle, just treat it as an ear.  Also, if we've tried every possible candidate
		// vertex looking for an ear, go ahead and just treat the current vertex as an ear.  This can happen when 
		// vertices are collinear or other degenerate cases.
		if (RemainingVertexCount > 3 && EarTestCount < RemainingVertexCount)
		{
			const TVector<T>& PrevVertexPosition = VertexPositions[PrevVertexNumbers[EarVertexNumber]];
			const TVector<T>& EarVertexPosition = VertexPositions[EarVertexNumber];
			const TVector<T>& NextVertexPosition = VertexPositions[NextVertexNumbers[EarVertexNumber]];

			// Figure out whether the potential ear triangle is facing the same direction as the polygon
			// itself.  If it's facing the opposite direction, then we're dealing with a concave triangle
			// and we'll skip it for now.
			if (!Local3::IsTriangleFlipped(
				PolygonNormal, PrevVertexPosition, EarVertexPosition, NextVertexPosition))
			{
				int32 TestVertexNumber = NextVertexNumbers[NextVertexNumbers[EarVertexNumber]];

				do
				{
					// Test every other remaining vertex to make sure that it doesn't lie inside our potential ear
					// triangle.  If we find a vertex that's inside the triangle, then it cannot actually be an ear.
					const TVector<T>& TestVertexPosition = VertexPositions[TestVertexNumber];
					if (Local3::PointInTriangle(PrevVertexPosition, EarVertexPosition, NextVertexPosition, TestVertexPosition, SMALL_NUMBER))
					{
						bIsEar = false;
						break;
					}

					TestVertexNumber = NextVertexNumbers[TestVertexNumber];
				} while (TestVertexNumber != PrevVertexNumbers[EarVertexNumber]);
			}
			else
			{
				bIsEar = false;
			}
		}

		if (bIsEar)
		{
			// OK, we found an ear!  Let's save this triangle in our output buffer.
			{
				int32 A = PrevVertexNumbers[EarVertexNumber]
					, B = EarVertexNumber
					, C = NextVertexNumbers[EarVertexNumber];
				OutTriangles.Add(bOrientAsHoleFill ? FIndex3i(A, C, B) : FIndex3i(A, B, C));
			}

			// Update our linked list.  We're effectively cutting off the ear by pointing the ear vertex's neighbors to
			// point at their next sequential neighbor, and reducing the remaining vertex count by one.
			{
				NextVertexNumbers[PrevVertexNumbers[EarVertexNumber]] = NextVertexNumbers[EarVertexNumber];
				PrevVertexNumbers[NextVertexNumbers[EarVertexNumber]] = PrevVertexNumbers[EarVertexNumber];
				--RemainingVertexCount;
			}

			// Move on to the previous vertex in the list, now that this vertex was cut
			EarVertexNumber = PrevVertexNumbers[EarVertexNumber];

			EarTestCount = 0;
		}
		else
		{
			// The vertex is not the ear vertex, because it formed a triangle that either had a normal which pointed in the opposite direction
			// of the polygon, or at least one of the other polygon vertices was found to be inside the triangle.  Move on to the next vertex.
			EarVertexNumber = NextVertexNumbers[EarVertexNumber];

			// Keep track of how many ear vertices we've tested, so that if we exhaust all remaining vertices, we can
			// fall back to clipping the triangle and adding it to our mesh anyway.  This is important for degenerate cases.
			++EarTestCount;
		}
	}

	check(OutTriangles.Num() > 0);
}