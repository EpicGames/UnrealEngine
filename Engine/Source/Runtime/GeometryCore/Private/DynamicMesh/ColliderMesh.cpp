// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/ColliderMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Distance/DistPoint3Triangle3.h"

using namespace UE::Geometry;


FColliderMesh::FColliderMesh(const FDynamicMesh3& SourceMesh, const FBuildOptions& BuildOptions)
{
	Initialize(SourceMesh, BuildOptions);
}

FColliderMesh::FColliderMesh()
{
	Reset(EAllowShrinking::No);
}

void FColliderMesh::Reset(EAllowShrinking AllowShrinking)
{
	Vertices.SetNum(0, AllowShrinking);
	SourceVertexIDs.SetNum(0, AllowShrinking);
	Triangles.SetNum(0, AllowShrinking);
	SourceTriangleIDs.SetNum(0, AllowShrinking);

	bSourceWasCompactV = bSourceWasCompactT = true;

	AABBTree.SetMesh(this, true);
}


void FColliderMesh::Initialize(const FDynamicMesh3& SourceMesh, const FBuildOptions& BuildOptions)
{
	bSourceWasCompactV = SourceMesh.IsCompactV();
	bSourceWasCompactT = SourceMesh.IsCompactT();

	int32 NumVertices = SourceMesh.VertexCount();
	Vertices.Reserve(NumVertices);
	if (BuildOptions.bBuildVertexMap && bSourceWasCompactV == false)
	{
		SourceVertexIDs.Reserve(NumVertices);
	}

	int32 NumTriangles = SourceMesh.TriangleCount();
	Triangles.Reserve(NumTriangles);
	if (BuildOptions.bBuildTriangleMap && bSourceWasCompactT == false)
	{
		SourceTriangleIDs.Reserve(NumTriangles);
	}

	TArray<int32> SourceToCompactMapV;

	if (bSourceWasCompactV)
	{
		for (FVector3d VertexPos : SourceMesh.VerticesItr())
		{
			Vertices.Add(VertexPos);
		}
	}
	else
	{
		SourceToCompactMapV.SetNum(SourceMesh.MaxVertexID());
		for (int32 VertexID : SourceMesh.VertexIndicesItr())
		{
			SourceToCompactMapV[VertexID] = Vertices.Num();
			Vertices.Add(SourceMesh.GetVertex(VertexID));
			if (BuildOptions.bBuildVertexMap)
			{
				SourceVertexIDs.Add(VertexID);
			}
		}
	}

	if (bSourceWasCompactT && bSourceWasCompactV)
	{
		for (FIndex3i TriIndices : SourceMesh.TrianglesItr())
		{
			Triangles.Add(TriIndices);
		}
	}
	else
	{
		for (int32 TriangleID : SourceMesh.TriangleIndicesItr())
		{
			FIndex3i Tri = SourceMesh.GetTriangle(TriangleID);
			if (!bSourceWasCompactV)
			{
				Tri.A = SourceToCompactMapV[Tri.A];
				Tri.B = SourceToCompactMapV[Tri.B];
				Tri.C = SourceToCompactMapV[Tri.C];
			}
			Triangles.Add(Tri);
			if (BuildOptions.bBuildTriangleMap)
			{
				SourceTriangleIDs.Add(TriangleID);
			}
		}
	}

	AABBTree.SetMesh(this, BuildOptions.bBuildAABBTree);

}


TMeshAABBTree3<FColliderMesh>* FColliderMesh::GetRawAABBTreeUnsafe()
{
	return &AABBTree;
}


bool FColliderMesh::FindNearestHitTriangle(const FRay3d& Ray, double& RayParameterOut, int& HitTriangleIDOut, FVector3d& BaryCoordsOut, const IMeshSpatial::FQueryOptions& Options) const
{
	if ( AABBTree.IsValid(false) )
	{
		return AABBTree.FindNearestHitTriangle(Ray, RayParameterOut, HitTriangleIDOut, BaryCoordsOut, Options);
	}
	return false;
}


int FColliderMesh::FindNearestTriangle(const FVector3d& Point, double& NearestDistSqrOut, const IMeshSpatial::FQueryOptions& Options) const
{
	if ( AABBTree.IsValid(false) )
	{
		return AABBTree.FindNearestTriangle(Point, NearestDistSqrOut, Options);
	}
	return IndexConstants::InvalidID;
}


int32 FColliderMesh::GetSourceVertexID(int32 VertexID) const 
{ 
	if (bSourceWasCompactV)
	{
		return VertexID;
	}
	else
	{
		return (VertexID >= 0 && VertexID < SourceVertexIDs.Num()) ? SourceVertexIDs[VertexID] : IndexConstants::InvalidID;
	}
}

int32 FColliderMesh::GetSourceTriangleID(int32 TriangleID) const 
{ 
	if (bSourceWasCompactT)
	{
		return TriangleID;
	}
	else
	{
		return (TriangleID >= 0 && TriangleID < SourceTriangleIDs.Num()) ? SourceTriangleIDs[TriangleID] : IndexConstants::InvalidID;
	}
}




FVector3d FColliderMeshProjectionTarget::Project(const FVector3d& Point, int Identifier)
{
	double DistSqr;
	int NearestTriID = ColliderMesh->FindNearestTriangle(Point, DistSqr);
	if (NearestTriID < 0)
	{
		return Point;
	}

	FTriangle3d Triangle;
	ColliderMesh->GetTriVertices(NearestTriID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

	FDistPoint3Triangle3d DistanceQuery(Point, Triangle);
	DistanceQuery.GetSquared();
	if (VectorUtil::IsFinite(DistanceQuery.ClosestTrianglePoint))
	{
		return DistanceQuery.ClosestTrianglePoint;
	}
	else
	{
		return Point;
	}
}



FVector3d FColliderMeshProjectionTarget::Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier)
{
	double DistSqr;
	int NearestTriID = ColliderMesh->FindNearestTriangle(Point, DistSqr);
	if (NearestTriID < 0)
	{
		return Point;
	}

	FTriangle3d Triangle;
	ColliderMesh->GetTriVertices(NearestTriID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

	ProjectNormalOut = Triangle.Normal();

	FDistPoint3Triangle3d DistanceQuery(Point, Triangle);
	DistanceQuery.GetSquared();
	if (VectorUtil::IsFinite(DistanceQuery.ClosestTrianglePoint))
	{
		return DistanceQuery.ClosestTrianglePoint;
	}
	else
	{
		return Point;
	}
}