// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicBoneAttribute.h"

using namespace UE::Geometry;

template<>
bool TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>::IsSameAs(const TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>& Other) const
{
	if (Num() != Other.Num())
	{
		return false;
	}

	for (int Idx = 0; Idx < Num(); ++Idx)
	{	
		if (GetValue(Idx).Equals(Other.GetValue(Idx)) == false)
		{
			return false;
		}
	}

	return true;
}

template<typename ParentType, typename AttribValueType>
TDynamicBoneAttributeBase<ParentType, AttribValueType>::TDynamicBoneAttributeBase(ParentType* ParentIn, const int InNumBones)
:
Parent(ParentIn)
{
	Resize(InNumBones);
}

template<typename ParentType, typename AttribValueType>
TDynamicBoneAttributeBase<ParentType, AttribValueType>::TDynamicBoneAttributeBase(ParentType* ParentIn, const int InNumBones, const AttribValueType& InitialValue)
:
Parent(ParentIn)
{
	Initialize(InNumBones, InitialValue);
}

template<typename ParentType, typename AttribValueType>
void TDynamicBoneAttributeBase<ParentType, AttribValueType>::Initialize(const int32 InNumBones, const AttribValueType& InitialValue)
{	
	AttribValues.Init(InitialValue, InNumBones);
}

template<typename ParentType, typename AttribValueType>
void TDynamicBoneAttributeBase<ParentType, AttribValueType>::Resize(const int32 InNumBones)
{	
	AttribValues.SetNumUninitialized(InNumBones);
}

template<typename ParentType, typename AttribValueType>
void TDynamicBoneAttributeBase<ParentType, AttribValueType>::Copy(const TDynamicBoneAttributeBase<ParentType, AttribValueType>& Copy)
{
	TDynamicAttributeBase<ParentType>::CopyParentClassData(Copy);
	AttribValues = Copy.AttribValues;
}

template<typename ParentType, typename AttribValueType>
TDynamicAttributeBase<ParentType>* TDynamicBoneAttributeBase<ParentType, AttribValueType>::MakeCopy(ParentType* ParentIn) const
{
	TDynamicBoneAttributeBase<ParentType, AttribValueType>* ToFill = new TDynamicBoneAttributeBase<ParentType, AttribValueType>(ParentIn);
	ToFill->Copy(*this);
	return ToFill;
}

template<typename ParentType, typename AttribValueType>
TDynamicAttributeBase<ParentType>* TDynamicBoneAttributeBase<ParentType, AttribValueType>::MakeNew(ParentType* ParentIn) const
{
	TDynamicBoneAttributeBase<ParentType, AttribValueType>* Matching = new TDynamicBoneAttributeBase<ParentType, AttribValueType>(ParentIn);
	return Matching;
}

template<typename ParentType, typename AttribValueType>
bool TDynamicBoneAttributeBase<ParentType, AttribValueType>::IsSameAs(const TDynamicBoneAttributeBase<ParentType, AttribValueType>& Other) const
{
	return AttribValues == Other.AttribValues;
}

template<typename ParentType, typename AttribValueType>
void TDynamicBoneAttributeBase<ParentType, AttribValueType>::Append(const AttribValueType& InValue) 
{
	AttribValues.Add(InValue);
}

template<typename ParentType, typename AttribValueType>
void TDynamicBoneAttributeBase<ParentType, AttribValueType>::Serialize(FArchive& Ar)
{	
	Ar << AttribValues;
}

// Instantiate for linker
namespace UE::Geometry
{
	template class TDynamicBoneAttributeBase<FDynamicMesh3, FName>;
	template class TDynamicBoneAttributeBase<FDynamicMesh3, int32>;
	template class TDynamicBoneAttributeBase<FDynamicMesh3, FVector4f>;
	template class TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>;
}