// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Generators/MeshShapeGenerator.h"
#include "Templates/UniquePtr.h"
#include "HAL/IConsoleManager.h"	// required for cvars

using namespace UE::Geometry;

// NB: These have to be here until C++17 allows inline variables
constexpr int       FDynamicMesh3::InvalidID;
constexpr int       FDynamicMesh3::NonManifoldID;
constexpr int       FDynamicMesh3::DuplicateTriangleID;
const FVector3d FDynamicMesh3::InvalidVertex = FVector3d(TNumericLimits<double>::Max(), 0.0, 0.0);
constexpr FIndex3i  FDynamicMesh3::InvalidTriangle;
constexpr FIndex2i  FDynamicMesh3::InvalidEdge;

FDynamicMesh3::FDynamicMesh3() : FDynamicMesh3(false, false, false, false) {}

FDynamicMesh3::FDynamicMesh3(bool bWantNormals, bool bWantColors, bool bWantUVs, bool bWantTriGroups)
{
	if ( bWantNormals )   { VertexNormals = TDynamicVector<FVector3f>{}; }
	if ( bWantColors )    { VertexColors = TDynamicVector<FVector3f>{}; }
	if ( bWantUVs )       { VertexUVs = TDynamicVector<FVector2f>{}; }
	if ( bWantTriGroups ) { TriangleGroups = TDynamicVector<int>{}; }
}

FDynamicMesh3::FDynamicMesh3(EMeshComponents flags)
	: FDynamicMesh3(((int)flags & (int)EMeshComponents::VertexNormals) != 0,
	                ((int)flags & (int)EMeshComponents::VertexColors) != 0,
	                ((int)flags & (int)EMeshComponents::VertexUVs) != 0,
	                ((int)flags & (int)EMeshComponents::FaceGroups) != 0)
{
}

// normals/colors/uvs will only be copied if they exist
FDynamicMesh3::FDynamicMesh3(const FDynamicMesh3& Other)
	:
	Vertices{ Other.Vertices },
	VertexRefCounts{ Other.VertexRefCounts },
	VertexNormals{ Other.VertexNormals },
	VertexColors{ Other.VertexColors },
	VertexUVs{ Other.VertexUVs },
	VertexEdgeLists{ Other.VertexEdgeLists },

	Triangles{ Other.Triangles },
	TriangleRefCounts{ Other.TriangleRefCounts },
	TriangleEdges{ Other.TriangleEdges },
	TriangleGroups{ Other.TriangleGroups },
	GroupIDCounter{ Other.GroupIDCounter },

	Edges{ Other.Edges },
	EdgeRefCounts{ Other.EdgeRefCounts }
{
	ChangeStampShape.Set(Other.ChangeStampShape.GetValue());
	ChangeStampTopology.Set(Other.ChangeStampTopology.GetValue());

	if (Other.HasAttributes())
	{
		EnableAttributes();
		AttributeSet->Copy(*Other.AttributeSet);
	}
}

FDynamicMesh3::FDynamicMesh3(FDynamicMesh3&& Other)
	:
	Vertices{ MoveTemp(Other.Vertices) },
	VertexRefCounts{ MoveTemp(Other.VertexRefCounts) },
	VertexNormals{ MoveTemp(Other.VertexNormals) },
	VertexColors{ MoveTemp(Other.VertexColors) },
	VertexUVs{ MoveTemp( Other.VertexUVs ) },
	VertexEdgeLists{ MoveTemp( Other.VertexEdgeLists ) },

	Triangles{ MoveTemp( Other.Triangles ) },
	TriangleRefCounts{ MoveTemp( Other.TriangleRefCounts ) },
	TriangleEdges{ MoveTemp( Other.TriangleEdges ) },
	TriangleGroups{ MoveTemp( Other.TriangleGroups ) },
	GroupIDCounter{ MoveTemp( Other.GroupIDCounter ) },

	Edges{ MoveTemp( Other.Edges ) },
	EdgeRefCounts{ MoveTemp( Other.EdgeRefCounts ) },

	AttributeSet{ MoveTemp( Other.AttributeSet ) }
{
	ChangeStampShape.Set(Other.ChangeStampShape.GetValue());
	ChangeStampTopology.Set(Other.ChangeStampTopology.GetValue());

	if (AttributeSet)
	{
		AttributeSet->Reparent(this);
	}
}

FDynamicMesh3::~FDynamicMesh3() = default;

const FDynamicMesh3& FDynamicMesh3::operator=(const FDynamicMesh3& CopyMesh)
{
	Copy(CopyMesh);
	return *this;
}

const FDynamicMesh3 & FDynamicMesh3::operator=(FDynamicMesh3 && Other)
{
	if (this != &Other)
	{
		Vertices = MoveTemp(Other.Vertices);
		VertexRefCounts = MoveTemp(Other.VertexRefCounts);
		VertexNormals = MoveTemp(Other.VertexNormals);
		VertexColors = MoveTemp(Other.VertexColors);
		VertexUVs = MoveTemp(Other.VertexUVs);
		VertexEdgeLists = MoveTemp(Other.VertexEdgeLists);

		Triangles = MoveTemp(Other.Triangles);
		TriangleRefCounts = MoveTemp(Other.TriangleRefCounts);
		TriangleEdges = MoveTemp(Other.TriangleEdges);
		TriangleGroups = MoveTemp(Other.TriangleGroups);
		GroupIDCounter = MoveTemp(Other.GroupIDCounter);

		Edges = MoveTemp(Other.Edges);
		EdgeRefCounts = MoveTemp(Other.EdgeRefCounts);

		AttributeSet = MoveTemp(Other.AttributeSet);
		if (AttributeSet)
		{
			AttributeSet->Reparent(this);
		}
	
		ChangeStampShape.Set(Other.ChangeStampShape.GetValue());
		ChangeStampTopology.Set(Other.ChangeStampTopology.GetValue());
	}

	return *this;
}

FDynamicMesh3::FDynamicMesh3(const FMeshShapeGenerator* Generator)
{
	Copy(Generator);
}

bool FDynamicMesh3::Copy(const FMeshShapeGenerator* Generator)
{
	Clear();

	EnableTriangleGroups();

	int NumVerts = Generator->Vertices.Num();
	for (int i = 0; i < NumVerts; ++i)
	{
		AppendVertex(Generator->Vertices[i]);
	}

	int NumTris = Generator->Triangles.Num();
	if (Generator->HasAttributes())
	{
		bool bSuccess = true;
		// First append all triangles w/out attributes enabled
		for (int32 TID = 0; TID < NumTris; ++TID)
		{
			int PolyID = Generator->TrianglePolygonIDs.Num() > 0 ? 1 + Generator->TrianglePolygonIDs[TID] : 0;
			int AppendedTID = AppendTriangle(Generator->Triangles[TID], PolyID);
			bSuccess &= bool(TID == AppendedTID);
		}
		// If they were successfully appended, enable and set attributes
		// (doing this as a post-process is faster)
		if (ensure(bSuccess))
		{
			EnableAttributes();
			FDynamicMeshUVOverlay* UVOverlay = Attributes()->PrimaryUV();
			FDynamicMeshNormalOverlay* NormalOverlay = Attributes()->PrimaryNormals();
			int NumUVs = Generator->UVs.Num();
			for (int i = 0; i < NumUVs; ++i)
			{
				UVOverlay->AppendElement(Generator->UVs[i]);
			}
			int NumNormals = Generator->Normals.Num();
			for (int i = 0; i < NumNormals; ++i)
			{
				NormalOverlay->AppendElement(Generator->Normals[i]);
			}

			for (int i = 0; i < NumTris; ++i)
			{
				UVOverlay->SetTriangle(i, Generator->TriangleUVs[i]);
				NormalOverlay->SetTriangle(i, Generator->TriangleNormals[i]);
			}
		}
	}
	else if (Generator->TrianglePolygonIDs.Num()) // no attributes, yes polygon ids
	{
		for (int i = 0; i < NumTris; ++i)
		{
			int tid = AppendTriangle(Generator->Triangles[i], 1 + Generator->TrianglePolygonIDs[i]);
			ensure(tid == i);
		}
	}
	else // no attribute and no polygon ids
	{
		for (int i = 0; i < NumTris; ++i)
		{
			int tid = AppendTriangle(Generator->Triangles[i], 0);
			ensure(tid == i);
		}
	}

	return (TriangleCount() == NumTris);
}

void FDynamicMesh3::Copy(const FDynamicMesh3& copy, bool bNormals, bool bColors, bool bUVs, bool bAttributes)
{
	if (this != &copy)
	{
		Vertices        = copy.Vertices;
		VertexNormals   = bNormals ? copy.VertexNormals : TOptional<TDynamicVector<FVector3f>>{};
		VertexColors    = bColors  ? copy.VertexColors : TOptional<TDynamicVector<FVector3f>>{};
		VertexUVs       = bUVs     ? copy.VertexUVs : TOptional<TDynamicVector<FVector2f>>{};
		VertexRefCounts = copy.VertexRefCounts;
		VertexEdgeLists = copy.VertexEdgeLists;

		Triangles         = copy.Triangles;
		TriangleEdges     = copy.TriangleEdges;
		TriangleRefCounts = copy.TriangleRefCounts;
		TriangleGroups    = copy.TriangleGroups;
		GroupIDCounter    = copy.GroupIDCounter;

		Edges         = copy.Edges;
		EdgeRefCounts = copy.EdgeRefCounts;

		// Note that we populate our existing AttributeSet when possible rather than building a
		// new one. For the common case of updating a mesh from its preview copy, it seems reasonable
		// for a client to be able to hold on to the AttributeSet pointer without expecting it to
		// get reset by the Copy() call.
		if (bAttributes && copy.HasAttributes())
		{
			EnableAttributes(); // does nothing if already enabled
			AttributeSet->Copy(*copy.AttributeSet);
		}
		else
		{
			DiscardAttributes();
		}

		ChangeStampShape.Set(copy.ChangeStampShape.GetValue());
		ChangeStampTopology.Set(copy.ChangeStampTopology.GetValue());
	}
}

void FDynamicMesh3::AppendWithOffsets(const FDynamicMesh3& ToAppend, FAppendInfo* OutAppendInfo)
{
	FAppendInfo LocalAppendInfo;
	FAppendInfo* UseAppendInfo = OutAppendInfo ? OutAppendInfo : &LocalAppendInfo;

	UseAppendInfo->VertexOffset = MaxVertexID();
	UseAppendInfo->TriangleOffset = MaxTriangleID();
	UseAppendInfo->EdgeOffset = MaxEdgeID();
	UseAppendInfo->GroupOffset = HasTriangleGroups() ? MaxGroupID() : 0;
	UseAppendInfo->NumVertex = ToAppend.MaxVertexID();
	UseAppendInfo->NumTriangle = ToAppend.MaxTriangleID();
	UseAppendInfo->NumEdge = ToAppend.MaxEdgeID();

	if (HasAttributes())
	{
		for (int32 NormalLayerIdx = 0, N = FMath::Min(3, Attributes()->NumNormalLayers()); NormalLayerIdx < N; ++NormalLayerIdx)
		{
			UseAppendInfo->NormalOverlayOffsets[NormalLayerIdx] = Attributes()->GetNormalLayer(NormalLayerIdx)->MaxElementID();
		}
	}

	Vertices.Add(ToAppend.Vertices);
	auto MatchOptional = []<typename T>(TOptional<TDynamicVector<T>>& Src, const TOptional<TDynamicVector<T>>& ToAppend, int32 NumDefault, T DefaultValue = T())
	{
		if (Src.IsSet())
		{
			if (ToAppend.IsSet())
			{
				Src->Add(*ToAppend);
			}
			else
			{
				Src->Resize(NumDefault, DefaultValue);
			}
		}
	};
	MatchOptional(VertexNormals, ToAppend.VertexNormals, Vertices.Num(), FVector3f::UnitY());
	MatchOptional(VertexColors, ToAppend.VertexColors, Vertices.Num(), FVector3f::One());
	MatchOptional(VertexUVs, ToAppend.VertexUVs, Vertices.Num(), FVector2f::Zero());

	VertexRefCounts.Append(ToAppend.VertexRefCounts);
	VertexEdgeLists.AppendWithElementOffset(ToAppend.VertexEdgeLists, UseAppendInfo->EdgeOffset);

	Triangles.Add(ToAppend.Triangles);
	for (int32 Idx = UseAppendInfo->TriangleOffset, N = Triangles.Num(); Idx < N; ++Idx)
	{
		Triangles[Idx].A += UseAppendInfo->VertexOffset;
		Triangles[Idx].B += UseAppendInfo->VertexOffset;
		Triangles[Idx].C += UseAppendInfo->VertexOffset;
	}
	// TriangleEdges should be 1:1 with Triangles, so this ensure should not fail ...
	// however due to a now-fixed bug, it is possible that a serialized mesh will 
	// have too many TriangleEdges; we can recover by resizing to match before appending
	if (!ensure(TriangleEdges.Num() == UseAppendInfo->TriangleOffset))
	{
		// Resize to recover from a too-large TriangleEdges array
		TriangleEdges.Resize(UseAppendInfo->TriangleOffset);
	}
	TriangleEdges.Add(ToAppend.TriangleEdges);
	for (int32 Idx = UseAppendInfo->TriangleOffset, N = Triangles.Num(); Idx < N; ++Idx)
	{
		TriangleEdges[Idx].A += UseAppendInfo->EdgeOffset;
		TriangleEdges[Idx].B += UseAppendInfo->EdgeOffset;
		TriangleEdges[Idx].C += UseAppendInfo->EdgeOffset;
	}
	TriangleRefCounts.Append(ToAppend.TriangleRefCounts);

	if (TriangleGroups.IsSet())
	{
		GroupIDCounter += ToAppend.MaxGroupID();
		if (ToAppend.TriangleGroups.IsSet())
		{
			TriangleGroups->Add(*ToAppend.TriangleGroups);
			for (int32 Idx = UseAppendInfo->TriangleOffset, N = Triangles.Num(); Idx < N; ++Idx)
			{
				(*TriangleGroups)[Idx] += UseAppendInfo->GroupOffset;
			}
		}
		else
		{
			TriangleGroups->Resize(MaxTriangleID(), 0);
		}
	}

	Edges.Add(ToAppend.Edges);
	for (int32 Idx = UseAppendInfo->EdgeOffset, N = Edges.Num(); Idx < N; ++Idx)
	{
		Edges[Idx].Tri.A += UseAppendInfo->TriangleOffset;
		if (Edges[Idx].Tri.B != INDEX_NONE)
		{
			Edges[Idx].Tri.B += UseAppendInfo->TriangleOffset;
		}
		Edges[Idx].Vert.A += UseAppendInfo->VertexOffset;
		Edges[Idx].Vert.B += UseAppendInfo->VertexOffset;
	}
	EdgeRefCounts.Append(ToAppend.EdgeRefCounts);

	if (HasAttributes())
	{
		if (ToAppend.HasAttributes())
		{
			AttributeSet->Append(*ToAppend.AttributeSet, *UseAppendInfo);
		}
		else
		{
			AttributeSet->AppendDefaulted(*UseAppendInfo);
		}
	}
	
	UpdateChangeStamps(true, true);
}

void FDynamicMesh3::CompactCopy(const FDynamicMesh3& copy, bool bNormals, bool bColors, bool bUVs, bool bAttributes, FCompactMaps* CompactInfo)
{
	if (copy.IsCompact() && ((!bAttributes || !copy.HasAttributes()) || copy.Attributes()->IsCompact())) {
		Copy(copy, bNormals, bColors, bUVs, bAttributes);
		if (CompactInfo)
		{
			CompactInfo->SetIdentity(MaxVertexID(), MaxTriangleID());
		}
		return;
	}

	// currently cannot re-use existing attribute buffers
	Clear();
	if (bNormals && copy.HasVertexNormals())
	{
		EnableVertexNormals(FVector3f::UnitY());
	}
	if (bColors && copy.HasVertexColors())
	{
		EnableVertexColors(FVector3f::One());
	}
	if (bUVs && copy.HasVertexUVs())
	{
		EnableVertexUVs(FVector2f::Zero());
	}

	// Use a triangle map if we have a CompactInfo or we need to copy attributes.
	const bool bUseTriangleMap = CompactInfo != nullptr || (bAttributes && copy.HasAttributes());

	// If we don't have a CompactInfo, we'll make it refer to a local one.
	FCompactMaps LocalCompactInfo;
	if (!CompactInfo)
	{
		CompactInfo = &LocalCompactInfo;
	}

	CompactInfo->ResetVertexMap(copy.MaxVertexID(), false);
	FVertexInfo vinfo;
	for (int vid = 0, NumVid = copy.MaxVertexID(); vid < NumVid; vid++)
	{
		if (copy.IsVertex(vid))
		{
			copy.GetVertex(vid, vinfo, bNormals, bColors, bUVs);
			CompactInfo->SetVertexMapping(vid, AppendVertex(vinfo));
		}
		else
		{
			CompactInfo->SetVertexMapping(vid, FCompactMaps::InvalidID);
		}
	}

	// [TODO] would be much faster to explicitly copy triangle & edge data structures!!
	if (copy.HasTriangleGroups())
	{
		EnableTriangleGroups(0);
	}

	// need the triangle map to be computed if we have attributes and/or the FCompactMaps flag was set to request it
	
	if (bUseTriangleMap)
	{
		CompactInfo->ResetTriangleMap(bUseTriangleMap ? copy.MaxTriangleID() : 0, true);
	}
	for (int tid : copy.TriangleIndicesItr())
	{
		const FIndex3i t = CompactInfo->GetVertexMapping(copy.GetTriangle(tid));
		const int g = (copy.HasTriangleGroups()) ? copy.GetTriangleGroup(tid) : InvalidID;
		const int NewTID = AppendTriangle(t, g);
		GroupIDCounter = FMath::Max(GroupIDCounter, g + 1);
		if (bUseTriangleMap)
		{
			CompactInfo->SetTriangleMapping(tid, NewTID);
		}
	}

	if (bAttributes && copy.HasAttributes())
	{
		EnableAttributes();
		AttributeSet->EnableMatchingAttributes(*copy.Attributes());
		AttributeSet->CompactCopy(*CompactInfo, *copy.Attributes());
	}

	ChangeStampShape.Set(copy.ChangeStampShape.GetValue());
	ChangeStampTopology.Set(copy.ChangeStampTopology.GetValue());
}

void FDynamicMesh3::Clear()
{
	Vertices.Clear();
	VertexRefCounts.Clear();
	VertexNormals.Reset();
	VertexColors.Reset(); 
	VertexUVs.Reset();
	VertexEdgeLists.Reset();

	Triangles.Clear();
	TriangleRefCounts.Clear();
	TriangleEdges.Clear();
	TriangleGroups.Reset();
	GroupIDCounter = 0;

	Edges.Clear();
	EdgeRefCounts.Clear();

	AttributeSet.Reset();

	ChangeStampShape.Set(1);
	ChangeStampTopology.Set(1);
}


void FDynamicMesh3::EnableMatchingAttributes(const FDynamicMesh3& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes)
{
	bool bWantVertexNormals = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasVertexNormals() : ( ToMatch.HasVertexNormals() || this->HasVertexNormals() );
	if (bClearExisting || bWantVertexNormals == false )
	{
		DiscardVertexNormals();
	}
	if (bWantVertexNormals)
	{
		EnableVertexNormals(FVector3f::UnitZ());
	}

	bool bWantVertexColors = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasVertexColors() : ( ToMatch.HasVertexColors() || this->HasVertexColors() );
	if (bClearExisting || bWantVertexColors == false )
	{
		DiscardVertexColors();
	}
	if (bWantVertexColors)
	{
		EnableVertexColors(FVector3f::Zero());
	}

	bool bWantVertexUVs = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasVertexUVs() : ( ToMatch.HasVertexUVs() || this->HasVertexUVs() );
	if (bClearExisting || bWantVertexUVs == false )
	{
		DiscardVertexUVs();
	}
	if (bWantVertexUVs)
	{
		EnableVertexUVs(FVector2f::Zero());
	}

	bool bWantTriangleGroups = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasTriangleGroups() : ( ToMatch.HasTriangleGroups() || this->HasTriangleGroups() );
	if (bClearExisting || bWantTriangleGroups == false )
	{
		DiscardTriangleGroups();
	}
	if (bWantTriangleGroups)
	{
		EnableTriangleGroups();
	}

	bool bWantAttributes = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasAttributes() : ( ToMatch.HasAttributes() || this->HasAttributes() );
	if (bClearExisting || bWantAttributes == false )
	{
		DiscardAttributes();
	}
	if (bWantAttributes)
	{
		EnableAttributes();
	}
	if (HasAttributes() && ToMatch.HasAttributes())
	{
		Attributes()->EnableMatchingAttributes(*ToMatch.Attributes(), bClearExisting, bDiscardExtraAttributes);
	}
}



int FDynamicMesh3::GetComponentsFlags() const
{
	int c = 0;
	if (HasVertexNormals())
	{
		c |= (int)EMeshComponents::VertexNormals;
	}
	if (HasVertexColors())
	{
		c |= (int)EMeshComponents::VertexColors;
	}
	if (HasVertexUVs())
	{
		c |= (int)EMeshComponents::VertexUVs;
	}
	if (HasTriangleGroups())
	{
		c |= (int)EMeshComponents::FaceGroups;
	}
	return c;
}

void FDynamicMesh3::EnableMeshComponents(int MeshComponentsFlags)
{
	if (int(EMeshComponents::FaceGroups) & MeshComponentsFlags)
	{
		EnableTriangleGroups(0);
	}
	else
	{
		DiscardTriangleGroups();
	}
	if (int(EMeshComponents::VertexColors) & MeshComponentsFlags)
	{
		EnableVertexColors(FVector3f(1, 1, 1));
	}
	else
	{
		DiscardVertexColors();
	}
	if (int(EMeshComponents::VertexNormals) & MeshComponentsFlags)
	{
		EnableVertexNormals(FVector3f::UnitY());
	}
	else
	{
		DiscardVertexNormals();
	}
	if (int(EMeshComponents::VertexUVs) & MeshComponentsFlags)
	{
		EnableVertexUVs(FVector2f(0, 0));
	}
	else
	{
		DiscardVertexUVs();
	}
}

void FDynamicMesh3::EnableVertexNormals(const FVector3f& InitialNormal)
{
	if (HasVertexNormals())
	{
		return;
	}

	TDynamicVector<FVector3f> NewNormals;
	int NV = MaxVertexID();
	NewNormals.Resize(NV);
	for (int i = 0; i < NV; ++i)
	{
		NewNormals[i] = InitialNormal;
	}
	VertexNormals = MoveTemp(NewNormals);
}

void FDynamicMesh3::DiscardVertexNormals()
{
	VertexNormals.Reset();
}

void FDynamicMesh3::EnableVertexColors(const FVector3f& InitialColor)
{
	if (HasVertexColors())
	{
		return;
	}
	VertexColors = TDynamicVector<FVector3f>();
	int NV = MaxVertexID();
	VertexColors->Resize(NV);
	for (int i = 0; i < NV; ++i)
	{
		VertexColors.GetValue()[i] = InitialColor;
	}
}

void FDynamicMesh3::DiscardVertexColors()
{
	VertexColors.Reset();
}

void FDynamicMesh3::EnableVertexUVs(const FVector2f& InitialUV)
{
	if (HasVertexUVs())
	{
		return;
	}
	VertexUVs = TDynamicVector<FVector2f>();
	int NV = MaxVertexID();
	VertexUVs->Resize(NV);
	for (int i = 0; i < NV; ++i)
	{
		VertexUVs.GetValue()[i] = InitialUV;
	}
}

void FDynamicMesh3::DiscardVertexUVs()
{
		VertexUVs.Reset();
}

void FDynamicMesh3::EnableTriangleGroups(int InitialGroup)
{
	if (HasTriangleGroups())
	{
		return;
	}
	checkSlow(InitialGroup >= 0);
	TriangleGroups = TDynamicVector<int>();
	int NT = MaxTriangleID();
	TriangleGroups->Resize(NT);
	for (int i = 0; i < NT; ++i)
	{
		TriangleGroups.GetValue()[i] = InitialGroup;
	}
	GroupIDCounter = InitialGroup + 1;
}

void FDynamicMesh3::DiscardTriangleGroups()
{
	TriangleGroups.Reset();
	GroupIDCounter = 0;
}

void FDynamicMesh3::EnableAttributes()
{
	if (HasAttributes())
	{
		return;
	}
	AttributeSet = MakeUnique<FDynamicMeshAttributeSet>(this);
	AttributeSet->Initialize(MaxVertexID(), MaxTriangleID());
}

void FDynamicMesh3::DiscardAttributes()
{
	AttributeSet = nullptr;
}

bool FDynamicMesh3::GetVertex(int vID, FVertexInfo& vinfo, bool bWantNormals, bool bWantColors, bool bWantUVs) const
{
	if (VertexRefCounts.IsValid(vID) == false)
	{
		return false;
	}
	vinfo.Position = Vertices[vID];
	vinfo.bHaveN = vinfo.bHaveUV = vinfo.bHaveC = false;
	if (HasVertexNormals() && bWantNormals)
	{
		vinfo.bHaveN = true;
		const TDynamicVector<FVector3f>& NormalVec = VertexNormals.GetValue();
		vinfo.Normal = NormalVec[vID];
	}
	if (HasVertexColors() && bWantColors)
	{
		vinfo.bHaveC = true;
		const TDynamicVector<FVector3f>& ColorVec = VertexColors.GetValue();
		vinfo.Color = ColorVec[vID];
	}
	if (HasVertexUVs() && bWantUVs)
	{
		vinfo.bHaveUV = true;
		const TDynamicVector<FVector2f>& UVVec = VertexUVs.GetValue();
		vinfo.UV = UVVec[vID];
	}
	return true;
}

int FDynamicMesh3::GetMaxVtxEdgeCount() const
{
	int max = 0;
	for (int vid : VertexIndicesItr())
	{
		max = FMath::Max(max, VertexEdgeLists.GetCount(vid));
	}
	return max;
}

FVertexInfo FDynamicMesh3::GetVertexInfo(int i) const
{
	FVertexInfo vi = FVertexInfo();
	vi.Position = GetVertex(i);
	vi.bHaveN = vi.bHaveC = vi.bHaveUV = false;
	if (HasVertexNormals())
	{
		vi.bHaveN = true;
		vi.Normal = GetVertexNormal(i);
	}
	if (HasVertexColors())
	{
		vi.bHaveC = true;
		vi.Color = GetVertexColor(i);
	}
	if (HasVertexUVs())
	{
		vi.bHaveUV = true;
		vi.UV = GetVertexUV(i);
	}
	return vi;
}

FIndex3i FDynamicMesh3::GetTriNeighbourTris(int tID) const
{
	if (TriangleRefCounts.IsValid(tID))
	{
		FIndex3i nbr_t = FIndex3i::Zero();
		for (int j = 0; j < 3; ++j)
		{
			FEdge Edge = Edges[TriangleEdges[tID][j]];
			nbr_t[j] = (Edge.Tri[0] == tID) ? Edge.Tri[1] : Edge.Tri[0];
		}
		return nbr_t;
	}
	else
	{
		return InvalidTriangle;
	}
}


void FDynamicMesh3::EnumerateVertexTriangles(int32 VertexID, TFunctionRef<void(int32)> ApplyFunc) const
{
	checkSlow(VertexRefCounts.IsValid(VertexID));
	if (!IsVertex(VertexID))
	{
		return;
	}

	VertexEdgeLists.Enumerate(VertexID, [&](int32 eid)
	{
		const FEdge Edge = Edges[eid];
		const int vOther = Edge.Vert.A == VertexID ? Edge.Vert.B : Edge.Vert.A;
		if (TriHasSequentialVertices(Edge.Tri[0], VertexID, vOther))
		{
			ApplyFunc(Edge.Tri[0]);
		}
		if (Edge.Tri[1] != InvalidID && TriHasSequentialVertices(Edge.Tri[1], VertexID, vOther))
		{
			ApplyFunc(Edge.Tri[1]);
		}
	});
}

int32 FDynamicMesh3::GetSingleVertexTriangle(int32 VID) const
{
	checkSlow(VertexRefCounts.IsValid(VID));
	if (!IsVertex(VID) || VertexEdgeLists.GetCount(VID) == 0)
	{
		return IndexConstants::InvalidID;
	}
	return VertexEdgeLists.First(VID);
}


void FDynamicMesh3::EnumerateEdgeTriangles(int32 EdgeID, TFunctionRef<void(int32)> ApplyFunc) const
{
	checkSlow(EdgeRefCounts.IsValid(EdgeID));
	if (IsEdge(EdgeID))
	{
		const FEdge Edge = Edges[EdgeID];
		ApplyFunc(Edge.Tri.A);
		if (Edge.Tri.B != IndexConstants::InvalidID)
		{
			ApplyFunc(Edge.Tri.B);
		}
	}
}


FString FDynamicMesh3::MeshInfoString() const
{
	FString VtxString = FString::Printf(TEXT("Vertices count %d max %d  %s  VtxEdges %s"),
		VertexCount(), MaxVertexID(), *VertexRefCounts.UsageStats(), *(VertexEdgeLists.MemoryUsage()));
	FString TriString = FString::Printf(TEXT("Triangles count %d max %d  %s"),
		TriangleCount(), MaxTriangleID(), *TriangleRefCounts.UsageStats());
	FString EdgeString = FString::Printf(TEXT("Edges count %d max %d  %s"),
		EdgeCount(), MaxEdgeID(), *EdgeRefCounts.UsageStats());
	FString AttribString = FString::Printf(TEXT("VtxNormals %d  VtxColors %d  VtxUVs %d  TriGroups %d  Attributes %d"),
		HasVertexNormals(), HasVertexColors(), HasVertexUVs(), HasTriangleGroups(), HasAttributes());
	FString InfoString = FString::Printf(TEXT("Closed %d  Compact %d  TopologyChangeStamp %d  MaxGroupID %d"),
		IsClosed(), IsCompact(), GetTopologyChangeStamp(), MaxGroupID());

	return VtxString + "\n" + TriString + "\n" + EdgeString + "\n" + AttribString + "\n" + InfoString;
}

SIZE_T FDynamicMesh3::GetByteCount() const
{
	SIZE_T ByteCount = VertexRefCounts.GetByteCount() + VertexEdgeLists.GetByteCount() + TriangleRefCounts.GetByteCount() + EdgeRefCounts.GetByteCount();
	if (HasAttributes())
	{
		ByteCount += Attributes()->GetByteCount();
	}
	return ByteCount;
}

bool FDynamicMesh3::IsSameAs(const FDynamicMesh3& m2, const FSameAsOptions& Options) const
{
	return IsSameAs_Helper(m2, Options, nullptr);
}
bool FDynamicMesh3::IsSameAs(const FDynamicMesh3& m2, const FSameAsOptions& Options, FMeshDifferenceInfo& OutMeshDifferenceInfo) const
{
	return IsSameAs_Helper(m2, Options, &OutMeshDifferenceInfo);
}
bool FDynamicMesh3::IsSameAs_Helper(const FDynamicMesh3& m2, const FSameAsOptions& Options, FMeshDifferenceInfo* OutMeshDifferenceInfo) const
{
	if (OutMeshDifferenceInfo)
	{
		// reset difference info to defaults
		*OutMeshDifferenceInfo = FMeshDifferenceInfo{};
	}

	auto SameVertex = [this, &m2, &Options](const int Vid, const int VidM2)
	{
		return VectorUtil::EpsilonEqual(GetVertexRef(Vid), m2.GetVertexRef(VidM2), Options.EpsilonDouble);
	};

	if (VertexCount() != m2.VertexCount())
	{
		if (OutMeshDifferenceInfo)
		{
			OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::VertexCount;
			OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Vertex Count: %d != %d"), VertexCount(), m2.VertexCount());
		}
		return false;
	}

	if (TriangleCount() != m2.TriangleCount())
	{
		if (OutMeshDifferenceInfo)
		{
			OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::TriangleCount;
			OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Triangle Count: %d != %d"), TriangleCount(), m2.TriangleCount());
		}
		return false;
	}

	TOptional<TDynamicVector<int>> VidMapping;
	TOptional<TDynamicVector<int>> TidMapping;

	if (!Options.bIgnoreDataLayout || (VertexRefCounts.IsDense() && m2.VertexRefCounts.IsDense() && TriangleRefCounts.IsDense() && m2.TriangleRefCounts.IsDense()))
	{
		for (const int Vid : VertexIndicesItr())
		{
			if (m2.IsVertex(Vid) == false || !SameVertex(Vid, Vid))
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Vertex;
					OutMeshDifferenceInfo->SetVID(Vid);
				}
				return false;
			}
		}

		for (const int Tid : TriangleIndicesItr())
		{
			if (m2.IsTriangle(Tid) == false || (GetTriangle(Tid) != m2.GetTriangle(Tid)))
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Triangle;
					OutMeshDifferenceInfo->SetTID(Tid);
				}
				return false;
			}
		}
	}
	else
	{
		// Ignore holes in vertex and triangle data, i.e. don't use specific IDs but still make sure they are still in the same order.

		// Vertices
		{
			FRefCountVector::IndexIterator ItVid = VertexRefCounts.BeginIndices();
			const FRefCountVector::IndexIterator ItEndVid = VertexRefCounts.EndIndices();
			FRefCountVector::IndexIterator ItVidM2 = m2.VertexRefCounts.BeginIndices();
			const FRefCountVector::IndexIterator ItEndVidM2 = m2.VertexRefCounts.EndIndices();

			VidMapping.Emplace();
			VidMapping->Resize(Vertices.Num(), InvalidID);

			while (ItVid != ItEndVid && ItVidM2 != ItEndVidM2)
			{
				(*VidMapping)[*ItVid] = *ItVidM2;

				if (!SameVertex(*ItVid, *ItVidM2))
				{
					// Vertices are not the same.
					if (OutMeshDifferenceInfo)
					{
						OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Vertex;
						OutMeshDifferenceInfo->SetVID(*ItVid, *ItVidM2);
					}
					return false;
				}
			
				++ItVid;
				++ItVidM2;
			}
		
			if (ItVid != ItEndVid || ItVidM2 != ItEndVidM2)
			{
				// Number of vertices is not the same.
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::VertexCount;
				}
				return false;
			}
		}

		// Triangles
		{
			FRefCountVector::IndexIterator ItTid = TriangleRefCounts.BeginIndices();
			const FRefCountVector::IndexIterator ItEndTid = TriangleRefCounts.EndIndices();
			FRefCountVector::IndexIterator ItTidM2 = m2.TriangleRefCounts.BeginIndices();
			const FRefCountVector::IndexIterator ItEndTidM2 = m2.TriangleRefCounts.EndIndices();

			TidMapping.Emplace();
			TidMapping->Resize(Triangles.Num(), InvalidID);
		
			while (ItTid != ItEndTid && ItTidM2 != ItEndTidM2)
			{
				const FIndex3i Tri = GetTriangle(*ItTid);
				const FIndex3i TriM2 = m2.GetTriangle(*ItTidM2);

				(*TidMapping)[*ItTid] = *ItTidM2;

				for (int i = 0; i < 3; ++i)
				{
					if (TriM2[i] != (VidMapping ? (*VidMapping)[Tri[i]] : Tri[i]))
					{
						// Triangle vertices are not the same.
						if (OutMeshDifferenceInfo)
						{
							OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Triangle;
							OutMeshDifferenceInfo->SetTID(*ItTid, *ItTidM2);
						}
						return false;
					}
				}

				++ItTid;
				++ItTidM2;
			}

			if (ItTid != ItEndTid || ItTidM2 != ItEndTidM2)
			{
				// Number of triangles is not the same.
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::TriangleCount;
				}
				return false;
			}
		}
	}

	if (Options.bCheckConnectivity)
	{
		if (EdgeCount() != m2.EdgeCount())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::EdgeCount;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Edge Count: %d != %d"), EdgeCount(), m2.EdgeCount());
			}
			return false;
		}
		for (int eid : EdgeIndicesItr())
		{
			const FEdge e = GetEdge(eid);
			const int Vert0 = VidMapping ? (*VidMapping)[e.Vert[0]] : e.Vert[0];
			const int Vert1 = VidMapping ? (*VidMapping)[e.Vert[1]] : e.Vert[1];
			const int other_eid = m2.FindEdge(Vert0, Vert1);
			if (other_eid == InvalidID)
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Connectivity;
					OutMeshDifferenceInfo->SetEID(eid);
				}
				return false;
			}
			const FEdge oe = m2.GetEdge(other_eid);
			const FIndex2i eTri = [&e, &TidMapping]
			{
				return !TidMapping
					       ? e.Tri
					       : FIndex2i{e.Tri[0] != InvalidID ? (*TidMapping)[e.Tri[0]] : InvalidID, e.Tri[1] != InvalidID ? (*TidMapping)[e.Tri[1]] : InvalidID};
			}();
			if (FMath::Min(eTri[0], eTri[1]) != FMath::Min(oe.Tri[0], oe.Tri[1]) ||
			    FMath::Max(eTri[0], eTri[1]) != FMath::Max(oe.Tri[0], oe.Tri[1]))
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Connectivity;
					OutMeshDifferenceInfo->SetEID(eid, other_eid);
				}
				return false;
			}
		}
	}
	if (Options.bCheckEdgeIDs)
	{
		if (EdgeCount() != m2.EdgeCount())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::EdgeCount;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Edge Count: %d != %d"), EdgeCount(), m2.EdgeCount());
			}
			return false;
		}
		for (const int Eid : EdgeIndicesItr())
		{
			if (m2.IsEdge(Eid) == false)
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Edge;
					OutMeshDifferenceInfo->SetEID(Eid);
				}
				return false;
			}
			FEdge Edge = GetEdge(Eid);
			if (TidMapping.IsSet())
			{
				Edge.Tri[0] = Edge.Tri[0] != InvalidID ? (*TidMapping)[Edge.Tri[0]] : InvalidID;
				Edge.Tri[1] = Edge.Tri[1] != InvalidID ? (*TidMapping)[Edge.Tri[1]] : InvalidID;
			}
			if (Edge != m2.GetEdge(Eid))
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Edge;
					OutMeshDifferenceInfo->SetEID(Eid);
				}
				return false;
			}
		}
	}
	if (Options.bCheckNormals)
	{
		if (HasVertexNormals() != m2.HasVertexNormals())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Normal;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Has Vertex Normals: %d != %d"), HasVertexNormals(), m2.HasVertexNormals());
			}
			return false;
		}
		if (HasVertexNormals())
		{
			for (int vid : VertexIndicesItr())
			{
				if (VectorUtil::EpsilonEqual(GetVertexNormal(vid), m2.GetVertexNormal(VidMapping ? (*VidMapping)[vid] : vid), Options.EpsilonFloat) == false)
				{
					if (OutMeshDifferenceInfo)
					{
						OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Normal;
						OutMeshDifferenceInfo->SetVID(vid);
					}
					return false;
				}
			}
		}
	}
	if (Options.bCheckColors)
	{
		if (HasVertexColors() != m2.HasVertexColors())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Color;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Has Vertex Colors: %d != %d"), HasVertexColors(), m2.HasVertexColors());
			}
			return false;
		}
		if (HasVertexColors())
		{
			for (int vid : VertexIndicesItr())
			{
				if (VectorUtil::EpsilonEqual(GetVertexColor(vid), m2.GetVertexColor(VidMapping ? (*VidMapping)[vid] : vid), Options.EpsilonFloat) == false)
				{
					if (OutMeshDifferenceInfo)
					{
						OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Color;
						OutMeshDifferenceInfo->SetVID(vid);
					}
					return false;
				}
			}
		}
	}
	if (Options.bCheckUVs)
	{
		if (HasVertexUVs() != m2.HasVertexUVs())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::UV;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Has Vertex UVs: %d != %d"), HasVertexUVs(), m2.HasVertexUVs());
			}
			return false;
		}
		if (HasVertexUVs())
		{
			for (int vid : VertexIndicesItr())
			{
				if (VectorUtil::EpsilonEqual(GetVertexUV(vid), m2.GetVertexUV(VidMapping ? (*VidMapping)[vid] : vid), Options.EpsilonFloat) == false)
				{
					if (OutMeshDifferenceInfo)
					{
						OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::UV;
						OutMeshDifferenceInfo->SetVID(vid);
					}
					return false;
				}
			}
		}
	}
	if (Options.bCheckGroups)
	{
		if (HasTriangleGroups() != m2.HasTriangleGroups())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Group;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Has Triangle Groups: %d != %d"), HasTriangleGroups(), m2.HasTriangleGroups());
			}
			return false;
		}
		if (HasTriangleGroups())
		{
			for (int Tid : TriangleIndicesItr())
			{
				const int TidM2 = TidMapping ? (*TidMapping)[Tid] : Tid;
				if (GetTriangleGroup(Tid) != m2.GetTriangleGroup(TidM2))
				{
					if (OutMeshDifferenceInfo)
					{
						OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Group;
						OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Triangle Group: %d != %d"), GetTriangleGroup(Tid), m2.GetTriangleGroup(TidM2));
						OutMeshDifferenceInfo->SetTID(Tid, TidM2);
					}
					return false;
				}
			}
		}
	}
	if (Options.bCheckAttributes)
	{
		if (HasAttributes() != m2.HasAttributes())
		{
			if (OutMeshDifferenceInfo)
			{
				OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Attribute;
				OutMeshDifferenceInfo->Detail = FString::Printf(TEXT("Has Attribute Set: %d != %d"), HasAttributes(), m2.HasAttributes());
			}
			return false;
		}
		if (HasAttributes())
		{
			if (!AttributeSet->IsSameAs(*m2.AttributeSet, Options.bIgnoreDataLayout))
			{
				if (OutMeshDifferenceInfo)
				{
					OutMeshDifferenceInfo->Reason = FMeshDifferenceInfo::EReason::Attribute;
				}
				return false;
			}
		}
	}
	return true;
}

bool FDynamicMesh3::CheckValidity(FValidityOptions ValidityOptions, EValidityCheckFailMode FailMode) const
{

	TArray<int> triToVtxRefs;
	triToVtxRefs.SetNum(MaxVertexID());

	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FDynamicMesh3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FDynamicMesh3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}

	// When ref counts are dense, the used ref count must match the size of the vertex/triangle vector. 
	CheckOrFailF(!VertexRefCounts.IsDense() || (VertexRefCounts.IsDense() && VertexRefCounts.GetCount() == Vertices.Num()));
	CheckOrFailF(!TriangleRefCounts.IsDense() || (TriangleRefCounts.IsDense() && TriangleRefCounts.GetCount() == Triangles.Num()));

	// TriangleEdges should be 1:1 with Triangles
	CheckOrFailF(TriangleEdges.Num() == Triangles.Num());

	for (int tID : TriangleIndicesItr())
	{
		CheckOrFailF(IsTriangle(tID));
		CheckOrFailF(TriangleRefCounts.GetRefCount(tID) == 1);

		// vertices must exist
		FIndex3i tv = GetTriangle(tID);
		for (int j = 0; j < 3; ++j)
		{
			CheckOrFailF(IsVertex(tv[j]));
			triToVtxRefs[tv[j]] += 1;
		}

		// edges must exist and reference this tri
		FIndex3i e;
		for (int j = 0; j < 3; ++j)
		{
			int a = tv[j], b = tv[(j + 1) % 3];
			e[j] = FindEdge(a, b);
			CheckOrFailF(e[j] != InvalidID);
			CheckOrFailF(EdgeHasTriangle(e[j], tID));
			CheckOrFailF(e[j] == FindEdgeFromTri(a, b, tID));
		}
		CheckOrFailF(e[0] != e[1] && e[0] != e[2] && e[1] != e[2]);

		// tri nbrs must exist and reference this tri, or same edge must be boundary edge
		FIndex3i te = GetTriEdges(tID);
		for (int j = 0; j < 3; ++j)
		{
			int eid = te[j];
			CheckOrFailF(IsEdge(eid));
			int tOther = GetOtherEdgeTriangle(eid, tID);
			if (tOther == InvalidID)
			{
				CheckOrFailF(IsBoundaryTriangle(tID));
				continue;
			}

			CheckOrFailF(TriHasNeighbourTri(tOther, tID) == true);

			// edge must have same two verts as tri for same index
			int a = tv[j], b = tv[(j + 1) % 3];
			FIndex2i ev = GetEdgeV(te[j]);
			CheckOrFailF(IndexUtil::SamePairUnordered(a, b, ev[0], ev[1]));

			// also check that nbr edge has opposite orientation
			if (ValidityOptions.bAllowAdjacentFacesReverseOrientation == false)
			{
				FIndex3i othertv = GetTriangle(tOther);
				int found = IndexUtil::FindTriOrderedEdge(b, a, othertv);
				CheckOrFailF(found != InvalidID);
			}
		}
	}

	if (HasTriangleGroups())
	{
		const TDynamicVector<int>& Groups = TriangleGroups.GetValue();
		// must have a group per triangle ID
		CheckOrFailF(Groups.Num() == MaxTriangleID());
		// group IDs must be in range [0, GroupIDCounter)
		for (int TID : TriangleIndicesItr())
		{
			CheckOrFailF(Groups[TID] >= 0);
			CheckOrFailF(Groups[TID] < GroupIDCounter);
		}
	}


	// edge verts/tris must exist
	for (int eID : EdgeIndicesItr())
	{
		CheckOrFailF(IsEdge(eID));
		CheckOrFailF(EdgeRefCounts.GetRefCount(eID) == 1);
		FIndex2i ev = GetEdgeV(eID);
		FIndex2i et = GetEdgeT(eID);
		CheckOrFailF(IsVertex(ev[0]));
		CheckOrFailF(IsVertex(ev[1]));
		CheckOrFailF(et[0] != InvalidID);
		CheckOrFailF(ev[0] < ev[1]);
		CheckOrFailF(IsTriangle(et[0]));
		if (et[1] != InvalidID)
		{
			CheckOrFailF(IsTriangle(et[1]));
		}
	}

	// verify compact check
	bool is_compact = VertexRefCounts.IsDense();
	if (is_compact)
	{
		for (int vid = 0; vid < (int)Vertices.GetLength(); ++vid)
		{
			CheckOrFailF(VertexRefCounts.IsValid(vid));
		}
	}

	// vertex edges must exist and reference this vert
	for (int vID : VertexIndicesItr())
	{
		CheckOrFailF(IsVertex(vID));

		FVector3d v = GetVertex(vID);
		CheckOrFailF(FMathd::IsNaN(v.SquaredLength()) == false);
		CheckOrFailF(FMathd::IsFinite(v.SquaredLength()));

		for (int edgeid : VertexEdgeLists.Values(vID))
		{
			CheckOrFailF(IsEdge(edgeid));
			CheckOrFailF(EdgeHasVertex(edgeid, vID));

			int otherV = GetOtherEdgeVertex(edgeid, vID);
			int e2 = FindEdge(vID, otherV);
			CheckOrFailF(e2 != InvalidID);
			CheckOrFailF(e2 == edgeid);
			e2 = FindEdge(otherV, vID);
			CheckOrFailF(e2 != InvalidID);
			CheckOrFailF(e2 == edgeid);
		}

		for (int nbr_vid : VtxVerticesItr(vID))
		{
			CheckOrFailF(IsVertex(nbr_vid));
			int edge = FindEdge(vID, nbr_vid);
			CheckOrFailF(IsEdge(edge));
		}

		FDynamicMesh3::FLocalIntArray vTris;
		GetVtxTriangles(vID, vTris);
		//System.Console.WriteLine(string.Format("{0} {1} {2}", vID, vTris.Count, GetVtxEdges(vID).Count));
		if (ValidityOptions.bAllowNonManifoldVertices)
		{
			CheckOrFailF(vTris.Num() <= GetVtxEdgeCount(vID));
		}
		else
		{
			CheckOrFailF(vTris.Num() == GetVtxEdgeCount(vID) || vTris.Num() == GetVtxEdgeCount(vID) - 1);
		}
		int32 VertexRefCount = VertexRefCounts.GetRefCount(vID);
		CheckOrFailF(VertexRefCount == (vTris.Num() + 1));
		CheckOrFailF(triToVtxRefs[vID] == vTris.Num());
		for (int tID : vTris)
		{
			CheckOrFailF(TriangleHasVertex(tID, vID));
		}

		// check that edges around vert only references tris above, and reference all of them!
		TArray<int> vRemoveTris(vTris);
		for (int edgeid : VertexEdgeLists.Values(vID))
		{
			FIndex2i edget = GetEdgeT(edgeid);
			CheckOrFailF(vTris.Contains(edget[0]));
			if (edget[1] != InvalidID)
			{
				CheckOrFailF(vTris.Contains(edget[1]));
			}
			vRemoveTris.Remove(edget[0]);
			if (edget[1] != InvalidID)
			{
				vRemoveTris.Remove(edget[1]);
			}
		}
		CheckOrFailF(vRemoveTris.Num() == 0);
	}

	if (HasAttributes())
	{
		CheckOrFailF(Attributes()->CheckValidity(true, FailMode));
	}

	return is_ok;
}




int FDynamicMesh3::AddTriangleInternal(int a, int b, int c, int e0, int e1, int e2)
{
	int tid = TriangleRefCounts.Allocate();
	Triangles.InsertAt(FIndex3i(a,b,c), tid);
	TriangleEdges.InsertAt(FIndex3i(e0, e1, e2), tid);
	return tid;
}


int FDynamicMesh3::ReplaceEdgeVertex(int eID, int vOld, int vNew)
{
	FIndex2i& Verts = Edges[eID].Vert;
	int a = Verts[0], b = Verts[1];
	if (a == vOld)
	{
		Verts[0] = FMath::Min(b, vNew);
		Verts[1] = FMath::Max(b, vNew);
		return 0;
	}
	else if (b == vOld)
	{
		Verts[0] = FMath::Min(a, vNew);
		Verts[1] = FMath::Max(a, vNew);
		return 1;
	}
	else
	{
		return -1;
	}
}


int FDynamicMesh3::ReplaceEdgeTriangle(int eID, int tOld, int tNew)
{
	FIndex2i& Tris = Edges[eID].Tri;
	int a = Tris[0], b = Tris[1];
	if (a == tOld) {
		if (tNew == InvalidID)
		{
			Tris[0] = b;
			Tris[1] = InvalidID;
		}
		else
		{
			Tris[0] = tNew;
		}
		return 0;
	}
	else if (b == tOld)
	{
		Tris[1] = tNew;
		return 1;
	}
	else
	{
		return -1;
	}
}

int FDynamicMesh3::ReplaceTriangleEdge(int tID, int eOld, int eNew)
{
	FIndex3i& TriEdgeIDs = TriangleEdges[tID];
	for ( int j = 0 ; j < 3 ; ++j )
	{
		if (TriEdgeIDs[j] == eOld)
		{
			TriEdgeIDs[j] = eNew;
			return j;
		}
	}
	return -1;
}



//! returns edge ID
int FDynamicMesh3::FindTriangleEdge(int tID, int vA, int vB) const
{
	const FIndex3i Triangle = Triangles[tID];
	if (IndexUtil::SamePairUnordered(Triangle[0], Triangle[1], vA, vB)) return TriangleEdges[tID][0];
	if (IndexUtil::SamePairUnordered(Triangle[1], Triangle[2], vA, vB)) return TriangleEdges[tID][1];
	if (IndexUtil::SamePairUnordered(Triangle[2], Triangle[0], vA, vB)) return TriangleEdges[tID][2];
	return InvalidID;
}


int32 FDynamicMesh3::FindEdgeInternal(int32 vA, int32 vB, bool& bIsBoundary) const
{
	// edge vertices must be sorted (min,max), that means we only need one index-check in inner loop.
	int32 vMax = vA, vMin = vB;
	if (vB > vA)
	{
		vMax = vB; vMin = vA;
	}
	return VertexEdgeLists.Find(vMin, [&](int32 eid)
	{
		const FEdge Edge = Edges[eid];
		if (Edge.Vert[1] == vMax)
		{
			bIsBoundary = (Edge.Tri[1] == InvalidID);
			return true;
		}
		return false;
	}, InvalidID);
}



int FDynamicMesh3::FindEdge(int vA, int vB) const
{
	checkSlow(IsVertex(vA));
	checkSlow(IsVertex(vB));
	if (vA == vB)
	{
		// self-edges are not allowed, and if we fall through to the search below on a self edge we will incorrectly
		// sometimes return an arbitrary edge if queried for a self-edge, due to the optimization of only checking one side of the edge
		return InvalidID;
	}

	// edge vertices must be sorted (min,max),
	//   that means we only need one index-check in inner loop.
	int32 vMax = vA, vMin = vB;
	if (vB > vA)
	{
		vMax = vB; vMin = vA;
	}
	if (IsVertex(vMin))
	{
		return VertexEdgeLists.Find(vMin, [&](int32 eid)
		{
			return (Edges[eid].Vert[1] == vMax);
		}, InvalidID);
	}
	else
	{
		return InvalidID;
	}

	// this is slower, likely because it creates func<> every time. can we do w/o that?
	//return VertexEdgeLists.Find(vI, (eid) => { return Edges[4 * eid + 1] == vO; }, InvalidID);
}

int FDynamicMesh3::FindEdgeFromTri(int vA, int vB, int tID) const
{
	const FIndex3i& Triangle = Triangles[tID];
	const FIndex3i& TriangleEdgeIDs = TriangleEdges[tID];
	if (IndexUtil::SamePairUnordered(vA, vB, Triangle[0], Triangle[1]))
	{
		return TriangleEdgeIDs[0];
	}
	if (IndexUtil::SamePairUnordered(vA, vB, Triangle[1], Triangle[2]))
	{
		return TriangleEdgeIDs[1];
	}
	if (IndexUtil::SamePairUnordered(vA, vB, Triangle[2], Triangle[0]))
	{
		return TriangleEdgeIDs[2];
	}
	return InvalidID;
}

int FDynamicMesh3::FindEdgeFromTriPair(int TriA, int TriB) const
{
	if (TriangleRefCounts.IsValid(TriA) && TriangleRefCounts.IsValid(TriB))
	{
		for (int j = 0; j < 3; ++j)
		{
			int EdgeID = TriangleEdges[TriA][j];
			const FEdge Edge = Edges[EdgeID];
			int NbrT = (Edge.Tri[0] == TriA) ? Edge.Tri[1] : Edge.Tri[0];
			if (NbrT == TriB)
			{
				return EdgeID;
			}
		}
	}
	return InvalidID;
}





static TAutoConsoleVariable<bool> CVarDynamicMeshDebugMeshesEnabled(
	TEXT("geometry.DynamicMesh.EnableDebugMeshes"),
	false,
	TEXT("Enable/Disable FDynamicMesh3 Global Debug Mesh support. Debug Mesh support is only available in the Editor."));

static FAutoConsoleCommand DynamicMeshClearDebugMeshesCmd(
	TEXT("geometry.DynamicMesh.ClearDebugMeshes"),
	TEXT("Discard all debug meshes currently stored in the FDynamicMesh3 Global Debug Mesh set. This command only works in the Editor."),
	FConsoleCommandDelegate::CreateStatic(UE::Geometry::Debug::ClearAllDebugMeshes) );


namespace UELocal
{
#if WITH_EDITOR
	TMap<FString, TUniquePtr<FDynamicMesh3>> GlobalDebugMeshes;
#endif
}

void UE::Geometry::Debug::ClearAllDebugMeshes()
{
#if WITH_EDITOR
	if (CVarDynamicMeshDebugMeshesEnabled.GetValueOnAnyThread() == false )
	{
		UE_LOGF(LogGeometry, Warning, "ClearAllDebugMeshes() called but geometry.DynamicMesh.EnableDebugMeshes CVar is disabled");
		return;
	}

	UELocal::GlobalDebugMeshes.Reset();
#else
	UE_LOGF(LogGeometry, Warning, "DynamicMesh3 Global Debug Mesh support is only available in-Editor");
#endif
}

void UE::Geometry::Debug::StashDebugMesh(const FDynamicMesh3& Mesh, FString DebugMeshName)
{
#if WITH_EDITOR
	if (CVarDynamicMeshDebugMeshesEnabled.GetValueOnAnyThread() == false )
	{
		UE_LOGF(LogGeometry, Warning, "StashDebugMesh() called but geometry.DynamicMesh.EnableDebugMeshes CVar is disabled")
		return;
	}

	UELocal::GlobalDebugMeshes.Add( DebugMeshName, MakeUnique<FDynamicMesh3>(Mesh) );
#else
	UE_LOGF(LogGeometry, Warning, "DynamicMesh3 Global Debug Mesh support is only available in-Editor");
#endif
}

bool UE::Geometry::Debug::FetchDebugMesh(FString DebugMeshName, FDynamicMesh3& MeshOut, bool bClear)
{
#if WITH_EDITOR
	if (CVarDynamicMeshDebugMeshesEnabled.GetValueOnAnyThread() == false )
	{
		UE_LOGF(LogGeometry, Warning, "FetchDebugMesh() called but geometry.DynamicMesh.EnableDebugMeshes CVar is disabled");
		return false;
	}

	TUniquePtr<FDynamicMesh3>* FoundMesh = UELocal::GlobalDebugMeshes.Find(DebugMeshName);
	if (FoundMesh == nullptr)
	{
		return false;
	}

	if (bClear)
	{
		MeshOut = MoveTemp(**FoundMesh);
		UELocal::GlobalDebugMeshes.Remove(DebugMeshName);
	}
	else
	{
		MeshOut = **FoundMesh;
	}
	return true;
#else
	UE_LOGF(LogGeometry, Warning, "DynamicMesh3 Global Debug Mesh support is only available in-Editor");
	return false;
#endif
}

