// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;


int FDynamicMesh3::AppendVertex(const FVertexInfo& VtxInfo)
{
	int vid = VertexRefCounts.Allocate();
	Vertices.InsertAt(VtxInfo.Position, vid);

	if (HasVertexNormals())
	{
		FVector3f n = (VtxInfo.bHaveN) ? VtxInfo.Normal : FVector3f::UnitY();
		VertexNormals->InsertAt(n, vid);
	}

	if (HasVertexColors())
	{
		FVector3f c = (VtxInfo.bHaveC) ? VtxInfo.Color : FVector3f::One();
		VertexColors->InsertAt(c, vid);
	}

	if (HasVertexUVs())
	{
		FVector2f u = (VtxInfo.bHaveUV) ? VtxInfo.UV : FVector2f::Zero();
		VertexUVs->InsertAt(u, vid);
	}

	AllocateEdgesList(vid);

	if (HasAttributes())
	{
		Attributes()->OnNewVertex(vid, false);
	}

	UpdateChangeStamps(true, true);
	return vid;
}



int FDynamicMesh3::AppendVertex(const FDynamicMesh3& from, int fromVID)
{
	const int vid = VertexRefCounts.Allocate();
	Vertices.InsertAt(from.Vertices[fromVID], vid);

	if (HasVertexNormals())
	{
		if (from.HasVertexNormals())
		{
			const TDynamicVector<FVector3f>& FromNormals = from.VertexNormals.GetValue();
			VertexNormals->InsertAt(FromNormals[fromVID], vid);
		}
		else
		{
			VertexNormals->InsertAt({0,1,0}, vid);   // y-up
		}
	}

	if (HasVertexColors())
	{
		if (from.HasVertexColors())
		{
			const TDynamicVector<FVector3f>& FromColors = from.VertexColors.GetValue();
			VertexColors->InsertAt(FromColors[fromVID], vid);
		}
		else
		{
			VertexColors->InsertAt({1,1,1}, vid);   // white
		}
	}

	if (HasVertexUVs())
	{
		if (from.HasVertexUVs())
		{
			const TDynamicVector<FVector2f>& FromUVs = from.VertexUVs.GetValue();
			VertexUVs->InsertAt(FromUVs[fromVID], vid);
		}
		else
		{
			VertexUVs->InsertAt({0,0}, vid);
		}
	}

	AllocateEdgesList(vid);

	if (HasAttributes())
	{
		Attributes()->OnNewVertex(vid, false);
	}

	UpdateChangeStamps(true, true);
	return vid;
}



EMeshResult FDynamicMesh3::InsertVertex(int vid, const FVertexInfo& info, bool bUnsafe)
{
	if (VertexRefCounts.IsValid(vid))
	{
		return EMeshResult::Failed_VertexAlreadyExists;
	}

	bool bOK = (bUnsafe) ? VertexRefCounts.AllocateAtUnsafe(vid) :
		VertexRefCounts.AllocateAt(vid);
	if (bOK == false)
	{
		return EMeshResult::Failed_CannotAllocateVertex;
	}

	Vertices.InsertAt(info.Position, vid);

	if (HasVertexNormals())
	{
		FVector3f n = (info.bHaveN) ? info.Normal : FVector3f::UnitY();
		VertexNormals->InsertAt(n, vid);
	}

	if (HasVertexColors())
	{
		FVector3f c = (info.bHaveC) ? info.Color : FVector3f::One();
		VertexColors->InsertAt(c, vid);
	}

	if (HasVertexUVs())
	{
		FVector2f u = (info.bHaveUV) ? info.UV : FVector2f::Zero();
		VertexUVs->InsertAt(u, vid);
	}

	AllocateEdgesList(vid);

	if (HasAttributes())
	{
		Attributes()->OnNewVertex(vid, true);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}




int FDynamicMesh3::AppendTriangle(const FIndex3i& tv, int gid)
{
	if (IsVertex(tv[0]) == false || IsVertex(tv[1]) == false || IsVertex(tv[2]) == false)
	{
		checkSlow(false);
		return InvalidID;
	}
	if (tv[0] == tv[1] || tv[0] == tv[2] || tv[1] == tv[2])
	{
		checkSlow(false);
		return InvalidID;
	}

	// look up edges. if any already have two triangles, this would
	// create non-manifold geometry and so we do not allow it
	bool boundary0, boundary1, boundary2;
	int e0 = FindEdgeInternal(tv[0], tv[1], boundary0);
	int e1 = FindEdgeInternal(tv[1], tv[2], boundary1);
	int e2 = FindEdgeInternal(tv[2], tv[0], boundary2);
	if ((e0 != InvalidID && boundary0 == false)
		|| (e1 != InvalidID && boundary1 == false)
		|| (e2 != InvalidID && boundary2 == false))
	{
		return NonManifoldID;
	}

	// if all edges were already present, check for duplicate triangle
	if (e0 != InvalidID && e1 != InvalidID && e2 != InvalidID)
	{
		// check if the triangle attached to edge e0 (tv[0] - tv[1]) also contains tv[2]
		int ti = Edges[e0].Tri[0];
		if (Triangles[ti][0] == tv[2] || Triangles[ti][1] == tv[2] || Triangles[ti][2] == tv[2])
		{
			return DuplicateTriangleID;
		}
		// there's no other triangle on the edge to check -- if there were, we would have already returned NonManifoldID
		checkSlow(Edges[e0].Tri[1] == InvalidID);
	}

	bool bHasGroups = HasTriangleGroups();  // have to check before changing .triangles

	// now safe to insert triangle
	int tid = TriangleRefCounts.Allocate();
	Triangles.InsertAt(tv, tid);
	if (bHasGroups)
	{
		TriangleGroups->InsertAt(gid, tid);
		GroupIDCounter = FMath::Max(GroupIDCounter, gid + 1);
	}

	// increment ref counts and update/create edges
	VertexRefCounts.Increment(tv[0]);
	VertexRefCounts.Increment(tv[1]);
	VertexRefCounts.Increment(tv[2]);

	AddTriangleEdge(tid, tv[0], tv[1], 0, e0);
	AddTriangleEdge(tid, tv[1], tv[2], 1, e1);
	AddTriangleEdge(tid, tv[2], tv[0], 2, e2);

	if (HasAttributes())
	{
		Attributes()->OnNewTriangle(tid, false);
	}

	UpdateChangeStamps(true, true);
	return tid;
}




EMeshResult FDynamicMesh3::InsertTriangle(int tid, const FIndex3i& tv, int gid, bool bUnsafe)
{
	if (TriangleRefCounts.IsValid(tid))
	{
		return EMeshResult::Failed_TriangleAlreadyExists;
	}

	if (IsVertex(tv[0]) == false || IsVertex(tv[1]) == false || IsVertex(tv[2]) == false)
	{
		checkSlow(false);
		return EMeshResult::Failed_NotAVertex;
	}
	if (tv[0] == tv[1] || tv[0] == tv[2] || tv[1] == tv[2])
	{
		checkSlow(false);
		return EMeshResult::Failed_InvalidNeighbourhood;
	}

	// look up edges. if any already have two triangles, this would
	// create non-manifold geometry and so we do not allow it
	int e0 = FindEdge(tv[0], tv[1]);
	int e1 = FindEdge(tv[1], tv[2]);
	int e2 = FindEdge(tv[2], tv[0]);
	if ((e0 != InvalidID && IsBoundaryEdge(e0) == false)
		|| (e1 != InvalidID && IsBoundaryEdge(e1) == false)
		|| (e2 != InvalidID && IsBoundaryEdge(e2) == false))
	{
		return EMeshResult::Failed_WouldCreateNonmanifoldEdge;
	}

	bool bOK = (bUnsafe) ? TriangleRefCounts.AllocateAtUnsafe(tid) :
		TriangleRefCounts.AllocateAt(tid);
	if (bOK == false)
	{
		return EMeshResult::Failed_CannotAllocateTriangle;
	}

	// now safe to insert triangle
	Triangles.InsertAt(tv, tid);
	if (HasTriangleGroups())
	{
		TriangleGroups->InsertAt(gid, tid);
		GroupIDCounter = FMath::Max(GroupIDCounter, gid + 1);
	}

	// increment ref counts and update/create edges
	VertexRefCounts.Increment(tv[0]);
	VertexRefCounts.Increment(tv[1]);
	VertexRefCounts.Increment(tv[2]);

	AddTriangleEdge(tid, tv[0], tv[1], 0, e0);
	AddTriangleEdge(tid, tv[1], tv[2], 1, e1);
	AddTriangleEdge(tid, tv[2], tv[0], 2, e2);

	if (HasAttributes())
	{
		Attributes()->OnNewTriangle(tid, true);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}






int32 FDynamicMesh3::RemoveUnusedVertices()
{
	int32 NumRemoved = 0;
	for (int32 VID = 0; VID < MaxVertexID(); ++VID)
	{
		// If vertex exists but is not referenced by any triangles
		if (VertexRefCounts.GetRefCount(VID) == 1)
		{
			NumRemoved++;
			VertexRefCounts.Decrement(VID);

			if (HasAttributes())
			{
				Attributes()->OnRemoveVertex(VID);
			}
			checkSlow(VertexRefCounts.IsValid(VID) == false); // vertex should now not be valid
			checkSlow(VertexEdgeLists.GetCount(VID) == 0); // vertex should not have had any edges attached
		}
	}

	if (NumRemoved > 0)
	{
		UpdateChangeStamps(true, true);
	}
	return NumRemoved;
}

bool FDynamicMesh3::HasUnusedVertices() const
{
	for (int32 VID = 0; VID < MaxVertexID(); ++VID)
	{
		// If vertex exists but is not referenced by any triangles
		if (VertexRefCounts.GetRefCount(VID) == 1)
		{
			return true;
		}
	}
	
	return false;
}


void FDynamicMesh3::CompactInPlace(FCompactMaps* CompactInfo)
{
	// Initialize CompactInfo

	// If we need a CompactInfo for compacting attributes but we don't have one, we'll make it refer to a local one.
	FCompactMaps LocalCompactInfo;
	if (HasAttributes() && !CompactInfo)
	{
		CompactInfo = &LocalCompactInfo;
	}

	if (CompactInfo)
	{
		// starts as identity (except at gaps); sparsely remapped below
		CompactInfo->Reset(MaxVertexID(), MaxTriangleID(), false);
		for (int VID = 0, NumVID = MaxVertexID(); VID < NumVID; VID++)
		{
			CompactInfo->SetVertexMapping(VID, IsVertex(VID) ? VID : -1);
		}
		for (int TID = 0, NumTID = MaxTriangleID(); TID < NumTID; TID++)
		{
			CompactInfo->SetTriangleMapping(TID, IsTriangle(TID) ? TID : -1);
		}
	}

	// find first free vertex, and last used vertex
	int iLastV = MaxVertexID() - 1, iCurV = 0;
	while (iLastV >= 0 && VertexRefCounts.IsValidUnsafe(iLastV) == false)
	{
		iLastV--;
	}
	while (iCurV < iLastV && VertexRefCounts.IsValidUnsafe(iCurV))
	{
		iCurV++;
	}

	TDynamicVector<unsigned short> &vref = VertexRefCounts.GetRawRefCountsUnsafe();

	while (iCurV < iLastV)
	{
		Vertices[iCurV] = Vertices[iLastV];

		//const int kc = iCurV * 3;
		//const int kl = iLastV * 3;
		if (HasVertexNormals())
		{
			TDynamicVector<FVector3f>& Normals = VertexNormals.GetValue();
			Normals[iCurV] = Normals[iLastV];
		}
		if (HasVertexColors())
		{
			TDynamicVector<FVector3f>& Colors = VertexColors.GetValue();
			Colors[iCurV] =     Colors[iLastV];
		}
		if (HasVertexUVs())
		{
			TDynamicVector<FVector2f>& UVs = VertexUVs.GetValue();
			UVs[iCurV] =     UVs[iLastV];
		}

		for (int eid : VertexEdgeLists.Values(iLastV))
		{
			// replace vertex in edges
			ReplaceEdgeVertex(eid, iLastV, iCurV);

			// replace vertex in triangles
			const FIndex2i Tris = Edges[eid].Tri;
			ReplaceTriangleVertex(Tris[0], iLastV, iCurV);
			if (Tris[1] != InvalidID)
			{
				ReplaceTriangleVertex(Tris[1], iLastV, iCurV);
			}
		}

		// shift vertex refcount to position
		vref[iCurV] = vref[iLastV];
		vref[iLastV] = FRefCountVector::INVALID_REF_COUNT;

		// move edge list
		VertexEdgeLists.Move(iLastV, iCurV);

		if (CompactInfo != nullptr)
		{
			CompactInfo->SetVertexMapping(iLastV, iCurV);
		}

		// move cur forward one, last back one, and  then search for next valid
		iLastV--; iCurV++;
		while (iLastV >= 0 && VertexRefCounts.IsValidUnsafe(iLastV) == false)
		{
			iLastV--;
		}
		while (iCurV < iLastV && VertexRefCounts.IsValidUnsafe(iCurV))
		{
			iCurV++;
		}
	}

	// trim vertices data structures
	VertexRefCounts.Trim(VertexCount());
	Vertices.Resize(VertexCount());
	if (HasVertexNormals())
	{
		VertexNormals->Resize(VertexCount() * 3);
	}
	if (HasVertexColors())
	{
		VertexColors->Resize(VertexCount() * 3);
	}
	if (HasVertexUVs())
	{
		VertexUVs->Resize(VertexCount() * 2);
	}

	VertexEdgeLists.Compact(VertexCount());

	/** shift triangles **/

	// find first free triangle, and last valid triangle
	int iLastT = MaxTriangleID() - 1, iCurT = 0;
	while (iLastT >= 0 && TriangleRefCounts.IsValidUnsafe(iLastT) == false)
	{
		iLastT--;
	}
	while (iCurT < iLastT && TriangleRefCounts.IsValidUnsafe(iCurT))
	{
		iCurT++;
	}

	TDynamicVector<unsigned short> &tref = TriangleRefCounts.GetRawRefCountsUnsafe();

	while (iCurT < iLastT)
	{
		// shift triangle
		Triangles[iCurT] = Triangles[iLastT];
		TriangleEdges[iCurT] = TriangleEdges[iLastT];

		if (HasTriangleGroups())
		{
			TriangleGroups.GetValue()[iCurT] = TriangleGroups.GetValue()[iLastT];
		}

		// update edges
		for (int j = 0; j < 3; ++j)
		{
			int eid = TriangleEdges[iCurT][j];
			ReplaceEdgeTriangle(eid, iLastT, iCurT);
		}

		// shift triangle refcount to position
		tref[iCurT] = tref[iLastT];
		tref[iLastT] = FRefCountVector::INVALID_REF_COUNT;

		if (CompactInfo != nullptr)
		{
			CompactInfo->SetTriangleMapping(iLastT, iCurT);
		}

		// move cur forward one, last back one, and  then search for next valid
		iLastT--; iCurT++;
		while (iLastT >= 0 && TriangleRefCounts.IsValidUnsafe(iLastT) == false)
		{
			iLastT--;
		}
		while (iCurT < iLastT && TriangleRefCounts.IsValidUnsafe(iCurT))
		{
			iCurT++;
		}
	}

	// trim triangles data structures
	TriangleRefCounts.Trim(TriangleCount());
	Triangles.Resize(TriangleCount());
	TriangleEdges.Resize(TriangleCount());
	if (HasTriangleGroups())
	{
		TriangleGroups->Resize(TriangleCount());
	}

	/** shift edges **/

	// find first free edge, and last used edge
	int iLastE = MaxEdgeID() - 1, iCurE = 0;
	while (iLastE >= 0 && EdgeRefCounts.IsValidUnsafe(iLastE) == false)
	{
		iLastE--;
	}
	while (iCurE < iLastE && EdgeRefCounts.IsValidUnsafe(iCurE))
	{
		iCurE++;
	}

	TDynamicVector<unsigned short> &eref = EdgeRefCounts.GetRawRefCountsUnsafe();

	while (iCurE < iLastE)
	{
		Edges[iCurE] = Edges[iLastE];

		// replace edge in vertex edges lists
		int v0 = Edges[iCurE].Vert[0], v1 = Edges[iCurE].Vert[1];
		VertexEdgeLists.Replace(v0, [iLastE](int eid) { return eid == iLastE; }, iCurE);
		VertexEdgeLists.Replace(v1, [iLastE](int eid) { return eid == iLastE; }, iCurE);

		// replace edge in triangles
		ReplaceTriangleEdge(Edges[iCurE].Tri[0], iLastE, iCurE);
		if (Edges[iCurE].Tri[1] != InvalidID)
		{
			ReplaceTriangleEdge(Edges[iCurE].Tri[1], iLastE, iCurE);
		}

		// shift triangle refcount to position
		eref[iCurE] = eref[iLastE];
		eref[iLastE] = FRefCountVector::INVALID_REF_COUNT;

		// move cur forward one, last back one, and  then search for next valid
		iLastE--; iCurE++;
		while (iLastE >= 0 && EdgeRefCounts.IsValidUnsafe(iLastE) == false)
		{
			iLastE--;
		}
		while (iCurE < iLastE && EdgeRefCounts.IsValidUnsafe(iCurE))
		{
			iCurE++;
		}
	}

	// trim edge data structures
	EdgeRefCounts.Trim(EdgeCount());
	Edges.Resize(EdgeCount());

	if (HasAttributes())
	{
		checkSlow(CompactInfo);		// can this ever fail?
		AttributeSet->CompactInPlace(*CompactInfo);
	}
}







EMeshResult FDynamicMesh3::ReverseTriOrientation(int tID)
{
	if (!IsTriangle(tID))
	{
		return EMeshResult::Failed_NotATriangle;
	}
	ReverseTriOrientationInternal(tID);
	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}

void FDynamicMesh3::ReverseTriOrientationInternal(int tID)
{
	FIndex3i t = GetTriangle(tID);
	SetTriangleInternal(tID, t[1], t[0], t[2]);
	FIndex3i te = GetTriEdges(tID);
	SetTriangleEdgesInternal(tID, te[0], te[2], te[1]);
	if (HasAttributes())
	{
		Attributes()->OnReverseTriOrientation(tID);
	}
}

void FDynamicMesh3::ReverseOrientation(bool bFlipNormals)
{
	for (int tid : TriangleIndicesItr())
	{
		ReverseTriOrientationInternal(tid);
	}
	if (bFlipNormals && HasVertexNormals())
	{
		for (int vid : VertexIndicesItr())
		{
			TDynamicVector<FVector3f>& Normals = VertexNormals.GetValue();
			Normals[vid] = -Normals[vid];
		}
	}
	UpdateChangeStamps(true, true);
}

EMeshResult FDynamicMesh3::RemoveVertex(int vID, bool bPreserveManifold)
{
	if (VertexRefCounts.IsValid(vID) == false)
	{
		return EMeshResult::Failed_NotAVertex;
	}

	// if any one-ring vtx is a boundary vtx and one of its outer-ring edges is an
	// interior edge then we will create a bowtie if we remove that triangle
	if (bPreserveManifold)
	{
		for (int tid : VtxTrianglesItr(vID))
		{
			FIndex3i tri = GetTriangle(tid);
			int j = IndexUtil::FindTriIndex(vID, tri);
			int oa = tri[(j + 1) % 3], ob = tri[(j + 2) % 3];
			int eid = FindEdge(oa, ob);
			if (IsBoundaryEdge(eid))
			{
				continue;
			}
			if (IsBoundaryVertex(oa) || IsBoundaryVertex(ob))
			{
				return EMeshResult::Failed_WouldCreateBowtie;
			}
		}
	}

	// Remove incident triangles
	FDynamicMesh3::FLocalIntArray tris;
	GetVtxTriangles(vID, tris);
	for (int tID : tris)
	{
		EMeshResult result = RemoveTriangle(tID, false, bPreserveManifold);
		if (result != EMeshResult::Ok)
		{
			return result;
		}
	}

	if (VertexRefCounts.GetRefCount(vID) != 1)
	{
		return EMeshResult::Failed_VertexStillReferenced;
	}

	VertexRefCounts.Decrement(vID);
	ensure(VertexRefCounts.IsValid(vID) == false);
	VertexEdgeLists.Clear(vID);

	if (HasAttributes())
	{
		Attributes()->OnRemoveVertex(vID);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}








EMeshResult FDynamicMesh3::RemoveTriangle(int tID, bool bRemoveIsolatedVertices, bool bPreserveManifold)
{
	if (!TriangleRefCounts.IsValid(tID))
	{
		ensure(false);
		return EMeshResult::Failed_NotATriangle;
	}

	FIndex3i tv = GetTriangle(tID);
	FIndex3i te = GetTriEdges(tID);

	// if any tri vtx is a boundary vtx connected to two interior edges, then
	// we cannot remove this triangle because it would create a bowtie vertex!
	// (that vtx already has 2 boundary edges, and we would add two more)
	if (bPreserveManifold)
	{
		for (int j = 0; j < 3; ++j)
		{
			if (IsBoundaryVertex(tv[j]))
			{
				if (IsBoundaryEdge(te[j]) == false && IsBoundaryEdge(te[(j + 2) % 3]) == false)
				{
					return EMeshResult::Failed_WouldCreateBowtie;
				}
			}
		}
	}

	// Remove triangle from its edges. if edge has no triangles left,
	// then it is removed.
	for (int j = 0; j < 3; ++j)
	{
		int eid = te[j];
		ReplaceEdgeTriangle(eid, tID, InvalidID);
		const FEdge Edge = Edges[eid];
		if (Edge.Tri[0] == InvalidID)
		{
			int a = Edge.Vert[0];
			VertexEdgeLists.Remove(a, eid);

			int b = Edge.Vert[1];
			VertexEdgeLists.Remove(b, eid);

			EdgeRefCounts.Decrement(eid);
		}
	}

	// free this triangle
	TriangleRefCounts.Decrement(tID);
	checkSlow(TriangleRefCounts.IsValid(tID) == false);

	// Decrement vertex refcounts. If any hit 1 and we got remove-isolated flag,
	// we need to remove that vertex
	for (int j = 0; j < 3; ++j)
	{
		int vid = tv[j];
		VertexRefCounts.Decrement(vid);
		if (bRemoveIsolatedVertices && VertexRefCounts.GetRefCount(vid) == 1)
		{
			VertexRefCounts.Decrement(vid);
			checkSlow(VertexRefCounts.IsValid(vid) == false);
			VertexEdgeLists.Clear(vid);
		}
	}

	if (HasAttributes())
	{
		Attributes()->OnRemoveTriangle(tID);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}







EMeshResult FDynamicMesh3::SetTriangle(int tID, const FIndex3i& newv, bool bRemoveIsolatedVertices)
{
	// @todo support this.
	if (ensure(HasAttributes()) == false)
	{
		return EMeshResult::Failed_Unsupported;
	}

	FIndex3i tv = GetTriangle(tID);
	FIndex3i te = GetTriEdges(tID);
	if (tv[0] == newv[0] && tv[1] == newv[1])
	{
		te[0] = -1;
	}
	if (tv[1] == newv[1] && tv[2] == newv[2])
	{
		te[1] = -1;
	}
	if (tv[2] == newv[2] && tv[0] == newv[0])
	{
		te[2] = -1;
	}

	if (!TriangleRefCounts.IsValid(tID))
	{
		checkSlow(false);
		return EMeshResult::Failed_NotATriangle;
	}
	if (IsVertex(newv[0]) == false || IsVertex(newv[1]) == false || IsVertex(newv[2]) == false)
	{
		checkSlow(false);
		return EMeshResult::Failed_NotAVertex;
	}
	if (newv[0] == newv[1] || newv[0] == newv[2] || newv[1] == newv[2])
	{
		checkSlow(false);
		return EMeshResult::Failed_BrokenTopology;
	}
	// look up edges. if any already have two triangles, this would
	// create non-manifold geometry and so we do not allow it
	int e0 = FindEdge(newv[0], newv[1]);
	int e1 = FindEdge(newv[1], newv[2]);
	int e2 = FindEdge(newv[2], newv[0]);
	if ((te[0] != -1 && e0 != InvalidID && IsBoundaryEdge(e0) == false)
		|| (te[1] != -1 && e1 != InvalidID && IsBoundaryEdge(e1) == false)
		|| (te[2] != -1 && e2 != InvalidID && IsBoundaryEdge(e2) == false))
	{
		return EMeshResult::Failed_BrokenTopology;
	}


	// [TODO] check that we are not going to create invalid stuff...

	// Remove triangle from its edges. if edge has no triangles left, then it is removed.
	for (int j = 0; j < 3; ++j)
	{
		int eid = te[j];
		if (eid == -1)      // we don't need to modify this edge
		{
			continue;
		}
		ReplaceEdgeTriangle(eid, tID, InvalidID);
		const FEdge Edge = GetEdge(eid);
		if (Edge.Tri[0] == InvalidID)
		{
			int a = Edge.Vert[0];
			VertexEdgeLists.Remove(a, eid);

			int b = Edge.Vert[1];
			VertexEdgeLists.Remove(b, eid);

			EdgeRefCounts.Decrement(eid);
		}
	}

	// Decrement vertex refcounts. If any hit 1 and we got remove-isolated flag,
	// we need to remove that vertex
	for (int j = 0; j < 3; ++j)
	{
		int vid = tv[j];
		if (vid == newv[j])     // we don't need to modify this vertex
		{
			continue;
		}
		VertexRefCounts.Decrement(vid);
		if (bRemoveIsolatedVertices && VertexRefCounts.GetRefCount(vid) == 1)
		{
			VertexRefCounts.Decrement(vid);
			checkSlow(VertexRefCounts.IsValid(vid) == false);
			VertexEdgeLists.Clear(vid);
		}
	}


	// ok now re-insert with vertices
	for (int j = 0; j < 3; ++j)
	{
		if (newv[j] != tv[j])
		{
			Triangles[tID][j] = newv[j];
			VertexRefCounts.Increment(newv[j]);
		}
	}

	if (te[0] != -1)
	{
		AddTriangleEdge(tID, newv[0], newv[1], 0, e0);
	}
	if (te[1] != -1)
	{
		AddTriangleEdge(tID, newv[1], newv[2], 1, e1);
	}
	if (te[2] != -1)
	{
		AddTriangleEdge(tID, newv[2], newv[0], 2, e2);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}









EMeshResult FDynamicMesh3::SplitEdge(int eab, FEdgeSplitInfo& SplitInfo, double split_t)
{
	SplitInfo = FEdgeSplitInfo();

	if (!IsEdge(eab))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	// look up primary edge & triangle
	const FEdge Edge = Edges[eab];
	int a = Edge.Vert[0], b = Edge.Vert[1];
	int t0 = Edge.Tri[0];
	if (t0 == InvalidID)
	{
		return EMeshResult::Failed_BrokenTopology;
	}
	FIndex3i T0tv = GetTriangle(t0);
	int c = IndexUtil::OrientTriEdgeAndFindOtherVtx(a, b, T0tv);

	// RefCount overflow check. Conservatively leave room for
	// extra increments from other operations.
	if (VertexRefCounts.GetRawRefCount(c) > FRefCountVector::INVALID_REF_COUNT - 3)
	{
		return EMeshResult::Failed_HitValenceLimit;
	}
	if (a != Edge.Vert[0])
	{
		split_t = 1.0 - split_t;    // if we flipped a/b order we need to reverse t
	}

	SplitInfo.OriginalEdge = eab;
	SplitInfo.OriginalVertices = FIndex2i(a, b);   // this is the oriented a,b
	SplitInfo.OriginalTriangles = FIndex2i(t0, InvalidID);
	SplitInfo.SplitT = split_t;

	// quite a bit of code is duplicated between boundary and non-boundary case, but it
	//  is too hard to follow later if we factor it out...
	if (IsBoundaryEdge(eab))
	{
		// create vertex
		FVector3d vNew = Lerp(GetVertex(a), GetVertex(b), split_t);
		int f = AppendVertex(vNew);
		if (HasVertexNormals())
		{
			SetVertexNormal(f, Normalized(Lerp(GetVertexNormal(a), GetVertexNormal(b), (float)split_t)) );
		}
		if (HasVertexColors())
		{
			SetVertexColor(f, Lerp(GetVertexColor(a), GetVertexColor(b), (float)split_t));
		}
		if (HasVertexUVs())
		{
			SetVertexUV(f, Lerp(GetVertexUV(a), GetVertexUV(b), (float)split_t));
		}

		// look up edge bc, which needs to be modified
		FIndex3i T0te = GetTriEdges(t0);
		int ebc = T0te[IndexUtil::FindEdgeIndexInTri(b, c, T0tv)];

		// rewrite existing triangle
		ReplaceTriangleVertex(t0, b, f);

		// add second triangle
		int t2 = AddTriangleInternal(f, b, c, InvalidID, InvalidID, InvalidID);
		if (HasTriangleGroups())
		{
			int group0 = TriangleGroups.GetValue()[t0];
			TriangleGroups->InsertAt(group0, t2);
		}

		// rewrite edge bc, create edge af
		ReplaceEdgeTriangle(ebc, t0, t2);
		int eaf = eab;
		ReplaceEdgeVertex(eaf, b, f);
		VertexEdgeLists.Remove(b, eab);
		VertexEdgeLists.Insert(f, eaf);

		// create edges fb and fc
		int efb = AddEdgeInternal(f, b, t2);
		int efc = AddEdgeInternal(f, c, t0, t2);

		// update triangle edge-nbrs
		ReplaceTriangleEdge(t0, ebc, efc);
		SetTriangleEdgesInternal(t2, efb, ebc, efc);

		// update vertex refcounts
		VertexRefCounts.Increment(c);
		VertexRefCounts.Increment(f, 2);

		SplitInfo.bIsBoundary = true;
		SplitInfo.OtherVertices = FIndex2i(c, InvalidID);
		SplitInfo.NewVertex = f;
		SplitInfo.NewEdges = FIndex3i(efb, efc, InvalidID);
		SplitInfo.NewTriangles = FIndex2i(t2, InvalidID);

		if (HasAttributes())
		{
			Attributes()->OnSplitEdge(SplitInfo);
		}

		UpdateChangeStamps(true, true);
		return EMeshResult::Ok;

	}
	else 		// interior triangle branch
	{
		// look up other triangle
		int t1 = Edges[eab].Tri[1];
		SplitInfo.OriginalTriangles.B = t1;
		FIndex3i T1tv = GetTriangle(t1);
		int d = IndexUtil::FindTriOtherVtx(a, b, T1tv);

		// RefCount overflow check. Conservatively leave room for
		// extra increments from other operations.
		if (VertexRefCounts.GetRawRefCount(d) > FRefCountVector::INVALID_REF_COUNT - 3)
		{
			return EMeshResult::Failed_HitValenceLimit;
		}

		// create vertex
		FVector3d vNew = Lerp(GetVertex(a), GetVertex(b), split_t);
		int f = AppendVertex(vNew);
		if (HasVertexNormals())
		{
			SetVertexNormal(f, Normalized(Lerp(GetVertexNormal(a), GetVertexNormal(b), (float)split_t)) );
		}
		if (HasVertexColors())
		{
			SetVertexColor(f, Lerp(GetVertexColor(a), GetVertexColor(b), (float)split_t));
		}
		if (HasVertexUVs())
		{
			SetVertexUV(f, Lerp(GetVertexUV(a), GetVertexUV(b), (float)split_t));
		}

		// look up edges that we are going to need to update
		// [TODO OPT] could use ordering to reduce # of compares here
		FIndex3i T0te = GetTriEdges(t0);
		int ebc = T0te[IndexUtil::FindEdgeIndexInTri(b, c, T0tv)];
		FIndex3i T1te = GetTriEdges(t1);
		int edb = T1te[IndexUtil::FindEdgeIndexInTri(d, b, T1tv)];

		// rewrite existing triangles
		ReplaceTriangleVertex(t0, b, f);
		ReplaceTriangleVertex(t1, b, f);

		// add two triangles to close holes we just created
		int t2 = AddTriangleInternal(f, b, c, InvalidID, InvalidID, InvalidID);
		int t3 = AddTriangleInternal(f, d, b, InvalidID, InvalidID, InvalidID);
		if (HasTriangleGroups())
		{
			int group0 = TriangleGroups.GetValue()[t0];
			TriangleGroups->InsertAt(group0, t2);
			int group1 = TriangleGroups.GetValue()[t1];
			TriangleGroups->InsertAt(group1, t3);
		}

		// update the edges we found above, to point to triangles
		ReplaceEdgeTriangle(ebc, t0, t2);
		ReplaceEdgeTriangle(edb, t1, t3);

		// edge eab became eaf
		int eaf = eab; //Edge * eAF = eAB;
		ReplaceEdgeVertex(eaf, b, f);

		// update a/b/f vertex-edges
		VertexEdgeLists.Remove(b, eab);
		VertexEdgeLists.Insert(f, eaf);

		// create edges connected to f  (also updates vertex-edges)
		int efb = AddEdgeInternal(f, b, t2, t3);
		int efc = AddEdgeInternal(f, c, t0, t2);
		int edf = AddEdgeInternal(d, f, t1, t3);

		// update triangle edge-nbrs
		ReplaceTriangleEdge(t0, ebc, efc);
		ReplaceTriangleEdge(t1, edb, edf);
		SetTriangleEdgesInternal(t2, efb, ebc, efc);
		SetTriangleEdgesInternal(t3, edf, edb, efb);

		// update vertex refcounts
		VertexRefCounts.Increment(c);
		VertexRefCounts.Increment(d);
		VertexRefCounts.Increment(f, 4);

		SplitInfo.bIsBoundary = false;
		SplitInfo.OtherVertices = FIndex2i(c, d);
		SplitInfo.NewVertex = f;
		SplitInfo.NewEdges = FIndex3i(efb, efc, edf);
		SplitInfo.NewTriangles = FIndex2i(t2, t3);

		if (HasAttributes())
		{
			Attributes()->OnSplitEdge(SplitInfo);
		}

		UpdateChangeStamps(true, true);
		return EMeshResult::Ok;
	}

}


EMeshResult FDynamicMesh3::SplitEdge(int vA, int vB, FEdgeSplitInfo& SplitInfo)
{
	int eid = FindEdge(vA, vB);
	if (eid == InvalidID)
	{
		SplitInfo = FEdgeSplitInfo();
		return EMeshResult::Failed_NotAnEdge;
	}
	return SplitEdge(eid, SplitInfo);
}








EMeshResult FDynamicMesh3::FlipEdge(int eab, FEdgeFlipInfo& FlipInfo)
{
	FlipInfo = FEdgeFlipInfo();

	if (!IsEdge(eab))
	{
		return EMeshResult::Failed_NotAnEdge;
	}
	if (IsBoundaryEdge(eab))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}

	// find oriented edge [a,b], tris t0,t1, and other verts c in t0, d in t1
	const FEdge Edge = Edges[eab];
	int a = Edge.Vert[0], b = Edge.Vert[1];
	int t0 = Edge.Tri[0], t1 = Edge.Tri[1];
	FIndex3i T0tv = GetTriangle(t0), T1tv = GetTriangle(t1);
	int c = IndexUtil::OrientTriEdgeAndFindOtherVtx(a, b, T0tv);
	int d = IndexUtil::FindTriOtherVtx(a, b, T1tv);
	if (c == InvalidID || d == InvalidID)
	{
		return EMeshResult::Failed_BrokenTopology;
	}

	int flipped = FindEdge(c, d);
	if (flipped != InvalidID)
	{
		return EMeshResult::Failed_FlippedEdgeExists;
	}

	// find edges bc, ca, ad, db
	int ebc = FindTriangleEdge(t0, b, c);
	int eca = FindTriangleEdge(t0, c, a);
	int ead = FindTriangleEdge(t1, a, d);
	int edb = FindTriangleEdge(t1, d, b);

	// update triangles
	SetTriangleInternal(t0, c, d, b);
	SetTriangleInternal(t1, d, c, a);

	// update edge AB, which becomes flipped edge CD
	SetEdgeVerticesInternal(eab, c, d);
	SetEdgeTrianglesInternal(eab, t0, t1);
	int ecd = eab;

	// update the two other edges whose triangle nbrs have changed
	if (ReplaceEdgeTriangle(eca, t0, t1) == -1)
	{
		checkfSlow(false, TEXT("FDynamicMesh3.FlipEdge: first ReplaceEdgeTriangle failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}
	if (ReplaceEdgeTriangle(edb, t1, t0) == -1)
	{
		checkfSlow(false, TEXT("FDynamicMesh3.FlipEdge: second ReplaceEdgeTriangle failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}

	// update triangle nbr lists (these are edges)
	SetTriangleEdgesInternal(t0, ecd, edb, ebc);
	SetTriangleEdgesInternal(t1, ecd, eca, ead);

	// remove old eab from verts a and b, and Decrement ref counts
	if (VertexEdgeLists.Remove(a, eab) == false)
	{
		checkfSlow(false, TEXT("FDynamicMesh3.FlipEdge: first edge list remove failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}
	if (VertexEdgeLists.Remove(b, eab) == false)
	{
		checkfSlow(false, TEXT("FDynamicMesh3.FlipEdge: second edge list remove failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}
	VertexRefCounts.Decrement(a);
	VertexRefCounts.Decrement(b);
	if (IsVertex(a) == false || IsVertex(b) == false)
	{
		checkfSlow(false, TEXT("FDynamicMesh3.FlipEdge: either a or b is not a vertex?"));
		return EMeshResult::Failed_UnrecoverableError;
	}

	// add edge ecd to verts c and d, and increment ref counts
	VertexEdgeLists.Insert(c, ecd);
	VertexEdgeLists.Insert(d, ecd);
	VertexRefCounts.Increment(c);
	VertexRefCounts.Increment(d);

	// success! collect up results
	FlipInfo.EdgeID = eab;
	FlipInfo.OriginalVerts = FIndex2i(a, b);
	FlipInfo.OpposingVerts = FIndex2i(c, d);
	FlipInfo.Triangles = FIndex2i(t0, t1);

	if (HasAttributes())
	{
		Attributes()->OnFlipEdge(FlipInfo);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}


EMeshResult FDynamicMesh3::FlipEdge(int vA, int vB, FEdgeFlipInfo& FlipInfo)
{
	int eid = FindEdge(vA, vB);
	if (eid == InvalidID)
	{
		FlipInfo = FEdgeFlipInfo();
		return EMeshResult::Failed_NotAnEdge;
	}
	return FlipEdge(eid, FlipInfo);
}




EMeshResult FDynamicMesh3::SplitVertex(int VertexID, const TArrayView<const int>& TrianglesToUpdate, FVertexSplitInfo& SplitInfo)
{
	if (!ensure(IsVertex(VertexID)))
	{
		return EMeshResult::Failed_NotAVertex;
	}

	SplitInfo.OriginalVertex = VertexID;
	SplitInfo.NewVertex = AppendVertex(*this, VertexID);

	// TODO: consider making a TSet copy of TrianglesToUpdate for membership tests, if TrianglesToUpdate is large
	auto ProcessEdge = [this, &TrianglesToUpdate, &SplitInfo](int TriID, FIndex3i& UpdatedTri, FIndex3i& TriEdges, int SubIdx)
	{
		int EdgeID = TriEdges[SubIdx];
		int OtherTri = GetOtherEdgeTriangle(EdgeID, TriID);
		bool bNewBoundary = OtherTri >= 0 && !TrianglesToUpdate.Contains(OtherTri); // processing this edge will create a new boundary
		if (bNewBoundary) // there *is* a triangle across from this edge and we do need to separate from it
		{
			ReplaceEdgeTriangle(EdgeID, TriID, InvalidID); // remove TriID from original edge, disconnecting it from OtherTri
			// add a new edge for TriID connecting the updated tri vertices
			AddTriangleEdge(TriID, UpdatedTri[SubIdx], UpdatedTri[(SubIdx+1)%3], SubIdx, InvalidID); // adds to vertexedgelists
		}
		else // othertri invalid or also in set
		{
			// if OtherTri already was processed and replaced edge, ReplaceEdgeVertex will return InvalidID and do nothing
			if (ReplaceEdgeVertex(EdgeID, SplitInfo.OriginalVertex, SplitInfo.NewVertex) != InvalidID)
			{
				// if replace edge actually happened, also update VertexEdgeLists accordingly
				ensure(VertexEdgeLists.Remove(SplitInfo.OriginalVertex, EdgeID));
				VertexEdgeLists.Insert(SplitInfo.NewVertex, EdgeID);
			}
		}
	};
	for (int TriID : TrianglesToUpdate)
	{
		FIndex3i Triangle = GetTriangle(TriID);
		int SubIdx = Triangle.IndexOf(VertexID);
		if (SubIdx < 0)
		{
			continue;
		}
		Triangle[SubIdx] = SplitInfo.NewVertex; // update local copy w/ new vertex, for use by ProcessEdge helper
		FIndex3i TriEdges = GetTriEdges(TriID);
		ProcessEdge(TriID, Triangle, TriEdges, SubIdx);
		ProcessEdge(TriID, Triangle, TriEdges, (SubIdx+2)%3);

		Triangles[TriID][SubIdx] = SplitInfo.NewVertex;
		VertexRefCounts.Decrement(SplitInfo.OriginalVertex); // remove the triangle from the original vertex
		VertexRefCounts.Increment(SplitInfo.NewVertex);
	}

	if (HasAttributes())
	{
		Attributes()->OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}


bool FDynamicMesh3::SplitVertexWouldLeaveIsolated(int VertexID, const TArrayView<const int>& TrianglesToUpdate)
{
	for (int TID : VtxTrianglesItr(VertexID))
	{
		if (!TrianglesToUpdate.Contains(TID))
		{
			return false; // at least one triangle will keep the old VertexID
		}
	}
	return true; // no triangles founds that keep old VertexID
}


EMeshResult FDynamicMesh3::CanCollapseEdgeInternal(int vKeep, int vRemove, double collapse_t, FEdgeCollapseInfo* OutCollapseInfo) const
{
	return CanCollapseEdgeInternal(vKeep, vRemove, collapse_t, FCollapseEdgeOptions(),OutCollapseInfo);
}

EMeshResult UE::Geometry::FDynamicMesh3::CanCollapseEdgeInternal(int vKeep, int vRemove, double collapse_t, 
	const FCollapseEdgeOptions& Options, FEdgeCollapseInfo* OutCollapseInfo) const
{
	if (IsVertex(vKeep) == false || IsVertex(vRemove) == false)
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	int b = vKeep;		// renaming for sanity. We remove a and keep b
	int a = vRemove;

	int eab = FindEdge(a, b);
	if (eab == InvalidID)
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	const FEdge EdgeAB = Edges[eab];
	int t0 = EdgeAB.Tri[0];
	if (t0 == InvalidID)
	{
		return EMeshResult::Failed_BrokenTopology;
	}
	FIndex3i T0tv = GetTriangle(t0);
	int c = IndexUtil::FindTriOtherVtx(a, b, T0tv);

	// look up opposing triangle/vtx if we are not in boundary case
	bool bIsBoundaryEdge = false;
	int d = InvalidID;
	int t1 = EdgeAB.Tri[1];
	if (t1 != InvalidID)
	{
		FIndex3i T1tv = GetTriangle(t1);
		d = IndexUtil::FindTriOtherVtx(a, b, T1tv);
		if (c == d)
		{
			return EMeshResult::Failed_FoundDuplicateTriangle;
		}
	}
	else
	{
		bIsBoundaryEdge = true;
	}

	// We cannot collapse if there is some other vertex x that is connected to both a and b,
	//  and either xa or xb is an interior edge. In other words, if there are more than two
	//  triangles that have edge xa or xb, then after collapsing a to b, we'll end up with more
	//  than two triangles trying to share edge xb, which is disallowed.
	// Additionally, depending on options, we might not even allow such a collapse even if the
	//  both edges are boundary edges.
	int edges_a_count = VertexEdgeLists.GetCount(a);
	int eac = InvalidID, ead = InvalidID, ebc = InvalidID, ebd = InvalidID;
	for (int eid_a : VertexEdgeLists.Values(a))
	{
		int vax = GetOtherEdgeVertex(eid_a, a);
		if (vax == c)
		{
			eac = eid_a;
			continue;
		}
		if (vax == d)
		{
			ead = eid_a;
			continue;
		}
		if (vax == b)
		{
			continue;
		}
		for (int eid_b : VertexEdgeLists.Values(b))
		{
			if (GetOtherEdgeVertex(eid_b, b) == vax)
			{
				if (!Options.bAllowHoleCollapse || !IsBoundaryEdge(eid_b) || !IsBoundaryEdge(eid_a))
				{
					return EMeshResult::Failed_InvalidNeighbourhood;
				}
				break;
			}
		}
	}

	// We may not allow collapse if we have a tetrahedron. In this case a has 3 nbr edges,
	//  and edge cd exists. But that is not conclusive, we also have to check that
	//  cd is an internal edge, and that each of its tris contain a or b
	if (!Options.bAllowTetrahedronCollapse && edges_a_count == 3 && bIsBoundaryEdge == false)
	{
		int edc = FindEdge(d, c);
		if (edc != InvalidID)
		{
			const FEdge EdgeDC = Edges[edc];
			if (EdgeDC.Tri[1] != InvalidID)
			{
				int edc_t0 = EdgeDC.Tri[0];
				int edc_t1 = EdgeDC.Tri[1];

				if ((TriangleHasVertex(edc_t0, a) && TriangleHasVertex(edc_t1, b))
					|| (TriangleHasVertex(edc_t0, b) && TriangleHasVertex(edc_t1, a)))
				{
					return EMeshResult::Failed_CollapseTetrahedron;
				}
			}
		}
	}
	else if (bIsBoundaryEdge == true && IsBoundaryEdge(eac))
	{
		// Cannot collapse edge if we are down to a single triangle
		ebc = FindEdgeFromTri(b, c, t0);
		if (IsBoundaryEdge(ebc))
		{
			return EMeshResult::Failed_CollapseTriangle;
		}
	}

	// We might not allow collapsing an edge where both vertices are boundary vertices
	//  because that would sometimes create a bowtie
	//
	// NOTE: potentially scanning all edges here...couldn't we
	//  pick up eac/bc/ad/bd as we go? somehow?
	if (!Options.bAllowCollapsingInternalEdgeWithBoundaryVertices && !bIsBoundaryEdge
		&& IsBoundaryVertex(a) && IsBoundaryVertex(b))
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}

	// If we're allowing internal edge collapse with boundary vertices, we open the possibility
	//  to another place where we would end up collapsing away an entire component- a quad. As
	//  with single and double sided triangles, we currently disallow collapse in this case.
	// Note that if we ever add an option to allow this, we have to deal with the possibility
	//  that our kept vert (vKeep) may not actually survive a collapse.
	if (Options.bAllowCollapsingInternalEdgeWithBoundaryVertices && !bIsBoundaryEdge
		&& IsBoundaryEdge(eac) && IsBoundaryEdge(ead))
	{
		ebc = FindEdgeFromTri(b, c, t0);
		if (IsBoundaryEdge(ebc))
		{
			ebd = FindEdgeFromTri(b, d, t1);
			if (IsBoundaryEdge(ebd))
			{
				return EMeshResult::Failed_CollapseQuad;
			}
		}
	}

	if (OutCollapseInfo)
	{
		OutCollapseInfo->OpposingVerts = FIndex2i(c, d);
		OutCollapseInfo->KeptVertex = b;
		OutCollapseInfo->RemovedVertex = a;
		OutCollapseInfo->bIsBoundary = bIsBoundaryEdge;
		OutCollapseInfo->CollapsedEdge = eab;
		OutCollapseInfo->RemovedTris = FIndex2i(t0, t1);
		OutCollapseInfo->RemovedEdges = FIndex2i(eac, ead);
		OutCollapseInfo->KeptEdges = FIndex2i(ebc, ebd);
		OutCollapseInfo->CollapseT = collapse_t;
	}

	return EMeshResult::Ok;
}


EMeshResult FDynamicMesh3::CanCollapseEdge(int vKeep, int vRemove, double collapse_t) const
{
	return CanCollapseEdgeInternal(vKeep, vRemove, 0, FCollapseEdgeOptions(), nullptr);
}

EMeshResult FDynamicMesh3::CanCollapseEdge(int vKeep, int vRemove, const FCollapseEdgeOptions& Options) const
{
	return CanCollapseEdgeInternal(vKeep, vRemove, 0, Options, nullptr);
}

EMeshResult FDynamicMesh3::CollapseEdge(int vKeep, int vRemove, double collapse_t, FEdgeCollapseInfo& CollapseInfo)
{
	return CollapseEdge(vKeep, vRemove, collapse_t, FCollapseEdgeOptions(), CollapseInfo);
}

EMeshResult FDynamicMesh3::CollapseEdge(int vKeep, int vRemove, double collapse_t, const FCollapseEdgeOptions& Options, FEdgeCollapseInfo& CollapseInfo)
{
	CollapseInfo = FEdgeCollapseInfo();

	const EMeshResult CanCollapseResult = CanCollapseEdgeInternal(vKeep, vRemove, collapse_t, Options, &CollapseInfo);
	if (CanCollapseResult != EMeshResult::Ok)
	{
		return CanCollapseResult;
	}

	const int b = vKeep;		// renaming for sanity. We remove a and keep b
	const int a = vRemove;
	const int c = CollapseInfo.OpposingVerts[0];
	const int d = CollapseInfo.OpposingVerts[1];
	const int t0 = CollapseInfo.RemovedTris[0];
	const int t1 = CollapseInfo.RemovedTris[1];
	const int eab = CollapseInfo.CollapsedEdge;
	const bool bIsBoundaryEdge = CollapseInfo.bIsBoundary;
	const int eac = CollapseInfo.RemovedEdges[0];
	const int ead = CollapseInfo.RemovedEdges[1];

	// This may or may not have been computed already
	int ebc = CollapseInfo.KeptEdges[0];
	int ebd = InvalidID;


	// save vertex positions before we delete removed (can defer kept?)
	FVector3d KeptPos = GetVertex(vKeep);
	FVector3d RemovedPos = GetVertex(vRemove);
	FVector2f RemovedUV;
	if (HasVertexUVs())
	{
		RemovedUV = GetVertexUV(vRemove);
	}
	FVector3f RemovedNormal;
	if (HasVertexNormals())
	{
		RemovedNormal = GetVertexNormal(vRemove);
	}
	FVector3f RemovedColor;
	if (HasVertexColors())
	{
		RemovedColor = GetVertexColor(vRemove);
	}

	// 1) remove edge ab from vtx b
	// 2) find edges ad and ac, and tris tad, tac across those edges  (will use later)
	// 3) for other edges, replace a with b, and add that edge to b
	// 4) replace a with b in all triangles connected to a
	int tad = InvalidID, tac = InvalidID;
	for (int eid : VertexEdgeLists.Values(a))
	{
		int o = GetOtherEdgeVertex(eid, a);
		if (o == b)
		{
			if (VertexEdgeLists.Remove(b, eid) != true)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove case o == b"));
				return EMeshResult::Failed_UnrecoverableError;
			}
		}
		else if (o == c)
		{
			if (VertexEdgeLists.Remove(c, eid) != true)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove case o == c"));
				return EMeshResult::Failed_UnrecoverableError;
			}
			tac = GetOtherEdgeTriangle(eid, t0);
		}
		else if (o == d)
		{
			if (VertexEdgeLists.Remove(d, eid) != true)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove case o == c, step 1"));
				return EMeshResult::Failed_UnrecoverableError;
			}
			tad = GetOtherEdgeTriangle(eid, t1);
		}
		else
		{
			// This is some edge oa, not in a triangle with ab, that we can in most cases change to be ob. However it's
			//  possible that ob already exists as a boundary edge, and we need to instead weld the triangle incident to
			//  oa to that edge. This situation is only permitted by CanCollapseEdgeInternal if bAllowHoleCollapse
			//  is true and eid is a boundary edge.
			int32 ExistingEdge = InvalidID;
			if (Options.bAllowHoleCollapse && IsBoundaryEdge(eid) && (ExistingEdge = FindEdge(o, b)) != InvalidID)
			{
				int32 WeldedTriangle = GetEdgeT(eid).A;
				if (ReplaceTriangleEdge(WeldedTriangle, eid, ExistingEdge) == -1
					|| ReplaceEdgeTriangle(ExistingEdge, InvalidID, WeldedTriangle) == -1)
				{
					checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove case else"));
					return EMeshResult::Failed_UnrecoverableError;
				}
				// Edge (o,a) should no longer exist
				VertexEdgeLists.Remove(o, eid);
				EdgeRefCounts.Decrement(eid);
				checkSlow(EdgeRefCounts.IsValid(eid) == false);
			}
			else
			{
				if (ReplaceEdgeVertex(eid, a, b) == -1)
				{
					checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove case else"));
					return EMeshResult::Failed_UnrecoverableError;
				}
				VertexEdgeLists.Insert(b, eid);
			}
		}

		// [TODO] perhaps we can already have unique tri list because of the manifold-nbrhood check we need to do...
		const FEdge Edge = Edges[eid];
		for (int j = 0; j < 2 ; ++j)
		{
			int t_j = Edge.Tri[j];
			if (t_j != InvalidID && t_j != t0 && t_j != t1)
			{
				if (TriangleHasVertex(t_j, a))
				{
					if (ReplaceTriangleVertex(t_j, a, b) == -1)
					{
						checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at remove last check"));
						return EMeshResult::Failed_UnrecoverableError;
					}
					VertexRefCounts.Increment(b);
					VertexRefCounts.Decrement(a);
				}
			}
		}
	}

	if (bIsBoundaryEdge == false)
	{
		// remove all edges from vtx a, then remove vtx a
		VertexEdgeLists.Clear(a);
		checkSlow(VertexRefCounts.GetRefCount(a) == 3);		// in t0,t1, and initial ref
		VertexRefCounts.Decrement(a, 3);
		checkSlow(VertexRefCounts.IsValid(a) == false);

		// remove triangles T0 and T1, and update b/c/d refcounts
		TriangleRefCounts.Decrement(t0);
		TriangleRefCounts.Decrement(t1);
		VertexRefCounts.Decrement(c);
		VertexRefCounts.Decrement(d);
		VertexRefCounts.Decrement(b, 2);
		checkSlow(TriangleRefCounts.IsValid(t0) == false);
		checkSlow(TriangleRefCounts.IsValid(t1) == false);

		// remove edges ead, eab, eac
		EdgeRefCounts.Decrement(ead);
		EdgeRefCounts.Decrement(eab);
		EdgeRefCounts.Decrement(eac);
		checkSlow(EdgeRefCounts.IsValid(ead) == false);
		checkSlow(EdgeRefCounts.IsValid(eab) == false);
		checkSlow(EdgeRefCounts.IsValid(eac) == false);

		// replace t0 and t1 in edges ebd and ebc that we kept
		ebd = FindEdgeFromTri(b, d, t1);
		if (ebc == InvalidID)   // we may have already looked this up
		{
			ebc = FindEdgeFromTri(b, c, t0);
		}

		if (ReplaceEdgeTriangle(ebd, t1, tad) == -1)
		{
			checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=false branch, ebd replace triangle"));
			return EMeshResult::Failed_UnrecoverableError;
		}

		if (ReplaceEdgeTriangle(ebc, t0, tac) == -1)
		{
			checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=false branch, ebc replace triangle"));
			return EMeshResult::Failed_UnrecoverableError;
		}

		// update tri-edge-nbrs in tad and tac
		if (tad != InvalidID)
		{
			if (ReplaceTriangleEdge(tad, ead, ebd) == -1)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=false branch, ebd replace triangle"));
				return EMeshResult::Failed_UnrecoverableError;
			}
		}
		if (tac != InvalidID)
		{
			if (ReplaceTriangleEdge(tac, eac, ebc) == -1)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=false branch, ebd replace triangle"));
				return EMeshResult::Failed_UnrecoverableError;
			}
		}

		// If both bd and ad were boundary edges, or both bc and ac, then the edge bd/bc will have
		//  no incident triangles and will need to be deleted. In that case, if vert c/d was not 
		//  kept alive by some bowtie, that vertex will also need deleting. 
		// This cannot happen if bAllowCollapsingInternalEdgeWithBoundaryVertices is false, and it
		//  cannot happen for a boundary ab edge because that would require a single triangle, which
		//  we currently disallow collapsing.
		if (Options.bAllowCollapsingInternalEdgeWithBoundaryVertices)
		{
			if (GetEdgeT(ebc).A == InvalidID)
			{
				VertexEdgeLists.Remove(b, ebc);
				EdgeRefCounts.Decrement(ebc);
				if (VertexRefCounts.GetRefCount(c) == 1)
				{
					VertexEdgeLists.Clear(c);
					VertexRefCounts.Decrement(c);
				}
				else
				{
					// The vert must still be part of a bowtie. Still need to remove the deleted edge.
					VertexEdgeLists.Remove(c, ebc);
				}
			}
			if (GetEdgeT(ebd).A == InvalidID)
			{
				VertexEdgeLists.Remove(b, ebd);
				VertexEdgeLists.Remove(d, ebd);
				EdgeRefCounts.Decrement(ebd);
				if (VertexRefCounts.GetRefCount(d) == 1)
				{
					VertexEdgeLists.Clear(d);
					VertexRefCounts.Decrement(d);
				}
				else
				{
					// The vert must still be part of a bowtie. Still need to remove the deleted edge.
					VertexEdgeLists.Remove(d, ebd);
				}
			}
		}
	}
	else
	{
		//  boundary-edge path. this is basically same code as above, just not referencing t1/d

		// remove all edges from vtx a, then remove vtx a
		VertexEdgeLists.Clear(a);
		checkSlow(VertexRefCounts.GetRefCount(a) == 2);		// in t0 and initial ref
		VertexRefCounts.Decrement(a, 2);
		checkSlow(VertexRefCounts.IsValid(a) == false);

		// remove triangle T0 and update b/c refcounts
		TriangleRefCounts.Decrement(t0);
		VertexRefCounts.Decrement(c);
		VertexRefCounts.Decrement(b);
		checkSlow(TriangleRefCounts.IsValid(t0) == false);

		// remove edges eab and eac
		EdgeRefCounts.Decrement(eab);
		EdgeRefCounts.Decrement(eac);
		checkSlow(EdgeRefCounts.IsValid(eab) == false);
		checkSlow(EdgeRefCounts.IsValid(eac) == false);

		// replace t0 in edge ebc that we kept
		ebc = FindEdgeFromTri(b, c, t0);
		if (ReplaceEdgeTriangle(ebc, t0, tac) == -1)
		{
			checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=false branch, ebc replace triangle"));
			return EMeshResult::Failed_UnrecoverableError;
		}

		// update tri-edge-nbrs in tac
		if (tac != InvalidID)
		{
			if (ReplaceTriangleEdge(tac, eac, ebc) == -1)
			{
				checkfSlow(false, TEXT("FDynamicMesh3::CollapseEdge: failed at isboundary=true branch, ebd replace triangle"));
				return EMeshResult::Failed_UnrecoverableError;
			}
		}
	}

	// set kept vertex to interpolated collapse position
	SetVertex(vKeep, Lerp(KeptPos, RemovedPos, collapse_t));
	if (HasVertexUVs())
	{
		SetVertexUV(vKeep, Lerp(GetVertexUV(vKeep), RemovedUV, (float)collapse_t));
	}
	if (HasVertexNormals())
	{
		SetVertexNormal(vKeep, Normalized(Lerp(GetVertexNormal(vKeep), RemovedNormal, (float)collapse_t)) );
	}
	if (HasVertexColors())
	{
		SetVertexColor(vKeep, Lerp(GetVertexColor(vKeep), RemovedColor, (float)collapse_t));
	}

	CollapseInfo.KeptEdges = FIndex2i(ebc, ebd);

	if (HasAttributes())
	{
		Attributes()->OnCollapseEdge(CollapseInfo);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}












EMeshResult FDynamicMesh3::MergeEdges(int KeepEdgeID, int DiscardEdgeID, FMergeEdgesInfo& MergeInfo, bool bCheckValidOrientation)
{
	return MergeEdges(KeepEdgeID, DiscardEdgeID, 0, MergeInfo, bCheckValidOrientation);
}

EMeshResult UE::Geometry::FDynamicMesh3::MergeEdges(int eKeep, int eDiscard, double InterpolationT, FMergeEdgesInfo& MergeInfo, bool bCheckValidOrientation)
{
	MergeInfo = FMergeEdgesInfo();
	MergeInfo.InterpolationT = InterpolationT;

	if (IsEdge(eKeep) == false || IsEdge(eDiscard) == false)
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	const FEdge edgeinfo_keep = GetEdge(eKeep);
	const FEdge edgeinfo_discard = GetEdge(eDiscard);
	if (edgeinfo_keep.Tri[1] != InvalidID || edgeinfo_discard.Tri[1] != InvalidID)
	{
		return EMeshResult::Failed_NotABoundaryEdge;
	}

	int a = edgeinfo_keep.Vert[0], b = edgeinfo_keep.Vert[1];
	int tab = edgeinfo_keep.Tri[0];
	int eab = eKeep;
	int c = edgeinfo_discard.Vert[0], d = edgeinfo_discard.Vert[1];
	int tcd = edgeinfo_discard.Tri[0];
	int ecd = eDiscard;

	// Need to correctly orient a,b and c,d and then check that
	// we will not join triangles with incompatible winding order
	// I can't see how to do this purely topologically.
	// So relying on closest-pairs testing.
	int OppAB = IndexUtil::OrientTriEdgeAndFindOtherVtx(a, b, GetTriangle(tab));
	int OppCD = IndexUtil::OrientTriEdgeAndFindOtherVtx(c, d, GetTriangle(tcd));

	// Refuse to merge if doing so would create a duplicate triangle
	if (OppAB == OppCD)
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}

	int x = c; c = d; d = x;   // joinable bdry edges have opposing orientations, so flip to get ac and b/d correspondences
	FVector3d Va = GetVertex(a), Vb = GetVertex(b), Vc = GetVertex(c), Vd = GetVertex(d);
	if (bCheckValidOrientation && 
		((Va - Vc).SquaredLength() + (Vb - Vd).SquaredLength()) >
		((Va - Vd).SquaredLength() + (Vb - Vc).SquaredLength()))
	{
		return EMeshResult::Failed_SameOrientation;
	}

	// alternative that detects normal flip of triangle tcd. This is a more
	// robust geometric test, but fails if tri is degenerate...also more expensive
	//FVector3d otherv = GetVertex(tcd_otherv);
	//FVector3d Ncd = VectorUtil::NormalDirection(GetVertex(c), GetVertex(d), otherv);
	//FVector3d Nab = VectorUtil::NormalDirection(GetVertex(a), GetVertex(b), otherv);
	//if (Ncd.Dot(Nab) < 0)
	//return EMeshResult::Failed_SameOrientation;

	MergeInfo.KeptEdge = eab;
	MergeInfo.RemovedEdge = ecd;

	// if a/c or b/d are connected by an existing edge, we can't merge
	if (a != c && FindEdge(a, c) != InvalidID)
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}
	if (b != d && FindEdge(b, d) != InvalidID)
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}
	// the un-matched edge vertices, a/d and b/c, should also not be directly connected
	// (unless the edges share a vertex, in which case they are always connected by one of the two merge edges)
	if (a != c && b != d && FindEdge(a, d) != InvalidID)
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}
	if (a != c && b != d && FindEdge(b, c) != InvalidID)
	{
		return EMeshResult::Failed_InvalidNeighbourhood;
	}

	// if vertices at either end already share a common neighbour vertex, and we
	// do the merge, that would create duplicate edges. This is something like the
	// 'link condition' in edge collapses.
	// Note that we have to catch cases where both edges to the shared vertex are
	// boundary edges, in that case we will also merge this edge later on
	int MaxAdjBoundaryMerges[2]{ 0,0 };
	if (a != c)
	{
		int ea = 0, ec = 0, other_v = (b == d) ? b : -1;
		for (int cnbr : VtxVerticesItr(c))
		{
			if (cnbr != other_v && (ea = FindEdge(a, cnbr)) != InvalidID)
			{
				ec = FindEdge(c, cnbr);
				if (IsBoundaryEdge(ea) == false || IsBoundaryEdge(ec) == false)
				{
					return EMeshResult::Failed_InvalidNeighbourhood;
				}
				else
				{
					MaxAdjBoundaryMerges[0]++;
				}
			}
		}
	}
	if (b != d)
	{
		int eb = 0, ed = 0, other_v = (a == c) ? a : -1;
		for (int dnbr : VtxVerticesItr(d))
		{
			if (dnbr != other_v && (eb = FindEdge(b, dnbr)) != InvalidID)
			{
				ed = FindEdge(d, dnbr);
				if (IsBoundaryEdge(eb) == false || IsBoundaryEdge(ed) == false)
				{
					return EMeshResult::Failed_InvalidNeighbourhood;
				}
				else
				{
					MaxAdjBoundaryMerges[1]++;
				}
			}
		}
	}

	auto ApplyInterpolation = [this, InterpolationT](int32 KeepVid, int32 RemoveVid)
	{
		SetVertex(KeepVid, Lerp(GetVertex(KeepVid), GetVertex(RemoveVid), InterpolationT));
		if (HasVertexUVs())
		{
			SetVertexUV(KeepVid, Lerp(GetVertexUV(KeepVid), GetVertexUV(RemoveVid), (float)InterpolationT));
		}
		if (HasVertexNormals())
		{
			SetVertexNormal(KeepVid, Normalized(Lerp(GetVertexNormal(KeepVid), GetVertexNormal(RemoveVid), (float)InterpolationT)));
		}
		if (HasVertexColors())
		{
			SetVertexColor(KeepVid, Lerp(GetVertexColor(KeepVid), GetVertexColor(RemoveVid), (float)InterpolationT));
		}
	};

	// [TODO] this acts on each interior tri twice. could avoid using vtx-tri iterator?
	if (a != c)
	{
		if (InterpolationT != 0)
		{
			// Do the interpolation before we remove c
			ApplyInterpolation(a, c);
		}

		// replace c w/ a in edges and tris connected to c, and move edges to a
		for (int eid : VertexEdgeLists.Values(c))
		{
			if (eid == eDiscard)
			{
				continue;
			}
			ReplaceEdgeVertex(eid, c, a);
			short rc = 0;
			const FEdge Edge = Edges[eid];
			if (ReplaceTriangleVertex(Edge.Tri[0], c, a) >= 0)
			{
				rc++;
			}
			if (Edge.Tri[1] != InvalidID)
			{
				if (ReplaceTriangleVertex(Edge.Tri[1], c, a) >= 0)
				{
					rc++;
				}
			}
			VertexEdgeLists.Insert(a, eid);
			if (rc > 0)
			{
				VertexRefCounts.Increment(a, rc);
				VertexRefCounts.Decrement(c, rc);
			}
		}
		VertexEdgeLists.Clear(c);
		VertexRefCounts.Decrement(c);
		MergeInfo.RemovedVerts[0] = c;
	}
	else
	{
		VertexEdgeLists.Remove(a, ecd);
		MergeInfo.RemovedVerts[0] = InvalidID;
	}
	MergeInfo.KeptVerts[0] = a;

	if (d != b)
	{
		if (InterpolationT != 0)
		{
			// Do the interpolation before we remove d
			ApplyInterpolation(b, d);
		}

		// replace d w/ b in edges and tris connected to d, and move edges to b
		for (int eid : VertexEdgeLists.Values(d))
		{
			if (eid == eDiscard)
			{
				continue;
			}
			ReplaceEdgeVertex(eid, d, b);
			short rc = 0;
			const FEdge Edge = Edges[eid];
			if (ReplaceTriangleVertex(Edge.Tri[0], d, b) >= 0)
			{
				rc++;
			}
			if (Edge.Tri[1] != InvalidID)
			{
				if (ReplaceTriangleVertex(Edge.Tri[1], d, b) >= 0)
				{
					rc++;
				}
			}
			VertexEdgeLists.Insert(b, eid);
			if (rc > 0)
			{
				VertexRefCounts.Increment(b, rc);
				VertexRefCounts.Decrement(d, rc);
			}

		}
		VertexEdgeLists.Clear(d);
		VertexRefCounts.Decrement(d);
		MergeInfo.RemovedVerts[1] = d;
	}
	else
	{
		VertexEdgeLists.Remove(b, ecd);
		MergeInfo.RemovedVerts[1] = InvalidID;
	}
	MergeInfo.KeptVerts[1] = b;

	// replace edge cd with edge ab in triangle tcd
	ReplaceTriangleEdge(tcd, ecd, eab);
	EdgeRefCounts.Decrement(ecd);

	// update edge-tri adjacency
	SetEdgeTrianglesInternal(eab, tab, tcd);

	// Once we merge ab to cd, there may be additional edges (now) connected
	// to either a or b that are connected to the same vertex on their 'other' side.
	// So we now have two boundary edges connecting the same two vertices - disaster!
	// We need to find and merge these edges.
	MergeInfo.ExtraRemovedEdges = FIndex2i(InvalidID, InvalidID);
	MergeInfo.ExtraKeptEdges = FIndex2i(InvalidID, InvalidID);
	for (int vi = 0; vi < 2; ++vi)
	{
		int v1 = a, v2 = c;   // vertices of merged edge
		if (vi == 1)
		{
			v1 = b; v2 = d;
		}
		if (v1 == v2)
		{
			continue;
		}

		FDynamicMesh3::FLocalIntArray edges_v;
		GetVertexEdgesList(v1, edges_v);
		int Nedges = (int)edges_v.Num();
		int FoundNum = 0;
		// in this loop, we compare 'other' vert_1 and vert_2 of edges around v1.
		// problem case is when vert_1 == vert_2  (ie two edges w/ same other vtx).
		for (int i = 0; i < Nedges && FoundNum < MaxAdjBoundaryMerges[vi]; ++i)
		{
			int edge_1 = edges_v[i];
			// Skip any non-boundary edge, or edge we've already removed via merging
			if (!EdgeRefCounts.IsValidUnsafe(edge_1) || !IsBoundaryEdge(edge_1))
			{
				continue;
			}
			int vert_1 = GetOtherEdgeVertex(edge_1, v1);
			for (int j = i + 1; j < Nedges; ++j)
			{
				int edge_2 = edges_v[j];
				// Skip any non-boundary edge, or edge we've already removed via merging
				if (!EdgeRefCounts.IsValidUnsafe(edge_2) || !IsBoundaryEdge(edge_2))
				{
					continue;
				}
				int vert_2 = GetOtherEdgeVertex(edge_2, v1);
				if (vert_1 == vert_2)
				{
					// replace edge_2 w/ edge_1 in tri, update edge and vtx-edge-nbr lists
					int tri_1 = Edges[edge_1].Tri[0];
					int tri_2 = Edges[edge_2].Tri[0];
					ReplaceTriangleEdge(tri_2, edge_2, edge_1);
					SetEdgeTrianglesInternal(edge_1, tri_1, tri_2);
					VertexEdgeLists.Remove(v1, edge_2);
					VertexEdgeLists.Remove(vert_1, edge_2);
					EdgeRefCounts.Decrement(edge_2);
					if (FoundNum == 0)
					{
						MergeInfo.ExtraRemovedEdges[vi] = edge_2;
						MergeInfo.ExtraKeptEdges[vi] = edge_1;
					}
					else
					{
						MergeInfo.BowtiesRemovedEdges.Add(edge_2);
						MergeInfo.BowtiesKeptEdges.Add(edge_1);
					}

					FoundNum++;				  // exit outer i loop if we've found all possible merge edges
					break;					  // exit inner j loop; we won't merge anything else to edge_1
				}
			}
		}
	}

	if (HasAttributes())
	{
		Attributes()->OnMergeEdges(MergeInfo);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}


EMeshResult UE::Geometry::FDynamicMesh3::MergeVertices(int KeepVid, int DiscardVid, double InterpolationT, 
	const FMergeVerticesOptions& Options, FMergeVerticesInfo& MergeInfo)
{
	MergeInfo = FMergeVerticesInfo();
	MergeInfo.InterpolationT = InterpolationT;
	MergeInfo.KeptVertex = KeepVid;
	MergeInfo.RemovedVertex = DiscardVid;

	if (!IsVertex(KeepVid) || !IsVertex(DiscardVid))
	{
		return EMeshResult::Failed_NotAVertex;
	}
	if (KeepVid == DiscardVid)
	{
		return EMeshResult::Failed_VertexAlreadyExists;
	}

	// See if we can resolve this as an edge collapse. 
	// Note that this should be checked before trying to resolve as an edge weld because edge welding
	//  will currently fail if there is an edge connecting the removed and kept vert (i.e. welding two
	//  sides of a single triangle hole will fail, but collapse of the intervening edge will succeed
	//  with our permissiveness options).
	if (FindEdge(KeepVid, DiscardVid) != InvalidID)
	{
		FCollapseEdgeOptions CollapseOptions;
		CollapseOptions.bAllowCollapsingInternalEdgeWithBoundaryVertices = true;
		CollapseOptions.bAllowHoleCollapse = true;
		CollapseOptions.bAllowTetrahedronCollapse = true;
		MergeInfo.EdgeCollapseInfo.Emplace();
		return CollapseEdge(KeepVid, DiscardVid, InterpolationT, CollapseOptions, MergeInfo.EdgeCollapseInfo.GetValue());
	}

	// See if we can resolve this as an edge weld
	for (int32 KeepAdjacentEid : VertexEdgeLists.Values(KeepVid))
	{
		int32 KeepAdjacentVid = GetOtherEdgeVertex(KeepAdjacentEid, KeepVid);
		
		// See if the adjacent vert is also adjacent to DiscardVid
		for (int32 DiscardAdjacentEid : VertexEdgeLists.Values(DiscardVid))
		{
			if (GetOtherEdgeVertex(DiscardAdjacentEid, DiscardVid) == KeepAdjacentVid)
			{
				// We've found a V shape (a vertex adjacent to both of the vertices we're working with).
				//  Neither of the edges in the V shape can be a non boundary edge, else the merge would
				//  create a non-manifold edge- we do this check ourselves so that we return Failed_InvalidNeighbourhood
				//  instead of Failed_NotABoundaryEdge.
				if (!IsBoundaryEdge(KeepAdjacentEid) || !IsBoundaryEdge(DiscardAdjacentEid))
				{
					return EMeshResult::Failed_InvalidNeighbourhood;
				}
				// The MergeEdges operation will do the other neighbor checks for us

				MergeInfo.MergeEdgesInfo.Emplace();
				return MergeEdges(KeepAdjacentEid, DiscardAdjacentEid, InterpolationT, MergeInfo.MergeEdgesInfo.GetValue(), false);
			}
		}// end for each adjacent vid to KeepVid
	}//end V shape search

	// If we got to here, the vertices are at least three edges apart, and we're creating a bowtie.

	if (!Options.bAllowNonBoundaryBowtieCreation
		&& (!IsBoundaryVertex(KeepVid) || !IsBoundaryVertex(DiscardVid)))
	{
		return EMeshResult::Failed_WouldCreateBowtie;
	}

	// Apply interpolation first, before removing DiscardVid
	if (InterpolationT != 0)
	{
		SetVertex(KeepVid, Lerp(GetVertex(KeepVid), GetVertex(DiscardVid), InterpolationT));
		if (HasVertexUVs())
		{
			SetVertexUV(KeepVid, Lerp(GetVertexUV(KeepVid), GetVertexUV(DiscardVid), (float)InterpolationT));
		}
		if (HasVertexNormals())
		{
			SetVertexNormal(KeepVid, Normalized(Lerp(GetVertexNormal(KeepVid), GetVertexNormal(DiscardVid), (float)InterpolationT)));
		}
		if (HasVertexColors())
		{
			SetVertexColor(KeepVid, Lerp(GetVertexColor(KeepVid), GetVertexColor(DiscardVid), (float)InterpolationT));
		}
	}

	// Replace DiscardVid w/ KeepVid in edges and tris connected to DiscardVid, and move edges to KeepVid
	for (int Eid : VertexEdgeLists.Values(DiscardVid))
	{
		ReplaceEdgeVertex(Eid, DiscardVid, KeepVid);
		short ReplaceCount = 0;
		const FEdge Edge = Edges[Eid];
		if (ReplaceTriangleVertex(Edge.Tri[0], DiscardVid, KeepVid) >= 0)
		{
			ReplaceCount++;
		}
		if (Edge.Tri[1] != InvalidID)
		{
			if (ReplaceTriangleVertex(Edge.Tri[1], DiscardVid, KeepVid) >= 0)
			{
				ReplaceCount++;
			}
		}
		VertexEdgeLists.Insert(KeepVid, Eid);
		if (ReplaceCount > 0)
		{
			VertexRefCounts.Increment(KeepVid, ReplaceCount);
			VertexRefCounts.Decrement(DiscardVid, ReplaceCount);
		}
	}
	VertexEdgeLists.Clear(DiscardVid);
	VertexRefCounts.Decrement(DiscardVid);

	if (HasAttributes())
	{
		Attributes()->OnMergeVertices(MergeInfo);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}


EMeshResult FDynamicMesh3::PokeTriangle(int TriangleID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeResult)
{
	PokeResult = FPokeTriangleInfo();

	if (!IsTriangle(TriangleID))
	{
		return EMeshResult::Failed_NotATriangle;
	}

	FIndex3i tv = GetTriangle(TriangleID);
	FIndex3i te = GetTriEdges(TriangleID);

	// create vertex with interpolated vertex attribs
	FVertexInfo vinfo;
	GetTriBaryPoint(TriangleID, BaryCoordinates[0], BaryCoordinates[1], BaryCoordinates[2], vinfo);
	int center = AppendVertex(vinfo);

	// add in edges to center vtx, do not connect to triangles yet
	int eaC = AddEdgeInternal(tv[0], center, -1, -1);
	int ebC = AddEdgeInternal(tv[1], center, -1, -1);
	int ecC = AddEdgeInternal(tv[2], center, -1, -1);
	VertexRefCounts.Increment(tv[0]);
	VertexRefCounts.Increment(tv[1]);
	VertexRefCounts.Increment(tv[2]);
	VertexRefCounts.Increment(center, 3);

	// old triangle becomes tri along first edge
	SetTriangleInternal(TriangleID, tv[0], tv[1], center);
	SetTriangleEdgesInternal(TriangleID, te[0], ebC, eaC);

	// add two triangles
	int t1 = AddTriangleInternal(tv[1], tv[2], center, te[1], ecC, ebC);
	int t2 = AddTriangleInternal(tv[2], tv[0], center, te[2], eaC, ecC);

	// second and third edges of original tri have neighbours
	ReplaceEdgeTriangle(te[1], TriangleID, t1);
	ReplaceEdgeTriangle(te[2], TriangleID, t2);

	// set the triangles for the edges we created above
	SetEdgeTrianglesInternal(eaC, TriangleID, t2);
	SetEdgeTrianglesInternal(ebC, TriangleID, t1);
	SetEdgeTrianglesInternal(ecC, t1, t2);

	// transfer groups
	if (HasTriangleGroups())
	{
		int g = TriangleGroups.GetValue()[TriangleID];
		TriangleGroups->InsertAt(g, t1);
		TriangleGroups->InsertAt(g, t2);
	}

	PokeResult.OriginalTriangle = TriangleID;
	PokeResult.TriVertices = tv;
	PokeResult.NewVertex = center;
	PokeResult.NewTriangles = FIndex2i(t1,t2);
	PokeResult.NewEdges = FIndex3i(eaC, ebC, ecC);
	PokeResult.BaryCoords = BaryCoordinates;

	if (HasAttributes())
	{
		Attributes()->OnPokeTriangle(PokeResult);
	}

	UpdateChangeStamps(true, true);
	return EMeshResult::Ok;
}




