// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

FIndex2i FDynamicMesh3::GetEdgeOpposingV(int eID) const
{
	const FEdge& Edge = Edges[eID];
	int a = Edge.Vert[0];
	int b = Edge.Vert[1];

	// ** it is important that verts returned maintain [c,d] order!!
	int c = IndexUtil::FindTriOtherVtxUnsafe(a, b, Triangles[Edge.Tri[0]]);
	if (Edge.Tri[1] != InvalidID)
	{
		int d = IndexUtil::FindTriOtherVtxUnsafe(a, b, Triangles[Edge.Tri[1]]);
		return FIndex2i(c, d);
	}
	else
	{
		return FIndex2i(c, InvalidID);
	}
}


int FDynamicMesh3::GetVtxBoundaryEdges(int vID, int& e0, int& e1) const
{
	if (VertexRefCounts.IsValid(vID))
	{
		int count = 0;
		for (int eid : VertexEdgeLists.Values(vID))
		{
			if (Edges[eid].Tri[1] == InvalidID)
			{
				if (count == 0)
				{
					e0 = eid;
				}
				else if (count == 1)
				{
					e1 = eid;
				}
				count++;
			}
		}
		return count;
	}
	return 0;
}

template<typename ArrayType>
int FDynamicMesh3::GetAllVtxBoundaryEdges(int vID, ArrayType& EdgeListOut) const
{
	if (VertexRefCounts.IsValid(vID))
	{
		int count = 0;
		for (int eid : VertexEdgeLists.Values(vID))
		{
			if (Edges[eid].Tri[1] == InvalidID)
			{
				EdgeListOut.Add(eid);
				count++;
			}
		}
		return count;
	}
	return 0;
}

template GEOMETRYCORE_API int FDynamicMesh3::GetAllVtxBoundaryEdges<TArray<int32>>(int vID, TArray<int32>& EdgeListOut) const;
template GEOMETRYCORE_API int FDynamicMesh3::GetAllVtxBoundaryEdges<FDynamicMesh3::FLocalIntArray>(int vID, FDynamicMesh3::FLocalIntArray& EdgeListOut) const;


void FDynamicMesh3::GetVtxNbrhood(int eID, int vID, int& vOther, int& oppV1, int& oppV2, int& t1, int& t2) const
{
	const FEdge Edge = Edges[eID];
	vOther = (Edge.Vert[0] == vID) ? Edge.Vert[1] : Edge.Vert[0];
	t1 = Edge.Tri[0];
	oppV1 = IndexUtil::FindTriOtherVtx(vID, vOther, Triangles, t1);
	t2 = Edge.Tri[1];
	if (t2 != InvalidID)
	{
		oppV2 = IndexUtil::FindTriOtherVtx(vID, vOther, Triangles, t2);
	}
	else
	{
		t2 = InvalidID;
	}
}


int FDynamicMesh3::GetVtxTriangleCount(int vID) const
{
	if (!IsVertex(vID))
	{
		return -1;
	}
	int N = 0;
	VertexEdgeLists.Enumerate(vID, [&](int32 eid)
	{
		const FEdge Edge = Edges[eid];
		const int vOther = Edge.Vert.A == vID ? Edge.Vert.B : Edge.Vert.A;
		if (TriHasSequentialVertices(Edge.Tri[0], vID, vOther))
		{
			N++;
		}
		const int et1 = Edge.Tri[1];
		if (Edge.Tri[1] != InvalidID && TriHasSequentialVertices(Edge.Tri[1], vID, vOther))
		{
			N++;
		}
	});
	return N;
}


int FDynamicMesh3::GetVtxSingleTriangle(int VertexID) const
{
	if (!IsVertex(VertexID))
	{
		return FDynamicMesh3::InvalidID;
	}

	PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
	for (int EID : VertexEdgeLists.Values(VertexID))
	{
		return Edges[EID].Tri[0];
	}
	PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

	return FDynamicMesh3::InvalidID;
}

template<typename ArrayType>
EMeshResult FDynamicMesh3::GetVtxTriangles(int vID, ArrayType& TrianglesOut) const
{
	if (!IsVertex(vID))
	{
		return EMeshResult::Failed_NotAVertex;
	}

	if (VertexEdgeLists.GetCount(vID) > 20)
	{
		VertexEdgeLists.Enumerate(vID, [&](int32 eid)
		{
			const FEdge Edge = Edges[eid];
			const int vOther = Edge.Vert.A == vID ? Edge.Vert.B : Edge.Vert.A;
			if (TriHasSequentialVertices(Edge.Tri[0], vID, vOther))
			{
				TrianglesOut.Add(Edge.Tri[0]);
			}
			if (Edge.Tri[1] != InvalidID && TriHasSequentialVertices(Edge.Tri[1], vID, vOther))
			{
				TrianglesOut.Add(Edge.Tri[1]);
			}
		});
	}
	else
	{
		VertexEdgeLists.Enumerate(vID, [&](int32 eid)
		{
			const FEdge Edge = Edges[eid];
			TrianglesOut.AddUnique(Edge.Tri[0]);
			if (Edge.Tri[1] != InvalidID)
			{
				TrianglesOut.AddUnique(Edge.Tri[1]);
			}
		});
	}
	return EMeshResult::Ok;
}

template GEOMETRYCORE_API EMeshResult FDynamicMesh3::GetVtxTriangles<TArray<int32>>(int vID, TArray<int32>& TrianglesOut) const;
template GEOMETRYCORE_API EMeshResult FDynamicMesh3::GetVtxTriangles<FDynamicMesh3::FLocalIntArray>(int vID, FDynamicMesh3::FLocalIntArray& TrianglesOut) const;

template<typename IntArray, typename BoolArray>
EMeshResult FDynamicMesh3::GetVtxContiguousTriangles(int VertexID, IntArray& TrianglesOut, IntArray& SpanLengths, BoolArray& IsLoop) const
{
	TrianglesOut.Reset();
	SpanLengths.Reset();
	IsLoop.Reset();

	if (!ensure(IsVertex(VertexID)))
	{
		return EMeshResult::Failed_NotAVertex;
	}

	int NumEdges = VertexEdgeLists.GetCount(VertexID);
	if (NumEdges == 0)
	{
		return EMeshResult::Ok;
	}

	// The number of StartEdgeIDs for a vertex is NumBoundaryEdges + NumRingsWithoutBoundary
	// This is only higher than 2 at non-manifold (e.g., "bowtie") vertices, and
	// it should be very rare for it to exceed this inline allocation of 8
	FLocalIntArray StartEdgeIDs;
	// initial starting edge candidates == boundary edges
	for (int EID : VertexEdgeLists.Values(VertexID))
	{
		if (Edges[EID].Tri[1] == InvalidID)
		{
			StartEdgeIDs.Push(EID);
		}
	}
	bool bHasBoundaries = StartEdgeIDs.Num() != 0;

	if (!bHasBoundaries)
	{
		for (int EID : VertexEdgeLists.Values(VertexID))
		{
			StartEdgeIDs.Push(EID);
			break;
		}
	}

	int WalkedEdges = 0;
	bool bHasRemainingBoundaries = bHasBoundaries;
	while (StartEdgeIDs.Num() || WalkedEdges < NumEdges)
	{
		if (!StartEdgeIDs.Num())
		{
			bHasRemainingBoundaries = false;
			// fallback for (hopefully very rare) case of a non-manifold vertex where there are separate one-rings w/ no boundary edges --
			//  brute force search for an edge that hasn't already been walked
			for (int EID : VertexEdgeLists.Values(VertexID))
			{
				int AttachedTriID = Edges[EID].Tri[0];
				bool UsedEdge = TrianglesOut.Contains(AttachedTriID);
				if (!UsedEdge)
				{
					StartEdgeIDs.Push(EID);
					break;
				}
			}
		}

		// walk starting from this edge, add the found span

		int StartEID = StartEdgeIDs.Pop();
		int PrevEID = StartEID;
		WalkedEdges++;
		int WalkTri = Edges[StartEID].Tri[0];
		int32 SpanStart = TrianglesOut.Num();
		IsLoop.Add(!bHasRemainingBoundaries);
		while (true)
		{
			TrianglesOut.Add(WalkTri);

			int TriIdx = WalkTri;
			const FIndex3i& TriVIDs = Triangles[TriIdx];
			const FIndex3i& TriEIDs = TriangleEdges[TriIdx];
			int VertSubIdx = IndexUtil::FindTriIndex(VertexID, TriVIDs);
			int NextEID = TriEIDs[VertSubIdx];
			if (NextEID == PrevEID)
			{
				NextEID = TriEIDs[(VertSubIdx + 2) % 3];
			}
			if (NextEID == StartEID)
			{
				break;
			}

			WalkedEdges++;

			int NextTriID = Edges[NextEID].Tri[0];
			if (NextTriID == WalkTri)
			{
				NextTriID = Edges[NextEID].Tri[1];
			}
			if (NextTriID == InvalidID)
			{
				// remove the corresponding boundary
				checkSlow(StartEdgeIDs.Num() > 0);
				if (StartEdgeIDs.Num() > 0)
				{
					StartEdgeIDs.RemoveSingleSwap(NextEID);
				}
				break;
			}
			WalkTri = NextTriID;
			PrevEID = NextEID;
		}
		SpanLengths.Add(TrianglesOut.Num() - SpanStart);
	}

	return ensure(SpanLengths.Num() == IsLoop.Num()) ? EMeshResult::Ok : EMeshResult::Failed_InvalidNeighbourhood;
}

template GEOMETRYCORE_API EMeshResult FDynamicMesh3::GetVtxContiguousTriangles<TArray<int32>, TArray<bool>>
	(int VertexID, TArray<int32>& TrianglesOut, TArray<int32>& SpanLengths, TArray<bool>& IsLoop) const;
template GEOMETRYCORE_API EMeshResult FDynamicMesh3::GetVtxContiguousTriangles<FDynamicMesh3::FLocalIntArray, FDynamicMesh3::FLocalBoolArray>
	(int VertexID, FDynamicMesh3::FLocalIntArray& TrianglesOut, FDynamicMesh3::FLocalIntArray& SpanLengths, FDynamicMesh3::FLocalBoolArray& IsLoop) const;

bool FDynamicMesh3::IsBoundaryVertex(int vID) const
{
	checkSlow(IsVertex(vID));
	if ( IsVertex(vID) )
	{
		for (int eid : VertexEdgeLists.Values(vID))
		{
			if (Edges[eid].Tri[1] == InvalidID)
			{
				return true;
			}
		}
	}
	return false;
}

bool FDynamicMesh3::IsBoundaryTriangle(int tID) const
{
	checkSlow(IsTriangle(tID));
	if (IsTriangle(tID))
	{
		const FIndex3i& TriEdgeIDs = TriangleEdges[tID];
		return IsBoundaryEdge(TriEdgeIDs[0]) || IsBoundaryEdge(TriEdgeIDs[1]) || IsBoundaryEdge(TriEdgeIDs[2]);
	}
	else
	{
		return false;
	}
}

FIndex2i FDynamicMesh3::GetOrientedBoundaryEdgeV(int eID) const
{
	if (EdgeRefCounts.IsValid(eID))
	{
		const FEdge Edge = Edges[eID];
		if (Edge.Tri[1] == InvalidID)
		{
			int a = Edge.Vert[0], b = Edge.Vert[1];
			int ti = Edge.Tri[0];
			const FIndex3i& tri = Triangles[ti];
			int ai = IndexUtil::FindEdgeIndexInTri(a, b, tri);
			return FIndex2i(tri[ai], tri[(ai + 1) % 3]);
		}
	}
	checkSlow(false);
	return InvalidEdge;
}


bool FDynamicMesh3::IsGroupBoundaryEdge(int eID) const
{
	if (!HasTriangleGroups()) return false;

	const FEdge Edge = Edges[eID];
	int et1 = Edge.Tri[1];
	if (et1 == InvalidID)
	{
		return false;
	}
	int g1 = TriangleGroups.GetValue()[et1];
	int et0 = Edge.Tri[0];
	int g0 = TriangleGroups.GetValue()[et0];
	return g1 != g0;
}

bool FDynamicMesh3::IsGroupBoundaryVertex(int vID) const
{
	if (!HasTriangleGroups()) return false;

	int group_id = InvalidID;
	for (int eID : VertexEdgeLists.Values(vID))
	{
		const FEdge Edge = Edges[eID];
		int et0 = Edge.Tri[0];
		int g0 = TriangleGroups.GetValue()[et0];
		if (group_id != g0)
		{
			if (group_id == InvalidID)
			{
				group_id = g0;
			}
			else
			{
				return true;        // saw multiple group IDs
			}
		}
		int et1 = Edge.Tri[1];
		if (et1 != InvalidID)
		{
			int g1 = TriangleGroups.GetValue()[et1];
			if (group_id != g1)
			{
				return true;        // saw multiple group IDs
			}
		}
	}
	return false;
}



bool FDynamicMesh3::IsGroupJunctionVertex(int vID) const
{
	if (!HasTriangleGroups()) return false;

	FIndex2i groups(InvalidID, InvalidID);
	for (int eID : VertexEdgeLists.Values(vID))
	{
		const FEdge Edge = Edges[eID];
		FIndex2i et = Edge.Tri;
		for (int k = 0; k < 2; ++k)
		{
			if (et[k] == InvalidID)
			{
				continue;
			}
			int g0 = TriangleGroups.GetValue()[et[k]];
			if (g0 != groups[0] && g0 != groups[1])
			{
				if (groups[0] != InvalidID && groups[1] != InvalidID)
				{
					return true;
				}
				if (groups[0] == InvalidID)
				{
					groups[0] = g0;
				}
				else
				{
					groups[1] = g0;
				}
			}
		}
	}
	return false;
}


bool FDynamicMesh3::GetVertexGroups(int vID, FIndex4i& groups) const
{
	groups = FIndex4i(InvalidID, InvalidID, InvalidID, InvalidID);
	if (!HasTriangleGroups()) return false;
	int ng = 0;

	for (int eID : VertexEdgeLists.Values(vID))
	{
		const FEdge Edge = Edges[eID];

		int et0 = Edge.Tri[0];
		int g0 = TriangleGroups.GetValue()[et0];
		if (groups.Contains(g0) == false)
		{
			groups[ng++] = g0;
		}
		if (ng == 4)
		{
			return false;
		}
		int et1 = Edge.Tri[1];
		if (et1 != InvalidID)
		{
			int g1 = TriangleGroups.GetValue()[et1];
			if (groups.Contains(g1) == false)
			{
				groups[ng++] = g1;
			}
			if (ng == 4)
			{
				return false;
			}
		}
	}
	return true;
}


template<typename ArrayType>
bool FDynamicMesh3::GetAllVertexGroups(int vID, ArrayType& GroupsOut) const
{
	if (!HasTriangleGroups()) return false;

	for (int eID : VertexEdgeLists.Values(vID))
	{
		const FEdge Edge = Edges[eID];
		int et0 = Edge.Tri[0];
		int g0 = TriangleGroups.GetValue()[et0];
		GroupsOut.AddUnique(g0);

		int et1 = Edge.Tri[1];
		if (et1 != InvalidID)
		{
			int g1 = TriangleGroups.GetValue()[et1];
			GroupsOut.AddUnique(g1);
		}
	}
	return true;
}

template GEOMETRYCORE_API bool FDynamicMesh3::GetAllVertexGroups<TArray<int32>>(int vID, TArray<int32>& GroupsOut) const;
template GEOMETRYCORE_API bool FDynamicMesh3::GetAllVertexGroups<FDynamicMesh3::FLocalIntArray>(int vID, FDynamicMesh3::FLocalIntArray& GroupsOut) const;



/**
 * returns true if vID is a "bowtie" vertex, ie multiple disjoint triangle sets in one-ring
 */
bool FDynamicMesh3::IsBowtieVertex(int vID) const
{
	if (VertexRefCounts.IsValid(vID) == false)
	{
		return false;
	}

	int nEdges = VertexEdgeLists.GetCount(vID);
	if (nEdges == 0)
	{
		return false;
	}

	// find a boundary edge to start at
	int start_eid = -1;
	bool start_at_boundary = false;
	for (int eid : VertexEdgeLists.Values(vID))
	{
		const FEdge Edge = Edges[eid];
		if (Edge.Tri[1] == InvalidID)
		{
			start_at_boundary = true;
			start_eid = eid;
			break;
		}
	}
	// if no boundary edge, start at arbitrary edge
	if (start_eid == -1)
	{
		start_eid = VertexEdgeLists.First(vID);
	}
	// initial triangle
	int start_tid = Edges[start_eid].Tri[0];

	int prev_tid = start_tid;
	int prev_eid = start_eid;

	// walk forward to next edge. if we hit start edge or boundary edge,
	// we are done the walk. count number of edges as we go.
	int count = 1;
	while (true)
	{
		int i = prev_tid;
		const FIndex3i& tv = Triangles[i];
		const FIndex3i& te = TriangleEdges[i];
		int vert_idx = IndexUtil::FindTriIndex(vID, tv);
		int e1 = te[vert_idx], e2 = te[(vert_idx + 2) % 3];
		int next_eid = (e1 == prev_eid) ? e2 : e1;
		if (next_eid == start_eid)
		{
			break;
		}
		FIndex2i next_eid_tris = GetEdgeT(next_eid);
		int next_tid = (next_eid_tris[0] == prev_tid) ? next_eid_tris[1] : next_eid_tris[0];
		if (next_tid == InvalidID)
		{
			break;
		}
		prev_eid = next_eid;
		prev_tid = next_tid;
		count++;
	}

	// if we did not see all edges at vertex, we have a bowtie
	int target_count = (start_at_boundary) ? nEdges - 1 : nEdges;
	bool is_bowtie = (target_count != count);
	return is_bowtie;
}




int FDynamicMesh3::FindTriangle(int a, int b, int c) const
{
	int eid = FindEdge(a, b);
	if (eid == InvalidID)
	{
		return InvalidID;
	}
	const FEdge Edge = Edges[eid];

	// triangles attached to edge [a,b] must contain verts a and b...
	int ti = Edge.Tri[0];
	if (Triangles[ti][0] == c || Triangles[ti][1] == c || Triangles[ti][2] == c)
	{
		return Edge.Tri[0];
	}
	if (Edge.Tri[1] != InvalidID)
	{
		ti = Edge.Tri[1];
		if (Triangles[ti][0] == c || Triangles[ti][1] == c || Triangles[ti][2] == c)
		{
			return Edge.Tri[1];
		}
	}

	return InvalidID;
}



/**
 * Computes bounding box of all vertices.
 */
FAxisAlignedBox3d FDynamicMesh3::GetBounds(bool bParallel) const
{
	if (VertexCount() == 0)
	{
		return FAxisAlignedBox3d::Empty();
	}

	FAxisAlignedBox3d ResultBounds = FAxisAlignedBox3d::Empty();
	int32 MaxVID = MaxVertexID();
	if (bParallel == false || MaxVID < 50000)
	{
		FVector3d MinVec = Vertices[*(VertexIndicesItr().begin())];
		FVector3d MaxVec = MinVec;
		for (int vi : VertexIndicesItr())
		{
			MinVec = Min(MinVec, Vertices[vi]);
			MaxVec = Max(MaxVec, Vertices[vi]);
		}
		return FAxisAlignedBox3d(MinVec, MaxVec);
	}
	else
	{
		constexpr int ChunkSize = 10000;
		int32 NumChunks = (MaxVID / ChunkSize) + 1;
		FCriticalSection BoundsLock;
		ParallelFor(NumChunks, [&](int ci)
		{
			FAxisAlignedBox3d ChunkBounds = FAxisAlignedBox3d::Empty();
			int Start = ci * ChunkSize;
			for (int k = 0; k < ChunkSize; ++k)
			{
				int vid = Start + k;
				if (IsVertex(vid))
				{
					ChunkBounds.Contain(Vertices[vid]);
				}
			}
			BoundsLock.Lock();
			ResultBounds.Contain(ChunkBounds);
			BoundsLock.Unlock();
		});
	}
	return ResultBounds;
}

/**
 * Computes bounding box of selected vertices.
 */
FAxisAlignedBox3d FDynamicMesh3::GetBoundsForVertexSelection(TConstArrayView<int32> VertexIDs) const
{
	int32 NumVertices = VertexIDs.Num();
	if (NumVertices == 0)
	{
		return FAxisAlignedBox3d::Empty();
	}

	FAxisAlignedBox3d ResultBounds = FAxisAlignedBox3d::Empty();

	constexpr int32 ChunkSize = 4096*2;
	constexpr int32 MinCountToThread = ChunkSize * 3;
	
	constexpr bool bParallel = true;
	if (!bParallel || NumVertices < MinCountToThread)
	{
		FVector3d MinVec = Vertices[VertexIDs[0]];
		FVector3d MaxVec = MinVec;
		for (int Idx = 1; Idx < VertexIDs.Num(); ++Idx)
		{
			int32 VID = VertexIDs[Idx];
			MinVec = Min(MinVec, Vertices[VID]);
			MaxVec = Max(MaxVec, Vertices[VID]);
		}
		return FAxisAlignedBox3d(MinVec, MaxVec);
	}
	else
	{
		int32 NumChunks = (NumVertices / ChunkSize) + 1;
		FCriticalSection BoundsLock;
		ParallelFor(NumChunks, [&](int ci)
		{
			FAxisAlignedBox3d ChunkBounds = FAxisAlignedBox3d::Empty();
			int32 Start = ci * ChunkSize;
			int32 End = FMath::Min(Start + ChunkSize, NumVertices);
			for (int Idx = Start; Idx < End; ++Idx)
			{
				int32 VID = VertexIDs[Idx];
				ChunkBounds.Contain(Vertices[VID]);
			}
			BoundsLock.Lock();
			ResultBounds.Contain(ChunkBounds);
			BoundsLock.Unlock();
		});
	}
	return ResultBounds;
}

FAxisAlignedBox3d FDynamicMesh3::GetBoundsForTriangleSelection(TConstArrayView<int32> TriangleIDs) const
{
	int32 NumTriangles = TriangleIDs.Num();
	if (NumTriangles == 0)
	{
		return FAxisAlignedBox3d::Empty();
	}

	auto UpdateBoundsWithTriangles = [this](FAxisAlignedBox3d& Bounds, TConstArrayView<int32> IndexArray)
	{
		for (int32 TID : IndexArray)
		{
			const FIndex3i& Tri = Triangles[TID];
			Bounds.Contain(Vertices[Tri.A]);
			Bounds.Contain(Vertices[Tri.B]);
			Bounds.Contain(Vertices[Tri.C]);
		}
	};

	FAxisAlignedBox3d ResultBounds = FAxisAlignedBox3d::Empty();

	constexpr int32 ChunkSize = 4096;
	constexpr int32 MinCountToThread = ChunkSize * 3;

	constexpr bool bParallel = true;
	if (!bParallel || NumTriangles < MinCountToThread)
	{
		UpdateBoundsWithTriangles(ResultBounds, TriangleIDs);
	}
	else
	{
		int32 NumChunks = (NumTriangles / ChunkSize) + 1;
		FCriticalSection BoundsLock;
		ParallelFor(NumChunks, [&](int ci)
		{
			FAxisAlignedBox3d ChunkBounds = FAxisAlignedBox3d::Empty();
			int32 Start = ci * ChunkSize;
			int32 End = FMath::Min(TriangleIDs.Num(), Start + ChunkSize);
			TConstArrayView<int32> IndexArrayView(TriangleIDs.GetData() + Start, End - Start);
			UpdateBoundsWithTriangles(ChunkBounds, IndexArrayView);
			BoundsLock.Lock();
			ResultBounds.Contain(ChunkBounds);
			BoundsLock.Unlock();
		});
	}
	return ResultBounds;
}


bool FDynamicMesh3::IsClosed() const
{
	if (TriangleCount() == 0)
	{
		return false;
	}

	int N = MaxEdgeID();
	for (int i = 0; i < N; ++i)
	{
		if (EdgeRefCounts.IsValid(i) && IsBoundaryEdge(i))
		{
			return false;
		}
	}
	return true;
}

// average of 1 or 2 face normals
FVector3d FDynamicMesh3::GetEdgeNormal(int eID) const
{
	if (EdgeRefCounts.IsValid(eID))
	{
		const FIndex2i Tris = Edges[eID].Tri;
		FVector3d n = GetTriNormal(Tris[0]);
		if (Tris[1] != InvalidID)
		{
			n += GetTriNormal(Tris[1]);
			Normalize(n);
		}
		return n;
	}
	checkSlow(false);
	return FVector3d::Zero();
}

FVector3d FDynamicMesh3::GetEdgePoint(int eID, double t) const
{
	t = VectorUtil::Clamp(t, 0.0, 1.0);
	if (EdgeRefCounts.IsValid(eID))
	{
		FIndex2i Verts = Edges[eID].Vert;
		const int iv0 = Verts[0];
		const int iv1 = Verts[1];
		double mt = 1.0 - t;
		return mt*Vertices[iv0] + t * Vertices[iv1];
	}
	checkSlow(false);
	return FVector3d::Zero();
}


void FDynamicMesh3::GetVtxOneRingCentroid(int vID, FVector3d& centroid) const
{
	centroid = FVector3d::Zero();
	if (VertexRefCounts.IsValid(vID))
	{
		int n = 0;
		for (int eid : VertexEdgeLists.Values(vID))
		{
			int other_idx = GetOtherEdgeVertex(eid, vID);
			centroid += Vertices[other_idx];
			n++;
		}
		if (n > 0)
		{
			centroid *= 1.0 / n;
		}
	}
}


FFrame3d FDynamicMesh3::GetVertexFrame(int vID, bool bFrameNormalY, FVector3d* UseNormal) const
{
	if (HasVertexNormals() == false && UseNormal == nullptr)
	{
		return FFrame3d();
	}

	FVector3d v = Vertices[vID];
	FVector3d normal = (UseNormal != nullptr) ? Normalized(*UseNormal) : FVector3d(VertexNormals.GetValue()[vID]);
	int eid = VertexEdgeLists.First(vID);
	int ovi = GetOtherEdgeVertex(eid, vID);
	FVector3d ov = Vertices[ovi];
	FVector3d edge = (ov - v);
	Normalize(edge);

	FVector3d other = normal.Cross(edge);
	edge = other.Cross(normal);
	if (bFrameNormalY)
	{
		return FFrame3d(v, edge, normal, -other);
	}
	else
	{
		return FFrame3d(v, edge, other, normal);
	}
}



FVector3d FDynamicMesh3::GetTriNormal(int tID) const
{
	FVector3d v0, v1, v2;
	GetTriVertices(tID, v0, v1, v2);
	return VectorUtil::Normal(v0, v1, v2);
}

double FDynamicMesh3::GetTriArea(int tID) const
{
	FVector3d v0, v1, v2;
	GetTriVertices(tID, v0, v1, v2);
	return VectorUtil::Area(v0, v1, v2);
}



void FDynamicMesh3::GetTriInfo(int tID, FVector3d& Normal, double& Area, FVector3d& Centroid) const
{
	FVector3d v0, v1, v2;
	GetTriVertices(tID, v0, v1, v2);
	Centroid = (v0 + v1 + v2) * (1.0 / 3.0);
	Normal = VectorUtil::NormalArea(v0, v1, v2, Area);
}


FVector3d FDynamicMesh3::GetTriBaryPoint(int tID, double bary0, double bary1, double bary2) const
{
	const FIndex3i& tIDs = Triangles[tID];
	return bary0 * Vertices[tIDs[0]] +
		bary1 * Vertices[tIDs[1]] +
		bary2 * Vertices[tIDs[2]];
}


FVector3d FDynamicMesh3::GetTriBaryNormal(int tID, double bary0, double bary1, double bary2) const
{
	checkSlow(HasVertexNormals());
	if (HasVertexNormals())
	{
		const FIndex3i& tIDs = Triangles[tID];
		const TDynamicVector<FVector3f>& normalsR = VertexNormals.GetValue();
		FVector3d n = FVector3d(bary0 * normalsR[tIDs[0]] + bary1 * normalsR[tIDs[1]] + bary2 * normalsR[tIDs[2]]);
		Normalize(n);
		return n;
	}
	return FVector3d::Zero();
}

FVector3d FDynamicMesh3::GetTriCentroid(int tID) const
{
	const FIndex3i& tIDs = Triangles[tID];
	double f = (1.0 / 3.0);
	return (Vertices[tIDs[0]] + Vertices[tIDs[1]] + Vertices[tIDs[2]]) * f;
}

void FDynamicMesh3::GetTriBaryPoint(int tID, double bary0, double bary1, double bary2, FVertexInfo& vinfo) const
{
	vinfo                = FVertexInfo();
	const FIndex3i& tIDs = Triangles[tID];
	vinfo.Position       = bary0 * Vertices[tIDs[0]] + bary1 * Vertices[tIDs[1]] + bary2 * Vertices[tIDs[2]];
	vinfo.bHaveN         = HasVertexNormals();
	if (vinfo.bHaveN)
	{
		const TDynamicVector<FVector3f>& normalsR = this->VertexNormals.GetValue();
		vinfo.Normal =
		    (float)bary0 * normalsR[tIDs[0]] + (float)bary1 * normalsR[tIDs[1]] + (float)bary2 * normalsR[tIDs[2]];
		Normalize(vinfo.Normal);
	}
	vinfo.bHaveC = HasVertexColors();
	if (vinfo.bHaveC)
	{
		const TDynamicVector<FVector3f>& colorsR = this->VertexColors.GetValue();
		vinfo.Color =
		    (float)bary0 * colorsR[tIDs[0]] + (float)bary1 * colorsR[tIDs[1]] + (float)bary2 * colorsR[tIDs[2]];
	}
	vinfo.bHaveUV = HasVertexUVs();
	if (vinfo.bHaveUV)
	{
		const TDynamicVector<FVector2f>& uvR = this->VertexUVs.GetValue();
		vinfo.UV = (float)bary0 * uvR[tIDs[0]] + (float)bary1 * uvR[tIDs[1]] + (float)bary2 * uvR[tIDs[2]];
	}
}


FAxisAlignedBox3d FDynamicMesh3::GetTriBounds(int tID) const
{
	const FIndex3i& tIDs = Triangles[tID];
	const FVector3d& A = Vertices[tIDs.A];
	const FVector3d& B = Vertices[tIDs.B];
	const FVector3d& C = Vertices[tIDs.C];
	return FAxisAlignedBox3d(A, B, C);
}

FFrame3d FDynamicMesh3::GetTriFrame(int tID, int nEdge) const
{
	const FIndex3i& tIDs = Triangles[tID];
	const FVector3d TriVerts[3] = {Vertices[tIDs[nEdge % 3]], Vertices[tIDs[(nEdge + 1) % 3]],
	                               Vertices[tIDs[(nEdge + 2) % 3]]};

	FVector3d edge1 = TriVerts[1] - TriVerts[0];
	Normalize(edge1);
	FVector3d edge2 = TriVerts[2] - TriVerts[1];
	Normalize(edge2);
	FVector3d normal = edge2.Cross(edge1);
	Normalize(normal);
	FVector3d other  = normal.Cross(edge1);
	FVector3d center = (TriVerts[0] + TriVerts[1] + TriVerts[2]) / 3.0;

	return FFrame3d(center, edge1, other, normal);
}

double FDynamicMesh3::GetTriSolidAngle(int tID, const FVector3d& p) const
{
	// inlined version of GetTriVertices & VectorUtil::TriSolidAngle
	const FIndex3i& Triangle = Triangles[tID];
	const FVector3d TV[3] = {Vertices[Triangle[0]] - p, Vertices[Triangle[1]] - p, Vertices[Triangle[2]] - p};

	double la = TV[0].Length(), lb = TV[1].Length(), lc = TV[2].Length();
	double top    = (la * lb * lc) + TV[0].Dot(TV[1]) * lc + TV[1].Dot(TV[2]) * la + TV[2].Dot(TV[0]) * lb;
	double bottom = TV[0].X * (TV[1].Y * TV[2].Z - TV[2].Y * TV[1].Z) -
	    TV[0].Y * (TV[1].X * TV[2].Z - TV[2].X * TV[1].Z) + TV[0].Z * (TV[1].X * TV[2].Y - TV[2].X * TV[1].Y);
	// -2 instead of 2 to account for UE winding
	return -2.0 * atan2(bottom, top);
}

double FDynamicMesh3::GetTriInternalAngleR(int tID, int i) const
{
	const FIndex3i& Triangle = Triangles[tID];
	const FVector3d TV[3] = {Vertices[Triangle[0]], Vertices[Triangle[1]], Vertices[Triangle[2]]};
	if (i == 0)
	{
		return AngleR( Normalized(TV[1] - TV[0]), Normalized(TV[2] - TV[0]) );
	}
	else if (i == 1)
	{
		return AngleR( Normalized(TV[0] - TV[1]), Normalized(TV[2] - TV[1]) );
	}
	else
	{
		return AngleR( Normalized(TV[0] - TV[2]), Normalized(TV[1] - TV[2]) );
	}
}

FVector3d FDynamicMesh3::GetTriInternalAnglesR(int tID) const
{
	const FIndex3i& Triangle = Triangles[tID];
	return VectorUtil::TriangleInternalAngles(Vertices[Triangle[0]], Vertices[Triangle[1]], Vertices[Triangle[2]]);
}



double FDynamicMesh3::CalculateWindingNumber(const FVector3d& QueryPoint) const
{
	double sum = 0;
	for (int tid : TriangleIndicesItr())
	{
		sum += GetTriSolidAngle(tid, QueryPoint);
	}
	return sum / FMathd::FourPi;
}
