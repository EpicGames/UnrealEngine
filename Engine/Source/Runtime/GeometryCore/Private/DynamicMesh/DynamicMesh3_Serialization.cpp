// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "HAL/PlatformTime.h"
#include "UObject/UE5MainStreamObjectVersion.h"

using namespace UE::Geometry;

namespace FDynamicMesh3Serialization_Local
{
enum EDynamicMeshSerializationVersion
{
	InitialVersion = 1,
	CompactAndCompress = 2
};

// Encapsulates our serialization options, and selects the serialization variant for a given set of options. 
struct FDynamicMesh3SerializationOptions
{
	bool bPreserveDataLayout; //< Preserve the data layout, i.e. external vertex/triangle/edge indices are still valid after roundtrip serialization.
	bool bCompactData; //< Remove any holes or padding in the data layout, and discard/recompute any redundant data. 
	bool bUseCompression; //< Compress all data buffers to minimize memory footprint.

	explicit FDynamicMesh3SerializationOptions(const FArchive& Ar)
		: bPreserveDataLayout(Ar.IsTransacting())
		, bCompactData(!Ar.IsTransacting())
		, bUseCompression(Ar.IsPersistent())
	{
		checkSlow(bPreserveDataLayout != bCompactData); // Preserving the data layout and compacting the data are mutually exclusive.
	}

	enum EImplementationVariant
	{
		Default = 0,
		CompactData = 1 << 16
	};

	int ImplementationVariant() const
	{
		return bCompactData * CompactData;
	}

	friend FArchive& operator<<(FArchive& Ar, FDynamicMesh3SerializationOptions& Options)
	{
		Ar << Options.bPreserveDataLayout;
		Ar << Options.bCompactData;
		Ar << Options.bUseCompression;
		return Ar;
	}
};

// Serializes a generic dynamic vector containing bulk data.
template <typename T>
void SerializeVector(FArchive& Ar, TDynamicVector<T>& Vector, const FDynamicMesh3SerializationOptions& Options)
{
	if (Options.bUseCompression)
	{
		Vector.template Serialize<true /* bForceBulkSerialization */, true /* bUseCompression */>(Ar);
	}
	else
	{
		Vector.template Serialize<true /* bForceBulkSerialization */, false /* bUseCompression */>(Ar);
	}
}

// Serializes an optional generic dynamic vector containing bulk data.
// We need to do our own serialization since using the default serialization for TOptional would not allow us to customize the dynamic vector serialization.
template <typename T>
void SerializeOptionalVector(FArchive& Ar, TOptional<TDynamicVector<T>>& OptionalVector, const FDynamicMesh3SerializationOptions& Options)
{
	bool bHasOptionalVector = OptionalVector.IsSet();
	Ar << bHasOptionalVector;
	if (bHasOptionalVector)
	{
		if (Ar.IsLoading())
		{
			OptionalVector.Emplace();
		}
		SerializeVector(Ar, OptionalVector.GetValue(), Options);
	}
}

// Serializes all unique data related to vertices.
void SerializeUniqueVertexData(FArchive& Ar, TDynamicVector<FVector3d>& Vertices, TOptional<TDynamicVector<FVector3f>>& VertexNormals,
							TOptional<TDynamicVector<FVector3f>>& VertexColors, TOptional<TDynamicVector<FVector2f>>& VertexUVs,
							const FDynamicMesh3SerializationOptions& Options)
{
	SerializeVector(Ar, Vertices, Options);
	SerializeOptionalVector(Ar, VertexNormals, Options);
	SerializeOptionalVector(Ar, VertexColors, Options);
	SerializeOptionalVector(Ar, VertexUVs, Options);
}

// Serializes all unique data related to triangles.
void SerializeUniqueTriangleData(FArchive& Ar, TDynamicVector<FIndex3i>& Triangles, TOptional<TDynamicVector<int32>>& TriangleGroups, int&
                                 GroupIDCounter,
                                 const FDynamicMesh3SerializationOptions& Options)
{
	SerializeVector(Ar, Triangles, Options);
	SerializeOptionalVector(Ar, TriangleGroups, Options);
	Ar << GroupIDCounter;
}

// Serializes a ref count vector.
void SerializeRefCounts(FArchive& Ar, FRefCountVector& RefCounts, const FDynamicMesh3SerializationOptions& Options)
{
	RefCounts.Serialize(Ar, Options.bCompactData, Options.bUseCompression);
}

// Resets a ref count vector for dense triangle data, i.e. all ref count values are 1.
void ResetDenseTriangleRefCounts(size_t Num, FRefCountVector& TriangleRefCounts)
{
	TriangleRefCounts.Trim(Num);
	TriangleRefCounts.GetRawRefCountsUnsafe().Fill(1);
}

// Recomputes vertex ref counts for a given set of triangle indices.
void RecomputeVertexRefCounts(size_t NumVertices, const TDynamicVector<FIndex3i>& Triangles, const FRefCountVector& TriangleRefCounts,
                              FRefCountVector& VertexRefCounts)
{
	// Callable that iterates over all triangles.
	// This gets called from within Rebuild() with another callable as parameter that we in turn call to update the ref count for each vertex in our triangles.
	const auto IterateTriangles = [&Triangles, &TriangleRefCounts](auto&& UpdateVertexRefCount)
	{
		for (const int TriIndex : TriangleRefCounts.Indices())
		{
			const FIndex3i& Tri = Triangles[TriIndex];
			UpdateVertexRefCount(Tri[0]);
			UpdateVertexRefCount(Tri[1]);
			UpdateVertexRefCount(Tri[2]);
		}
	};

	// Callable that allocates the initial ref count value for a vertex. This gets called from within Rebuild().
	constexpr auto AllocateRefCountFunc = [](unsigned short& RefCount)
	{
		RefCount = 2;
	};

	// Callable that increments the ref count value for a vertex. This gets called from within Rebuild().
	constexpr auto IncrementRefCountFunc = [](unsigned short& RefCount)
	{
		++RefCount;
	};

	// Rebuild function using our callables.
	VertexRefCounts.Rebuild(NumVertices, IterateTriangles, AllocateRefCountFunc, IncrementRefCountFunc);
}

// Serializes a small list set.
void SerializeSmallListSet(FArchive& Ar, FSmallListSet& SmallListSet, const FDynamicMesh3SerializationOptions& Options)
{
	SmallListSet.Serialize(Ar, Options.bCompactData, Options.bUseCompression);
}

// Recomputes all the redundant edge data for given sets of vertices and triangles.
template <typename FindEdgeFunc, typename AddEdgeInternalFunc>
void RecomputeEdgeData(const TDynamicVector<FVector3d>& Vertices, const TDynamicVector<FIndex3i>& Triangles, const FRefCountVector& TriangleRefCounts,
                       TDynamicVector<FDynamicMesh3::FEdge>& Edges, FRefCountVector& EdgeRefCounts, TDynamicVector<FIndex3i>& TriangleEdges,
                       FSmallListSet& VertexEdgeLists, FindEdgeFunc&& FindEdge, AddEdgeInternalFunc&& AddEdgeInternal)
{
	Edges.Clear();
	EdgeRefCounts.Clear();

	TriangleEdges.Resize(Triangles.Num());
	TriangleEdges.Fill({FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID});

	VertexEdgeLists.Reset();
	VertexEdgeLists.Resize(Vertices.Num());

	auto AddEdge = [&Edges, &FindEdge, &AddEdgeInternal](const int VertA, const int VertB, const int Tid) -> int
	{
		int Eid = Forward<FindEdgeFunc>(FindEdge)(VertA, VertB);
		if (Eid == FDynamicMesh3::InvalidID)
		{
			Eid = Forward<AddEdgeInternalFunc>(AddEdgeInternal)(VertA, VertB, Tid);
		}
		else
		{
			Edges[Eid].Tri[1] = Tid;
		}
		return Eid;
	};

	for (const int Tid : TriangleRefCounts.Indices())
	{
		const FIndex3i& Tri = Triangles[Tid];
		const FIndex3i TriEdges = {
			AddEdge(Tri[0], Tri[1], Tid),
			AddEdge(Tri[1], Tri[2], Tid),
			AddEdge(Tri[2], Tri[0], Tid)
		};
		TriangleEdges[Tid] = TriEdges;
	}
}

// Serializes the optional attribute set for a given mesh.
// We need to do our own serialization since using the default serialization for TUniquePtr would not allow us to customize the attribute set serialization.
void SerializeAttributeSet(FArchive& Ar, FDynamicMesh3* Mesh, const FCompactMaps* CompactMaps, const FDynamicMesh3SerializationOptions& Options)
{
	bool bHasAttributes = Mesh->Attributes() != nullptr;
	Ar << bHasAttributes;
	if (Ar.IsLoading())
	{
		bHasAttributes ? Mesh->EnableAttributes() : Mesh->DiscardAttributes();
	}
	if (bHasAttributes)
	{
		Mesh->Attributes()->Serialize(Ar, CompactMaps, Options.bUseCompression);
	}
}

// Creates a generic optional dynamic vector of a given size if and only if another optional vector is set.
// This reduces code duplication for creating compacting vectors for optional mesh data such as normals or colors.
template <typename T>
TOptional<TDynamicVector<T>> CreateOptionalVector(const TOptional<TDynamicVector<T>>& ExistingOptionalVector, size_t Num)
{
	TOptional<TDynamicVector<T>> OptionalVector;
	if (ExistingOptionalVector.IsSet())
	{
		OptionalVector.Emplace();
		OptionalVector->SetNum(Num);
	}
	return MoveTemp(OptionalVector);
};
} // namespace FDynamicMesh3Serialization_Local

using namespace FDynamicMesh3Serialization_Local;

// Declaration of all variants of our serialization. The integer index is derived from the version an serialization options, if available.
template <> void FDynamicMesh3::SerializeInternal<InitialVersion>(FArchive&, void*);
template <> void FDynamicMesh3::SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::Default>(FArchive&, void*);
template <> void FDynamicMesh3::SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::CompactData>(FArchive&, void*);

void FDynamicMesh3::Serialize(FArchive& Ar)
{
	const double TimeStart = FPlatformTime::Seconds();

	// Skip serialization if the archive is only looking for UObject references.
	if (Ar.IsObjectReferenceCollector())
	{
		return;
	}

	constexpr bool bCheckValidityForDebugging = false;

	// Check validity for debugging before saving data.
	checkSlow(!bCheckValidityForDebugging || (Ar.IsLoading() || CheckValidity(FValidityOptions(), EValidityCheckFailMode::Ensure)));

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
	{
		SerializeInternal<InitialVersion>(Ar, nullptr);
	}
	else
	{
		FDynamicMesh3SerializationOptions Options(Ar);
		Ar << Options;
		switch (Options.ImplementationVariant())
		{
		case FDynamicMesh3SerializationOptions::Default:
			SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::Default>(Ar, &Options);
			break;
		case FDynamicMesh3SerializationOptions::CompactData:
			SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::CompactData>(Ar, &Options);
			break;
		default:
			ensureAlwaysMsgf(false, TEXT("Unhandled FDynamicMesh3 serialization variant '%d'."), Options.ImplementationVariant());
		}
	}

	// Check validity for debugging after loading data.
	checkSlow(!bCheckValidityForDebugging || (!Ar.IsLoading() || CheckValidity(FValidityOptions(), EValidityCheckFailMode::Ensure)));

	UE_LOGF(LogGeometry, Verbose, "NEW %ls of dynamic mesh took %2.3f ms", ANSI_TO_TCHAR(Ar.IsLoading() ? "deserialization" : "serialization"),
	       1000.0f * static_cast<float>(FPlatformTime::Seconds() - TimeStart));
}

template <>
void FDynamicMesh3::SerializeInternal<InitialVersion>(FArchive& Ar, void*)
{
	// This is not a current variant; only allow loading.
	ensure(Ar.IsLoading());

	int32 SerializationVersion;
	Ar << SerializationVersion;

	Ar << Vertices;
	Ar << VertexRefCounts;
	Ar << VertexNormals;
	Ar << VertexColors;
	Ar << VertexUVs;
	Ar << VertexEdgeLists;
	Ar << Triangles;
	Ar << TriangleRefCounts;
	Ar << TriangleEdges;
	Ar << TriangleGroups;
	Ar << GroupIDCounter;
	Ar << Edges;
	Ar << EdgeRefCounts;

	bool bHasAttributes;
	Ar << bHasAttributes;
	if (bHasAttributes)
	{
		EnableAttributes();
		Ar << *AttributeSet;
	}
}

// This 'CompactAndCompress' variant is relatively fast, but allows for discarding some redundant data.
// You can chose to preserve the data layout in the serialization options, which will keep any externally held vertex/triangle/edge indices valid.
template <>
void FDynamicMesh3::SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::Default>(FArchive& Ar, void* OptionsPtr)
{
	checkSlow(OptionsPtr != nullptr);
	const FDynamicMesh3SerializationOptions& Options = *static_cast<FDynamicMesh3SerializationOptions*>(OptionsPtr);

	// Vertex data.
	SerializeUniqueVertexData(Ar, Vertices, VertexNormals, VertexColors, VertexUVs, Options);

	// Triangle data.
	SerializeUniqueTriangleData(Ar, Triangles, TriangleGroups, GroupIDCounter, Options);

	// Triangle ref counts.
	// If the triangle vector has holes then we simply need to serialize the ref count data. If all allocated triangles are used, we can easily restore a ref
	// count vector containing all ones, which is not slower than naively serializing the data but saves some memory.
	bool bTriangleVectorHasHoles = !TriangleRefCounts.IsDense();
	Ar << bTriangleVectorHasHoles;
	if (bTriangleVectorHasHoles)
	{
		SerializeRefCounts(Ar, TriangleRefCounts, Options);
	}
	else if (Ar.IsLoading())
	{
		ResetDenseTriangleRefCounts(Triangles.Num(), TriangleRefCounts);
	}
	
	// Vertex ref counts.
	// The vertex ref counts are redundant to the triangle data, and hence we can recompute it even if we need to preserve the data layout.
	// Considering that recomputing it is about as fast as serializing it, we always recompute it to save memory.
	if (Ar.IsLoading())
	{
		RecomputeVertexRefCounts(Vertices.Num(), Triangles, TriangleRefCounts, VertexRefCounts);
	}

	// Edge data.
	// All edge data can be derived from the triangle and vertex data. If we need to preserve the data layout though, we simply serialize everything as is.
	if (Options.bPreserveDataLayout)
	{
		SerializeVector(Ar, Edges, Options);
		SerializeRefCounts(Ar, EdgeRefCounts, Options);
		SerializeVector(Ar, TriangleEdges, Options);
		SerializeSmallListSet(Ar, VertexEdgeLists, Options);
	}
	else
	{
		if (Ar.IsLoading())
		{
			RecomputeEdgeData(Vertices, Triangles, TriangleRefCounts, Edges, EdgeRefCounts, TriangleEdges, VertexEdgeLists,
			                  [this](int VertA, int VertB) -> int { return FindEdge(VertA, VertB); },
			                  [this](int VertA, int VertB, int Tid) -> int { return AddEdgeInternal(VertA, VertB, Tid); });
		}
	}

	// Attribute set data.
	SerializeAttributeSet(Ar, this, nullptr, Options);
}

// This 'CompactAndCompress' variant will significantly reduce the memory footprint of the serialized data by compacting the unique data, and disregarding any
// redundant data, e.g. ref counts and edges. Optionally, it can also compress all data using Oodle, although at a significant runtime cost. 
// Note that this will very likely invalidate externally held vertex/triangle/edge indices.
template <>
void FDynamicMesh3::SerializeInternal<CompactAndCompress + FDynamicMesh3SerializationOptions::CompactData>(FArchive& Ar, void* OptionsPtr)
{
	checkSlow(OptionsPtr != nullptr);
	const FDynamicMesh3SerializationOptions& Options = *static_cast<FDynamicMesh3SerializationOptions*>(OptionsPtr);

	FCompactMaps CompactMaps;

	// Vertex data.
	if (Ar.IsLoading() || (!Ar.IsLoading() && VertexRefCounts.IsDense()))
	{
		SerializeUniqueVertexData(Ar, Vertices, VertexNormals, VertexColors, VertexUVs, Options);
	}
	else
	{
		const size_t NumValidVertices = VertexRefCounts.GetCount();

		TDynamicVector<FVector3d> VerticesCompact;
		VerticesCompact.SetNum(NumValidVertices);

		TOptional<TDynamicVector<FVector3f>> VertexNormalsCompact = CreateOptionalVector(VertexNormals, NumValidVertices);
		TOptional<TDynamicVector<FVector3f>> VertexColorsCompact = CreateOptionalVector(VertexColors, NumValidVertices);
		TOptional<TDynamicVector<FVector2f>> VertexUVsCompact = CreateOptionalVector(VertexUVs, NumValidVertices);

		CompactMaps.ResetVertexMap(Vertices.Num(), true);

		size_t VidCompact = 0;
		for (const size_t Vid : VertexRefCounts.Indices())
		{
			VerticesCompact[VidCompact] = Vertices[Vid];
			if (VertexNormalsCompact) { (*VertexNormalsCompact)[VidCompact] = (*VertexNormals)[Vid]; }
			if (VertexColorsCompact) { (*VertexColorsCompact)[VidCompact] = (*VertexColors)[Vid]; }
			if (VertexUVsCompact) { (*VertexUVsCompact)[VidCompact] = (*VertexUVs)[Vid]; }
			CompactMaps.SetVertexMapping(Vid, VidCompact);
			++VidCompact;
		}

		SerializeUniqueVertexData(Ar, VerticesCompact, VertexNormalsCompact, VertexColorsCompact, VertexUVsCompact, Options);
	}

	// Triangle data.
	if (Ar.IsLoading() || (!Ar.IsLoading() && TriangleRefCounts.IsDense() && VertexRefCounts.IsDense()))
	{
		SerializeUniqueTriangleData(Ar, Triangles, TriangleGroups, GroupIDCounter, Options);
	}
	else
	{
		const size_t NumValidTriangles = TriangleRefCounts.GetCount();

		TDynamicVector<FIndex3i> TrianglesCompact;
		TrianglesCompact.SetNum(NumValidTriangles);

		TOptional<TDynamicVector<int>> TriangleGroupsCompact = CreateOptionalVector(TriangleGroups, NumValidTriangles);

		CompactMaps.ResetTriangleMap(Triangles.Num(), true);

		const bool bHasVertexMapping = CompactMaps.VertexMapIsSet();
		
		size_t TidCompact = 0;
		for (const size_t Tid : TriangleRefCounts.Indices())
		{
			if (!bHasVertexMapping)
			{
				TrianglesCompact[TidCompact] = Triangles[Tid];
			}
			else
			{
				const FIndex3i& Tri = Triangles[Tid];
				FIndex3i& TriCompact = TrianglesCompact[TidCompact];
				TriCompact[0] = CompactMaps.GetVertexMapping(Tri[0]);
				TriCompact[1] = CompactMaps.GetVertexMapping(Tri[1]);
				TriCompact[2] = CompactMaps.GetVertexMapping(Tri[2]);
			}
			if (TriangleGroupsCompact) { (*TriangleGroupsCompact)[TidCompact] = (*TriangleGroups)[Tid]; }
			CompactMaps.SetTriangleMapping(Tid, TidCompact);
			++TidCompact;
		}

		SerializeUniqueTriangleData(Ar, TrianglesCompact, TriangleGroupsCompact, GroupIDCounter, Options);
	}

	// When loading, restore redundant ref count and edge data.
	if (Ar.IsLoading())
	{
		ResetDenseTriangleRefCounts(Triangles.Num(), TriangleRefCounts);

		RecomputeVertexRefCounts(Vertices.Num(), Triangles, TriangleRefCounts, VertexRefCounts);

		RecomputeEdgeData(Vertices, Triangles, TriangleRefCounts, Edges, EdgeRefCounts, TriangleEdges, VertexEdgeLists,
		                  [this](int VertA, int VertB) -> int { return FindEdge(VertA, VertB); },
		                  [this](int VertA, int VertB, int Tid) -> int { return AddEdgeInternal(VertA, VertB, Tid); });
	}

	// Attribute set data.
	SerializeAttributeSet(Ar, this, CompactMaps.VertexMapIsSet() || CompactMaps.TriangleMapIsSet() ? &CompactMaps : nullptr, Options);
}
