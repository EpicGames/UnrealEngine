// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "IndexTypes.h"
#include "Tasks/Task.h"
#include "Serialization/NameAsStringProxyArchive.h"

#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

using namespace UE::Geometry;


FDynamicMeshAttributeSet::FDynamicMeshAttributeSet(FDynamicMesh3* Mesh)
	: ParentMesh(Mesh)
{
	FDynamicMeshAttributeSet::SetNumUVLayers(1);
	FDynamicMeshAttributeSet::SetNumNormalLayers(1);
}

FDynamicMeshAttributeSet::FDynamicMeshAttributeSet(FDynamicMesh3* Mesh, int32 NumUVLayers, int32 NumNormalLayers)
	: ParentMesh(Mesh)
{
	FDynamicMeshAttributeSet::SetNumUVLayers(NumUVLayers);
	FDynamicMeshAttributeSet::SetNumNormalLayers(NumNormalLayers);
}

FDynamicMeshAttributeSet::~FDynamicMeshAttributeSet()
{
}

void FDynamicMeshAttributeSet::Copy(const FDynamicMeshAttributeSet& Copy)
{
	SetNumUVLayers(Copy.NumUVLayers());
	for (int UVIdx = 0; UVIdx < NumUVLayers(); UVIdx++)
	{
		UVLayers[UVIdx].Copy(Copy.UVLayers[UVIdx]);
	}
	SetNumNormalLayers(Copy.NumNormalLayers());
	for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers(); NormalLayerIndex++)
	{
		NormalLayers[NormalLayerIndex].Copy(Copy.NormalLayers[NormalLayerIndex]);
	}
	if (Copy.ColorLayer)
	{
		EnablePrimaryColors();
		ColorLayer->Copy(*(Copy.ColorLayer));
	}
	else
	{
		DisablePrimaryColors();
	}
	if (Copy.MaterialIDAttrib)
	{
		EnableMaterialID();
		MaterialIDAttrib->Copy(*(Copy.MaterialIDAttrib));
	}
	else
	{
		DisableMaterialID();
	}

	SetNumPolygroupLayers(Copy.NumPolygroupLayers());
	for (int GroupIdx = 0; GroupIdx < NumPolygroupLayers(); ++GroupIdx)
	{
		PolygroupLayers[GroupIdx].Copy(Copy.PolygroupLayers[GroupIdx]);
	}

	SetNumWeightLayers(Copy.NumWeightLayers());
	for (int WeightLayerIdx = 0; WeightLayerIdx < NumWeightLayers(); ++WeightLayerIdx)
	{
		WeightLayers[WeightLayerIdx].Copy(Copy.WeightLayers[WeightLayerIdx]);
	}

	ResetRegisteredAttributes();

	TriangleLabelAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair: Copy.TriangleLabelAttributes)
	{
		AttachTriangleLabelAttribute(AttribPair.Key,
			static_cast<FDynamicMeshTriangleLabelAttribute*>(AttribPair.Value->MakeCopy(ParentMesh)));
	}
	
	SkinWeightAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : Copy.SkinWeightAttributes)
	{
		AttachSkinWeightsAttribute(AttribPair.Key,
			static_cast<FDynamicMeshVertexSkinWeightsAttribute *>(AttribPair.Value->MakeCopy(ParentMesh)));
	}

	MorphTargetAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : Copy.MorphTargetAttributes)
	{
		AttachMorphTargetAttribute(AttribPair.Key,
			static_cast<FDynamicMeshMorphTargetAttribute*>(AttribPair.Value->MakeCopy(ParentMesh)));
	}

	CopyBoneAttributes(Copy);
	
	GenericAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : Copy.GenericAttributes)
	{
		AttachAttribute(AttribPair.Key, AttribPair.Value->MakeCopy(ParentMesh));
	}

	SculptLayers.Copy(this, Copy.SculptLayers);

	// parent mesh is *not* copied!
}


bool FDynamicMeshAttributeSet::IsCompact() const
{
	for (int UVIdx = 0; UVIdx < NumUVLayers(); UVIdx++)
	{
		if (!UVLayers[UVIdx].IsCompact())
		{
			return false;
		}
	}
	for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers(); NormalLayerIndex++)
	{
		if (!NormalLayers[NormalLayerIndex].IsCompact())
		{
			return false;
		}
	}
	if (HasPrimaryColors())
	{
		if (!ColorLayer->IsCompact())
		{
			return false;
		}
	}
	
	// material ID and generic per-element attributes currently cannot be non-compact
	return true;
}



void FDynamicMeshAttributeSet::CompactCopy(const FCompactMaps& CompactMaps, const FDynamicMeshAttributeSet& Copy)
{
	SetNumUVLayers(Copy.NumUVLayers());
	for (int UVIdx = 0; UVIdx < NumUVLayers(); UVIdx++)
	{
		UVLayers[UVIdx].CompactCopy(CompactMaps, Copy.UVLayers[UVIdx]);
	}
	SetNumNormalLayers(Copy.NumNormalLayers());
	for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers(); NormalLayerIndex++)
	{
		NormalLayers[NormalLayerIndex].CompactCopy(CompactMaps, Copy.NormalLayers[NormalLayerIndex]);
	}
	if (Copy.ColorLayer)
	{
		EnablePrimaryColors();
		ColorLayer->CompactCopy(CompactMaps, *(Copy.ColorLayer));
	}
	else
	{
		DisablePrimaryColors();
	}

	if (Copy.MaterialIDAttrib)
	{
		EnableMaterialID();
		MaterialIDAttrib->CompactCopy(CompactMaps, *(Copy.MaterialIDAttrib));
	}
	else
	{
		DisableMaterialID();
	}

	SetNumPolygroupLayers(Copy.NumPolygroupLayers());
	for (int GroupIdx = 0; GroupIdx < NumPolygroupLayers(); ++GroupIdx)
	{
		PolygroupLayers[GroupIdx].CompactCopy(CompactMaps, Copy.PolygroupLayers[GroupIdx]);
	}
	
	SetNumWeightLayers(Copy.NumWeightLayers());
	for (int WeightLayerIdx = 0; WeightLayerIdx < NumWeightLayers(); ++WeightLayerIdx)
	{
		WeightLayers[WeightLayerIdx].CompactCopy(CompactMaps, Copy.WeightLayers[WeightLayerIdx]);
	}

	ResetRegisteredAttributes();

	TriangleLabelAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair: Copy.TriangleLabelAttributes)
	{
		AttachTriangleLabelAttribute(AttribPair.Key,
			static_cast<FDynamicMeshTriangleLabelAttribute *>(AttribPair.Value->MakeCompactCopy(CompactMaps, ParentMesh)));
	}
	
	SkinWeightAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : Copy.SkinWeightAttributes)
	{
		AttachSkinWeightsAttribute(AttribPair.Key,
			static_cast<FDynamicMeshVertexSkinWeightsAttribute *>(AttribPair.Value->MakeCompactCopy(CompactMaps, ParentMesh)));
	}

	MorphTargetAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : Copy.MorphTargetAttributes)
	{
		AttachMorphTargetAttribute(AttribPair.Key,
			static_cast<FDynamicMeshMorphTargetAttribute*>(AttribPair.Value->MakeCompactCopy(CompactMaps, ParentMesh)));
	}
	
	CopyBoneAttributes(Copy);
	
	GenericAttributes.Reset();
	for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : Copy.GenericAttributes)
	{
		AttachAttribute(AttribPair.Key, AttribPair.Value->MakeCompactCopy(CompactMaps, ParentMesh));
	}

	SculptLayers.CompactCopy(this, CompactMaps, Copy.SculptLayers);

	// parent mesh is *not* copied!
}


void FDynamicMeshAttributeSet::Append(const FDynamicMeshAttributeSet& ToAppend, const FDynamicMesh3::FAppendInfo& AppendInfo)
{
	auto AppendHelper = [&AppendInfo]<typename T>(T & Target, const T * ToAppend)
	{
		if (ToAppend)
		{
			Target.Append(*ToAppend, AppendInfo);
		}
		else
		{
			Target.AppendDefaulted(AppendInfo);
		}
	};

	for (int32 Idx = 0; Idx < NumUVLayers(); ++Idx)
	{
		AppendHelper(UVLayers[Idx], ToAppend.GetUVLayer(Idx));
	}
	for (int32 Idx = 0; Idx < NumNormalLayers(); ++Idx)
	{
		AppendHelper(NormalLayers[Idx], ToAppend.GetNormalLayer(Idx));
	}
	if (ColorLayer.IsValid())
	{
		AppendHelper(*ColorLayer, ToAppend.ColorLayer.Get());
	}
	if (MaterialIDAttrib.IsValid())
	{
		AppendHelper(*MaterialIDAttrib, ToAppend.MaterialIDAttrib.Get());
	}

	for (int Idx = 0; Idx < NumPolygroupLayers(); ++Idx)
	{
		AppendHelper(PolygroupLayers[Idx], Idx < ToAppend.NumPolygroupLayers() ? ToAppend.GetPolygroupLayer(Idx) : nullptr);
	}
	
	for (TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair: TriangleLabelAttributes)
	{
		AppendHelper(*AttribPair.Value, ToAppend.FindTriangleLabelAttribute(AttribPair.Key));
	}
	
	for (int Idx = 0; Idx < NumWeightLayers(); ++Idx)
	{
		AppendHelper(WeightLayers[Idx], Idx < ToAppend.NumWeightLayers() ? ToAppend.GetWeightLayer(Idx) : nullptr);
	}

	// if both have bones, attempt to combine the bones and update weights accordingly
	if (HasBones() && ToAppend.HasBones())
	{
		const bool bSameSkeletons = GetBoneNames()->IsSameAs(*ToAppend.GetBoneNames());
		if (!bSameSkeletons)
		{
			AppendBonesUnique(ToAppend);
		}
		for (TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : SkinWeightAttributes)
		{
			FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = ToAppend.GetSkinWeightsAttribute(AttribPair.Key);
			if (ToAttrib && !bSameSkeletons)
			{
				// Make a copy of the to-append skinning weights so we can reindex them w.r.t the new skeleton
				// TODO: add a method to do this reindexing in-place on just the appended weights
				FDynamicMeshVertexSkinWeightsAttribute CopyAppendMeshAttrib;
				CopyAppendMeshAttrib.Copy(*ToAttrib);
				CopyAppendMeshAttrib.ReindexBoneIndicesToSkeleton(ToAppend.GetBoneNames()->GetAttribValues(), GetBoneNames()->GetAttribValues());
				AppendHelper(*AttribPair.Value, &CopyAppendMeshAttrib);
			}
			else
			{
				AppendHelper(*AttribPair.Value, ToAttrib);
			}
		}
	}
	else
	{
		// If only one had bones, we don't need to update bones; just transfer skin weights via standard append logic
		for (TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : SkinWeightAttributes)
		{
			AppendHelper(*AttribPair.Value, ToAppend.GetSkinWeightsAttribute(AttribPair.Key));
		}
	}

	for (TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : MorphTargetAttributes)
	{
		AppendHelper(*AttribPair.Value, ToAppend.GetMorphTargetAttribute(AttribPair.Key));
	}

	for (TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : GenericAttributes)
	{
		const TUniquePtr<FDynamicMeshAttributeBase>* AppendAttr = ToAppend.GenericAttributes.Find(AttribPair.Key);
		AppendHelper(*AttribPair.Value, AppendAttr ? AppendAttr->Get() : nullptr);
	}

	for (int32 Idx = 0; Idx < SculptLayers.NumLayers(); ++Idx)
	{
		AppendHelper(SculptLayers.Layers[Idx], ToAppend.SculptLayers.GetLayer(Idx));
	}
}

void FDynamicMeshAttributeSet::AppendDefaulted(const FDynamicMesh3::FAppendInfo& AppendInfo)
{
	auto AppendHelper = [&AppendInfo]<typename T>(T & Target)
	{
		Target.AppendDefaulted(AppendInfo);
	};

	for (int32 Idx = 0; Idx < NumUVLayers(); ++Idx)
	{
		AppendHelper(UVLayers[Idx]);
	}
	for (int32 Idx = 0; Idx < NumNormalLayers(); ++Idx)
	{
		AppendHelper(NormalLayers[Idx]);
	}
	if (ColorLayer.IsValid())
	{
		AppendHelper(*ColorLayer);
	}
	if (MaterialIDAttrib.IsValid())
	{
		AppendHelper(*MaterialIDAttrib);
	}

	for (int Idx = 0; Idx < NumPolygroupLayers(); ++Idx)
	{
		AppendHelper(PolygroupLayers[Idx]);
	}

	for (int Idx = 0; Idx < NumWeightLayers(); ++Idx)
	{
		AppendHelper(WeightLayers[Idx]);
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair : TriangleLabelAttributes)
	{
		AppendHelper(*AttribPair.Value);
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : SkinWeightAttributes)
	{
		AppendHelper(*AttribPair.Value);
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : MorphTargetAttributes)
	{
		AppendHelper(*AttribPair.Value);
	}

	// Note: In the 'defaulted' case, there is nothing to append re the Bone attributes (since they are not tied to the base mesh elements)

	for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : GenericAttributes)
	{
		AppendHelper(*AttribPair.Value);
	}

	for (int32 Idx = 0; Idx < SculptLayers.NumLayers(); ++Idx)
	{
		AppendHelper(SculptLayers.Layers[Idx]);
	}
}



void FDynamicMeshAttributeSet::CompactInPlace(const FCompactMaps& CompactMaps)
{
	for (int UVIdx = 0; UVIdx < NumUVLayers(); UVIdx++)
	{
		UVLayers[UVIdx].CompactInPlace(CompactMaps);
	}
	for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers(); NormalLayerIndex++)
	{
		NormalLayers[NormalLayerIndex].CompactInPlace(CompactMaps);
	}
	if (ColorLayer.IsValid())
	{
		ColorLayer->CompactInPlace(CompactMaps);
	}
	if (MaterialIDAttrib.IsValid())
	{
		MaterialIDAttrib->CompactInPlace(CompactMaps);
	}

	for (int GroupIdx = 0; GroupIdx < NumPolygroupLayers(); ++GroupIdx)
	{
		PolygroupLayers[GroupIdx].CompactInPlace(CompactMaps);
	}
	
	for (auto& [AttrName, TriangleLabelAttr]: TriangleLabelAttributes)
	{
		TriangleLabelAttr->CompactInPlace(CompactMaps);
	}
	
	for (int WeightLayerIdx = 0; WeightLayerIdx < NumWeightLayers(); ++WeightLayerIdx)
	{
		WeightLayers[WeightLayerIdx].CompactInPlace(CompactMaps);
	}
	
	for (FDynamicMeshAttributeBase* RegAttrib : RegisteredAttributes)
	{
		RegAttrib->CompactInPlace(CompactMaps);
	}
}



void FDynamicMeshAttributeSet::SplitAllBowties(bool bParallel)
{
	int32 UVLayerCount = NumUVLayers();
	int32 NormalLayerCount = NumNormalLayers();
	
	TArray<UE::Tasks::FTask> Pending;
	auto ASyncOrRunSplit = [&Pending, bParallel](auto Overlay)->void
	{
		if (bParallel)
		{	
			UE::Tasks::FTask AsyncTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Overlay]() 
			{
				Overlay->SplitBowties();
			});
			Pending.Add(MoveTemp(AsyncTask));
		}
		else
		{
			Overlay->SplitBowties();
		}
	};

	for (int32 i = 0; i < UVLayerCount; ++i)
	{	
		FDynamicMeshUVOverlay* UVLayer = GetUVLayer(i);
		ASyncOrRunSplit(UVLayer);
	}
	for (int32 i = 0; i < NormalLayerCount; ++i)
	{
		FDynamicMeshNormalOverlay* NormalLayer = GetNormalLayer(i);
		ASyncOrRunSplit(NormalLayer);
	}
	if (HasPrimaryColors())
	{
		FDynamicMeshColorOverlay* Colors = PrimaryColors();
		ASyncOrRunSplit(Colors);
	}

	// this array will be empty if bParallel == false
	UE::Tasks::Wait(Pending);
}



void FDynamicMeshAttributeSet::EnableMatchingAttributes(const FDynamicMeshAttributeSet& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes)
{
	int32 ExistingUVLayers = NumUVLayers();
	int32 RequiredUVLayers = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.NumUVLayers() : FMath::Max(ExistingUVLayers, ToMatch.NumUVLayers());
	SetNumUVLayers(RequiredUVLayers);
	for (int32 k = bClearExisting ? 0 : ExistingUVLayers; k < NumUVLayers(); k++)
	{
		UVLayers[k].ClearElements();
	}

	int32 ExistingNormalLayers = NumNormalLayers();
	int32 RequiredNormalLayers = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.NumNormalLayers() : FMath::Max(ExistingNormalLayers, ToMatch.NumNormalLayers());
	SetNumNormalLayers(RequiredNormalLayers);
	for (int32 k = bClearExisting ? 0 : ExistingNormalLayers; k < NumNormalLayers(); k++)
	{
		NormalLayers[k].ClearElements();
	}

	bool bWantColorLayer = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasPrimaryColors() : ( ToMatch.HasPrimaryColors() || this->HasPrimaryColors() );
	if (bClearExisting || bWantColorLayer == false)
	{
		DisablePrimaryColors();
	}
	if (bWantColorLayer)
	{
		EnablePrimaryColors();
	}

	bool bWantMaterialID = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.HasMaterialID() : ( ToMatch.HasMaterialID() || this->HasMaterialID() );
	if (bClearExisting || bWantMaterialID == false)
	{
		DisableMaterialID();
	}
	if (bWantMaterialID)
	{
		EnableMaterialID();
	}

	// polygroup layers are handled by count, not by name...maybe wrong
	int32 ExistingPolygroupLayers = NumPolygroupLayers();
	int32 RequiredPolygroupLayers = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.NumPolygroupLayers() : FMath::Max(ExistingPolygroupLayers, ToMatch.NumPolygroupLayers());
	SetNumPolygroupLayers(RequiredPolygroupLayers);
	for (int32 k = bClearExisting ? 0 : ExistingPolygroupLayers; k < NumPolygroupLayers(); k++)
	{
		PolygroupLayers[k].Initialize((int32)0);
		if (k < ToMatch.NumPolygroupLayers() && PolygroupLayers[k].GetName() == NAME_None)
		{
			PolygroupLayers[k].SetName( ToMatch.GetPolygroupLayer(k)->GetName() );
		}
	}

	// weightmap layers are handled by count, not by name...maybe wrong
	int32 ExistingWeightLayers = NumWeightLayers();
	int32 RequiredWeightLayers = (bClearExisting || bDiscardExtraAttributes) ? ToMatch.NumWeightLayers() : FMath::Max(ExistingWeightLayers, ToMatch.NumWeightLayers());
	SetNumWeightLayers(RequiredWeightLayers);
	for (int32 k = bClearExisting ? 0 : ExistingWeightLayers; k < NumWeightLayers(); k++)
	{
		WeightLayers[k].Initialize(0.0f);
		if (k < ToMatch.NumWeightLayers() && WeightLayers[k].GetName() == NAME_None)
		{
			WeightLayers[k].SetName( ToMatch.GetWeightLayer(k)->GetName() );
		}
	}

	// SkinWeights and GenericAttributes require more complex handling...
	if (bClearExisting)
	{
		// discard existing
		TriangleLabelAttributes.Reset();
		SkinWeightAttributes.Reset();
		MorphTargetAttributes.Reset();
		GenericAttributes.Reset();
		ResetRegisteredAttributes();

		// register new attributes by name
		for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair: ToMatch.TriangleLabelAttributes)
		{
			AttachTriangleLabelAttribute(AttribPair.Key, 
				static_cast<FDynamicMeshTriangleLabelAttribute *>(AttribPair.Value->MakeNew(ParentMesh)));
		}
		for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : ToMatch.SkinWeightAttributes)
		{
			AttachSkinWeightsAttribute(AttribPair.Key,
				static_cast<FDynamicMeshVertexSkinWeightsAttribute *>(AttribPair.Value->MakeNew(ParentMesh)));	
		}
		for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : ToMatch.MorphTargetAttributes)
		{
			AttachMorphTargetAttribute(AttribPair.Key,
				static_cast<FDynamicMeshMorphTargetAttribute *>(AttribPair.Value->MakeNew(ParentMesh)));	
		}
		for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : ToMatch.GenericAttributes)
		{
			AttachAttribute(AttribPair.Key, AttribPair.Value->MakeNew(ParentMesh));
		}
	}
	else
	{
		// get rid of any attributes in current SkinWeights and Generic sets that are not in ToMatch
		if (bDiscardExtraAttributes)
		{
			TriangleLabelAttributeMap ExistingTriangleLabels = MoveTemp(TriangleLabelAttributes);
			SkinWeightAttributesMap ExistingSkinWeights = MoveTemp(SkinWeightAttributes);
			GenericAttributesMap ExistingGenericAttributes = MoveTemp(GenericAttributes);
			TriangleLabelAttributes.Reset();
			SkinWeightAttributes.Reset();
			GenericAttributes.Reset();
			
			ResetRegisteredAttributes();
			for (TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair : ExistingTriangleLabels)
			{
				if ( ToMatch.TriangleLabelAttributes.Contains(AttribPair.Key) )
				{
					AttachTriangleLabelAttribute(AttribPair.Key, AttribPair.Value.Release());
				}
			}
			for (TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : ExistingSkinWeights)
			{
				if ( ToMatch.SkinWeightAttributes.Contains(AttribPair.Key) )
				{
					AttachSkinWeightsAttribute(AttribPair.Key, AttribPair.Value.Release());
				}
			}
			for (TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : ExistingGenericAttributes)
			{
				if ( ToMatch.GenericAttributes.Contains(AttribPair.Key) )
				{
					AttachAttribute(AttribPair.Key, AttribPair.Value.Release());
				}
			}
		}

		// add any new triangle label attribute that did not previously exist
		for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair : ToMatch.TriangleLabelAttributes)
		{
			if (!TriangleLabelAttributes.Contains(AttribPair.Key))
			{
				AttachTriangleLabelAttribute(AttribPair.Key,
					static_cast<FDynamicMeshTriangleLabelAttribute *>(AttribPair.Value->MakeNew(ParentMesh)));	
			}
		}
		
		// add any new SkinWeight attributes that did not previously exist
		for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : ToMatch.SkinWeightAttributes)
		{
			TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>* FoundMatch = SkinWeightAttributes.Find(AttribPair.Key);
			if (FoundMatch == nullptr)
			{
				AttachSkinWeightsAttribute(AttribPair.Key,
					static_cast<FDynamicMeshVertexSkinWeightsAttribute *>(AttribPair.Value->MakeNew(ParentMesh)));	
			}
		}

		// add any new Generic attributes that did not previously exist, matching by name
		for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : ToMatch.GenericAttributes)
		{
			TUniquePtr<FDynamicMeshAttributeBase>* FoundMatch = GenericAttributes.Find(AttribPair.Key);
			if (FoundMatch == nullptr)
			{
				AttachAttribute(AttribPair.Key, AttribPair.Value->MakeNew(ParentMesh));	
			}
		}
	}

	EnableMatchingBoneAttributes(ToMatch, bClearExisting, bDiscardExtraAttributes);

	SculptLayers.EnableMatching(this, ToMatch.SculptLayers, bClearExisting, bDiscardExtraAttributes);
}



void FDynamicMeshAttributeSet::Reparent(FDynamicMesh3* NewParent)
{
	ParentMesh = NewParent;

	for (int UVIdx = 0; UVIdx < NumUVLayers(); UVIdx++)
	{
		UVLayers[UVIdx].Reparent(NewParent);
	}
	for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers(); NormalLayerIndex++)
	{
		NormalLayers[NormalLayerIndex].Reparent(NewParent);
	}
	if (ColorLayer)
	{
		ColorLayer->Reparent(NewParent);
	}

	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->Reparent(NewParent);
	}

	for (int GroupIdx = 0; GroupIdx < NumPolygroupLayers(); ++GroupIdx)
	{
		PolygroupLayers[GroupIdx].Reparent(NewParent);
	}

	for (int WeightLayerIdx = 0; WeightLayerIdx < NumWeightLayers(); ++WeightLayerIdx)
	{
		WeightLayers[WeightLayerIdx].Reparent(NewParent);
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& AttribPair : TriangleLabelAttributes)
	{
		AttribPair.Value.Get()->Reparent(NewParent);
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : SkinWeightAttributes)
	{
		AttribPair.Value.Get()->Reparent(NewParent);
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : MorphTargetAttributes)
	{
		AttribPair.Value.Get()->Reparent(NewParent);
	}

	if (BoneNameAttrib)
	{
		BoneNameAttrib->Reparent(NewParent);
	}

	if (BoneParentIndexAttrib)
	{
		BoneParentIndexAttrib->Reparent(NewParent);
	}

	if (BonePoseAttrib)
	{
		BonePoseAttrib->Reparent(NewParent);
	}

	if (BoneColorAttrib)
	{
		BoneColorAttrib->Reparent(NewParent);
	}
	
	for (FDynamicMeshAttributeBase* RegAttrib : RegisteredAttributes)
	{
		RegAttrib->Reparent(NewParent);
	}
}



void FDynamicMeshAttributeSet::SetNumUVLayers(int Num)
{
	if (UVLayers.Num() == Num)
	{
		return;
	}
	if (Num >= UVLayers.Num())
	{
		for (int i = (int)UVLayers.Num(); i < Num; ++i)
		{
			FDynamicMeshUVOverlay* NewUVLayer = new FDynamicMeshUVOverlay(ParentMesh);
			NewUVLayer->InitializeTriangles(ParentMesh->MaxTriangleID());
			UVLayers.Add(NewUVLayer);
		}
	}
	else
	{
		UVLayers.RemoveAt(Num, UVLayers.Num() - Num);
	}
	ensure(UVLayers.Num() == Num);
}



void FDynamicMeshAttributeSet::EnableTangents()
{
	SetNumNormalLayers(3);
}

void FDynamicMeshAttributeSet::DisableTangents()
{
	SetNumNormalLayers(1);
}


void FDynamicMeshAttributeSet::SetNumNormalLayers(int Num)
{
	if (NormalLayers.Num() == Num)
	{
		return;
	}
	if (Num >= NormalLayers.Num())
	{
		for (int32 i = NormalLayers.Num(); i < Num; ++i)
		{
			FDynamicMeshNormalOverlay* NewNormalLayer = new FDynamicMeshNormalOverlay(ParentMesh);
			NewNormalLayer->InitializeTriangles(ParentMesh->MaxTriangleID());
			NormalLayers.Add(NewNormalLayer);
		}
	}
	else
	{
		NormalLayers.RemoveAt(Num, NormalLayers.Num() - Num);
	}
	ensure(NormalLayers.Num() == Num);
}

void FDynamicMeshAttributeSet::EnablePrimaryColors()
{
	if (HasPrimaryColors() == false)
	{ 
		ColorLayer = MakeUnique<FDynamicMeshColorOverlay>(ParentMesh);
		ColorLayer->InitializeTriangles(ParentMesh->MaxTriangleID());
	}
}

void FDynamicMeshAttributeSet::DisablePrimaryColors()
{
	ColorLayer.Reset();
}

int32 FDynamicMeshAttributeSet::NumPolygroupLayers() const
{
	return PolygroupLayers.Num();
}


void FDynamicMeshAttributeSet::SetNumPolygroupLayers(int32 Num)
{
	if (PolygroupLayers.Num() == Num)
	{
		return;
	}
	if (Num >= PolygroupLayers.Num())
	{
		for (int i = (int)PolygroupLayers.Num(); i < Num; ++i)
		{
			PolygroupLayers.Add(new FDynamicMeshPolygroupAttribute(ParentMesh));
		}
	}
	else
	{
		PolygroupLayers.RemoveAt(Num, PolygroupLayers.Num() - Num);
	}
	ensure(PolygroupLayers.Num() == Num);
}

FDynamicMeshPolygroupAttribute* FDynamicMeshAttributeSet::GetPolygroupLayer(int Index)
{
	return &PolygroupLayers[Index];
}

const FDynamicMeshPolygroupAttribute* FDynamicMeshAttributeSet::GetPolygroupLayer(int Index) const
{
	return &PolygroupLayers[Index];
}


int32 FDynamicMeshAttributeSet::NumWeightLayers() const
{
	return WeightLayers.Num();
}

void FDynamicMeshAttributeSet::SetNumWeightLayers(int32 Num)
{
	if (WeightLayers.Num() == Num)
	{
		return;
	}
	if (Num >= WeightLayers.Num())
	{
		for (int i = (int)WeightLayers.Num(); i < Num; ++i)
		{
			WeightLayers.Add(new FDynamicMeshWeightAttribute(ParentMesh));
		}
	}
	else
	{
		WeightLayers.RemoveAt(Num, WeightLayers.Num() - Num);
	}
	ensure(WeightLayers.Num() == Num);
}

void FDynamicMeshAttributeSet::RemoveWeightLayer(int32 Index)
{
	WeightLayers.RemoveAt(Index);
}

FDynamicMeshWeightAttribute* FDynamicMeshAttributeSet::GetWeightLayer(int Index)
{
	return &WeightLayers[Index];
}

const FDynamicMeshWeightAttribute* FDynamicMeshAttributeSet::GetWeightLayer(int Index) const
{
	return &WeightLayers[Index];
}

void FDynamicMeshAttributeSet::EnableMatchingWeightLayersByNames(const FDynamicMeshAttributeSet* ToMatch, bool bDiscardUnmatched)
{
	if (!ToMatch)
	{
		if (bDiscardUnmatched)
		{
			SetNumWeightLayers(0);
		}
		return;
	}

	for (int32 MatchLayerIdx = 0; MatchLayerIdx < ToMatch->WeightLayers.Num(); ++MatchLayerIdx)
	{
		const FDynamicMeshWeightAttribute& MatchLayer = ToMatch->WeightLayers[MatchLayerIdx];
		// Forward-search for a matching layer, and if found swap it back as needed
		bool bFound = false;
		for (int32 LayerIdx = MatchLayerIdx; LayerIdx < WeightLayers.Num(); ++LayerIdx)
		{
			FDynamicMeshWeightAttribute& Layer = WeightLayers[LayerIdx];
			if (MatchLayer.GetName() == Layer.GetName())
			{
				bFound = true;
				if (LayerIdx > MatchLayerIdx)
				{
					Swap(WeightLayers[LayerIdx], WeightLayers[MatchLayerIdx]);
				}
				break;
			}
		}
		if (!bFound)
		{
			if (MatchLayerIdx >= WeightLayers.Num())
			{
				// Add all remaining reference layers
				SetNumWeightLayers(ToMatch->WeightLayers.Num());
				for (int32 Idx = MatchLayerIdx; Idx < ToMatch->WeightLayers.Num(); ++Idx)
				{
					WeightLayers[Idx].SetName(ToMatch->WeightLayers[Idx].GetName());
				}
				break;
			}
			else
			{
				// Insert the missing layer
				FDynamicMeshWeightAttribute* ToInsert = new FDynamicMeshWeightAttribute(ParentMesh);
				ToInsert->SetName(MatchLayer.GetName());
				WeightLayers.Insert(ToInsert, MatchLayerIdx);
			}
		}
	}
	if (bDiscardUnmatched)
	{
		SetNumWeightLayers(ToMatch->WeightLayers.Num());
	}
}


void FDynamicMeshAttributeSet::EnableMaterialID()
{
	if (HasMaterialID() == false)
	{
		MaterialIDAttrib = MakeUnique<FDynamicMeshMaterialAttribute>(ParentMesh);
		MaterialIDAttrib->Initialize((int32)0);
	}
}

void FDynamicMeshAttributeSet::DisableMaterialID()
{
	MaterialIDAttrib.Reset();
}

void FDynamicMeshAttributeSet::AttachTriangleLabelAttribute(FName InName, FDynamicMeshTriangleLabelAttribute* InAttribute)
{
	RemoveTriangleLabelAttribute(InName);

	// Ensure proper ownership.
	InAttribute->Reparent(ParentMesh);
	InAttribute->SetName(InName);
	TriangleLabelAttributes.Add(InName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>(InAttribute));

	RegisterExternalAttribute(InAttribute);
}

void FDynamicMeshAttributeSet::RemoveTriangleLabelAttribute(FName InName)
{
	if (TriangleLabelAttributes.Contains(InName))
	{
		UnregisterExternalAttribute(TriangleLabelAttributes[InName].Get());
		TriangleLabelAttributes.Remove(InName);
	}
}

void FDynamicMeshAttributeSet::RemoveAllTriangleLabelAttributes()
{
	for (const auto&[Name, AttrUniquePtr]: TriangleLabelAttributes)
	{
		UnregisterExternalAttribute(AttrUniquePtr.Get());
	}
	TriangleLabelAttributes.Reset();
}

bool FDynamicMeshAttributeSet::HasTriangleLabelAttribute(FName InName) const
{
	return TriangleLabelAttributes.Contains(InName);
}

FDynamicMeshTriangleLabelAttribute* FDynamicMeshAttributeSet::FindTriangleLabelAttribute(FName InName) const
{
	const TUniquePtr<FDynamicMeshTriangleLabelAttribute>* Found = TriangleLabelAttributes.Find(InName);
	return Found && Found->IsValid() ? Found->Get() : nullptr;
}
	
const FDynamicMeshAttributeSet::TriangleLabelAttributeMap& FDynamicMeshAttributeSet::GetTriangleLabelAttributes() const
{
	return TriangleLabelAttributes;
}

void FDynamicMeshAttributeSet::AttachSkinWeightsAttribute(FName InProfileName, FDynamicMeshVertexSkinWeightsAttribute* InAttribute)
{
	RemoveSkinWeightsAttribute(InProfileName);

	// Ensure proper ownership.
	static_cast<FDynamicMeshAttributeBase *>(InAttribute)->Reparent(ParentMesh);
	SkinWeightAttributes.Add(InProfileName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>(InAttribute));

	RegisterExternalAttribute(InAttribute);
}

void FDynamicMeshAttributeSet::RemoveSkinWeightsAttribute(FName InProfileName)
{
	if (SkinWeightAttributes.Contains(InProfileName))
	{
		UnregisterExternalAttribute(SkinWeightAttributes[InProfileName].Get());
		SkinWeightAttributes.Remove(InProfileName);
	}
}

void FDynamicMeshAttributeSet::RemoveAllSkinWeightsAttributes()
{
	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& SkinWeightAttribute : SkinWeightAttributes)
	{
		UnregisterExternalAttribute(SkinWeightAttribute.Value.Get());
	}

	SkinWeightAttributes.Reset();	
}

void FDynamicMeshAttributeSet::AttachMorphTargetAttribute(FName InMorphTargetName, FDynamicMeshMorphTargetAttribute* InAttribute)
{
	RemoveMorphTargetAttribute(InMorphTargetName);

	// Ensure proper ownership.
	static_cast<FDynamicMeshAttributeBase *>(InAttribute)->Reparent(ParentMesh);
	MorphTargetAttributes.Add(InMorphTargetName, TUniquePtr<FDynamicMeshMorphTargetAttribute>(InAttribute));

	RegisterExternalAttribute(InAttribute);
}

void FDynamicMeshAttributeSet::RemoveMorphTargetAttribute(FName InMorphTargetName)
{
	if (MorphTargetAttributes.Contains(InMorphTargetName))
	{
		UnregisterExternalAttribute(MorphTargetAttributes[InMorphTargetName].Get());
		MorphTargetAttributes.Remove(InMorphTargetName);
	}
}

void FDynamicMeshAttributeSet::RemoveAllMorphTargetAttributes()
{
	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& MorphTargetAttribute : MorphTargetAttributes)
	{
		UnregisterExternalAttribute(MorphTargetAttribute.Value.Get());
	}

	MorphTargetAttributes.Reset();
}


bool FDynamicMeshAttributeSet::IsSeamEdge(int eid) const
{
	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		if (UVLayer.IsSeamEdge(eid))
		{
			return true;
		}
	}

	for (const FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		if (NormalLayer.IsSeamEdge(eid))
		{
			return true;
		}
	}

	if (ColorLayer && ColorLayer->IsSeamEdge(eid))
	{
		return true;
	}
	
	return false;
}

bool FDynamicMeshAttributeSet::IsSeamEndEdge(int eid) const
{
	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		if (UVLayer.IsSeamEndEdge(eid))
		{
			return true;
		}
	}

	for (const FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		if (NormalLayer.IsSeamEndEdge(eid))
		{
			return true;
		}
	}

	if (ColorLayer && ColorLayer->IsSeamEndEdge(eid))
	{
		return true;
	}
	return false;
}

bool FDynamicMeshAttributeSet::IsSeamEdge(int EdgeID, bool& bIsUVSeamOut, bool& bIsNormalSeamOut, bool& bIsColorSeamOut) const
{
	bool bIsTangentSeam;
	bool IsSeam = IsSeamEdge(EdgeID, bIsUVSeamOut, bIsNormalSeamOut, bIsColorSeamOut, bIsTangentSeam);
	bIsNormalSeamOut = bIsNormalSeamOut || bIsTangentSeam;
	return IsSeam;
}

bool FDynamicMeshAttributeSet::IsSeamEdge(int EdgeID, bool& bIsUVSeamOut, bool& bIsNormalSeamOut, bool& bIsColorSeamOut, bool& bIsTangentSeamOut) const
{
	bIsUVSeamOut = false;
	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		if (UVLayer.IsSeamEdge(EdgeID))
		{
			bIsUVSeamOut = true;
		}
	}

	bIsNormalSeamOut = !NormalLayers.IsEmpty() && NormalLayers[0].IsSeamEdge(EdgeID);
	bIsTangentSeamOut = false;
	for (int32 LayerIdx = 1; LayerIdx < NormalLayers.Num(); ++LayerIdx)
	{
		const FDynamicMeshNormalOverlay& NormalLayer = NormalLayers[LayerIdx];
		if (NormalLayer.IsSeamEdge(EdgeID))
		{
			bIsTangentSeamOut = true;
		}
	}

	bIsColorSeamOut = false;
	if (ColorLayer && ColorLayer->IsSeamEdge(EdgeID))
	{
		bIsColorSeamOut = true;
	}
	return (bIsUVSeamOut || bIsNormalSeamOut || bIsColorSeamOut || bIsTangentSeamOut);
}


bool FDynamicMeshAttributeSet::IsSeamVertex(int VID, bool bBoundaryIsSeam) const
{
	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		if (UVLayer.IsSeamVertex(VID, bBoundaryIsSeam))
		{
			return true;
		}
	}
	for (const FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		if (NormalLayer.IsSeamVertex(VID, bBoundaryIsSeam))
		{
			return true;
		}
	}
	if (ColorLayer && ColorLayer->IsSeamVertex(VID, bBoundaryIsSeam))
	{
		return true;
	}
	return false;
}

bool FDynamicMeshAttributeSet::IsSeamIntersectionVertex(int32 VertexID) const
{
	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		if (UVLayer.IsSeamIntersectionVertex(VertexID))
		{
			return true;
		}
	}
	for (const FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		if (NormalLayer.IsSeamIntersectionVertex(VertexID))
		{
			return true;
		}
	}
	if (ColorLayer && ColorLayer->IsSeamIntersectionVertex(VertexID))
	{
		return true;
	}
	return false;
}

bool FDynamicMeshAttributeSet::IsMaterialBoundaryEdge(int EdgeID) const
{
	if ( MaterialIDAttrib == nullptr )
	{
		return false;
	}
	checkSlow(ParentMesh->IsEdge(EdgeID));
	if (ParentMesh->IsEdge(EdgeID))
	{
		const FDynamicMesh3::FEdge Edge = ParentMesh->GetEdge(EdgeID);
		const int Tri0 = Edge.Tri[0];
		const int Tri1 = Edge.Tri[1];
		if ((Tri0 == IndexConstants::InvalidID) || (Tri1 == IndexConstants::InvalidID))
		{
			return false;
		}
		const int MatID0 = MaterialIDAttrib->GetValue(Tri0);
		const int MatID1 = MaterialIDAttrib->GetValue(Tri1);
		return MatID0 != MatID1;
	}
	return false;
}

void FDynamicMeshAttributeSet::OnNewVertex(int VertexID, bool bInserted)
{
	FDynamicMeshAttributeSetBase::OnNewVertex(VertexID, bInserted);

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		float NewWeight = 0.0f;
		WeightLayer.SetNewValue(VertexID, &NewWeight);
	}

	for (TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : GenericAttributes)
	{
		AttribPair.Value->OnNewVertex(VertexID, bInserted);
	}

	// When adding a new vertex (note: not via a edge split or triangle poke!) want to default layer 0 to use the new position
	// even if we're operating on a different layer, so we don't end up with the base geometry collapsed at the origin
	// (need to handle this here instead of in the layer itself since it depends on the layer index / active layer info)
	if (NumSculptLayers() > 0)
	{
		checkSlow(ParentMesh);
		SculptLayers.Layers[0].SetValue(VertexID, ParentMesh->GetVertex(VertexID));
	}
}


void FDynamicMeshAttributeSet::OnRemoveVertex(int VertexID)
{
	FDynamicMeshAttributeSetBase::OnRemoveVertex(VertexID);

	// no standard attributes need to update on vertex removals, but custom generic attributes may need to

	for (TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : GenericAttributes)
	{
		AttribPair.Value->OnRemoveVertex(VertexID);
	}
}


void FDynamicMeshAttributeSet::OnNewTriangle(int TriangleID, bool bInserted)
{
	FDynamicMeshAttributeSetBase::OnNewTriangle(TriangleID, bInserted);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.InitializeNewTriangle(TriangleID);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.InitializeNewTriangle(TriangleID);
	}
	if (ColorLayer)
	{
		ColorLayer->InitializeNewTriangle(TriangleID);
	}
	if (MaterialIDAttrib)
	{
		int NewValue = 0;
		MaterialIDAttrib->SetNewValue(TriangleID, &NewValue);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		int32 NewGroup = 0;
		PolygroupLayer.SetNewValue(TriangleID, &NewGroup);
	}

	const FName NewLabel = NAME_None;
	for (auto &[Name, TriangleLabelAttr]: TriangleLabelAttributes)
	{
		TriangleLabelAttr->SetNewValue(TriangleID, &NewLabel);
	}
}


void FDynamicMeshAttributeSet::OnRemoveTriangle(int TriangleID)
{
	FDynamicMeshAttributeSetBase::OnRemoveTriangle(TriangleID);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnRemoveTriangle(TriangleID);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnRemoveTriangle(TriangleID);
	}
	if (ColorLayer)
	{
		ColorLayer->OnRemoveTriangle(TriangleID);
	}
	
	for (auto &[Name, TriangleLabelAttr]: TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnRemoveTriangle(TriangleID);
	}
}

void FDynamicMeshAttributeSet::OnReverseTriOrientation(int TriangleID)
{
	FDynamicMeshAttributeSetBase::OnReverseTriOrientation(TriangleID);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnReverseTriOrientation(TriangleID);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnReverseTriOrientation(TriangleID);
	}
	if (ColorLayer)
	{
		ColorLayer->OnReverseTriOrientation(TriangleID);
	}
}

void FDynamicMeshAttributeSet::OnSplitEdge(const FDynamicMesh3::FEdgeSplitInfo& SplitInfo)
{
	FDynamicMeshAttributeSetBase::OnSplitEdge(SplitInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnSplitEdge(SplitInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnSplitEdge(SplitInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnSplitEdge(SplitInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnSplitEdge(SplitInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnSplitEdge(SplitInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnSplitEdge(SplitInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnSplitEdge(SplitInfo);
	}
}

void FDynamicMeshAttributeSet::OnFlipEdge(const FDynamicMesh3::FEdgeFlipInfo & flipInfo)
{
	FDynamicMeshAttributeSetBase::OnFlipEdge(flipInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnFlipEdge(flipInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnFlipEdge(flipInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnFlipEdge(flipInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnFlipEdge(flipInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnFlipEdge(flipInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnFlipEdge(flipInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnFlipEdge(flipInfo);
	}
}


void FDynamicMeshAttributeSet::OnCollapseEdge(const FDynamicMesh3::FEdgeCollapseInfo & collapseInfo)
{
	FDynamicMeshAttributeSetBase::OnCollapseEdge(collapseInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnCollapseEdge(collapseInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnCollapseEdge(collapseInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnCollapseEdge(collapseInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnCollapseEdge(collapseInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnCollapseEdge(collapseInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnCollapseEdge(collapseInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnCollapseEdge(collapseInfo);
	}
}

void FDynamicMeshAttributeSet::OnPokeTriangle(const FDynamicMesh3::FPokeTriangleInfo & pokeInfo)
{
	FDynamicMeshAttributeSetBase::OnPokeTriangle(pokeInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnPokeTriangle(pokeInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnPokeTriangle(pokeInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnPokeTriangle(pokeInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnPokeTriangle(pokeInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnPokeTriangle(pokeInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnPokeTriangle(pokeInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnPokeTriangle(pokeInfo);
	}
}

void FDynamicMeshAttributeSet::OnMergeEdges(const FDynamicMesh3::FMergeEdgesInfo & mergeInfo)
{
	FDynamicMeshAttributeSetBase::OnMergeEdges(mergeInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnMergeEdges(mergeInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnMergeEdges(mergeInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnMergeEdges(mergeInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnMergeEdges(mergeInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnMergeEdges(mergeInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnMergeEdges(mergeInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnMergeEdges(mergeInfo);
	}
}

void FDynamicMeshAttributeSet::OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& MergeInfo)
{
	FDynamicMeshAttributeSetBase::OnMergeVertices(MergeInfo);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnMergeVertices(MergeInfo);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnMergeVertices(MergeInfo);
	}
	if (ColorLayer)
	{
		ColorLayer->OnMergeVertices(MergeInfo);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnMergeVertices(MergeInfo);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnMergeVertices(MergeInfo);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnMergeVertices(MergeInfo);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnMergeVertices(MergeInfo);
	}
}

void FDynamicMeshAttributeSet::OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate)
{
	FDynamicMeshAttributeSetBase::OnSplitVertex(SplitInfo, TrianglesToUpdate);

	for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		UVLayer.OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
	for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		NormalLayer.OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
	if (ColorLayer)
	{
		ColorLayer->OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
	if (MaterialIDAttrib)
	{
		MaterialIDAttrib->OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}

	for (FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		PolygroupLayer.OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}

	for (FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		WeightLayer.OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
	
	for (auto& [Name, TriangleLabelAttr] : TriangleLabelAttributes)
	{
		TriangleLabelAttr->OnSplitVertex(SplitInfo, TrianglesToUpdate);
	}
}

bool FDynamicMeshAttributeSet::IsSameAs(const FDynamicMeshAttributeSet& Other, bool bIgnoreDataLayout) const
{
	if (UVLayers.Num() != Other.UVLayers.Num() ||
		NormalLayers.Num() != Other.NormalLayers.Num() ||
		PolygroupLayers.Num() != Other.PolygroupLayers.Num() ||
		WeightLayers.Num() != Other.WeightLayers.Num() ||
		TriangleLabelAttributes.Num() != Other.TriangleLabelAttributes.Num())
	{
		return false;
	}

	for (int Idx = 0; Idx < UVLayers.Num(); Idx++)
	{
		if (!UVLayers[Idx].IsSameAs(Other.UVLayers[Idx], bIgnoreDataLayout))
		{
			return false;
		}
	}

	for (int Idx = 0; Idx < NormalLayers.Num(); Idx++)
	{
		if (!NormalLayers[Idx].IsSameAs(Other.NormalLayers[Idx], bIgnoreDataLayout))
		{
			return false;
		}
	}

	for (int Idx = 0; Idx < PolygroupLayers.Num(); Idx++)
	{
		if (!PolygroupLayers[Idx].IsSameAs(Other.PolygroupLayers[Idx], bIgnoreDataLayout))
		{
			return false;
		}
	}

	for (int Idx = 0; Idx < WeightLayers.Num(); Idx++)
	{
		if (!WeightLayers[Idx].IsSameAs(Other.WeightLayers[Idx], bIgnoreDataLayout))
		{
			return false;
		}
	}

	if (HasPrimaryColors() != Other.HasPrimaryColors())
	{
		return false;
	}
	if (HasPrimaryColors())
	{
		if (!ColorLayer->IsSameAs(*Other.ColorLayer, bIgnoreDataLayout))
		{
			return false;
		}
	}

	if (HasMaterialID() != Other.HasMaterialID())
	{
		return false;
	}
	if (HasMaterialID())
	{
		if (!MaterialIDAttrib->IsSameAs(*Other.MaterialIDAttrib, bIgnoreDataLayout))
		{
			return false;
		}
	}

	if (!TriangleLabelAttributes.IsEmpty())
	{
		for (const auto&[Name, LabelAttribute]: TriangleLabelAttributes)
		{
			const TUniquePtr<FDynamicMeshTriangleLabelAttribute>* Found = Other.TriangleLabelAttributes.Find(Name);
			const FDynamicMeshTriangleLabelAttribute* OtherLabelAttribute = Found ? Found->Get() : nullptr;
			if (!OtherLabelAttribute)
			{
				return false;
			}
			if (!LabelAttribute->IsSameAs(*OtherLabelAttribute, bIgnoreDataLayout))
			{
				return false;
			}
		}
	}
	
	if (SkinWeightAttributes.Num() != Other.SkinWeightAttributes.Num())
	{
		return false;
	}
	if (!SkinWeightAttributes.IsEmpty())
	{
		auto VertexBoneWeightsAreIdentical = [](const AnimationCore::FBoneWeights& BoneWeights, const AnimationCore::FBoneWeights& BoneWeightsOther) -> bool
		{
			if (BoneWeights.Num() != BoneWeightsOther.Num())
			{
				return false;
			}

			for (int32 Index = 0; Index < BoneWeights.Num(); ++Index)
			{
				// If the weight is the same, the order is nondeterministic. Hence, we need to "manually" look for the same values.
				const int32 IndexOther = BoneWeightsOther.FindWeightIndexByBone(BoneWeights[Index].GetBoneIndex());
				if (IndexOther == INDEX_NONE || BoneWeights[Index].GetRawWeight() != BoneWeightsOther[IndexOther].GetRawWeight())
				{
					return false;
				}
			}

			return true;
		};
		
		SkinWeightAttributesMap::TConstIterator It(SkinWeightAttributes);
		SkinWeightAttributesMap::TConstIterator ItOther(Other.SkinWeightAttributes);
		while (It && ItOther)
		{
			if (It.Key() != ItOther.Key())
			{
				return false;
			}
			const FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = It->Value.Get();
			const FDynamicMeshVertexSkinWeightsAttribute* SkinWeightsOther = ItOther->Value.Get();

			if (!bIgnoreDataLayout)
			{
				if (SkinWeights->VertexBoneWeights.Num() != SkinWeightsOther->VertexBoneWeights.Num())
				{
					return false;
				}

				for (int32 i = 0, Num = SkinWeights->VertexBoneWeights.Num(); i < Num; ++i)
				{
					if (!VertexBoneWeightsAreIdentical(SkinWeights->VertexBoneWeights[i], SkinWeightsOther->VertexBoneWeights[i]))
					{
						return false;
					}
				}
			}
			else
			{
				const FRefCountVector& VertexRefCounts = SkinWeights->Parent->GetVerticesRefCounts();
				const FRefCountVector& VertexRefCountsOther = SkinWeightsOther->Parent->GetVerticesRefCounts();
				
				if (VertexRefCounts.GetCount() != VertexRefCountsOther.GetCount())
				{
					return false;
				}

				FRefCountVector::IndexIterator ItVid = VertexRefCounts.BeginIndices();
				const FRefCountVector::IndexIterator ItVidEnd = VertexRefCounts.EndIndices();
				FRefCountVector::IndexIterator ItVidOther = VertexRefCountsOther.BeginIndices();
				const FRefCountVector::IndexIterator ItVidEndOther = VertexRefCountsOther.EndIndices();

				while (ItVid != ItVidEnd && ItVidOther != ItVidEndOther)
				{
					if (!VertexBoneWeightsAreIdentical(SkinWeights->VertexBoneWeights[*ItVid], SkinWeightsOther->VertexBoneWeights[*ItVidOther]))
					{
						return false;
					}

					++ItVid;
					++ItVidOther;
				}
			}

			++It;
			++ItOther;
		}
	}

	if (MorphTargetAttributes.Num() != Other.MorphTargetAttributes.Num())
	{
		return false;
	}
	
	if (!MorphTargetAttributes.IsEmpty())
	{
		MorphTargetAttributesMap::TConstIterator It(MorphTargetAttributes);
		MorphTargetAttributesMap::TConstIterator ItOther(Other.MorphTargetAttributes);
		while (It && ItOther)
		{
			if (It.Key() != ItOther.Key())
			{
				return false;
			}
			const FDynamicMeshMorphTargetAttribute* MorphTarget = It->Value.Get();
			const FDynamicMeshMorphTargetAttribute* MorphTargetOther = ItOther->Value.Get();

			if (!MorphTarget->IsSameAs(*MorphTargetOther, bIgnoreDataLayout))
			{
				return false;
			}
			
			++It;
			++ItOther;
		}	
	}
	
	
	if (!IsSameBoneAttributesAs(Other))
	{
		return false;
	}
	
	// TODO: Test GenericAttributes

	return true;
}

namespace FDynamicMeshAttributeSet_Local
{
template <typename LayerType>
void SerializeLayers(TIndirectArray<LayerType>& Layers, FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
{
	int32 Num = Layers.Num();
	Ar << Num;
	if (Ar.IsLoading())
	{
		Layers.Empty(Num);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Layers.Add(new typename TIndirectArray<LayerType>::ElementType);
		}
	}
	for (int32 Index = 0; Index < Num; ++Index)
	{
		Layers[Index].Serialize(Ar, CompactMaps, bUseCompression);
	}
}
	
void SerializeLabels(FArchive& Ar, FDynamicMeshTriangleLabelAttribute& Labels)
{
	enum class EIndexSize : uint8 { U8 = 0, U16 = 1, U32 = 2 };
	
	if (Ar.IsLoading())
	{
		// read names count
		uint32 NumNames = 0;
		Ar << NumNames;
		
		constexpr uint32 ArrayMax = static_cast<uint32>(std::numeric_limits<TArray<FName>::SizeType>::max());
		if (!ensure(NumNames <= ArrayMax))
		{
			Ar.SetError();
			return;
		}
		
		if (NumNames == 0)
		{
			return;
		}
		
		// ret unique names
		uint32 NumUniqueNames = 0;
		Ar << NumUniqueNames;
		
		if (!ensure(NumUniqueNames <= ArrayMax))
		{
			Ar.SetError();
			return;
		}
		
		TArray<FName> UniqueNames;
		UniqueNames.Reserve(NumUniqueNames);
		
		for (uint32 Index = 0; Index < NumUniqueNames; ++Index)
		{
			FString NameStr;
			Ar << NameStr;
			UniqueNames.Add(FName(NameStr));
		}

		// single FName so assign the same for all triangles
		// or all names are unique so just assign them 
		if (NumUniqueNames == 1 || NumUniqueNames == NumNames)
		{
			const bool bSingle = NumUniqueNames == 1;	
			for (uint32 Index = 0; Index < NumNames; ++Index)
			{
				Labels.SetValue(Index, bSingle ? UniqueNames[0] : UniqueNames[Index]);
			}
			return;
		}

		// read index size
		EIndexSize IndexSize = EIndexSize::U8;
		Ar << IndexSize;

		// read indices relative to UniqueNames 
		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			uint32 NameIndex = 0;
			if (IndexSize == EIndexSize::U8)
			{
				uint8 v = 0;
				Ar << v;
				NameIndex = v;
			}
			else if (IndexSize == EIndexSize::U16)
			{
				uint16 v = 0;
				Ar << v;
				NameIndex = v;
			}
			else
			{
				Ar << NameIndex;
			}

			if (ensure(UniqueNames.IsValidIndex(NameIndex)))
			{
				Labels.SetValue(Index, UniqueNames[NameIndex]);
			}
		}
	}
	else
	{
		// write names count
		uint32 NumNames = Labels.GetParentMesh()->MaxTriangleID();
		Ar << NumNames;
		
		if (NumNames == 0)
		{
			return;
		}
		
		// get unique names
		TMap<FName, uint32> NameToIndex;
		TArray<FString> UniqueNames;
		TArray<uint32> NameIndices;
		NameIndices.Init(0, NumNames);

		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			const FName Name = Labels.GetValue(Index);
			if (const uint32* Found = NameToIndex.Find(Name))
			{
				NameIndices[Index] = *Found;
			}
			else
			{
				NameIndices[Index] = UniqueNames.Num();
				NameToIndex.Add(Name, UniqueNames.Num());
				UniqueNames.Emplace(Name.ToString());
			}
		}
		
		// write unique count and names
		uint32 NumUniqueNames = UniqueNames.Num();
		Ar << NumUniqueNames;
		for (uint32 Index = 0; Index < NumUniqueNames; ++Index)
		{
			Ar << UniqueNames[Index];
		}
		
		// single FName or all names are unique so nothing else to do
		if (NumUniqueNames == 1 || NumUniqueNames == NumNames)
		{
			return;
		}

		// choose a convenient index size and write it
		EIndexSize IndexSize = (NumUniqueNames <= MAX_uint8) ? EIndexSize::U8 : (NumUniqueNames <= MAX_uint16) ? EIndexSize::U16 : EIndexSize::U32;
		Ar << IndexSize;

		// write indices relative to UniqueNames 
		for (uint32 Index = 0; Index < NumNames; ++Index)
		{
			if (IndexSize == EIndexSize::U8)
			{
				uint8 v = static_cast<uint8>(NameIndices[Index]);
				Ar << v;
			}
			else if (IndexSize == EIndexSize::U16)
			{
				uint16 v = static_cast<uint16>(NameIndices[Index]);
				Ar << v;
			}
			else
			{
				Ar << NameIndices[Index];
			}
		}
	}
}

} // namespace FDynamicMeshAttributeSet_Local

void FDynamicMeshAttributeSet::Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
{
	using namespace FDynamicMeshAttributeSet_Local;
	
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	const bool bUseLegacySerialization = Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization;

	if (bUseLegacySerialization)
	{
		// Use serialization in TIndirectArray.
		Ar << UVLayers;
		Ar << NormalLayers;
		Ar << PolygroupLayers;
	}
	else
	{
		Ar << bUseCompression;

		// We do our own serialization since using the serialization in TIndirectArray will not allow us to do compacting/compression.
		SerializeLayers(UVLayers, Ar, CompactMaps, bUseCompression);
		SerializeLayers(NormalLayers, Ar, CompactMaps, bUseCompression);
		SerializeLayers(PolygroupLayers, Ar, CompactMaps, bUseCompression);

		const bool bSerializeWeightLayers = !Ar.IsLoading() || Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::DynamicMeshAttributesWeightMapsAndNames;
		if (bSerializeWeightLayers)
		{
			SerializeLayers(WeightLayers, Ar, CompactMaps, bUseCompression);
		}
	}

	if (Ar.IsLoading())
	{
		// Manually populate ParentMesh since deserialization of the individual
		// layers cannot populate the pointer.
		for (FDynamicMeshUVOverlay& Overlay : UVLayers)
		{
			Overlay.ParentMesh = ParentMesh;
		}
		for (FDynamicMeshNormalOverlay& Overlay : NormalLayers)
		{
			Overlay.ParentMesh = ParentMesh;
		}
		for (FDynamicMeshPolygroupAttribute& Attr : PolygroupLayers)
		{
			Attr.ParentMesh = ParentMesh;
		}
		for (FDynamicMeshWeightAttribute& Attr : WeightLayers)
		{
			Attr.Parent = ParentMesh;
		}
	}

	// Use int32 here to future-proof for multiple color layers.
	int32 NumColorLayers = HasPrimaryColors() ? 1 : 0;
	Ar << NumColorLayers;
	if (Ar.IsLoading())
	{
		NumColorLayers > 0 ? EnablePrimaryColors() : DisablePrimaryColors();
	}
	if (NumColorLayers > 0)
	{
		ColorLayer->Serialize(Ar, CompactMaps, bUseCompression);
	}

	bool bHasMaterialID = HasMaterialID();
	Ar << bHasMaterialID;
	if (Ar.IsLoading())
	{
		bHasMaterialID ? EnableMaterialID() : DisableMaterialID();
	}
	if (bHasMaterialID)
	{
		MaterialIDAttrib->Serialize(Ar, CompactMaps, bUseCompression);
	}

	if (!bUseLegacySerialization)
	{
		int32 NumSkinWeightAttributes = SkinWeightAttributes.Num();
		Ar << NumSkinWeightAttributes;

		if (Ar.IsLoading())
		{
			SkinWeightAttributes.Reset();

			for (int32 i = 0; i < NumSkinWeightAttributes; ++i)
			{
				FName Key;
				FNameAsStringProxyArchive ProxyArchive(Ar);
				ProxyArchive << Key;

				TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>& Value = SkinWeightAttributes.Emplace(Key, nullptr);
				bool IsValid;
				Ar << IsValid;
				if (IsValid)
				{
					Value.Reset(new FDynamicMeshVertexSkinWeightsAttribute(ParentMesh, false));
					Value.Get()->Serialize(Ar, CompactMaps, bUseCompression);
					RegisterExternalAttribute(Value.Get());
				}
			}
		}
		else
		{
			for (SkinWeightAttributesMap::TIterator It(SkinWeightAttributes); It; ++It)
			{
				FNameAsStringProxyArchive ProxyArchive(Ar);
				ProxyArchive << It.Key();

				const TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>& Value = It.Value();
				bool bIsValid = Value.IsValid();
				Ar << bIsValid;
				if (bIsValid)
				{
					Value.Get()->Serialize(Ar, CompactMaps, bUseCompression);
				}
			}
		}
	}

	const bool bSerializeMorphTargets = !Ar.IsLoading() || Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DynamicMeshAttributesMorphTargets;
	if (bSerializeMorphTargets)
	{
		int32 NumMorphTargetAttributes = MorphTargetAttributes.Num();
		Ar << NumMorphTargetAttributes;

		if (Ar.IsLoading())
		{
			MorphTargetAttributes.Reset();

			for (int32 i = 0; i < NumMorphTargetAttributes; ++i)
			{
				FName Key;
				FNameAsStringProxyArchive ProxyArchive(Ar);
				ProxyArchive << Key;

				TUniquePtr<FDynamicMeshMorphTargetAttribute>& Value = MorphTargetAttributes.Emplace(Key, nullptr);
				bool IsValid;
				Ar << IsValid;
				if (IsValid)
				{
					Value.Reset(new FDynamicMeshMorphTargetAttribute(ParentMesh, false));
					Value.Get()->Serialize(Ar, CompactMaps, bUseCompression);
					RegisterExternalAttribute(Value.Get());
				}
			}
		}
		else
		{
			for (MorphTargetAttributesMap::TIterator It(MorphTargetAttributes); It; ++It)
			{
				FNameAsStringProxyArchive ProxyArchive(Ar);
				ProxyArchive << It.Key();

				const TUniquePtr<FDynamicMeshMorphTargetAttribute>& Value = It.Value();
				bool bIsValid = Value.IsValid();
				Ar << bIsValid;
				if (bIsValid)
				{
					Value.Get()->Serialize(Ar, CompactMaps, bUseCompression);
				}
			}
		}	
	}
	
	
	const bool bSerializeBones = !Ar.IsLoading() || Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::DynamicMeshAttributesSerializeBones;
	if (bSerializeBones)
	{
		bool bHasBones = BoneNameAttrib && BoneParentIndexAttrib && BonePoseAttrib && BoneColorAttrib;
		Ar << bHasBones;

		if (Ar.IsLoading())
		{
			bHasBones ? EnableBones(0) : DisableBones();
		}
		if (bHasBones)
		{
			FNameAsStringProxyArchive ProxyArchive(Ar); BoneNameAttrib->Serialize(ProxyArchive);
			BoneParentIndexAttrib->Serialize(Ar);
			BonePoseAttrib->Serialize(Ar);
			BoneColorAttrib->Serialize(Ar);
		}
	}

	const bool bSerializeSculptLayers = !Ar.IsLoading() || Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DynamicMeshSerializeSculptLayers;
	if (bSerializeSculptLayers)
	{
		int32 NumLayers = NumSculptLayers();
		Ar << NumLayers;

		if (Ar.IsLoading())
		{
			DiscardSculptLayers();
			if (NumLayers > 0)
			{
				EnableSculptLayers(NumLayers);
			}
		}
		if (NumLayers > 0)
		{
			Ar << SculptLayers.ActiveLayer;
			Ar << SculptLayers.LayerWeights;
			for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
			{
				SculptLayers.Layers[LayerIdx].Serialize(Ar, CompactMaps, bUseCompression);
			}
		}
	}
	
	const bool bSerializeTriangleLabels = !Ar.IsLoading() ||
			Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DynamicMeshAttributesTriangleLabels;
	if (bSerializeTriangleLabels)
	{		
		int32 NumTriangleLabelAttributes = TriangleLabelAttributes.Num();
		Ar << NumTriangleLabelAttributes;
		
		FNameAsStringProxyArchive ProxyArchive(Ar);
		
		if (Ar.IsLoading())
		{
			RemoveAllTriangleLabelAttributes();

			for (int32 Index = 0; Index < NumTriangleLabelAttributes; ++Index)
			{
				FName Key;
				ProxyArchive << Key;

				bool IsValid;
				Ar << IsValid;

				if (IsValid)
				{
					TUniquePtr<FDynamicMeshTriangleLabelAttribute>& Value =
						TriangleLabelAttributes.Emplace(Key, new FDynamicMeshTriangleLabelAttribute(ParentMesh));
					SerializeLabels(Ar, *Value.Get());
					Value->SetName(Key);
					RegisterExternalAttribute(Value.Get());
				}
			}
		}
		else
		{
			for (TriangleLabelAttributeMap::TIterator It(TriangleLabelAttributes); It; ++It)
			{
				ProxyArchive << It.Key();

				const TUniquePtr<FDynamicMeshTriangleLabelAttribute>& Value = It.Value();
				bool bIsValid = Value.IsValid();
				Ar << bIsValid;
				if (bIsValid)
				{
					SerializeLabels(Ar, *Value.Get());
				}
			}
		}	
	}
	
	//Ar << GenericAttributes; // TODO
}

SIZE_T FDynamicMeshAttributeSet::GetByteCount() const
{
	SIZE_T ByteCount = 0;

	for (const FDynamicMeshUVOverlay& UVLayer : UVLayers)
	{
		ByteCount += UVLayer.GetByteCount();
	}

	for (const FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
	{
		ByteCount += NormalLayer.GetByteCount();
	}

	if (ColorLayer)
	{
		ByteCount += ColorLayer->GetByteCount();
	}

	if (MaterialIDAttrib)
	{
		ByteCount += MaterialIDAttrib->GetByteCount();
	}

	for (const FDynamicMeshPolygroupAttribute& PolygroupLayer : PolygroupLayers)
	{
		ByteCount += PolygroupLayer.GetByteCount();
	}

	for (const FDynamicMeshWeightAttribute& WeightLayer : WeightLayers)
	{
		ByteCount += WeightLayer.GetByteCount();
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& TriangleLabelAttribute: TriangleLabelAttributes)
	{
		ByteCount += TriangleLabelAttribute.Value->GetByteCount();
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& SkinWeightAttribute : SkinWeightAttributes)
	{
		ByteCount += SkinWeightAttribute.Value->GetByteCount();
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& MorphTargetAttribute : MorphTargetAttributes)
	{
		ByteCount += MorphTargetAttribute.Value->GetByteCount();
	}

	if (BoneNameAttrib)
	{
		ByteCount += BoneNameAttrib->GetByteCount();
	}
	if (BoneParentIndexAttrib)
	{
		ByteCount += BoneParentIndexAttrib->GetByteCount();
	}

	if (BoneColorAttrib)
	{
		ByteCount += BoneColorAttrib->GetByteCount();
	}
	
	if (BonePoseAttrib)
	{
		ByteCount += BonePoseAttrib->GetByteCount();
	}
	
	if (BoneColorAttrib)
	{
		ByteCount += BoneColorAttrib->GetByteCount();
	}

	for (int SculptLayerIndex = 0; SculptLayerIndex < NumSculptLayers(); ++SculptLayerIndex)
	{
		ByteCount += SculptLayers.Layers[SculptLayerIndex].GetByteCount();
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& GenericAttribute : GenericAttributes)
	{
		ByteCount += GenericAttribute.Value->GetByteCount();
	}

	return ByteCount;
}


void FDynamicMeshAttributeSet::EnableSculptLayers(int32 MinLayers)
{
	if (!ensure(ParentMesh))
	{
		return;
	}

	SculptLayers.Enable(this, MinLayers);
}

void FDynamicMeshAttributeSet::DiscardSculptLayers()
{
	SculptLayers.Discard(this);
}


bool FDynamicMeshAttributeSet::CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const
{
	bool bValid = FDynamicMeshAttributeSetBase::CheckValidity(bAllowNonmanifold, FailMode);
	for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers(); UVLayerIndex++)
	{
		bValid = GetUVLayer(UVLayerIndex)->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	}
	bValid = PrimaryNormals()->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	if (ColorLayer)
	{
		bValid = ColorLayer->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	}
	if (MaterialIDAttrib)
	{
		bValid = MaterialIDAttrib->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	}
	for (int PolygroupLayerIndex = 0; PolygroupLayerIndex < NumPolygroupLayers(); PolygroupLayerIndex++)
	{
		bValid = GetPolygroupLayer(PolygroupLayerIndex)->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	}
	for (int WeightLayerIndex = 0; WeightLayerIndex < NumWeightLayers(); WeightLayerIndex++)
	{
		bValid = GetWeightLayer(WeightLayerIndex)->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& Pair : TriangleLabelAttributes)
	{
		if (Pair.Value)
		{
			bValid = Pair.Value->CheckValidity(bAllowNonmanifold, FailMode) && bValid;

			// If non-null, triangle label attributes are in the RegisteredAttributes array
			int32 FoundIndex;
			const bool bFound = RegisteredAttributes.Find(Pair.Value.Get(), FoundIndex);
			bValid = bFound && bValid;
		}
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& Pair : SkinWeightAttributes)
	{
		if (Pair.Value)
		{
			bValid = Pair.Value->CheckValidity(bAllowNonmanifold, FailMode) && bValid;

			// If non-null, Skin Weight Attributes are in the RegisteredAttributes array
			int32 FoundIndex;
			const bool bFound = RegisteredAttributes.Find(Pair.Value.Get(), FoundIndex);
			bValid = bFound && bValid;
		}
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& Pair : MorphTargetAttributes)
	{
		if (Pair.Value)
		{
			bValid = Pair.Value->CheckValidity(bAllowNonmanifold, FailMode) && bValid;

			// If non-null, Morph Target Attributes are in the RegisteredAttributes array
			int32 FoundIndex;
			const bool bFound = RegisteredAttributes.Find(Pair.Value.Get(), FoundIndex);
			bValid = bFound && bValid;
		}
	}

	bValid = CheckBoneValidity(FailMode) && bValid;

	bValid = SculptLayers.CheckValidity(this, bAllowNonmanifold, FailMode) && bValid;

	return bValid;
}


//
// Bone Attributes Methods
//

namespace BoneAttributeHelpers
{	
	template <typename ParentType, typename AttribValueType>
 	void EnableIfMatching(FDynamicMesh3* Mesh,
						  TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>& Attribute,
						  const TDynamicBoneAttributeBase<ParentType, AttribValueType>* ToMatch,
						  const AttribValueType& InitialValue,
						  bool bClearExisting, 
						  bool bDiscardExtraAttributes)
	{
		const bool bToMatchIsNotNull = ToMatch != nullptr;
		const bool bWantAttrib = (bClearExisting || bDiscardExtraAttributes) ? bToMatchIsNotNull : (bToMatchIsNotNull || Attribute.Get());
		if (bClearExisting || bWantAttrib == false)
		{
			Attribute.Reset();
		}
		if (bWantAttrib)
		{	
			const int32 NumBones = ToMatch ? ToMatch->Num() : 0;
			if (!Attribute)
			{
				TDynamicBoneAttributeBase<ParentType, AttribValueType>* Ptr = new TDynamicBoneAttributeBase<ParentType, AttribValueType>(Mesh);
				Attribute = TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>(Ptr);
				Attribute->Initialize(NumBones, InitialValue);
			}
		}
	}

	template <typename ParentType, typename AttribValueType>
 	void CopyBoneAttribute(FDynamicMesh3* Mesh,
						   TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>& Attribute,
						   const TDynamicBoneAttributeBase<ParentType, AttribValueType>* Copy)
	{
		if (Copy)
		{
			TDynamicBoneAttributeBase<ParentType, AttribValueType>* Ptr = new TDynamicBoneAttributeBase<ParentType, AttribValueType>(Mesh);
			Attribute = TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>(Ptr);
			Attribute->Copy(*(Copy));
		}
		else
		{
			Attribute.Reset();
		}
	}

	template <typename ParentType, typename AttribValueType>
 	void SparseCopyBoneAttribute(
 		FDynamicMesh3* Mesh,
		TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>& Attribute,
		const TDynamicBoneAttributeBase<ParentType, AttribValueType>* Copy,
		TConstArrayView<int32> InIndicesToCopy
		)
	{
		if (Copy && !InIndicesToCopy.IsEmpty())
		{
			TDynamicBoneAttributeBase<ParentType, AttribValueType>* Ptr = new TDynamicBoneAttributeBase<ParentType, AttribValueType>(Mesh);
			Attribute = TUniquePtr<TDynamicBoneAttributeBase<ParentType, AttribValueType>>(Ptr);

			Attribute->Resize(InIndicesToCopy.Num());
			int32 TargetIndex = 0;
			for (int32 SourceIndex: InIndicesToCopy)
			{
				Attribute->SetValue(TargetIndex++, Copy->GetValue(SourceIndex));
			}
		}
		else
		{
			Attribute.Reset();
		}
	}
}

int32 FDynamicMeshAttributeSet::GetNumBones() const
{
	return HasBones() ? GetBoneNames()->Num() : 0;
}

void FDynamicMeshAttributeSet::CopyBoneAttributes(const FDynamicMeshAttributeSet& Copy)
{
	BoneAttributeHelpers::CopyBoneAttribute(ParentMesh, BoneNameAttrib, Copy.GetBoneNames());
	BoneAttributeHelpers::CopyBoneAttribute(ParentMesh, BoneParentIndexAttrib, Copy.GetBoneParentIndices());
	BoneAttributeHelpers::CopyBoneAttribute(ParentMesh, BonePoseAttrib, Copy.GetBonePoses());
	BoneAttributeHelpers::CopyBoneAttribute(ParentMesh, BoneColorAttrib, Copy.GetBoneColors());
}

void FDynamicMeshAttributeSet::CopyBoneAttributesWithRemapping(const FDynamicMeshAttributeSet& Copy, const TMap<FName, FName>& BoneHierarchy)
{
	// Create an index array to mark which attribute values to copy. If there's no bone name attribute, we copy nothing.
	TArray<int32> IndicesToCopy;
	TMap<FName, int32> NameToIndexMap;
	
	if (Copy.GetBoneNames())
	{
		const TArray<FName>& SourceBoneNames = Copy.GetBoneNames()->GetAttribValues();
		IndicesToCopy.Reserve(BoneHierarchy.Num());
		int32 SourceIndex = 0;
		for (FName BoneName: SourceBoneNames)
		{
			if (!BoneName.IsNone() && BoneHierarchy.Contains(BoneName))
			{
				NameToIndexMap.Add(BoneName, IndicesToCopy.Num());
				IndicesToCopy.Add(SourceIndex);
			}
			SourceIndex++;
		}
	}

	BoneAttributeHelpers::SparseCopyBoneAttribute(ParentMesh, BoneNameAttrib, Copy.GetBoneNames(), IndicesToCopy);
	BoneAttributeHelpers::SparseCopyBoneAttribute(ParentMesh, BoneParentIndexAttrib, Copy.GetBoneParentIndices(), IndicesToCopy);
	BoneAttributeHelpers::SparseCopyBoneAttribute(ParentMesh, BonePoseAttrib, Copy.GetBonePoses(), IndicesToCopy);
	BoneAttributeHelpers::SparseCopyBoneAttribute(ParentMesh, BoneColorAttrib, Copy.GetBoneColors(), IndicesToCopy);

	// If we copied parent indices, and we have bone names on the source, remap the parent bone index list to match.
	// If the parent name is NAME_None or does not exist in our bone list, then we map that index to INDEX_NONE.
	if (Copy.GetBoneNames() && BoneParentIndexAttrib)
	{
		const TArray<FName>& SourceBoneNames = Copy.GetBoneNames()->GetAttribValues();
		for (int32& ParentIndex: BoneParentIndexAttrib->AttribValues)
		{
			const FName ParentName = SourceBoneNames.IsValidIndex(ParentIndex) ? SourceBoneNames[ParentIndex] : NAME_None;

			if (const int32* BoneIndex = NameToIndexMap.Find(ParentName))
			{
				ParentIndex = *BoneIndex;
			}
			else
			{
				ParentIndex = INDEX_NONE;
			}
		}
	}
}

void FDynamicMeshAttributeSet::EnableMatchingBoneAttributes(const FDynamicMeshAttributeSet& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes)
{
	BoneAttributeHelpers::EnableIfMatching(ParentMesh, BoneNameAttrib, ToMatch.GetBoneNames(), (FName)NAME_None, bClearExisting, bDiscardExtraAttributes);
	BoneAttributeHelpers::EnableIfMatching(ParentMesh, BoneParentIndexAttrib, ToMatch.GetBoneParentIndices(), (int32)INDEX_NONE, bClearExisting, bDiscardExtraAttributes);
	BoneAttributeHelpers::EnableIfMatching(ParentMesh, BonePoseAttrib, ToMatch.GetBonePoses(), FTransform::Identity, bClearExisting, bDiscardExtraAttributes);
	BoneAttributeHelpers::EnableIfMatching(ParentMesh, BoneColorAttrib, ToMatch.GetBoneColors(), FVector4f::One(), bClearExisting, bDiscardExtraAttributes);
}

void FDynamicMeshAttributeSet::EnableBones(const int InBonesNum)
{
	if (HasBones() == false || GetNumBones() != InBonesNum)
	{
		BoneNameAttrib = MakeUnique<FDynamicMeshBoneNameAttribute>(ParentMesh, InBonesNum, NAME_None);
		BoneParentIndexAttrib = MakeUnique<FDynamicMeshBoneParentIndexAttribute>(ParentMesh, InBonesNum, INDEX_NONE);
		BonePoseAttrib = MakeUnique<FDynamicMeshBonePoseAttribute>(ParentMesh, InBonesNum, FTransform::Identity);
		BoneColorAttrib = MakeUnique<FDynamicMeshBoneColorAttribute>(ParentMesh, InBonesNum, FVector4f::One());
	}
}

void FDynamicMeshAttributeSet::DisableBones()
{
	BoneNameAttrib.Reset();
	BoneParentIndexAttrib.Reset();
	BonePoseAttrib.Reset();
	BoneColorAttrib.Reset();
}

bool FDynamicMeshAttributeSet::IsSameBoneAttributesAs(const FDynamicMeshAttributeSet& Other) const
{
	if (HasBones() != Other.HasBones())
	{
		return false;
	}
	
	if (HasBones())
	{
		if (!BoneNameAttrib->IsSameAs(*Other.BoneNameAttrib))
		{
			return false;
		}

		if (!BoneParentIndexAttrib->IsSameAs(*Other.BoneParentIndexAttrib))
		{
			return false;
		}
	}

	return true;
}

bool FDynamicMeshAttributeSet::AppendBonesUnique(const FDynamicMeshAttributeSet& Other)
{
	if (!Other.CheckBoneValidity(EValidityCheckFailMode::ReturnOnly))
	{
		checkSlow(false);
		return false; // don't append from an invalid bone data
	}

	if (!Other.HasBones())
	{
		return true;
	}

	const FDynamicMeshBoneNameAttribute* OtherBoneNameAttrib = Other.GetBoneNames();

	if (!HasBones())
	{
		EnableBones(0);
	}

	// Used to check if bone with a given name in Other already exists
	TSet<FName> HashSet;
	HashSet.Append(BoneNameAttrib->GetAttribValues());

	for (int BoneIdx = 0; BoneIdx < OtherBoneNameAttrib->Num(); ++BoneIdx)
	{
		if (HashSet.Contains(OtherBoneNameAttrib->GetValue(BoneIdx)) == false)
		{
			BoneNameAttrib->Append(OtherBoneNameAttrib->GetValue(BoneIdx));
			BoneParentIndexAttrib->Append(Other.GetBoneParentIndices()->GetValue(BoneIdx));
			BonePoseAttrib->Append(Other.GetBonePoses()->GetValue(BoneIdx));
			BoneColorAttrib->Append(Other.GetBoneColors()->GetValue(BoneIdx));
		}
	}

	return true;
}

bool FDynamicMeshAttributeSet::CheckBoneValidity(EValidityCheckFailMode FailMode) const
{
	bool bValid = true;
	
	if (!HasBones())
	{
		// if boneless, no bone-related attributes should be set
		bValid = !BoneNameAttrib && !BoneParentIndexAttrib && !BoneColorAttrib && !BonePoseAttrib;
	}
	else
	{
		const int32 NumBones = GetNumBones();
	
		bValid = (BoneParentIndexAttrib->Num() == NumBones || BoneParentIndexAttrib->IsEmpty()) && bValid;
		bValid = (BoneColorAttrib->Num() == NumBones || BoneColorAttrib->IsEmpty()) && bValid;
		bValid = (BonePoseAttrib->Num() == NumBones || BonePoseAttrib->IsEmpty()) && bValid;
	}

	if (FailMode == EValidityCheckFailMode::Check)
	{
		checkf(bValid, TEXT("FDynamicMeshAttributeSet::CheckBoneValidity failed!"));
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{ 
		ensureMsgf(bValid, TEXT("FDynamicMeshAttributeSet::CheckBoneValidity failed!"));
	}

	return bValid;
}
