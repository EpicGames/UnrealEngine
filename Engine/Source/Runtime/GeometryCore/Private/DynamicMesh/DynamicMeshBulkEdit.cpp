// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMeshBulkEdit.h"

#include <fstream>

namespace UE 
{
namespace Geometry
{

	FDynamicMeshBulkEdit::FDynamicMeshBulkEdit(FDynamicMesh3& EditMesh, bool bInBuildEdgeLists)
		: Mesh(EditMesh)
		, bBuildEdgeLists(bInBuildEdgeLists)
	{
	}

	FDynamicMeshBulkEdit::~FDynamicMeshBulkEdit()
	{
		Finalize();
	}

	void FDynamicMeshBulkEdit::Initialize(int VertexCount, int TriangleCount, int EdgeCount)
	{
		Mesh.Clear();
		check(Mesh.VertexEdgeLists.Size() == 0);

		Mesh.Vertices.Resize(VertexCount);
		Mesh.VertexRefCounts.InitDense(VertexCount);

		Mesh.Triangles.Resize(TriangleCount);
		Mesh.TriangleRefCounts.InitDense(TriangleCount);
		Mesh.TriangleEdges.Resize(TriangleCount);

		Mesh.Edges.Resize(EdgeCount);
		Mesh.EdgeRefCounts.InitDense(EdgeCount);
	}

	void FDynamicMeshBulkEdit::Finalize()
	{
		if (Mesh.Edges.IsEmpty())
		{
			BuildTopology();
		}

		if (bBuildEdgeLists && Mesh.VertexEdgeLists.Size() == 0)
		{
			BuildEdgeLists();
		}
	}

	void FDynamicMeshBulkEdit::BuildTopology()
	{
		check(Mesh.Edges.IsEmpty());
		check(Mesh.EdgeRefCounts.IsEmpty());

		struct FSortEdge
		{
			int32 HalfEdgeIndex;
			int32 VtxA, VtxB;

			FORCEINLINE bool operator<(const FSortEdge& other) const
			{
				return VtxA < other.VtxA || (VtxA == other.VtxA && VtxB < other.VtxB);
			}
		};

		TArray<FSortEdge> SortedHalfEdges;

		const int32 NumHalfEdges = Mesh.TriangleCount() * 3;
		SortedHalfEdges.Reserve(NumHalfEdges);
	
		for (int32 i = 0; i < NumHalfEdges; ++i)
		{
			const int32 V0 = Mesh.Triangles[i/3][ i   %3];
			const int32 V1 = Mesh.Triangles[i/3][(i+1)%3];
			SortedHalfEdges.Emplace( i, V0 < V1 ? V0 : V1, V0 < V1 ? V1 : V0 );
		}
		SortedHalfEdges.Sort();
	
		// linear walk to count number of edges
		int32 NumEdges = 0;
		for (int32 HID=0; HID < NumHalfEdges; ++HID)
		{
			NumEdges++;
			if (HID<NumHalfEdges-1 && 
				SortedHalfEdges[HID].VtxA == SortedHalfEdges[HID+1].VtxA &&
				SortedHalfEdges[HID].VtxB == SortedHalfEdges[HID+1].VtxB)
			{
				++HID;
			}
		}
	
		Mesh.Edges.Resize(NumEdges);
		Mesh.EdgeRefCounts.InitDense(NumEdges);

		// linear walk to assign edges and halfedges
		int32 EdgeIdx = 0;
		for (int32 HID=0; HID<NumHalfEdges; ++HID)
		{
			if (HID<NumHalfEdges-1 && 
				SortedHalfEdges[HID].VtxA == SortedHalfEdges[HID+1].VtxA && 
				SortedHalfEdges[HID].VtxB == SortedHalfEdges[HID+1].VtxB)
			{
				const int32 HalfEdgeIndex0 = SortedHalfEdges[HID  ].HalfEdgeIndex;
				const int32 HalfEdgeIndex1 = SortedHalfEdges[HID+1].HalfEdgeIndex;

				FDynamicMesh3::FEdge Edge;
				Edge.Vert[0] = Mesh.Triangles[HalfEdgeIndex0 / 3][HalfEdgeIndex0 % 3];
				Edge.Vert[1] = Mesh.Triangles[HalfEdgeIndex1 / 3][HalfEdgeIndex1 % 3];
				check(Edge.Vert[1] == Mesh.Triangles[HalfEdgeIndex0 / 3][ (HalfEdgeIndex0+1) % 3]);

				Edge.Tri[0] = HalfEdgeIndex0 / 3;
				Edge.Tri[1] = HalfEdgeIndex1 / 3;

				Mesh.Edges[EdgeIdx] = Edge;
				Mesh.TriangleEdges[Edge.Tri[0]][ HalfEdgeIndex0 % 3 ] = EdgeIdx;
				Mesh.TriangleEdges[Edge.Tri[1]][ HalfEdgeIndex1 % 3 ] = EdgeIdx;

				// we found a pair, skip one more time
				++HID;
			}
			else
			{
				const int32 HalfEdgeIndex = SortedHalfEdges[HID].HalfEdgeIndex;
			
				FDynamicMesh3::FEdge Edge;
				Edge.Tri[0] = HalfEdgeIndex / 3;
				Edge.Tri[1] = IndexConstants::InvalidID;

				Edge.Vert[0] = Mesh.Triangles[HalfEdgeIndex / 3][HalfEdgeIndex % 3];
				Edge.Vert[1] = Mesh.Triangles[HalfEdgeIndex / 3][(HalfEdgeIndex+1) % 3];

				Mesh.Edges[EdgeIdx] = Edge;
				Mesh.TriangleEdges[Edge.Tri[0]][HalfEdgeIndex % 3] = EdgeIdx;
			}
		
			EdgeIdx++;
		}
	}

	void FDynamicMeshBulkEdit::BuildEdgeLists()
	{
		// populate edge lists
		Mesh.VertexEdgeLists.ResizeAndAllocateBlocks(Mesh.VertexCount());
		for (typename TArray<FVector3d>::SizeType EdgeIdx=0; EdgeIdx<Mesh.Edges.Num(); ++EdgeIdx)
		{
			Mesh.VertexEdgeLists.Insert(Mesh.Edges[EdgeIdx].Vert[0], EdgeIdx);
			Mesh.VertexEdgeLists.Insert(Mesh.Edges[EdgeIdx].Vert[1], EdgeIdx);
		}
	}

} // namespace Geometry
} // namespace UE
