// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMeshChangeTracker.h"

using namespace UE::Geometry;

//
//
// TDynamicMeshAttributeChange - stores change in attribute overlay
// 
//



template<typename RealType,int ElementSize>
void TDynamicMeshAttributeChange<RealType,ElementSize>::SaveInitialElement(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int ElementID)
{
	FChangeElement Element;
	Element.ElementID = ElementID;
	Element.ParentVertexID = Overlay->GetParentVertex(ElementID);
	Element.DataIndex = OldElementData.Num();
	TStaticArray<RealType,ElementSize> ElemData;
	Overlay->GetElement(ElementID, &ElemData[0]);
	for (int j = 0; j < ElementSize; ++j)
	{
		OldElementData.Add(ElemData[j]);
	}
	OldElements.Add(Element);		// @todo emplace
}

template<typename RealType,int ElementSize>
void TDynamicMeshAttributeChange<RealType,ElementSize>::SaveInitialTriangle(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int TriangleID)
{
	FChangeTriangle Triangle;
	Triangle.TriangleID = TriangleID;
	Triangle.Elements = Overlay->GetTriangle(TriangleID);
	OldTriangles.Add(Triangle);
}

template<typename RealType,int ElementSize>
void TDynamicMeshAttributeChange<RealType,ElementSize>::StoreFinalElement(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int ElementID)
{
	FChangeElement Element;
	Element.ElementID = ElementID;
	Element.ParentVertexID = Overlay->GetParentVertex(ElementID);
	Element.DataIndex = NewElementData.Num();
	TStaticArray<RealType, ElementSize> ElemData;
	Overlay->GetElement(ElementID, &ElemData[0]);
	for (int j = 0; j < ElementSize; ++j)
	{
		NewElementData.Add(ElemData[j]);
	}
	NewElements.Add(Element);		// @todo emplace
}

template<typename RealType,int ElementSize>
void TDynamicMeshAttributeChange<RealType,ElementSize>::StoreFinalTriangle(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int TriangleID)
{
	FChangeTriangle Triangle;
	Triangle.TriangleID = TriangleID;
	Triangle.Elements = Overlay->GetTriangle(TriangleID);
	NewTriangles.Add(Triangle);
}

template<typename RealType,int ElementSize>
bool TDynamicMeshAttributeChange<RealType,ElementSize>::Apply(TDynamicMeshOverlay<RealType,ElementSize>* Overlay, bool bRevert) const
{
	if (bRevert)
	{
		ApplyReplaceChange(Overlay, NewTriangles, OldElements, OldElementData, OldTriangles);
	}
	else
	{
		ApplyReplaceChange(Overlay, OldTriangles, NewElements, NewElementData, NewTriangles);
	}

	return true;
}


template<typename RealType,int ElementSize>
void TDynamicMeshAttributeChange<RealType,ElementSize>::ApplyReplaceChange(TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
	const TArray<FChangeTriangle>& RemoveTris,
	const TArray<FChangeElement>& InsertElements,
	const TArray<RealType>& InsertElementData,
	const TArray<FChangeTriangle>& InsertTris) const
{
	// following block is disabled because DynamicMesh::RemoveTriangle() will remove the attribute triangles,
	// and discard elements. So we do not need to do this here. And *cannot* do this here. Which is not ideal...
	// @todo optionally do this part via a flag, so we can use TDynamicMeshAttributeChange outside of a parent FDynamicMeshChange?


	// clear triangles and discard any elements we don't need anymore
	//for (const FChangeTriangle& TriInfo : RemoveTris)
	//{
	//	Overlay->OnRemoveTriangle(TriInfo.TriangleID, true);
	//}

	// if this is true, we use unsafe insert, which means we take shortcuts
	// inside DMesh to speed up multiple inserts. This requires rebuilding
	// the refcount free lists after the insertions are done, which is O(N)
	bool bUseUnsafe = true;

	// insert missing vertices
	if (bUseUnsafe)
	{
		Overlay->BeginUnsafeElementsInsert();
	}
	for (const FChangeElement& ElemInfo : InsertElements)
	{
		// boundary elements may still exist
		if (Overlay->IsElement(ElemInfo.ElementID) == false)
		{
			const RealType* Data = &InsertElementData[ElemInfo.DataIndex];
			EMeshResult Result = Overlay->InsertElement(ElemInfo.ElementID, Data, bUseUnsafe);
			ensure(Result == EMeshResult::Ok);
		}
		else
		{
			const RealType* Data = &InsertElementData[ElemInfo.DataIndex];
			Overlay->SetElement(ElemInfo.ElementID, Data);
		}
	}
	if (bUseUnsafe)
	{
		Overlay->EndUnsafeElementsInsert();
	}

	// set new element triangles
	for (const FChangeTriangle& TriInfo : InsertTris)
	{
		checkSlow(Overlay->GetParentMesh()->IsTriangle(TriInfo.TriangleID));
		// existing overlay triangle should be empty...
		EMeshResult Result = Overlay->SetTriangle(TriInfo.TriangleID, TriInfo.Elements);
		ensure(Result == EMeshResult::Ok);
	}
}



namespace UE
{
namespace Geometry
{

// because ::ApplyReplaceChange() is in the .cpp we need to declare possible attribute types here
template class TDynamicMeshAttributeChange<float, 1>;
template class TDynamicMeshAttributeChange<double, 1>;
template class TDynamicMeshAttributeChange<int, 1>;
template class TDynamicMeshAttributeChange<float, 2>;
template class TDynamicMeshAttributeChange<double, 2>;
template class TDynamicMeshAttributeChange<int, 2>;
template class TDynamicMeshAttributeChange<float, 3>;
template class TDynamicMeshAttributeChange<double, 3>;
template class TDynamicMeshAttributeChange<int, 3>;
template class TDynamicMeshAttributeChange<float, 4>;
template class TDynamicMeshAttributeChange<double, 4>;

} // end namespace UE::Geometry
} // end namespace UE





//
//
// FDynamicMeshChange - stores change in dynamic mesh
// 
//




FDynamicMeshChange::~FDynamicMeshChange()
{
}


void FDynamicMeshChange::SaveInitialVertex(const FDynamicMesh3* Mesh, int VertexID)
{
	FChangeVertex Vertex;
	Vertex.VertexID = VertexID;
	Vertex.Info = Mesh->GetVertexInfo(VertexID);
	OldVertices.Add(Vertex);		// @todo emplace
}

void FDynamicMeshChange::SaveInitialTriangle(const FDynamicMesh3* Mesh, int TriangleID)
{
	FChangeTriangle Triangle;
	Triangle.TriangleID = TriangleID;
	Triangle.Vertices = Mesh->GetTriangle(TriangleID);
	Triangle.Edges = Mesh->GetTriEdges(TriangleID);
	Triangle.GroupID = Mesh->GetTriangleGroup(TriangleID);
	OldTriangles.Add(Triangle);
}


void FDynamicMeshChange::StoreFinalVertex(const FDynamicMesh3* Mesh, int VertexID)
{
	FChangeVertex Vertex;
	Vertex.VertexID = VertexID;
	Vertex.Info = Mesh->GetVertexInfo(VertexID);
	NewVertices.Add(Vertex);		// @todo emplace
}

void FDynamicMeshChange::StoreFinalTriangle(const FDynamicMesh3* Mesh, int TriangleID)
{
	FChangeTriangle Triangle;
	Triangle.TriangleID = TriangleID;
	Triangle.Vertices = Mesh->GetTriangle(TriangleID);
	Triangle.Edges = Mesh->GetTriEdges(TriangleID);
	Triangle.GroupID = Mesh->GetTriangleGroup(TriangleID);
	NewTriangles.Add(Triangle);
}


bool FDynamicMeshChange::Apply(FDynamicMesh3* Mesh, bool bRevert) const
{
	//Mesh->CheckValidity();

	if (bRevert)
	{
		ApplyReplaceChange(Mesh, NewTriangles, OldVertices, OldTriangles);
	}
	else
	{
		ApplyReplaceChange(Mesh, OldTriangles, NewVertices, NewTriangles);
	}

	if (AttributeChanges.IsValid())
	{
		AttributeChanges->Apply(Mesh->Attributes(), bRevert);
	}

	//Mesh->CheckValidity();

	return true;
}


void FDynamicMeshChange::ApplyReplaceChange(FDynamicMesh3* Mesh,
	const TArray<FChangeTriangle>& RemoveTris,
	const TArray<FChangeVertex>& InsertVerts,
	const TArray<FChangeTriangle>& InsertTris) const
{
	// remove inserted triangles
	for (const FChangeTriangle& TriInfo : RemoveTris)
	{
		EMeshResult Result = Mesh->RemoveTriangle(TriInfo.TriangleID);
		ensure(Result == EMeshResult::Ok);
	}

	// if this is true, we use unsafe insert, which means we take shortcuts
	// inside DMesh to speed up multiple inserts. This requires rebuilding
	// the refcount free lists after the insertions are done, which is O(N)
	bool bUseUnsafe = true;
	

	// insert missing vertices
	if (bUseUnsafe)
	{
		Mesh->BeginUnsafeVerticesInsert();
	}
	for (const FChangeVertex& VertInfo : InsertVerts)
	{
		// boundary vertices may still exist. If interior vertices still exist we are in trouble but what could we possibly do to recover?
		if (Mesh->IsVertex(VertInfo.VertexID) == false)
		{
			EMeshResult Result = Mesh->InsertVertex(VertInfo.VertexID, VertInfo.Info, bUseUnsafe);
			ensure(Result == EMeshResult::Ok);
		}
		else
		{
			Mesh->SetVertex(VertInfo.VertexID, VertInfo.Info.Position);
			if (VertInfo.Info.bHaveN)
			{
				Mesh->SetVertexNormal(VertInfo.VertexID, VertInfo.Info.Normal);
			}
			if (VertInfo.Info.bHaveC)
			{
				Mesh->SetVertexColor(VertInfo.VertexID, VertInfo.Info.Color);
			}
			if (VertInfo.Info.bHaveUV)
			{
				Mesh->SetVertexUV(VertInfo.VertexID, VertInfo.Info.UV);
			}
		}
	}
	if (bUseUnsafe)
	{
		Mesh->EndUnsafeVerticesInsert();
	}

	// insert new triangles
	if (bUseUnsafe)
	{
		Mesh->BeginUnsafeTrianglesInsert();
	}
	for (const FChangeTriangle& TriInfo : InsertTris)
	{
		EMeshResult Result = Mesh->InsertTriangle(TriInfo.TriangleID, TriInfo.Vertices, TriInfo.GroupID, bUseUnsafe);
		ensure(Result == EMeshResult::Ok);
	}
	if (bUseUnsafe)
	{
		Mesh->EndUnsafeTrianglesInsert();
	}
}


bool FDynamicMeshChange::HasSavedVertex(int VertexID)
{
	for (const FChangeVertex& VertInfo : OldVertices)
	{
		if (VertInfo.VertexID == VertexID)
			return true;
	}
	return false;
}


void FDynamicMeshChange::VerifySaveState() const
{
	TSet<int> SavedVertexIDs;
	for (const FChangeVertex& VertInfo : OldVertices)
	{
		SavedVertexIDs.Add(VertInfo.VertexID);
	}

	for (const FChangeTriangle& TriInfo : OldTriangles)
	{
		ensure(SavedVertexIDs.Contains(TriInfo.Vertices.A));
		ensure(SavedVertexIDs.Contains(TriInfo.Vertices.B));
		ensure(SavedVertexIDs.Contains(TriInfo.Vertices.C));
	}
}





//
//
// FDynamicMeshAttributeSetChangeTracker - tracks changes in all attribute overlays of a Mesh
// 
//




FDynamicMeshAttributeSetChangeTracker::FDynamicMeshAttributeSetChangeTracker(const FDynamicMeshAttributeSet* AttribsIn)
{
	this->Attribs = AttribsIn;
}

void FDynamicMeshAttributeSetChangeTracker::BeginChange()
{
	// initialize new attribute set change
	ensure(Change == nullptr);
	Change = new FDynamicMeshAttributeChangeSet();
	int NumUVLayers = Attribs->NumUVLayers();
	Change->UVChanges.SetNum(NumUVLayers);
	int NumNormalLayers = Attribs->NumNormalLayers();
	Change->NormalChanges.SetNum(NumNormalLayers);

	// initialize UV layer state tracking
	if (UVStates.Num() < NumUVLayers)
	{
		UVStates.SetNum(NumUVLayers);
	}
	for (int k = 0; k < NumUVLayers; ++k)
	{
		FElementState& State = UVStates[k];
		const FDynamicMeshUVOverlay* UVLayer = Attribs->GetUVLayer(k);
		State.MaxElementID = UVLayer->MaxElementID();
		State.StartElements.Init(false, State.MaxElementID);
		State.ChangedElements.Init(false, State.MaxElementID);
		for ( int ElemID : UVLayer->ElementIndicesItr())
		{
			State.StartElements[ElemID] = true;
		}
	}

	// initialize Normal layer state tracking
	if (NormalStates.Num() < NumNormalLayers)
	{
		NormalStates.SetNum(NumNormalLayers);
	}
	for (int k = 0; k < NumNormalLayers; ++k)
	{
		FElementState& State = NormalStates[k];
		const FDynamicMeshNormalOverlay* NormalLayer = Attribs->GetNormalLayer(k);
		State.MaxElementID = NormalLayer->MaxElementID();
		State.StartElements.Init(false, State.MaxElementID);
		State.ChangedElements.Init(false, State.MaxElementID);
		for (int ElemID : NormalLayer->ElementIndicesItr())
		{
			State.StartElements[ElemID] = true;
		}
	}

	// initialize Color layer state tracking if needed
	if (Attribs->HasPrimaryColors())
	{
		Change->ColorChange.Emplace();
		const FDynamicMeshColorOverlay* ColorLayer = Attribs->PrimaryColors();
		ColorState.MaxElementID = ColorLayer->MaxElementID();
		ColorState.StartElements.Init(false, ColorState.MaxElementID);
		ColorState.ChangedElements.Init(false,  ColorState.MaxElementID);
		for ( int ElemID : ColorLayer->ElementIndicesItr())
		{
			ColorState.StartElements[ElemID] = true;
		}
	}

	if (Attribs->HasMaterialID())
	{
		Change->MaterialIDAttribChange.Emplace();
	}

	int NumPolygroupLayers = Attribs->NumPolygroupLayers();
	if (NumPolygroupLayers > 0)
	{
		Change->PolygroupChanges.SetNum(NumPolygroupLayers);
	}

	for (int k = 0, NumWeights = Attribs->NumWeightLayers(); k < NumWeights; k++)
	{
		Change->WeightChanges.Add(Attribs->GetWeightLayer(k)->NewBlankChange());
	}

	for (int k = 0, NumRegAttrib = Attribs->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		Change->RegisteredAttributeChanges.Add(Attribs->GetRegisteredAttribute(k)->NewBlankChange());
	}
}


TUniquePtr<FDynamicMeshAttributeChangeSet> FDynamicMeshAttributeSetChangeTracker::EndChange()
{
	TUniquePtr<FDynamicMeshAttributeChangeSet> Result(this->Change);
	return MoveTemp(Result);
}



void FDynamicMeshAttributeSetChangeTracker::SaveInitialTriangle(int TriangleID)
{
	int NumUVLayers = Attribs->NumUVLayers();
	for (int k = 0; k < NumUVLayers; ++k)
	{
		FDynamicMeshUVChange& UVChange = Change->UVChanges[k];
		const FDynamicMeshUVOverlay* UVLayer = Attribs->GetUVLayer(k);
		if (UVLayer->IsSetTriangle(TriangleID))
		{
			FIndex3i UVTriangle = UVLayer->GetTriangle(TriangleID);
			SaveElement(UVTriangle.A, UVStates[k], UVLayer, UVChange);
			SaveElement(UVTriangle.B, UVStates[k], UVLayer, UVChange);
			SaveElement(UVTriangle.C, UVStates[k], UVLayer, UVChange);
			UVChange.SaveInitialTriangle(UVLayer, TriangleID);
		}
	}

	int NumNormalLayers = Attribs->NumNormalLayers();
	for (int k = 0; k < NumNormalLayers; ++k)
	{
		FDynamicMeshNormalChange& NormalChange = Change->NormalChanges[k];
		const FDynamicMeshNormalOverlay* NormalLayer = Attribs->GetNormalLayer(k);
		if (NormalLayer->IsSetTriangle(TriangleID))
		{
			FIndex3i NormTriangle = NormalLayer->GetTriangle(TriangleID);
			SaveElement(NormTriangle.A, NormalStates[k], NormalLayer, NormalChange);
			SaveElement(NormTriangle.B, NormalStates[k], NormalLayer, NormalChange);
			SaveElement(NormTriangle.C, NormalStates[k], NormalLayer, NormalChange);
			NormalChange.SaveInitialTriangle(NormalLayer, TriangleID);
		}
	}
	if (Change->ColorChange.IsSet())
	{
		FDynamicMeshColorChange& ColorChange = *Change->ColorChange;
		const FDynamicMeshColorOverlay* ColorLayer = Attribs->PrimaryColors();
		if (ColorLayer->IsSetTriangle(TriangleID))
		{
			FIndex3i ColorTriangle = ColorLayer->GetTriangle(TriangleID);
			SaveElement(ColorTriangle.A, ColorState, ColorLayer, ColorChange);
			SaveElement(ColorTriangle.B, ColorState, ColorLayer, ColorChange);
			SaveElement(ColorTriangle.C, ColorState, ColorLayer, ColorChange);
			ColorChange.SaveInitialTriangle(ColorLayer, TriangleID);
		}
	}

	if (Change->MaterialIDAttribChange.IsSet())
	{
		Change->MaterialIDAttribChange->SaveInitialTriangle(Attribs->GetMaterialID(), TriangleID);
	}

	int NumPolygroupLayers = Attribs->NumPolygroupLayers();
	for (int k = 0; k < NumPolygroupLayers; ++k)
	{
		Change->PolygroupChanges[k].SaveInitialTriangle(Attribs->GetPolygroupLayer(k), TriangleID);
	}

	for (int k = 0, NumRegAttrib = Attribs->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		Change->RegisteredAttributeChanges[k]->SaveInitialTriangle(Attribs->GetRegisteredAttribute(k), TriangleID);
	}

	int NumWeightLayers = Attribs->NumWeightLayers();
	for (int k = 0; k < NumWeightLayers; ++k)
	{
		Change->WeightChanges[k]->SaveInitialTriangle(Attribs->GetWeightLayer(k), TriangleID);
	}

}


void FDynamicMeshAttributeSetChangeTracker::SaveInitialVertex(int VertexID)
{
	for (int k = 0, NumRegAttrib = Attribs->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		Change->RegisteredAttributeChanges[k]->SaveInitialVertex(Attribs->GetRegisteredAttribute(k), VertexID);
	}

	int NumWeightLayers = Attribs->NumWeightLayers();
	for (int k = 0; k < NumWeightLayers; ++k)
	{
		Change->WeightChanges[k]->SaveInitialVertex(Attribs->GetWeightLayer(k), VertexID);
	}

}


void FDynamicMeshAttributeSetChangeTracker::StoreAllFinalTriangles(const TArray<int>& TriangleIDs)
{
	TSet<int> StoredVertices;	// re-used
	
	// store final UV elements for all modified triangles, and final triangles
	int NumUVLayers = Attribs->NumUVLayers();
	for (int k = 0; k < NumUVLayers; ++k)
	{
		FDynamicMeshUVChange& UVChange = Change->UVChanges[k];
		const FDynamicMeshUVOverlay* UVLayer = Attribs->GetUVLayer(k);
		StoredVertices.Reset();

		for (int tid : TriangleIDs)
		{
			if (UVLayer->IsSetTriangle(tid))
			{
				FIndex3i Tri = UVLayer->GetTriangle(tid);
				for (int j = 0; j < 3; ++j)
				{
					if (StoredVertices.Contains(Tri[j]) == false)
					{
						UVChange.StoreFinalElement(UVLayer, Tri[j]);
						StoredVertices.Add(Tri[j]);
					}
				}
				UVChange.StoreFinalTriangle(UVLayer, tid);
			}
		}
	}


	// store final Normal elements for all modified triangles, and final triangles
	int NumNormalLayers = Attribs->NumNormalLayers();
	for (int k = 0; k < NumNormalLayers; ++k)
	{
		FDynamicMeshNormalChange& NormalChange = Change->NormalChanges[k];
		const FDynamicMeshNormalOverlay* NormalLayer = Attribs->GetNormalLayer(k);
		StoredVertices.Reset();

		for (int tid : TriangleIDs)
		{
			if (NormalLayer->IsSetTriangle(tid))
			{
				FIndex3i Tri = NormalLayer->GetTriangle(tid);
				for (int j = 0; j < 3; ++j)
				{
					if (StoredVertices.Contains(Tri[j]) == false)
					{
						NormalChange.StoreFinalElement(NormalLayer, Tri[j]);
						StoredVertices.Add(Tri[j]);
					}
				}
				NormalChange.StoreFinalTriangle(NormalLayer, tid);
			}
		}
	}

	// store final Color elements for all modified triangles, and final triangles if needed
	if (Change->ColorChange.IsSet())
	{
		FDynamicMeshColorChange& ColorChange = *Change->ColorChange;
		const FDynamicMeshColorOverlay* ColorLayer = Attribs->PrimaryColors();
		StoredVertices.Reset();

		for (int tid : TriangleIDs)
		{
			if (ColorLayer->IsSetTriangle(tid))
			{
				FIndex3i Tri = ColorLayer->GetTriangle(tid);
				for (int j = 0; j < 3; ++j)
				{
					if (StoredVertices.Contains(Tri[j]) == false)
					{ 
						ColorChange.StoreFinalElement(ColorLayer, Tri[j]);
						StoredVertices.Add(Tri[j]);
					}
				}
				ColorChange.StoreFinalTriangle(ColorLayer, tid);
			}
		}
	}

	if (Change->MaterialIDAttribChange.IsSet())
	{
		Change->MaterialIDAttribChange->StoreAllFinalTriangles(Attribs->GetMaterialID(), TriangleIDs);
	}

	int NumPolygroupLayers = Attribs->NumPolygroupLayers();
	for (int k = 0; k < NumPolygroupLayers; ++k)
	{
		Change->PolygroupChanges[k].StoreAllFinalTriangles(Attribs->GetPolygroupLayer(k), TriangleIDs);
	}

	for (int k = 0, NumRegAttrib = Attribs->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		Change->RegisteredAttributeChanges[k]->StoreAllFinalTriangles(Attribs->GetRegisteredAttribute(k), TriangleIDs);
	}

	for (int k = 0, NumWeightLayers = Attribs->NumWeightLayers(); k < NumWeightLayers; k++)
	{
		Change->WeightChanges[k]->StoreAllFinalTriangles(Attribs->GetWeightLayer(k), TriangleIDs);
	}
}


void FDynamicMeshAttributeSetChangeTracker::StoreAllFinalVertices(const TSet<int>& VertexIDs)
{
	for (int k = 0, NumRegAttrib = Attribs->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		Change->RegisteredAttributeChanges[k]->StoreAllFinalVertices(Attribs->GetRegisteredAttribute(k), VertexIDs);
	}

	for (int k = 0, NumWeightLayers = Attribs->NumWeightLayers(); k < NumWeightLayers; k++)
	{
		Change->WeightChanges[k]->StoreAllFinalVertices(Attribs->GetWeightLayer(k), VertexIDs);
	}
}


bool FDynamicMeshAttributeChangeSet::Apply(FDynamicMeshAttributeSet* Attributes, bool bRevert) const
{
	int NumUVLayers = Attributes->NumUVLayers();
	ensure(NumUVLayers == UVChanges.Num());
	NumUVLayers = FMath::Min(NumUVLayers, UVChanges.Num());
	for (int k = 0; k < NumUVLayers; ++k)
	{
		FDynamicMeshUVOverlay* UVLayer = Attributes->GetUVLayer(k);
		UVChanges[k].Apply(UVLayer, bRevert);
	}

	int NumNormalLayers = Attributes->NumNormalLayers();
	ensure(NumNormalLayers == NormalChanges.Num());
	NumNormalLayers = FMath::Min(NumNormalLayers, NormalChanges.Num());
	for (int k = 0; k < NumNormalLayers; ++k)
	{
		FDynamicMeshNormalOverlay* NormalLayer = Attributes->GetNormalLayer(k);
		NormalChanges[k].Apply(NormalLayer, bRevert);
	}

	for (int k = 0, NumRegAttrib = Attributes->NumRegisteredAttributes(); k < NumRegAttrib; k++)
	{
		RegisteredAttributeChanges[k]->Apply(Attributes->GetRegisteredAttribute(k), bRevert);
	}
	for (int k = 0, NumWeightLayers = Attributes->NumWeightLayers(); k < NumWeightLayers; k++)
	{
		WeightChanges[k]->Apply(Attributes->GetWeightLayer(k), bRevert);
	}

	if (ColorChange.IsSet())
	{
		FDynamicMeshColorOverlay* ColorLayer = Attributes->PrimaryColors();
		ColorChange->Apply(ColorLayer, bRevert);
	}
	if (MaterialIDAttribChange.IsSet())
	{
		MaterialIDAttribChange->Apply(Attributes->GetMaterialID(), bRevert);
	}

	int NumPolygroupLayers = Attributes->NumPolygroupLayers();
	for (int k = 0; k < NumPolygroupLayers; ++k)
	{
		PolygroupChanges[k].Apply(Attributes->GetPolygroupLayer(k), bRevert);
	}

	return true;
}








//
//
// FDynamicMeshChangeTracker - tracks all changes in mesh and attribute overlay
// 
//




FDynamicMeshChangeTracker::FDynamicMeshChangeTracker(const FDynamicMesh3* MeshIn)
{
	Mesh = MeshIn;
	if (Mesh->HasAttributes())
	{
		AttribChangeTracker = new FDynamicMeshAttributeSetChangeTracker(Mesh->Attributes());
	}
}

FDynamicMeshChangeTracker::~FDynamicMeshChangeTracker()
{
	if (AttribChangeTracker != nullptr)
	{
		delete AttribChangeTracker;
		AttribChangeTracker = nullptr;
	}
}


void FDynamicMeshChangeTracker::BeginChange()
{
	ensure(Change == nullptr);
	Change = new FDynamicMeshChange();

	// @todo should we do this on EndChange() so that delay is at end of stroke instead of beginning?
	// @todo use TSet instead of a bitmap? Or implement HBitmap?

	this->MaxVertexID = Mesh->MaxVertexID();
	StartVertices.Init(false, MaxVertexID);
	ChangedVertices.Init(false, MaxVertexID);
	for (int vi = 0; vi < MaxVertexID; ++vi)
	{
		if (Mesh->IsVertex(vi))
		{
			StartVertices[vi] = true;
		}
	}

	this->MaxTriangleID = Mesh->MaxTriangleID();
	StartTriangles.Init(false, MaxTriangleID);    // does this re-use memory? would be preferable
	ChangedTriangles.Init(false, MaxTriangleID);
	for (int ti = 0; ti < MaxTriangleID; ++ti)
	{
		if (Mesh->IsTriangle(ti))
		{
			StartTriangles[ti] = true;
		}
	}

	if (AttribChangeTracker != nullptr)
	{
		AttribChangeTracker->BeginChange();
	}
}



void FDynamicMeshChangeTracker::SaveVertex(int VertexID)
{
	// only save initial vertex state if it existed at BeginChange() and hasn't been modified yet
	if (VertexID >= MaxVertexID || ChangedVertices[VertexID] == true || StartVertices[VertexID] == false)
	{
		return;
	}

	Change->SaveInitialVertex(Mesh, VertexID);

	ChangedVertices[VertexID] = true;

	if (AttribChangeTracker != nullptr)
	{
		AttribChangeTracker->SaveInitialVertex(VertexID);
	}
}



void FDynamicMeshChangeTracker::SaveTriangle(int TriangleID, bool bSaveVertices)
{
	// only save initial triangle state if it existed at BeginChange() and hasn't been modified yet
	if (TriangleID >= MaxTriangleID || ChangedTriangles[TriangleID] == true || StartTriangles[TriangleID] == false)
	{
		return;
	}

	if (bSaveVertices)
	{
		FIndex3i Tri = Mesh->GetTriangle(TriangleID);
		SaveVertex(Tri.A);
		SaveVertex(Tri.B);
		SaveVertex(Tri.C);
	}

	Change->SaveInitialTriangle(Mesh, TriangleID);

	ChangedTriangles[TriangleID] = true;

	if (AttribChangeTracker != nullptr)
	{
		AttribChangeTracker->SaveInitialTriangle(TriangleID);
	}
}



TUniquePtr<FDynamicMeshChange> FDynamicMeshChangeTracker::EndChange()
{
	TSet<int> StoredVertices;   // @todo maybe should be TBitArray? 
	TArray<int> StoredTriangles;

	for (int tid : Mesh->TriangleIndicesItr())
	{
		// determine if this is a new or modified triangle, and if so, store it
		if (tid >= MaxTriangleID || StartTriangles[tid] == false || ChangedTriangles[tid] == true)
		{
			// only store new vertices 
			FIndex3i Tri = Mesh->GetTriangle(tid);
			for (int j = 0; j < 3; ++j)
			{
				if (StoredVertices.Contains(Tri[j]) == false)
				{
					Change->StoreFinalVertex(Mesh, Tri[j]);
					StoredVertices.Add(Tri[j]);
				}
			}

			Change->StoreFinalTriangle(Mesh, tid);
			StoredTriangles.Add(tid);
		}
	}
	TUniquePtr<FDynamicMeshChange> Result(Change);

	// handle attribute overlays
	if (AttribChangeTracker != nullptr)
	{
		AttribChangeTracker->StoreAllFinalTriangles(StoredTriangles);
		AttribChangeTracker->StoreAllFinalVertices(StoredVertices);
		Result->AttachAttributeChanges(AttribChangeTracker->EndChange());
	}

	return MoveTemp(Result);
}




void FDynamicMeshChangeTracker::VerifySaveState()
{
	Change->VerifySaveState();
}



void FDynamicMeshChange::GetSavedTriangleList(TArray<int>& TrianglesOut, bool bInitial) const
{
	const TArray<FChangeTriangle>& UseList = (bInitial) ? OldTriangles : NewTriangles;
	int32 N = UseList.Num();
	TrianglesOut.Reserve(TrianglesOut.Num() + N);
	for (int32 i = 0; i < N; ++i)
	{
		TrianglesOut.Add(UseList[i].TriangleID);
	}
}

void FDynamicMeshChange::GetAffectedVertices(TArray<int>& VerticesOut, bool bRevert) const
{
	const TArray<FChangeTriangle>& UseTris = (bRevert) ? OldTriangles : NewTriangles;
	const TArray<FChangeVertex>& UseVerts = (bRevert) ? OldVertices : NewVertices;
	TSet<int32> UniqueVIDs;
	for (const FChangeTriangle& Tri : UseTris)
	{
		UniqueVIDs.Add(Tri.Vertices.A);
		UniqueVIDs.Add(Tri.Vertices.B);
		UniqueVIDs.Add(Tri.Vertices.C);
	}
	for (const FChangeVertex& Vert : UseVerts)
	{
		UniqueVIDs.Add(Vert.VertexID);
	}
	VerticesOut = UniqueVIDs.Array();
}



void FDynamicMeshChange::CheckValidity(EValidityCheckFailMode FailMode) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FDynamicMeshChange::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FDynamicMeshChange::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}


	TSet<int> SavedOldV, SavedNewV;
	for (const FChangeVertex& ChangeVert : OldVertices)
	{
		SavedOldV.Add(ChangeVert.VertexID);
	}
	for (const FChangeVertex& ChangeVert : NewVertices)
	{
		SavedNewV.Add(ChangeVert.VertexID);
	}

	for (const FChangeTriangle& ChangedTri : OldTriangles)
	{
		CheckOrFailF(SavedOldV.Contains(ChangedTri.Vertices.A));
		CheckOrFailF(SavedOldV.Contains(ChangedTri.Vertices.B));
		CheckOrFailF(SavedOldV.Contains(ChangedTri.Vertices.C));
	}
	for (const FChangeTriangle& ChangedTri : NewTriangles)
	{
		CheckOrFailF(SavedNewV.Contains(ChangedTri.Vertices.A));
		CheckOrFailF(SavedNewV.Contains(ChangedTri.Vertices.B));
		CheckOrFailF(SavedNewV.Contains(ChangedTri.Vertices.C));
	}
}