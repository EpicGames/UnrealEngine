// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Async/ParallelFor.h"
#include "Async/Mutex.h"

using namespace UE::Geometry;


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::ClearElements()
{
	Elements.Clear();
	ElementsRefCounts = FRefCountVector();
	ParentVertices.Clear();
	InitializeTriangles(ParentMesh->MaxTriangleID());
}



template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::AppendElement(RealType ConstantValue)
{
	int vid = ElementsRefCounts.Allocate();
	int i = ElementSize * vid;
	for (int k = ElementSize - 1; k >= 0; --k)
	{
		Elements.InsertAt(ConstantValue, i + k);
	}
	ParentVertices.InsertAt(FDynamicMesh3::InvalidID, vid);

	//updateTimeStamp(true);
	return vid;
}


template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::AppendElement(const RealType* Value)
{
	int vid = ElementsRefCounts.Allocate();
	int i = ElementSize * vid;

	// insert in reverse order so that Resize() is only called once
	for (int k = ElementSize - 1; k >= 0; --k)
	{
		Elements.InsertAt(Value[k], i + k);
	}
	ParentVertices.InsertAt(FDynamicMesh3::InvalidID, vid);

	//updateTimeStamp(true);
	return vid;
}


template<typename RealType, int ElementSize>
EMeshResult TDynamicMeshOverlay<RealType, ElementSize>::InsertElement(int ElementID, const RealType* Value, bool bUnsafe)
{
	if (ElementsRefCounts.IsValid(ElementID))
	{
		return EMeshResult::Failed_VertexAlreadyExists;
	}

	bool bOK = (bUnsafe) ? ElementsRefCounts.AllocateAtUnsafe(ElementID) :
		ElementsRefCounts.AllocateAt(ElementID);
	if (bOK == false)
	{
		return EMeshResult::Failed_CannotAllocateVertex;
	}

	int i = ElementSize * ElementID;
	// insert in reverse order so that Resize() is only called once
	for (int k = ElementSize - 1; k >= 0; --k)
	{
		Elements.InsertAt(Value[k], i + k);
	}

	ParentVertices.InsertAt(FDynamicMesh3::InvalidID, ElementID, FDynamicMesh3::InvalidID);

	//UpdateTimeStamp(true, true);
	return EMeshResult::Ok;
}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::CreateFromPredicate(TFunctionRef<bool(int ParentVertexIdx, int TriIDA, int TriIDB)> TrisCanShareVertexPredicate, RealType InitElementValue)
{
	ClearElements(); // deletes all elements and initializes triangles to be 1:1 w/ parentmesh IDs
	FDynamicMesh3::FLocalIntArray TrisActiveSubGroup, AppendedElements;
	FDynamicMesh3::FLocalIntArray TriangleIDs, TriangleContigGroupLens;
	FDynamicMesh3::FLocalBoolArray GroupIsLoop;
	for (int VertexID : ParentMesh->VertexIndicesItr())
	{
		
		bool bActiveSubGroupBroken = false;
		ParentMesh->GetVtxContiguousTriangles(VertexID, TriangleIDs, TriangleContigGroupLens, GroupIsLoop);
		int GroupStart = 0;
		for (int GroupIdx = 0; GroupIdx < TriangleContigGroupLens.Num(); GroupIdx++)
		{
			bool bIsLoop = GroupIsLoop[GroupIdx];
			int GroupNum = TriangleContigGroupLens[GroupIdx];
			if (!ensure(GroupNum > 0)) // sanity check; groups should always have at least one element
			{
				continue;
			}

			TrisActiveSubGroup.Reset();
			AppendedElements.Reset();
			TrisActiveSubGroup.SetNumZeroed(GroupNum, EAllowShrinking::No);
			int CurrentGroupID = 0;
			int CurrentGroupRefSubIdx = 0;
			for (int TriSubIdx = 0; TriSubIdx+1 < GroupNum; TriSubIdx++)
			{
				int TriIDA = TriangleIDs[GroupStart + TriSubIdx];
				int TriIDB = TriangleIDs[GroupStart + TriSubIdx + 1];
				bool bCanShare = TrisCanShareVertexPredicate(VertexID, TriIDA, TriIDB);
				if (!bCanShare)
				{
					CurrentGroupID++;
					CurrentGroupRefSubIdx = TriSubIdx + 1;
				}
				
				TrisActiveSubGroup[TriSubIdx + 1] = CurrentGroupID;
			}

			// for loops, merge first and last group if needed
			int NumGroupID = CurrentGroupID + 1;
			if (bIsLoop && TrisActiveSubGroup[0] != TrisActiveSubGroup.Last())
			{
				if (TrisCanShareVertexPredicate(VertexID, TriangleIDs[GroupStart], TriangleIDs[GroupStart + GroupNum - 1]))
				{
					int EndGroupID = TrisActiveSubGroup[GroupNum - 1];
					int StartGroupID = TrisActiveSubGroup[0];
					int TriID0 = TriangleIDs[GroupStart];
					
					for (int Idx = GroupNum - 1; Idx >= 0 && TrisActiveSubGroup[Idx] == EndGroupID; Idx--)
					{
						TrisActiveSubGroup[Idx] = StartGroupID;
					}
					NumGroupID--;
				}
			}

			for (int Idx = 0; Idx < NumGroupID; Idx++)
			{
				AppendedElements.Add(AppendElement(InitElementValue));
			}
			for (int TriSubIdx = 0; TriSubIdx < GroupNum; TriSubIdx++)
			{
				int TriID = TriangleIDs[GroupStart + TriSubIdx];
				FIndex3i TriVertIDs = ParentMesh->GetTriangle(TriID);
				int VertSubIdx = IndexUtil::FindTriIndex(VertexID, TriVertIDs);
				int i = 3 * TriID;
				int ElementIndex = AppendedElements[TrisActiveSubGroup[TriSubIdx]];
				ElementTriangles.InsertAt(ElementIndex, i + VertSubIdx, FDynamicMesh3::InvalidID);
				ElementsRefCounts.Increment(ElementIndex);
				ParentVertices.InsertAt(VertexID, ElementIndex); // elements were appended one-by-one above, so default initialization not needed here
			}
			GroupStart += GroupNum;
		}
	}
}

template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::CreatePerVertex(RealType InitElementValue, bool bExactMapIDs)
{
	ClearElements(); // deletes all elements and initializes triangles to be 1:1 w/ parentmesh IDs
	TStaticArray<RealType, ElementSize> DefaultElement;
	for (int32 Idx = 0; Idx < ElementSize; ++Idx)
	{
		DefaultElement[Idx] = InitElementValue;
	}
	TArray<int32> VIDtoEID;
	bool bNeedsRemap = !bExactMapIDs && !ParentMesh->IsCompactV();
	if (bNeedsRemap)
	{
		VIDtoEID.SetNumUninitialized(ParentMesh->MaxVertexID());
	}
	for (int32 VertexID : ParentMesh->VertexIndicesItr())
	{
		int32 ElementID;
		if (bExactMapIDs)
		{
			ElementID = VertexID;
			EMeshResult Result = InsertElement(VertexID, DefaultElement.GetData());
			checkSlow(Result == EMeshResult::Ok); // because we allocate in increasing sequential order, shouldn't be possible InsertElement to return failure
		}
		else
		{
			ElementID = AppendElement(InitElementValue);
		}
		ParentVertices.InsertAt(VertexID, ElementID, FDynamicMesh3::InvalidID);
		if (bNeedsRemap)
		{
			VIDtoEID[VertexID] = ElementID;
		}
		else
		{
			checkSlow(VertexID == ElementID);
		}
	}

	for (int32 TriangleID : ParentMesh->TriangleIndicesItr())
	{
		FIndex3i Tri = ParentMesh->GetTriangle(TriangleID);
		int32 Start = 3 * TriangleID;
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			int32 VID = Tri[SubIdx];
			int32 EID = bNeedsRemap ? VIDtoEID[VID] : VID;
			ElementTriangles.InsertAt(EID, Start + SubIdx, FDynamicMesh3::InvalidID);
			ElementsRefCounts.Increment(EID);
		}
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::SplitVerticesWithPredicate(TFunctionRef<bool(int ElementID, int TriID)> ShouldSplitOutVertex, TFunctionRef<void(int ElementID, int TriID, RealType * FillVect)> GetNewElementValue)
{
	for (int TriID : ParentMesh->TriangleIndicesItr())
	{
		FIndex3i ElTri = GetTriangle(TriID);
		if (ElTri.A < 0)
		{
			// skip un-set triangles
			continue;
		}
		bool TriChanged = false;
		for (int SubIdx = 0; SubIdx < 3; SubIdx++)
		{
			int ElementID = ElTri[SubIdx];
			// by convention for overlays, a ref count of 2 means that only one triangle has the element -- can't split it out further
			if (ElementsRefCounts.GetRefCount(ElementID) <= 2)
			{
				// still set the new value though if the function wants to change it
				if (ShouldSplitOutVertex(ElementID, TriID))
				{
					RealType NewElementData[ElementSize];
					GetNewElementValue(ElementID, TriID, NewElementData);
					SetElement(ElementID, NewElementData);
				}
			}
			if (ShouldSplitOutVertex(ElementID, TriID))
			{
				TriChanged = true;
				RealType NewElementData[ElementSize];
				GetNewElementValue(ElementID, TriID, NewElementData);
				ElTri[SubIdx] = AppendElement(NewElementData);
			}
		}
		if (TriChanged)
		{
			InternalSetTriangle(TriID, ElTri, true);
		}
	}
}


template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::MergeElement(int SourceElementID, int TargetElementID)
{	
	if (SourceElementID == TargetElementID)
	{
		return false;
	}

	int SourceParentID = ParentVertices[SourceElementID];
	int TargetParentID = ParentVertices[TargetElementID];

	auto MergeElementForTriangle = [this, SourceElementID, TargetElementID](int32 TriID)
	{
		int ElementTriStart = TriID * 3;
		for (int SubIdx = 0; SubIdx < 3; SubIdx++)
		{
			int CurElID = ElementTriangles[ElementTriStart + SubIdx];
			if (CurElID == SourceElementID)
			{
				ElementsRefCounts.Decrement(SourceElementID);
				ElementsRefCounts.Increment(TargetElementID);
				ElementTriangles[ElementTriStart + SubIdx] = TargetElementID;
			}
		}
	};

	checkSlow(SourceParentID == TargetParentID);
	if (SourceParentID != TargetParentID)
	{
		return false;
	}

	ParentMesh->EnumerateVertexTriangles(SourceParentID, MergeElementForTriangle);

	checkSlow(ElementsRefCounts.IsValid(SourceElementID));
	checkSlow(ElementsRefCounts.GetRefCount(SourceElementID) == 1);
	if (ElementsRefCounts.GetRefCount(SourceElementID) == 1)
	{
		ElementsRefCounts.Decrement(SourceElementID);
		ParentVertices[SourceElementID] = FDynamicMesh3::InvalidID;
	}

	return true;
}


template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::SplitElement(int ElementID, const TArrayView<const int>& TrianglesToUpdate)
{
	int ParentID = ParentVertices[ElementID];
	return SplitElementWithNewParent(ElementID, ParentID, TrianglesToUpdate);
}


template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::SplitElementWithNewParent(int ElementID, int NewParentID, const TArrayView<const int>& TrianglesToUpdate)
{
	RealType SourceData[ElementSize];
	GetElement(ElementID, SourceData);
	int NewElID = AppendElement(SourceData);
	for (int TriID : TrianglesToUpdate)
	{
		int ElementTriStart = TriID * 3;
		for (int SubIdx = 0; SubIdx < 3; SubIdx++)
		{
			int CurElID = ElementTriangles[ElementTriStart + SubIdx];
			if (CurElID == ElementID)
			{
				ElementsRefCounts.Decrement(ElementID);
				ElementsRefCounts.Increment(NewElID);
				ElementTriangles[ElementTriStart + SubIdx] = NewElID;
			}
		}
	}
	ParentVertices.InsertAt(NewParentID, NewElID, FDynamicMesh3::InvalidID);

	checkSlow(ElementsRefCounts.IsValid(ElementID));

	// An element may have become isolated after changing all of its incident triangles. Delete such an element.
	if (ElementsRefCounts.GetRefCount(ElementID) == 1) 
	{
		ElementsRefCounts.Decrement(ElementID);
		ParentVertices[ElementID] = FDynamicMesh3::InvalidID;
	}

	return NewElID;
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::SplitBowties(bool bParallel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshOverlay_SplitBowties);

	if (bParallel)
	{
		// Bowties are typically rare, so we can test for bowties in parallel and only run the split on the (hopefully) few needed cases
		TArray<int32> BowtieVerts;
		FMutex BowtieCS;
		ParallelFor(ParentMesh->MaxVertexID(), [this, &BowtieVerts, &BowtieCS](int32 VertexID)
			{
				if (ParentMesh->IsVertex(VertexID) && IsBowtieInOverlay(VertexID))
				{
					BowtieCS.Lock();
					BowtieVerts.Add(VertexID);
					BowtieCS.Unlock();
				}
			}
		);
		for (int32 VertexID : BowtieVerts)
		{
			SplitBowtiesAtVertex(VertexID);
		}
	}
	else
	{
		for (int VertexID : ParentMesh->VertexIndicesItr())
		{
			SplitBowtiesAtVertex(VertexID);
		}
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::SplitBowtiesAtVertex(int32 VertexID, TArray<int32>* NewElementIDs)
{
	// arrays for storing contiguous triangle groups from parentmesh
	FDynamicMesh3::FLocalIntArray TrianglesOut, ContiguousGroupLengths;
	FDynamicMesh3::FLocalBoolArray GroupIsLoop;

	// per-vertex element group tracking data, reused in loop below
	FDynamicMesh3::FLocalIntArray ElementIDSeen;
	FDynamicMesh3::FLocalIntArray GroupElementIDs, // stores element IDs of this vertex, 1:1 w/ contiguous triangles in the parent mesh
		SubGroupID, // mapping from GroupElementIDs indices into SubGroupElementIDs indices, giving subgroup membership per triangle
		SubGroupElementIDs; // 1:1 w/ 'subgroups' in the group (e.g. if all triangles in the group all had the same element ID, this would be an array of length 1, just containing that element ID)

	ensure(EMeshResult::Ok == ParentMesh->GetVtxContiguousTriangles(VertexID, TrianglesOut, ContiguousGroupLengths, GroupIsLoop));
	int32 NumTris = TrianglesOut.Num();

	auto ElementIDFromTriangle = [VertexID, this](int32 TriID) -> int32
	{
		FIndex3i TriVIDs = ParentMesh->GetTriangle(TriID);
		FIndex3i TriEIDs = GetTriangle(TriID);
		int SubIdx = TriVIDs.IndexOf(VertexID);
		return TriEIDs[SubIdx];
	};

	// Early out in the easy non-bowtie case of 1 element, 1 contiguous group
	if (NumTris > 0 && ContiguousGroupLengths.Num() == 1)
	{
		int32 FirstElSeen = ElementIDFromTriangle(TrianglesOut[0]);
		bool bSingleElement = true;
		for (int32 Idx = 1; Idx < TrianglesOut.Num(); ++Idx)
		{
			if (FirstElSeen != ElementIDFromTriangle(TrianglesOut[Idx]))
			{
				bSingleElement = false;
			}
		}
		if (bSingleElement)
		{
			return;
		}
	}

	ElementIDSeen.Reset();
	// per contiguous group of triangles around vertex in ParentMesh, find contiguous sub-groups in overlay
	for (int32 GroupIdx = 0, NumGroups = ContiguousGroupLengths.Num(), TriSubStart = 0; GroupIdx < NumGroups; GroupIdx++)
	{
		bool bIsLoop = GroupIsLoop[GroupIdx];
		int TriInGroupNum = ContiguousGroupLengths[GroupIdx];
		if (ensure(TriInGroupNum > 0) == false)
		{
			continue;
		}
		int TriSubEnd = TriSubStart + TriInGroupNum;

		GroupElementIDs.Reset();
		for (int TriSubIdx = TriSubStart; TriSubIdx < TriSubEnd; TriSubIdx++)
		{
			int TriID = TrianglesOut[TriSubIdx];
			GroupElementIDs.Add(ElementIDFromTriangle(TriID));
		}

		auto IsConnected = [this, &GroupElementIDs, &TrianglesOut, &TriSubStart](int TriOutIdxA, int TriOutIdxB)
		{
			if (GroupElementIDs[TriOutIdxA - TriSubStart] != GroupElementIDs[TriOutIdxB - TriSubStart])
			{
				return false;
			}
			int EdgeID = ParentMesh->FindEdgeFromTriPair(TrianglesOut[TriOutIdxA], TrianglesOut[TriOutIdxB]);
			return EdgeID >= 0 && !IsSeamEdge(EdgeID);
		};

		SubGroupID.Reset(); SubGroupID.SetNum(TriInGroupNum);
		SubGroupElementIDs.Reset();
		int MaxSubID = 0;
		SubGroupID[0] = 0;
		SubGroupElementIDs.Add(GroupElementIDs[0]);

		// Iterate through tris in current group, except last one
		for (int TriSubIdx = TriSubStart; TriSubIdx + 1 < TriSubEnd; TriSubIdx++)
		{
			if (!IsConnected(TriSubIdx, TriSubIdx + 1))
			{
				SubGroupElementIDs.Add(GroupElementIDs[TriSubIdx + 1 - TriSubStart]);
				MaxSubID++;
			}
			SubGroupID[TriSubIdx - TriSubStart + 1] = MaxSubID;
		}
		// if group was a loop, need to check if the last sub-group and first sub-group were actually the same group
		if (bIsLoop && MaxSubID > 0 && IsConnected(TriSubStart, TriSubStart + TriInGroupNum - 1))
		{
			int LastGroupID = SubGroupID.Last();
			for (int32 Idx = SubGroupID.Num() - 1; Idx >= 0 && SubGroupID[Idx] == LastGroupID; Idx--)
			{
				SubGroupID[Idx] = 0;
			}
			MaxSubID--;
			SubGroupElementIDs.Pop(EAllowShrinking::No);
		}

		for (int SubID = 0; SubID < SubGroupElementIDs.Num(); SubID++)
		{
			int ElementID = SubGroupElementIDs[SubID];
			if (ElementID < 0)
			{
				continue;		// skip if this is an invalid ElementID (eg from an invalid triangle)
			}
			// split needed the *second* time we see a sub-group using a given ElementID
			if (ElementIDSeen.Contains(ElementID))
			{
				FDynamicMesh3::FLocalIntArray ConnectedTris;
				for (int TriSubIdx = TriSubStart; TriSubIdx < TriSubEnd; TriSubIdx++)
				{
					if (SubID == SubGroupID[TriSubIdx - TriSubStart])
					{
						ConnectedTris.Add(TrianglesOut[TriSubIdx]);
					}
				}
				int32 NewElementID = SplitElement(ElementID, ConnectedTris);
				if (NewElementIDs)
				{
					NewElementIDs->Add(NewElementID);
				}
			}
			else
			{
				ElementIDSeen.Add(ElementID);
			}
		}

		TriSubStart = TriSubEnd;
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::InitializeTriangles(int MaxTriangleID)
{
	ElementTriangles.SetNum(MaxTriangleID * 3);
	ElementTriangles.Fill(FDynamicMesh3::InvalidID);
}




template<typename RealType, int ElementSize>
EMeshResult TDynamicMeshOverlay<RealType, ElementSize>::SetTriangle(int tid, const FIndex3i& tv, bool bAllowElementFreeing)
{
	if (IsElement(tv[0]) == false || IsElement(tv[1]) == false || IsElement(tv[2]) == false)
	{
		checkSlow(false);
		return EMeshResult::Failed_NotAVertex;
	}
	if (tv[0] == tv[1] || tv[0] == tv[2] || tv[1] == tv[2]) 
	{
		checkSlow(false);
		return EMeshResult::Failed_InvalidNeighbourhood;
	}

	if (ParentMesh->IsTriangle(tid) == false)
	{
		checkSlow(false);
		return EMeshResult::Failed_NotATriangle;
	}

	InternalSetTriangle(tid, tv, true, bAllowElementFreeing);

	//updateTimeStamp(true);
	return EMeshResult::Ok;
}

template<typename RealType, int ElementSize>
void UE::Geometry::TDynamicMeshOverlay<RealType, ElementSize>::FreeUnusedElements(const TSet<int>* ElementsToCheck)
{
	auto FreeIfUnused = [this](int ElementID)
	{
		if (ElementsRefCounts.IsValid(ElementID) && ElementsRefCounts.GetRefCount(ElementID) == 1)
		{
			ElementsRefCounts.Decrement(ElementID);
			ParentVertices[ElementID] = FDynamicMesh3::InvalidID;
		}
	};

	if (ElementsToCheck)
	{
		for (int ElementID : *ElementsToCheck)
		{
			FreeIfUnused(ElementID);
		}
	}
	else
	{
		for (int ElementID : ElementsRefCounts.Indices())
		{
			FreeIfUnused(ElementID);
		}
	}
}

template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::UnsetTriangle(int TriangleID, bool bAllowElementFreeing)
{
	int i = 3 * TriangleID;
	if (ElementTriangles[i] == FDynamicMesh3::InvalidID)
	{
		return;
	}
	for (int SubIdx = 0; SubIdx < 3; SubIdx++)
	{
		ElementsRefCounts.Decrement(ElementTriangles[i + SubIdx]);

		if (bAllowElementFreeing && ElementsRefCounts.GetRefCount(ElementTriangles[i + SubIdx]) == 1)
		{
			ElementsRefCounts.Decrement(ElementTriangles[i + SubIdx]);
			ParentVertices[ElementTriangles[i + SubIdx]] = FDynamicMesh3::InvalidID;
		}
		ElementTriangles[i + SubIdx] = FDynamicMesh3::InvalidID;
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::InternalSetTriangle(int tid, const FIndex3i& tv, bool bUpdateRefCounts, bool bAllowElementFreeing)
{
	if (ensure(ParentMesh) == false)
	{
		return;
	}

	// If we have to decrement refcounts, we will do it at the end, because Decrement() frees
	// elements as soon as they lose their last reference, so a Decrement followed by Increment
	// can leave things in an invalid state.
	bool bNeedToDecrement = false;
	FIndex3i OldTriElements; // only used if need to decrement.

	int i = 3 * tid;

	// See if triangle existed and make it exist if not
	if (!ElementTriangles.SetMinimumSize(i + 3, FDynamicMesh3::InvalidID) && bUpdateRefCounts)
	{
		OldTriElements = GetTriangle(tid);
		bNeedToDecrement = (OldTriElements[0] != FDynamicMesh3::InvalidID);
	}

	ElementTriangles[i + 2] = tv[2];
	ElementTriangles[i + 1] = tv[1];
	ElementTriangles[i] = tv[0];

	if (bUpdateRefCounts)
	{
		ElementsRefCounts.Increment(tv[0]);
		ElementsRefCounts.Increment(tv[1]);
		ElementsRefCounts.Increment(tv[2]);

		if (bNeedToDecrement)
		{
			for (int j = 0; j < 3; ++j) 
			{
				ElementsRefCounts.Decrement(OldTriElements[j]);

				if (bAllowElementFreeing && ElementsRefCounts.GetRefCount(OldTriElements[j]) == 1)
				{
					ElementsRefCounts.Decrement(OldTriElements[j]);
					ParentVertices[OldTriElements[j]] = FDynamicMesh3::InvalidID;
				}
			};
		}
	}

	if (tv != FDynamicMesh3::InvalidTriangle)
	{
		// Set parent vertex IDs
		const FIndex3i ParentTriangle = ParentMesh->GetTriangle(tid);

		for (int VInd = 0; VInd < 3; ++VInd)
		{
			// Checks that the parent vertices of the elements that we're referencing in the overlay
			// triangle are either not yet set or already point to the vertices of the corresponding
			// mesh triangle (and so will remain unchanged). Remember that the same element is not
			// allowed to be used for multiple vertices.
			checkSlow(ParentVertices[tv[VInd]] == ParentTriangle[VInd] || ParentVertices[tv[VInd]] == FDynamicMesh3::InvalidID);

			ParentVertices.InsertAt(ParentTriangle[VInd], tv[VInd], FDynamicMesh3::InvalidID);
		}
	}
}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::InitializeNewTriangle(int tid)
{
	int i = 3 * tid;
	ElementTriangles.SetMinimumSize(i + 3, FDynamicMesh3::InvalidID);
	ElementTriangles[i + 2] = FDynamicMesh3::InvalidID;
	ElementTriangles[i + 1] = FDynamicMesh3::InvalidID;
	ElementTriangles[i] = FDynamicMesh3::InvalidID;

	//updateTimeStamp(true);
}




template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType,ElementSize>::IsSeamEdge(int eid, bool* bIsNonIntersecting) const
{
	if (bIsNonIntersecting != nullptr)
	{
		*bIsNonIntersecting = false;
	}
	if (ParentMesh->IsEdge(eid) == false)
	{
		return false;
	}

	FIndex2i et = ParentMesh->GetEdgeT(eid);
	if (et.B == FDynamicMesh3::InvalidID)
	{
		if (bIsNonIntersecting != nullptr)
		{
			FIndex2i ev = ParentMesh->GetEdgeV(eid);
			int CountA = CountVertexElements(ev.A);
			int CountB = CountVertexElements(ev.B);
			
			// will be false if another seam intersects is adjacent to either end of the seam edge
			*bIsNonIntersecting = (CountA == 1) && (CountB == 1);
		}

		return true;
	}

	FIndex2i ev = ParentMesh->GetEdgeV(eid);
	int base_a = ev.A, base_b = ev.B;

	bool bASet = IsSetTriangle(et.A), bBSet = IsSetTriangle(et.B);
	if (!bASet || !bBSet) // if either triangle is unset, need different logic for checking if this is a seam
	{
		return (bASet || bBSet); // consider it a seam if only one is unset
	}

	FIndex3i Triangle0 = GetTriangle(et.A);
	FIndex3i BaseTriangle0(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
	int idx_base_a0 = BaseTriangle0.IndexOf(base_a);
	int idx_base_b0 = BaseTriangle0.IndexOf(base_b);

	FIndex3i Triangle1 = GetTriangle(et.B);
	FIndex3i BaseTriangle1(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);
	int idx_base_a1 = BaseTriangle1.IndexOf(base_a);
	int idx_base_b1 = BaseTriangle1.IndexOf(base_b);

	int el_a_tri0 = Triangle0[idx_base_a0];
	int el_b_tri0 = Triangle0[idx_base_b0];
	int el_a_tri1 = Triangle1[idx_base_a1];
	int el_b_tri1 = Triangle1[idx_base_b1];

	bool bIsSeam = !IndexUtil::SamePairUnordered(el_a_tri0, el_b_tri0, el_a_tri1, el_b_tri1);

	if (bIsNonIntersecting != nullptr)
	{
		if ((el_a_tri0 == el_a_tri1 || el_a_tri0 == el_b_tri1) || (el_b_tri0 == el_b_tri1 || el_b_tri0 == el_a_tri1))
		{
			// seam edge "intersects" with end of the seam
			*bIsNonIntersecting = false;
		}
		else
		{
			// check that exactly two elements are associated with the vertices at each end of the edge
			int CountA = CountVertexElements(base_a);
			int CountB = CountVertexElements(base_b);

			// will be false if another seam intersects is adjacent to either end of the seam edge
			*bIsNonIntersecting = (CountA == 2) && (CountB == 2);
		}
	}
	
	return bIsSeam;
	


	// TODO: this doesn't seem to work but it should, and would be more efficient:
	//   - add ParentMesh->FindTriEdgeIndex(tid,eid)
	//   - SamePairUnordered query could directly index into ElementTriangles[]
	//FIndex3i TriangleA = GetTriangle(et.A);
	//FIndex3i TriangleB = GetTriangle(et.B);

	//FIndex3i BaseTriEdgesA = ParentMesh->GetTriEdges(et.A);
	//int WhichA = (BaseTriEdgesA.A == eid) ? 0 :
	//	((BaseTriEdgesA.B == eid) ? 1 : 2);

	//FIndex3i BaseTriEdgesB = ParentMesh->GetTriEdges(et.B);
	//int WhichB = (BaseTriEdgesB.A == eid) ? 0 :
	//	((BaseTriEdgesB.B == eid) ? 1 : 2);

	//return SamePairUnordered(
	//	TriangleA[WhichA], TriangleA[(WhichA + 1) % 3],
	//	TriangleB[WhichB], TriangleB[(WhichB + 1) % 3]);
}


template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::IsSeamEndEdge(int eid) const
{
	if (ParentMesh->IsEdge(eid) == false)
	{
		return false;
	}

	FIndex2i et = ParentMesh->GetEdgeT(eid);
	if (et.B == FDynamicMesh3::InvalidID)
	{
		return false;
	}

	FIndex2i ev = ParentMesh->GetEdgeV(eid);
	int base_a = ev.A, base_b = ev.B;

	bool bASet = IsSetTriangle(et.A), bBSet = IsSetTriangle(et.B);
	if (!bASet || !bBSet) 
	{
		return false; 
	}

	FIndex3i Triangle0 = GetTriangle(et.A);
	FIndex3i BaseTriangle0(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
	int idx_base_a0 = BaseTriangle0.IndexOf(base_a);
	int idx_base_b0 = BaseTriangle0.IndexOf(base_b);

	FIndex3i Triangle1 = GetTriangle(et.B);
	FIndex3i BaseTriangle1(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);
	int idx_base_a1 = BaseTriangle1.IndexOf(base_a);
	int idx_base_b1 = BaseTriangle1.IndexOf(base_b);

	int el_a_tri0 = Triangle0[idx_base_a0];
	int el_b_tri0 = Triangle0[idx_base_b0];
	int el_a_tri1 = Triangle1[idx_base_a1];
	int el_b_tri1 = Triangle1[idx_base_b1];

	bool bIsSeam = !IndexUtil::SamePairUnordered(el_a_tri0, el_b_tri0, el_a_tri1, el_b_tri1);

	bool bIsSeamEnd = false;
	if (bIsSeam)
	{
		// is only one of elements split?
		if ((el_a_tri0 == el_a_tri1 || el_a_tri0 == el_b_tri1) || (el_b_tri0 == el_b_tri1 || el_b_tri0 == el_a_tri1))
		{
			bIsSeamEnd = true;
		}
	}

	return bIsSeamEnd;
}

template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::HasInteriorSeamEdges() const
{
	for (int eid : ParentMesh->EdgeIndicesItr())
	{
		FIndex2i et = ParentMesh->GetEdgeT(eid);
		if (et.B != FDynamicMesh3::InvalidID)
		{
			bool bASet = IsSetTriangle(et.A), bBSet = IsSetTriangle(et.B);
			if (bASet != bBSet)
			{
				// seam between triangles with elements and triangles without
				return true;
			}
			else if (!bASet)
			{
				// neither triangle has set elements
				continue;
			}
			FIndex2i ev = ParentMesh->GetEdgeV(eid);
			int base_a = ev.A, base_b = ev.B;

			FIndex3i Triangle0 = GetTriangle(et.A);
			FIndex3i BaseTriangle0(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
			int idx_base_a1 = BaseTriangle0.IndexOf(base_a);
			int idx_base_b1 = BaseTriangle0.IndexOf(base_b);

			FIndex3i Triangle1 = GetTriangle(et.B);
			FIndex3i BaseTriangle1(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);
			int idx_base_a2 = BaseTriangle1.IndexOf(base_a);
			int idx_base_b2 = BaseTriangle1.IndexOf(base_b);

			if (!IndexUtil::SamePairUnordered(Triangle0[idx_base_a1], Triangle0[idx_base_b1], Triangle1[idx_base_a2], Triangle1[idx_base_b2]))
			{
				return true;
			}
		}
	}
	return false;
}



template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::IsSeamVertex(int vid, bool bBoundaryIsSeam) const
{
	// @todo can we do this more efficiently? At minimum we are looking up each triangle twice...
	for (int edgeid : ParentMesh->VtxEdgesItr(vid))
	{
		if (!bBoundaryIsSeam && ParentMesh->IsBoundaryEdge(edgeid))
		{
			continue;
		}
		if (IsSeamEdge(edgeid))
		{
			return true;
		}
	}
	return false;
}

template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::IsSeamIntersectionVertex(int32 VertexID) const
{
	int32 SeamCount = 0;
	for (int32 EdgeID : ParentMesh->VtxEdgesItr(VertexID))
	{
		SeamCount += int32(IsSeamEdge(EdgeID));
	}
	return SeamCount == 1 || SeamCount > 2;
}


template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::IsBowtieInOverlay(int32 VertexID) const
{
	// arrays for storing contiguous triangle groups from parentmesh
	FDynamicMesh3::FLocalIntArray TrianglesOut, ContiguousGroupLengths;
	FDynamicMesh3::FLocalBoolArray GroupIsLoop;

	// per-vertex element group tracking data, reused in loop below
	FDynamicMesh3::FLocalIntArray ElementIDSeen;
	FDynamicMesh3::FLocalIntArray GroupElementIDs, // stores element IDs of this vertex, 1:1 w/ contiguous triangles in the parent mesh
		SubGroupID, // mapping from GroupElementIDs indices into SubGroupElementIDs indices, giving subgroup membership per triangle
		SubGroupElementIDs; // 1:1 w/ 'subgroups' in the group (e.g. if all triangles in the group all had the same element ID, this would be an array of length 1, just containing that element ID)

	ensure(EMeshResult::Ok == ParentMesh->GetVtxContiguousTriangles(VertexID, TrianglesOut, ContiguousGroupLengths, GroupIsLoop));
	int32 NumTris = TrianglesOut.Num();

	auto ElementIDFromTriangle = [VertexID, this](int32 TriID) -> int32
	{
		FIndex3i TriVIDs = ParentMesh->GetTriangle(TriID);
		FIndex3i TriEIDs = GetTriangle(TriID);
		int SubIdx = TriVIDs.IndexOf(VertexID);
		return TriEIDs[SubIdx];
	};

	// Handle the easy case of 1 element, 1 contiguous group
	if (NumTris > 0 && ContiguousGroupLengths.Num() == 1)
	{
		int32 FirstElSeen = ElementIDFromTriangle(TrianglesOut[0]);
		bool bSingleElement = true;
		for (int32 Idx = 1; Idx < TrianglesOut.Num(); ++Idx)
		{
			if (FirstElSeen != ElementIDFromTriangle(TrianglesOut[Idx]))
			{
				bSingleElement = false;
			}
		}
		if (bSingleElement)
		{
			return false;
		}
	}

	// More complex case: Iterate through each contiguous group looking for a re-used element across groups
	ElementIDSeen.Reset();
	// per contiguous group of triangles around vertex in ParentMesh, find contiguous sub-groups in overlay
	for (int32 GroupIdx = 0, NumGroups = ContiguousGroupLengths.Num(), TriSubStart = 0; GroupIdx < NumGroups; GroupIdx++)
	{
		bool bIsLoop = GroupIsLoop[GroupIdx];
		int TriInGroupNum = ContiguousGroupLengths[GroupIdx];
		if (ensure(TriInGroupNum > 0) == false)
		{
			continue;
		}
		int TriSubEnd = TriSubStart + TriInGroupNum;

		GroupElementIDs.Reset();
		for (int TriSubIdx = TriSubStart; TriSubIdx < TriSubEnd; TriSubIdx++)
		{
			int TriID = TrianglesOut[TriSubIdx];
			GroupElementIDs.Add(ElementIDFromTriangle(TriID));
		}

		auto IsConnected = [this, &GroupElementIDs, &TrianglesOut, &TriSubStart](int TriOutIdxA, int TriOutIdxB)
		{
			if (GroupElementIDs[TriOutIdxA - TriSubStart] != GroupElementIDs[TriOutIdxB - TriSubStart])
			{
				return false;
			}
			int EdgeID = ParentMesh->FindEdgeFromTriPair(TrianglesOut[TriOutIdxA], TrianglesOut[TriOutIdxB]);
			return EdgeID >= 0 && !IsSeamEdge(EdgeID);
		};

		SubGroupID.Reset(); SubGroupID.SetNum(TriInGroupNum);
		SubGroupElementIDs.Reset();
		int MaxSubID = 0;
		SubGroupID[0] = 0;
		SubGroupElementIDs.Add(GroupElementIDs[0]);

		// Iterate through tris in current group, except last one
		for (int TriSubIdx = TriSubStart; TriSubIdx + 1 < TriSubEnd; TriSubIdx++)
		{
			if (!IsConnected(TriSubIdx, TriSubIdx + 1))
			{
				SubGroupElementIDs.Add(GroupElementIDs[TriSubIdx + 1 - TriSubStart]);
				MaxSubID++;
			}
			SubGroupID[TriSubIdx - TriSubStart + 1] = MaxSubID;
		}
		// if group was a loop, need to check if the last sub-group and first sub-group were actually the same group
		if (bIsLoop && MaxSubID > 0 && IsConnected(TriSubStart, TriSubStart + TriInGroupNum - 1))
		{
			int LastGroupID = SubGroupID.Last();
			for (int32 Idx = SubGroupID.Num() - 1; Idx >= 0 && SubGroupID[Idx] == LastGroupID; Idx--)
			{
				SubGroupID[Idx] = 0;
			}
			MaxSubID--;
			SubGroupElementIDs.Pop(EAllowShrinking::No);
		}

		for (int SubID = 0; SubID < SubGroupElementIDs.Num(); SubID++)
		{
			int ElementID = SubGroupElementIDs[SubID];
			if (ElementID < 0)
			{
				continue;		// skip if this is an invalid ElementID (eg from an invalid triangle)
			}
			// split needed the *second* time we see a sub-group using a given ElementID
			if (ElementIDSeen.Contains(ElementID))
			{
				return true;
			}
			else
			{
				ElementIDSeen.Add(ElementID);
			}
		}

		TriSubStart = TriSubEnd;
	}

	return false;
}


template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::AreTrianglesConnected(int TriangleID0, int TriangleID1) const
{
	FIndex3i NbrTris = ParentMesh->GetTriNeighbourTris(TriangleID0);
	int NbrIndex = IndexUtil::FindTriIndex(TriangleID1, NbrTris);
	if (NbrIndex != IndexConstants::InvalidID)
	{
		FIndex3i TriEdges = ParentMesh->GetTriEdges(TriangleID0);
		return IsSeamEdge(TriEdges[NbrIndex]) == false;
	}
	return false;
}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::GetVertexElements(int vid, TArray<int>& OutElements) const
{
	OutElements.Reset();
	for (int tid : ParentMesh->VtxTrianglesItr(vid))
	{
		if (!IsSetTriangle(tid))
		{
			continue;
		}
		FIndex3i Triangle = GetTriangle(tid);
		for (int j = 0; j < 3; ++j)
		{
			if (ParentVertices[Triangle[j]] == vid)
			{
				OutElements.AddUnique(Triangle[j]);
			}
		}
	}
}

template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::FindAnyElementIDAtVertex(int32 VertexID, int32& OutElementID) const
{
	OutElementID = IndexConstants::InvalidID;
	for (int TID : ParentMesh->VtxTrianglesItr(VertexID))
	{
		if (!IsSetTriangle(TID))
		{
			continue;
		}
		FIndex3i Triangle = GetTriangle(TID);
		for (int SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			if (ParentVertices[Triangle[SubIdx]] == VertexID)
			{
				OutElementID = Triangle[SubIdx];
				return true;
			}
		}
	}
	return false;
}




template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::CountVertexElements(int vid, bool bBruteForce) const
{
	FDynamicMesh3::FLocalIntArray VertexElements;
	FIndex3i Triangle;
	if (bBruteForce) 
	{
		for (int tid : ParentMesh->TriangleIndicesItr())
		{
			if (GetTriangleIfValid(tid, Triangle))
			{
				for (int j = 0; j < 3; ++j)
				{
					if (ParentVertices[Triangle[j]] == vid)
					{
						VertexElements.AddUnique(Triangle[j]);
					}
				}
			}
		}
	}
	else
	{
		for (int tid : ParentMesh->VtxTrianglesItr(vid))
		{
			if (GetTriangleIfValid(tid, Triangle))
			{
				for (int j = 0; j < 3; ++j)
				{
					if (ParentVertices[Triangle[j]] == vid)
					{
						VertexElements.AddUnique(Triangle[j]);
					}
				}
			}
		}
	}

	return VertexElements.Num();
}




template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::GetElementTriangles(int ElementID, TArray<int>& OutTriangles) const
{
	checkSlow(ElementsRefCounts.IsValid(ElementID));
	if (ElementsRefCounts.IsValid(ElementID))
	{
		int VertexID = ParentVertices[ElementID];

		for (int TriangleID : ParentMesh->VtxTrianglesItr(VertexID))
		{
			int i = 3 * TriangleID;
			if (ElementTriangles[i] == ElementID || ElementTriangles[i+1] == ElementID || ElementTriangles[i+2] == ElementID)
			{
				OutTriangles.Add(TriangleID);
			}
		}
	}
}

template<typename RealType, int ElementSize>
int TDynamicMeshOverlay<RealType, ElementSize>::GetElementIDAtVertex(int TriangleID, int VertexID) const
{
	FIndex3i Triangle = GetTriangle(TriangleID);
	// Check first element ID to see if triangle is unset (Note: triangles must be fully set or fully unset, so checking first element should be enough)
	if (Triangle.A == FDynamicMesh3::InvalidID)
	{
		return FDynamicMesh3::InvalidID;
	}
	for (int IDX = 0; IDX < 3; ++IDX)
	{	
		int ElementID = Triangle[IDX];
		if (ParentVertices[ElementID] == VertexID)
		{
			return ElementID;
		}
	}
	
	checkSlow(false);
	return FDynamicMesh3::InvalidID;
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnRemoveTriangle(int TriangleID)
{
	FIndex3i Triangle = GetTriangle(TriangleID);
	if (Triangle.A < 0 && Triangle.B < 0 && Triangle.C < 0)
	{
		// if whole triangle has no overlay vertices set, that's OK, just remove nothing
		// (if only *some* of the triangle vertices were < 0, that would be a bug / invalid overlay triangle)
		return;
	}
	InitializeNewTriangle(TriangleID);

	// decrement element refcounts, and free element if it is now unreferenced
	for (int j = 0; j < 3; ++j) 
	{
		int elemid = Triangle[j];
		ElementsRefCounts.Decrement(elemid);
		if (ElementsRefCounts.GetRefCount(elemid) == 1) 
		{
			ElementsRefCounts.Decrement(elemid);
			ParentVertices[elemid] = FDynamicMesh3::InvalidID;
			ensure(ElementsRefCounts.IsValid(elemid) == false);
		}
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnReverseTriOrientation(int TriangleID)
{
	FIndex3i Triangle = GetTriangle(TriangleID);
	int i = 3 * TriangleID;
	ElementTriangles[i] = Triangle[1];			// mirrors order in FDynamicMesh3::ReverseTriOrientationInternal
	ElementTriangles[i + 1] = Triangle[0];
	ElementTriangles[i + 2] = Triangle[2];
}




template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnSplitEdge(const FDynamicMesh3::FEdgeSplitInfo& splitInfo)
{
	int orig_t0 = splitInfo.OriginalTriangles.A;
	int orig_t1 = splitInfo.OriginalTriangles.B;
	int base_a = splitInfo.OriginalVertices.A;
	int base_b = splitInfo.OriginalVertices.B;

	// special handling if either triangle is unset
	bool bT0Set = IsSetTriangle(orig_t0), bT1Set = orig_t1 >= 0 && IsSetTriangle(orig_t1);
	// insert invalid triangle as needed
	if (!bT0Set)
	{
		InitializeNewTriangle(splitInfo.NewTriangles.A);
	}
	if (!bT1Set && splitInfo.NewTriangles.B >= 0)
	{
		// new triangle is invalid
		InitializeNewTriangle(splitInfo.NewTriangles.B);
	}
	// if neither tri was set, nothing else to do
	if (!bT0Set && !bT1Set)
	{
		return;
	}

	// look up current triangle 0, and infer base triangle 0
	FIndex3i Triangle0(-1, -1, -1);
	int idx_base_a1 = -1, idx_base_b1 = -1;
	int NewElemID = -1;
	if (bT0Set)
	{
		Triangle0 = GetTriangle(orig_t0);
		FIndex3i BaseTriangle0(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
		idx_base_a1 = BaseTriangle0.IndexOf(base_a);
		idx_base_b1 = BaseTriangle0.IndexOf(base_b);
		int idx_base_c = IndexUtil::GetOtherTriIndex(idx_base_a1, idx_base_b1);

		// create new element at lerp position
		NewElemID = AppendElement((RealType)0);
		SetElementFromLerp(NewElemID, Triangle0[idx_base_a1], Triangle0[idx_base_b1], (double)splitInfo.SplitT);

		// rewrite triangle 0
		ElementTriangles[3 * orig_t0 + idx_base_b1] = NewElemID;

		// create new triangle 2 w/ correct winding order
		FIndex3i NewTriangle2(NewElemID, Triangle0[idx_base_b1], Triangle0[idx_base_c]);  // mirrors DMesh3::SplitEdge [f,b,c]
		InternalSetTriangle(splitInfo.NewTriangles.A, NewTriangle2, false);

		// update ref counts
		ElementsRefCounts.Increment(NewElemID, 2); // for the two tris on the T0 side
		ElementsRefCounts.Increment(Triangle0[idx_base_c]);
	}

	if (orig_t1 == FDynamicMesh3::InvalidID)
	{
		return;  // we are done if this is a boundary triangle
	}

	// look up current triangle1 and infer base triangle 1
	if (bT1Set)
	{
		FIndex3i Triangle1 = GetTriangle(orig_t1);
		FIndex3i BaseTriangle1(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);
		int idx_base_a2 = BaseTriangle1.IndexOf(base_a);
		int idx_base_b2 = BaseTriangle1.IndexOf(base_b);
		int idx_base_d = IndexUtil::GetOtherTriIndex(idx_base_a2, idx_base_b2);

		int OtherNewElemID = NewElemID;

		// if we don't have a shared edge, we need to create another new UV for the other side
		bool bHasSharedUVEdge = bT0Set && IndexUtil::SamePairUnordered(Triangle0[idx_base_a1], Triangle0[idx_base_b1], Triangle1[idx_base_a2], Triangle1[idx_base_b2]);
		if (bHasSharedUVEdge == false)
		{
			// create new element at lerp position
			OtherNewElemID = AppendElement((RealType)0);
			SetElementFromLerp(OtherNewElemID, Triangle1[idx_base_a2], Triangle1[idx_base_b2], (double)splitInfo.SplitT);
		}

		// rewrite triangle 1
		ElementTriangles[3 * orig_t1 + idx_base_b2] = OtherNewElemID;

		// create new triangle 3 w/ correct winding order
		FIndex3i NewTriangle3(OtherNewElemID, Triangle1[idx_base_d], Triangle1[idx_base_b2]);  // mirrors DMesh3::SplitEdge [f,d,b]
		InternalSetTriangle(splitInfo.NewTriangles.B, NewTriangle3, false);

		// update ref counts
		ElementsRefCounts.Increment(OtherNewElemID, 2);
		ElementsRefCounts.Increment(Triangle1[idx_base_d]);
	}

}




template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnFlipEdge(const FDynamicMesh3::FEdgeFlipInfo& FlipInfo)
{
	int orig_t0 = FlipInfo.Triangles.A;
	int orig_t1 = FlipInfo.Triangles.B;
	bool bT0Set = IsSetTriangle(orig_t0), bT1Set = IsSetTriangle(orig_t1);
	int32 NumSet = (int32)bT0Set + (int32)bT1Set;
	if (NumSet == 0)
	{
		return; // nothing to do on the overlay if both triangles are unset
	}
	else if (NumSet == 1)
	{
		ensure(false); // flipping across a set/unset boundary is not allowed?
		// recover by just unsetting both triangles, since it's too late to prevent the flip
		UnsetTriangle(orig_t0);
		UnsetTriangle(orig_t1);
		return;
	}

	int base_a = FlipInfo.OriginalVerts.A;
	int base_b = FlipInfo.OriginalVerts.B;
	int base_c = FlipInfo.OpposingVerts.A;
	int base_d = FlipInfo.OpposingVerts.B;

	// look up triangle 0
	FIndex3i Triangle0 = GetTriangle(orig_t0);
	FIndex3i BaseTriangle0(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
	int idx_base_a1 = BaseTriangle0.IndexOf(base_a);
	int idx_base_b1 = BaseTriangle0.IndexOf(base_b);
	int idx_base_c = IndexUtil::GetOtherTriIndex(idx_base_a1, idx_base_b1);

	// look up triangle 1 (must exist because base mesh would never flip a boundary edge)
	FIndex3i Triangle1 = GetTriangle(orig_t1);
	FIndex3i BaseTriangle1(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);
	int idx_base_a2 = BaseTriangle1.IndexOf(base_a);
	int idx_base_b2 = BaseTriangle1.IndexOf(base_b);
	int idx_base_d = IndexUtil::GetOtherTriIndex(idx_base_a2, idx_base_b2);

	// sanity checks
	checkSlow(idx_base_c == BaseTriangle0.IndexOf(base_c));
	checkSlow(idx_base_d == BaseTriangle1.IndexOf(base_d));

	// we should not have been called on a non-shared edge!!
	bool bHasSharedUVEdge = (Triangle0[idx_base_a1] == Triangle1[idx_base_a2]) && (Triangle0[idx_base_b1] == Triangle1[idx_base_b2]);
	checkSlow(bHasSharedUVEdge);

	int A = Triangle0[idx_base_a1];
	int B = Triangle0[idx_base_b1];
	int C = Triangle0[idx_base_c];
	int D = Triangle1[idx_base_d];

	// set triangles to same index order as in FDynamicMesh::FlipEdge
	int i0 = 3 * orig_t0;
	ElementTriangles[i0] = C; ElementTriangles[i0+1] = D; ElementTriangles[i0+2] = B;
	int i1 = 3 * orig_t1;
	ElementTriangles[i1] = D; ElementTriangles[i1+1] = C; ElementTriangles[i1+2] = A;

	// update reference counts
	ElementsRefCounts.Increment(C);
	ElementsRefCounts.Increment(D);
	if (bHasSharedUVEdge)
	{
		ElementsRefCounts.Decrement(A);
		ElementsRefCounts.Decrement(B);
	}
	else // flipped a seam edge case
	{
		// ideally we wouldn't flip a seam edge, but if we have done so, may need to clean up a now-unreferenced element
		// (note it's too late to prevent the flip here)
		int A2 = Triangle1[idx_base_a2];
		int B2 = Triangle1[idx_base_b2];

		ElementsRefCounts.Decrement(A2);
		// free A2 if it's no longer referenced (can happen in the A != A2 case)
		if (ElementsRefCounts.GetRefCount(A2) == 1)
		{
			ElementsRefCounts.Decrement(A2);
			ParentVertices[A2] = FDynamicMesh3::InvalidID;
		}
		ElementsRefCounts.Decrement(B2);
		// free B2 if it's no longer referenced (can happen in the B != B2 case)
		if (ElementsRefCounts.GetRefCount(B2) == 1)
		{
			ElementsRefCounts.Decrement(B2);
			ParentVertices[B2] = FDynamicMesh3::InvalidID;
		}
	}
}






template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnCollapseEdge(const FDynamicMesh3::FEdgeCollapseInfo& collapseInfo)
{

	int tid_removed0 = collapseInfo.RemovedTris.A;
	int tid_removed1 = collapseInfo.RemovedTris.B;
	bool bT0Set = IsSetTriangle(tid_removed0), bT1Set = tid_removed1 >= 0 && IsSetTriangle(tid_removed1);

	int vid_base_kept = collapseInfo.KeptVertex;
	int vid_base_removed = collapseInfo.RemovedVertex;
	

	bool bIsSeam = false;
	bool bIsSeamEnd = false;

	
	// look up triangle 0
	FIndex3i Triangle0(-1,-1,-1), BaseTriangle0(-1,-1,-1);
	int idx_removed_tri0 = -1, idx_kept_tri0 = -1;
	if (bT0Set)
	{
		Triangle0 = GetTriangle(tid_removed0);
		BaseTriangle0 = FIndex3i(ParentVertices[Triangle0.A], ParentVertices[Triangle0.B], ParentVertices[Triangle0.C]);
		idx_kept_tri0 = BaseTriangle0.IndexOf(vid_base_kept);
		idx_removed_tri0    = BaseTriangle0.IndexOf(vid_base_removed);
	}

	// look up triangle 1 if this is not a boundary edge
	FIndex3i Triangle1(-1,-1,-1), BaseTriangle1(-1,-1,-1);
	int idx_removed_tri1 = -1, idx_kept_tri1 = -1;
	if (collapseInfo.bIsBoundary == false && bT1Set)
	{
		Triangle1 = GetTriangle(tid_removed1);
		BaseTriangle1 = FIndex3i(ParentVertices[Triangle1.A], ParentVertices[Triangle1.B], ParentVertices[Triangle1.C]);

		idx_kept_tri1 = BaseTriangle1.IndexOf(vid_base_kept);
		idx_removed_tri1 = BaseTriangle1.IndexOf(vid_base_removed);

		if (bT0Set)
		{
			int el_kept_tri0    = Triangle0[idx_kept_tri0];
			int el_removed_tri0 = Triangle0[idx_removed_tri0];
			int el_kept_tri1    = Triangle1[idx_kept_tri1];
			int el_removed_tri1 = Triangle1[idx_removed_tri1];

			// is this a seam?
			bIsSeam = !IndexUtil::SamePairUnordered(el_kept_tri0, el_removed_tri0, el_kept_tri1, el_removed_tri1);

			if (bIsSeam)
			{
				// is only one of elements split?
				if ((el_kept_tri0 == el_kept_tri1 || el_kept_tri0 == el_removed_tri1) || (el_removed_tri0 == el_kept_tri1 || el_removed_tri0 == el_removed_tri1))
				{
					bIsSeamEnd = true;
				}
			}
		}
	}

	// this should be protected against by calling code.
	//checkSlow(!bIsSeamEnd);

	// need to find the elementid for the "kept" and "removed" vertices that are connected by the edges of T0 and T1.
	// If this edge is :
	//       not a seam - just one kept and one removed element.
	//       a seam end - one (two) kept and two (one) removed elements.
	//       a seam     - two kept and two removed elements.
	// The collapse of a seam end must be protected against by the higher-level code.
	//     There is no sensible way to handle the collapse of a seam end.  Retaining two elements would require some arbitrary split
	//     of the removed element and conversely if one element is retained there is no reason to believe a single element value 
	//     would be a good approximation to collapsing the edges on both sides of the seam end.
	int kept_elemid[2]    = { FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID };
	int removed_elemid[2] = { FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID };
	bool bFoundRemovedElement[2] = { false,false };
	bool bFoundKeptElement[2]    = { false, false };
	if (bT0Set)
	{
		kept_elemid[0] = Triangle0[idx_kept_tri0];
		removed_elemid[0] = Triangle0[idx_removed_tri0];
		bFoundKeptElement[0] = bFoundRemovedElement[0] = true;

		checkSlow(kept_elemid[0] != FDynamicMesh3::InvalidID);
		checkSlow(removed_elemid[0] != FDynamicMesh3::InvalidID);
		
	}
	if ((bIsSeam || !bT0Set) && bT1Set)
	{
		kept_elemid[1] = Triangle1[idx_kept_tri1];
		removed_elemid[1] = Triangle1[idx_removed_tri1];
		bFoundKeptElement[1] = bFoundRemovedElement[1] = true;

		checkSlow(kept_elemid[1] != FDynamicMesh3::InvalidID);
		checkSlow(removed_elemid[1] != FDynamicMesh3::InvalidID);
		
	}

	// update value of kept elements
	for (int i = 0; i < 2; ++i)
	{
		if (kept_elemid[i] == FDynamicMesh3::InvalidID || removed_elemid[i] == FDynamicMesh3::InvalidID)
		{
			continue;
		}

		// Guard against seam-end edge collapse case or otherwise corrupted mesh data
		if (!ensure(IsElement(kept_elemid[i]) && IsElement(removed_elemid[i])))
		{
			continue;
		}

		SetElementFromLerp(kept_elemid[i], kept_elemid[i], removed_elemid[i], (double)collapseInfo.CollapseT);
	}

	// Helper for detaching from elements further below. Technically, the freeing gets done for us if the 
	// triangle unset call is the last detachment (which it should be as long as only elements on a removed 
	// overlay edge are ones that are removed), but it is saner to have it.
	auto DecrementAndFreeIfLast = [this](int32 elem_id)
	{
		ElementsRefCounts.Decrement(elem_id);
		if (ElementsRefCounts.GetRefCount(elem_id) == 1)
		{
			ElementsRefCounts.Decrement(elem_id);
			ParentVertices[elem_id] = FDynamicMesh3::InvalidID;
		}
	};


	// Look for still-existing triangles that have elements linked to the removed vertex and update them.
	// Note that this has to happen even if both triangles were unset, as the removed vertex may have had
	// other elements associated with it, so we need to look at its triangles (which are now attached to
	// vid_base_kept).
	for (int onering_tid : ParentMesh->VtxTrianglesItr(vid_base_kept))
	{
		if (!IsSetTriangle(onering_tid))
		{
			continue;
		}
		FIndex3i elem_tri = GetTriangle(onering_tid);
		for (int j = 0; j < 3; ++j)
		{
			int elem_id = elem_tri[j];
			if (ParentVertices[elem_id] == vid_base_removed)
			{
				if (elem_id == removed_elemid[0])
				{
					ElementTriangles[3 * onering_tid + j] = kept_elemid[0];
					if (bFoundKeptElement[0])
					{
						ElementsRefCounts.Increment(kept_elemid[0]);
					}
					DecrementAndFreeIfLast(elem_id);
				}
				else if (elem_id == removed_elemid[1])
				{
					ElementTriangles[3 * onering_tid + j] = kept_elemid[1];
					if (bFoundKeptElement[1])
					{
						ElementsRefCounts.Increment(kept_elemid[1]);
					}
					DecrementAndFreeIfLast(elem_id);
				}
				else
				{
					// this could happen if a split edge is adjacent to the edge we collapse
					ParentVertices[elem_id] = vid_base_kept;
				}
			}
		}
	}


	// clear the two triangles we removed
	if (bT0Set)
	{
		UnsetTriangle(tid_removed0, true);
	}
	if (collapseInfo.bIsBoundary == false && bT1Set)
	{
		UnsetTriangle(tid_removed1, true);
	}

	// if the edge was split, but still shared one element, this should be protected against in the calling code
	if (removed_elemid[1] == removed_elemid[0])
	{
		removed_elemid[1] = FDynamicMesh3::InvalidID;
	}

	// Note: the elements associated with the removed vertex should have been removed in the iteration or triangle unsetting above
#if UE_BUILD_DEBUG
	for (int k = 0; k < 2; ++k)
	{
		if (removed_elemid[k] != FDynamicMesh3::InvalidID)
		{
			int rc = ElementsRefCounts.GetRefCount(removed_elemid[k]);
			checkSlow(rc == 0);
		}
	}
#endif

}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnPokeTriangle(const FDynamicMesh3::FPokeTriangleInfo& PokeInfo)
{
	if (!IsSetTriangle(PokeInfo.OriginalTriangle))
	{
		InitializeNewTriangle(PokeInfo.NewTriangles.A);
		InitializeNewTriangle(PokeInfo.NewTriangles.B);
		return;
	}

	FIndex3i Triangle = GetTriangle(PokeInfo.OriginalTriangle);

	// create new element at barycentric position
	int CenterElemID = AppendElement((RealType)0);
	FVector3d BaryCoords((double)PokeInfo.BaryCoords.X, (double)PokeInfo.BaryCoords.Y, (double)PokeInfo.BaryCoords.Z);
	SetElementFromBary(CenterElemID, Triangle[0], Triangle[1], Triangle[2], BaryCoords);

	// update orig triangle and two new ones. Winding orders here mirror FDynamicMesh3::PokeTriangle
	InternalSetTriangle(PokeInfo.OriginalTriangle, FIndex3i(Triangle[0], Triangle[1], CenterElemID), false );
	InternalSetTriangle(PokeInfo.NewTriangles.A, FIndex3i(Triangle[1], Triangle[2], CenterElemID), false);
	InternalSetTriangle(PokeInfo.NewTriangles.B, FIndex3i(Triangle[2], Triangle[0], CenterElemID), false);

	ElementsRefCounts.Increment(Triangle[0]);
	ElementsRefCounts.Increment(Triangle[1]);
	ElementsRefCounts.Increment(Triangle[2]);
	ElementsRefCounts.Increment(CenterElemID, 3);
}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnMergeEdges(const FDynamicMesh3::FMergeEdgesInfo& MergeInfo)
{
	// MergeEdges just merges vertices. For now we will not also merge UVs. So all we need to
	// do is rewrite the UV parent vertices

	for (int i = 0; i < 2; i++)
	{
		int KeptVID = MergeInfo.KeptVerts[i];
		int RemovedVID = MergeInfo.RemovedVerts[i];
		if (RemovedVID == FDynamicMesh3::InvalidID)
		{
			continue;
		}
		// this for loop is very similar to GetVertexElements() but accounts for the base mesh already being updated
		for (int TID : ParentMesh->VtxTrianglesItr(KeptVID)) // only care about triangles connected to the *new* vertex; these are updated
		{
			if (!IsSetTriangle(TID))
			{
				continue;
			}
			FIndex3i Triangle = GetTriangle(TID);
			for (int j = 0; j < 3; ++j)
			{
				// though the ParentMesh vertex is NewVertex in the source mesh, it is still OriginalVertex in the ParentVertices array (since that hasn't been updated yet)
				if (Triangle[j] != FDynamicMesh3::InvalidID && ParentVertices[Triangle[j]] == RemovedVID)
				{
					ParentVertices[Triangle[j]] = KeptVID;
				}
			}
		}
	}
}

template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnMergeVertices(const FDynamicMesh3::FMergeVerticesInfo& MergeInfo)
{
	// Attributes don't change, but one of the vertices got removed, so any elements on that vertex
	//  should be reparented to the kept vertex. We have to iterate around the kept vertex since
	//  ParentMesh is already updated.
	for (int Tid : ParentMesh->VtxTrianglesItr(MergeInfo.KeptVertex))
	{
		if (!IsSetTriangle(Tid))
		{
			continue;
		}
		FIndex3i Triangle = GetTriangle(Tid);
		for (int j = 0; j < 3; ++j)
		{
			if (Triangle[j] != FDynamicMesh3::InvalidID && ParentVertices[Triangle[j]] == MergeInfo.RemovedVertex)
			{
				ParentVertices[Triangle[j]] = MergeInfo.KeptVertex;
			}
		}
	}
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate)
{
	TArray<int> OutElements;

	// this for loop is very similar to GetVertexElements() but accounts for the base mesh already being updated
	for (int tid : ParentMesh->VtxTrianglesItr(SplitInfo.NewVertex)) // only care about triangles connected to the *new* vertex; these are updated
	{
		if (!IsSetTriangle(tid))
		{
			continue;
		}
		FIndex3i Triangle = GetTriangle(tid);
		for (int j = 0; j < 3; ++j)
		{
			// though the ParentMesh vertex is NewVertex in the source mesh, it is still OriginalVertex in the ParentVertices array (since that hasn't been updated yet)
			if (Triangle[j] != FDynamicMesh3::InvalidID && ParentVertices[Triangle[j]] == SplitInfo.OriginalVertex)
			{
				OutElements.AddUnique(Triangle[j]);
			}
		}
	}

	for (int ElementID : OutElements)
	{
		// Note: TrianglesToUpdate will include triangles that don't include the element, but that's ok; it just won't find any elements to update for those
		//			(and this should be cheaper than constructing a new array for every element)
		SplitElementWithNewParent(ElementID, SplitInfo.NewVertex, TrianglesToUpdate);
	}
}



template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::SetElementFromLerp(int SetElement, int ElementA, int ElementB, double Alpha)
{
	int IndexSet = ElementSize * SetElement;
	int IndexA = ElementSize * ElementA;
	int IndexB = ElementSize * ElementB;
	double Beta = ((double)1 - Alpha);
	for (int i = 0; i < ElementSize; ++i)
	{
		double LerpValue =  Beta*(double)Elements[IndexA+i] + Alpha*(double)Elements[IndexB+i];
		Elements[IndexSet+i] = (RealType)LerpValue;
	}
}

template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::SetElementFromBary(int SetElement, int ElementA, int ElementB, int ElementC, const FVector3d& BaryCoords)
{
	int IndexSet = ElementSize * SetElement;
	int IndexA = ElementSize * ElementA;
	int IndexB = ElementSize * ElementB;
	int IndexC = ElementSize * ElementC;
	for (int i = 0; i < ElementSize; ++i)
	{
		double BaryValue = BaryCoords.X*(double)Elements[IndexA+i] + BaryCoords.Y*(double)Elements[IndexB+i] + BaryCoords.Z*(double)Elements[IndexC+i];
		Elements[IndexSet + i] = (RealType)BaryValue;			
	}
}



template<typename RealType, int ElementSize, typename VectorType>
bool TDynamicMeshVectorOverlay<RealType, ElementSize, VectorType>::EnumerateVertexElements(
	int VertexID,
	TFunctionRef<bool(int TriangleID, int ElementID, const VectorType& Value)> ProcessFunc,
	bool bFindUniqueElements) const
{
	if (this->ParentMesh->IsVertex(VertexID) == false) return false;

	TArray<int32, TInlineAllocator<16>> UniqueElements;

	int32 Count = 0;
	for (int tid : this->ParentMesh->VtxTrianglesItr(VertexID))
	{
		int32 BaseElemIdx = 3 * tid;
		bool bIsSetTriangle = (this->ElementTriangles[BaseElemIdx] >= 0);
		if (bIsSetTriangle)
		{
			Count++;

			for (int j = 0; j < 3; ++j)
			{
				int32 ElementIdx = this->ElementTriangles[BaseElemIdx + j];
				if (this->ParentVertices[ElementIdx] == VertexID)
				{
					int32 NumUnique = UniqueElements.Num();
					if (bFindUniqueElements == false || (UniqueElements.AddUnique(ElementIdx) == NumUnique) )
					{
						bool bContinue = ProcessFunc(tid, ElementIdx, this->GetElement(ElementIdx));
						if (!bContinue)
						{
							return true;
						}
					}
				}
			}

		}
	}
	return (Count > 0);
}






template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::CheckValidity(bool bAllowNonManifoldVertices, EValidityCheckFailMode FailMode) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("TDynamicMeshOverlay::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("TDynamicMeshOverlay::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}


	// @todo: check that all connected element-pairs are also edges in parentmesh

	// Check that the number of parent vertices is consistent with number of elements.
	CheckOrFailF(ParentVertices.Num() * ElementSize == Elements.Num());

	// Check that the per-triangle data matches the number of triangles in the parent mesh.
	CheckOrFailF(!ParentMesh || ElementTriangles.Num() == ParentMesh->MaxTriangleID() * 3);

	// check that parent vtx of a non-isolated element is actually a vertex
	for (int elemid : ElementIndicesItr())
	{
		int ParentVID = GetParentVertex(elemid);
		bool bParentIsVertex = ParentMesh->IsVertex(ParentVID);
		bool bElementIsIsolated = (ElementsRefCounts.GetRefCount(elemid) == 1);

		// bParentIsVertex XOR bElementIsIsolated
		if ( bParentIsVertex )
		{
			CheckOrFailF(!bElementIsIsolated);
		}
		else
		{
			CheckOrFailF(bElementIsIsolated);
		}
	}

	// check that parent vertices of each element triangle are the same as base triangle
	for (int tid : ParentMesh->TriangleIndicesItr())
	{
		FIndex3i ElemTri = GetTriangle(tid);
		FIndex3i BaseTri = ParentMesh->GetTriangle(tid);
		for (int j = 0; j < 3; ++j)
		{
			if (ElemTri[j] != FDynamicMesh3::InvalidID)
			{
				CheckOrFailF(GetParentVertex(ElemTri[j]) == BaseTri[j]);
			}
		}
	}

	// count references to each element
	TArray<int> RealRefCounts; RealRefCounts.Init(0, MaxElementID());
	for (int tid : ParentMesh->TriangleIndicesItr())
	{
		FIndex3i Tri = GetTriangle(tid);
		int ValidCount = 0;
		for (int j = 0; j < 3; ++j)
		{
			if (Tri[j] != FDynamicMesh3::InvalidID)
			{
				++ValidCount;
				RealRefCounts[Tri[j]] += 1;
			}
		}
		CheckOrFailF(ValidCount == 3 || ValidCount == 0); // element tri should be fully set or fully unset
	}
	// verify that refcount list counts are same as actual reference counts
	for (int32 ElementID = 0; ElementID < RealRefCounts.Num(); ++ElementID)
	{
		int32 CurRefCount = ElementsRefCounts.GetRefCount(ElementID);
		CheckOrFailF((RealRefCounts[ElementID] == 0 && CurRefCount == 0)
			|| (CurRefCount == RealRefCounts[ElementID] + 1));
	}

	return is_ok;
}


template<typename RealType, int ElementSize>
bool TDynamicMeshOverlay<RealType, ElementSize>::IsSameAs(const TDynamicMeshOverlay<RealType, ElementSize>& Other, bool bIgnoreDataLayout) const
{
	if (ElementsRefCounts.GetCount() != Other.ElementsRefCounts.GetCount())
	{
		return false;
	}

	if (!bIgnoreDataLayout || (ElementsRefCounts.IsDense() && Other.ElementsRefCounts.IsDense() && ParentMesh->MaxTriangleID() == ParentMesh->TriangleCount() &&
		Other.ParentMesh->MaxTriangleID() == Other.ParentMesh->TriangleCount()))
	{
		if (ElementsRefCounts.GetMaxIndex() != Other.ElementsRefCounts.GetMaxIndex() ||
			Elements.Num() != Other.Elements.Num() ||
			ParentVertices.Num() != Other.ParentVertices.Num() ||
			ElementTriangles.Num() != Other.ElementTriangles.Num())
		{
			return false;
		}

		if (Elements != Other.Elements)
		{
			return false;
		}

		for (int Idx = 0; Idx < ElementsRefCounts.GetMaxIndex(); Idx++)
		{
			if (ElementsRefCounts.GetRefCount(Idx) != Other.ElementsRefCounts.GetRefCount(Idx))
			{
				return false;
			}
		}

		if (ParentVertices != Other.ParentVertices)
		{
			return false;
		}

		if (ElementTriangles != Other.ElementTriangles)
		{
			return false;
		}
	}
	else
	{
		FRefCountVector::IndexIterator ItEid = ElementsRefCounts.BeginIndices();
		const FRefCountVector::IndexIterator ItEidEnd = ElementsRefCounts.EndIndices();
		FRefCountVector::IndexIterator ItEidOther = Other.ElementsRefCounts.BeginIndices();
		const FRefCountVector::IndexIterator ItEidEndOther = Other.ElementsRefCounts.EndIndices();

		TDynamicVector<int> EidMapping;
		EidMapping.Resize(ElementsRefCounts.GetMaxIndex(), FDynamicMesh3::InvalidID);

		while (ItEid != ItEidEnd && ItEidOther != ItEidEndOther)
		{
			for (int32 i = 0; i < ElementSize; ++i)
			{
				if (Elements[*ItEid * ElementSize + i] != Other.Elements[*ItEidOther * ElementSize + i])
				{
					// Element values are not the same.
					return false;
				}
			}

			if (ElementsRefCounts.GetRawRefCount(*ItEid) != Other.ElementsRefCounts.GetRefCount(*ItEidOther))
			{
				// Element ref counts are not the same.
				return false;
			}

			EidMapping[*ItEid] = *ItEidOther;

			++ItEid;
			++ItEidOther;
		}
		
		if (ItEid != ItEidEnd || ItEidOther != ItEidEndOther)
		{
			// Number of elements is not the same.
			return false;
		}
		
		if (ParentMesh->TriangleCount() != Other.ParentMesh->TriangleCount())
		{
			return false;
		}

		FRefCountVector::IndexIterator ItTid = ParentMesh->GetTrianglesRefCounts().BeginIndices();
		const FRefCountVector::IndexIterator ItTidEnd = ParentMesh->GetTrianglesRefCounts().EndIndices();
		FRefCountVector::IndexIterator ItTidOther = Other.ParentMesh->GetTrianglesRefCounts().BeginIndices();
		const FRefCountVector::IndexIterator ItTidEndOther = Other.ParentMesh->GetTrianglesRefCounts().EndIndices();

		while (ItTid != ItTidEnd && ItTidOther != ItTidEndOther)
		{
			const int ElementTrianglesIdx = *ItTid * 3;
			const int ElementTrianglesIdxOther = *ItTidOther * 3;

			for (int i = 0; i < 3; ++i)
			{
				const int Eid = ElementTriangles[ElementTrianglesIdx + i];
				const int EidOther = Other.ElementTriangles[ElementTrianglesIdxOther + i];
				if (EidOther != (Eid != FDynamicMesh3::InvalidID ? EidMapping[Eid] : FDynamicMesh3::InvalidID))
				{
					// Triangle elements do not index the same element value.
					return false;
				}
			}

			++ItTid;
			++ItTidOther;
		}

		if (ItTid != ItTidEnd || ItTidOther != ItTidEndOther)
		{
			// Number of element triangles is not the same.
			return false;
		}
	}

	return true;
}


template<typename RealType, int ElementSize>
void TDynamicMeshOverlay<RealType, ElementSize>::Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
	{
		Ar << ElementsRefCounts;
		Ar << Elements;
		Ar << ParentVertices;
		Ar << ElementTriangles;
	}
	else
	{
		auto SerializeVector = [](FArchive& Ar, auto& Vector, bool bUseCompression)
		{
			if (bUseCompression)
			{
				Vector.template Serialize<true, true>(Ar);
			}
			else
			{
				Vector.template Serialize<true, false>(Ar);
			}
		};
		
		TOptional<TArray<int>> ElementMap;

		if (Ar.IsLoading() || !CompactMaps || !CompactMaps->VertexMapIsSet())
		{
			ElementsRefCounts.Serialize(Ar, false, bUseCompression);

			SerializeVector(Ar, Elements, bUseCompression);
			SerializeVector(Ar, ParentVertices, bUseCompression);
		}
		else
		{
			const size_t NumElements = ElementsRefCounts.GetCount();

			FRefCountVector ElementsRefCountsCompact;
			ElementsRefCountsCompact.Trim(NumElements);
			TDynamicVector<unsigned short>& RawElementsRefCountsCompact = ElementsRefCountsCompact.GetRawRefCountsUnsafe();
			const TDynamicVector<unsigned short>& RawElementsRefCounts = ElementsRefCounts.GetRawRefCountsUnsafe();

			TDynamicVector<RealType> ElementsCompact;
			ElementsCompact.SetNum(NumElements * ElementSize);
			TDynamicVector<int> ParentVerticesCompact;
			ParentVerticesCompact.SetNum(NumElements);

			ElementMap.Emplace();
			ElementMap->Init(FDynamicMesh3::InvalidID, ElementsRefCounts.GetMaxIndex());
			TArray<int>& ElementMapping = *ElementMap;

			int EidCompact = 0;
			for (int Eid : ElementsRefCounts.Indices())
			{
				RawElementsRefCountsCompact[EidCompact] = RawElementsRefCounts[Eid];
				for (int32 i = 0; i < ElementSize; ++i)
				{
					ElementsCompact[EidCompact * ElementSize + i] = Elements[Eid * ElementSize + i];
				}
				ParentVerticesCompact[EidCompact] =
					(ParentVertices[Eid] >= 0) ? CompactMaps->GetVertexMapping(ParentVertices[Eid]) : FDynamicMesh3::InvalidID;
				ElementMapping[Eid] = EidCompact;
				++EidCompact;
			}

			ElementsRefCountsCompact.Serialize(Ar, true, bUseCompression);
			SerializeVector(Ar, ElementsCompact, bUseCompression);
			SerializeVector(Ar, ParentVerticesCompact, bUseCompression);
		}

		if (Ar.IsLoading() || ((!CompactMaps || !CompactMaps->TriangleMapIsSet()) && !ElementMap))
		{
			SerializeVector(Ar, ElementTriangles, bUseCompression);
		}
		else
		{
			TDynamicVector<int> ElementTrianglesCompact;
			ElementTrianglesCompact.SetNum(ParentMesh->TriangleCount() * 3);

			if (CompactMaps && CompactMaps->TriangleMapIsSet() && ElementMap)
			{
				TArray<int>& ElementMapping = *ElementMap;

				for (const int Tid : ParentMesh->TriangleIndicesItr())
				{
					const int TriIndex = Tid * 3;
					const int TriIndexCompact = CompactMaps->GetTriangleMapping(Tid) * 3;
					const int TriElement0 = ElementTriangles[TriIndex + 0];
					const int TriElement1 = ElementTriangles[TriIndex + 1];
					const int TriElement2 = ElementTriangles[TriIndex + 2];
					ElementTrianglesCompact[TriIndexCompact + 0] = TriElement0 != FDynamicMesh3::InvalidID
						                                               ? ElementMapping[TriElement0]
						                                               : FDynamicMesh3::InvalidID;
					ElementTrianglesCompact[TriIndexCompact + 1] = TriElement1 != FDynamicMesh3::InvalidID
						                                               ? ElementMapping[TriElement1]
						                                               : FDynamicMesh3::InvalidID;
					ElementTrianglesCompact[TriIndexCompact + 2] = TriElement2 != FDynamicMesh3::InvalidID
						                                               ? ElementMapping[TriElement2]
						                                               : FDynamicMesh3::InvalidID;
				}
			}
			else if (CompactMaps && CompactMaps->TriangleMapIsSet())
			{
				for (const int Tid : ParentMesh->TriangleIndicesItr())
				{
					const int TriIndex = Tid * 3;
					const int TriIndexCompact = CompactMaps->GetTriangleMapping(Tid) * 3;
					ElementTrianglesCompact[TriIndexCompact + 0] = ElementTriangles[TriIndex + 0];
					ElementTrianglesCompact[TriIndexCompact + 1] = ElementTriangles[TriIndex + 1];
					ElementTrianglesCompact[TriIndexCompact + 2] = ElementTriangles[TriIndex + 2];
				}
			}
			else
			{
				TArray<int>& ElementMapping = *ElementMap;

				for (const int Tid : ParentMesh->TriangleIndicesItr())
				{
					const int TriIndex = Tid * 3;
					const int TriElement0 = ElementTriangles[TriIndex + 0];
					const int TriElement1 = ElementTriangles[TriIndex + 1];
					const int TriElement2 = ElementTriangles[TriIndex + 2];
					ElementTrianglesCompact[TriIndex + 0] = TriElement0 != FDynamicMesh3::InvalidID ? ElementMapping[TriElement0] : FDynamicMesh3::InvalidID;
					ElementTrianglesCompact[TriIndex + 1] = TriElement1 != FDynamicMesh3::InvalidID ? ElementMapping[TriElement1] : FDynamicMesh3::InvalidID;
					ElementTrianglesCompact[TriIndex + 2] = TriElement2 != FDynamicMesh3::InvalidID ? ElementMapping[TriElement2] : FDynamicMesh3::InvalidID;
				}
			}

			SerializeVector(Ar, ElementTrianglesCompact, bUseCompression);
		}
	}
}


namespace UE
{
namespace Geometry
{

// These are explicit instantiations of the templates that are exported from the shared lib.
// Only these instantiations of the template can be used.
// This is necessary because we have placed most of the templated functions in this .cpp file, instead of the header.
template class GEOMETRYCORE_API TDynamicMeshOverlay<float, 1>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<double, 1>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<int, 1>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<float, 2>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<double, 2>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<int, 2>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<float, 3>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<double, 3>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<int, 3>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<float, 4>;
template class GEOMETRYCORE_API TDynamicMeshOverlay<double, 4>;

template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<float, 2, FVector2f>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<double, 2, FVector2d>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<int, 2, FVector2i>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<float, 3, FVector3f>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<double, 3, FVector3d>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<int, 3, FVector3i>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<float, 4, FVector4f>;
template class GEOMETRYCORE_API TDynamicMeshVectorOverlay<double, 4, FVector4d>;

} // end namespace UE::Geometry
} // end namespace UE