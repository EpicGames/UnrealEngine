// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/DynamicMeshSculptLayers.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;



int32 FDynamicMeshSculptLayers::SetActiveLayer(int32 LayerIndex)
{
	UpdateLayersFromMesh();
	ActiveLayer = LayerIndex;
	ValidateActiveLayer();
	return ActiveLayer;
}

bool FDynamicMeshSculptLayers::DiscardSculptLayer(int32 LayerIndex)
{
	if (!Layers.IsValidIndex(LayerIndex))
	{
		return false;
	}
	UpdateLayersFromMesh();
	Layers.RemoveAt(LayerIndex);
	LayerWeights.RemoveAt(LayerIndex);
	UpdateMeshFromLayers();
	ValidateActiveLayer();
	return true;
}

bool FDynamicMeshSculptLayers::MoveLayer(const int32 InitLayerIndex, const int32 TargetIndex)
{
	if (!Layers.IsValidIndex(InitLayerIndex) || !Layers.IsValidIndex(TargetIndex) || InitLayerIndex == TargetIndex)
	{
		return false;
	}
	UpdateLayersFromMesh();

	const int Dir = InitLayerIndex < TargetIndex ? 1 : -1;
	int32 CurrentMovedLayerIndex = InitLayerIndex;
	while (CurrentMovedLayerIndex != TargetIndex)
	{
		Layers.Swap(CurrentMovedLayerIndex, CurrentMovedLayerIndex  + Dir);
		LayerWeights.Swap(CurrentMovedLayerIndex, CurrentMovedLayerIndex + Dir);
		CurrentMovedLayerIndex += Dir;
	}

	UpdateMeshFromLayers();
	ValidateActiveLayer();
	return true;
}

bool FDynamicMeshSculptLayers::MergeSculptLayers(int32 StartIndex, int32 EndIndex, bool bUseWeights)
{
	if (!Layers.IsValidIndex(StartIndex) || !Layers.IsValidIndex(EndIndex) || StartIndex >= EndIndex)
	{
		return false;
	}
	double ActiveWeight = bUseWeights ? LayerWeights[StartIndex] : 1.0;
	if (ActiveWeight == 0)
	{
		// cannot bake into a zero-weight layer
		return false;
	}
	double ActiveWeightInv = 1.0 / ActiveWeight;
	UpdateLayersFromMesh();

	FDynamicMesh3* Mesh = Layers[StartIndex].GetParent();
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		FVector3d MergedSum(0, 0, 0);
		for (int32 LayerIdx = StartIndex + 1; LayerIdx <= EndIndex; ++LayerIdx)
		{
			FVector3d LayerOffset;
			Layers[LayerIdx].GetValue(VID, LayerOffset);
			double Wt = bUseWeights ? LayerWeights[LayerIdx] : 1.0;
			MergedSum += LayerOffset * Wt;
		}
		FVector3d StartInitial;
		Layers[StartIndex].GetValue(VID, StartInitial);
		Layers[StartIndex].SetValue(VID, StartInitial + MergedSum * ActiveWeightInv);
	}
	Layers.RemoveAt(StartIndex + 1, EndIndex - StartIndex);
	LayerWeights.RemoveAt(StartIndex + 1, EndIndex - StartIndex);
	if (!bUseWeights)
	{
		UpdateMeshFromLayers();
	}
	ValidateActiveLayer();
	return true;
}

void FDynamicMeshSculptLayers::UpdateLayerWeights(TConstArrayView<double> InLayerWeights)
{
	UpdateLayersFromMesh();
	for (int32 LayerIdx = 0, NumLayers = FMath::Min(InLayerWeights.Num(), LayerWeights.Num()); LayerIdx < NumLayers; ++LayerIdx)
	{
		LayerWeights[LayerIdx] = InLayerWeights[LayerIdx];
	}
	UpdateMeshFromLayers();
	ValidateActiveLayer();
}

void FDynamicMeshSculptLayers::Enable(FDynamicMeshAttributeSet* AttributeSet, int32 MinLayerCount)
{
	check(AttributeSet);

	int32 UseMinLayers = FMath::Max(MinLayerCount, 1);
	bool bWasEmpty = Layers.IsEmpty();

	if (UseMinLayers > Layers.Num())
	{
		FDynamicMesh3* ParentMesh = AttributeSet->GetParentMesh();
		Layers.Reserve(UseMinLayers);
		LayerWeights.Reserve(UseMinLayers);
		while (Layers.Num() < UseMinLayers)
		{
			FDynamicMeshSculptLayerAttribute* LayerToAdd = new FDynamicMeshSculptLayerAttribute(ParentMesh);
			Layers.Add(LayerToAdd);
			AttributeSet->RegisterExternalAttribute(LayerToAdd);
			LayerWeights.Add(1.0);
		}

		// Initialize base layer from parent mesh (if it was just created)
		if (bWasEmpty)
		{
			ActiveLayer = 0;
			for (int32 VID : ParentMesh->VertexIndicesItr())
			{
				Layers[0].SetValue(VID, ParentMesh->GetVertex(VID));
			}
		}
	}
}

void FDynamicMeshSculptLayers::Discard(FDynamicMeshAttributeSet* AttributeSet)
{
	check(AttributeSet);
	for (FDynamicMeshSculptLayerAttribute& Layer : Layers)
	{
		AttributeSet->UnregisterExternalAttribute(&Layer);
	}
	Layers.Reset();
	LayerWeights.Reset();
	ActiveLayer = -1;
}

void FDynamicMeshSculptLayers::Copy(FDynamicMeshAttributeSet* AttributeSet, const FDynamicMeshSculptLayers& Copy)
{
	Discard(AttributeSet);
	for (const FDynamicMeshSculptLayerAttribute& Layer : Copy.Layers)
	{
		FDynamicMeshSculptLayerAttribute* NewLayer = static_cast<FDynamicMeshSculptLayerAttribute*>(Layer.MakeCopy(AttributeSet->GetParentMesh()));
		Layers.Add(NewLayer);
		AttributeSet->RegisterExternalAttribute(NewLayer);
	}
	LayerWeights = Copy.LayerWeights;
	ActiveLayer = Copy.ActiveLayer;
}

void FDynamicMeshSculptLayers::CompactCopy(FDynamicMeshAttributeSet* AttributeSet, const FCompactMaps& CompactMaps, const FDynamicMeshSculptLayers& Copy)
{
	Discard(AttributeSet);
	for (const FDynamicMeshSculptLayerAttribute& Layer : Copy.Layers)
	{
		FDynamicMeshSculptLayerAttribute* NewLayer = static_cast<FDynamicMeshSculptLayerAttribute*>(Layer.MakeCompactCopy(CompactMaps, AttributeSet->GetParentMesh()));
		Layers.Add(NewLayer);
		AttributeSet->RegisterExternalAttribute(NewLayer);
	}
	LayerWeights = Copy.LayerWeights;
	ActiveLayer = Copy.ActiveLayer;
}

void FDynamicMeshSculptLayers::EnableMatching(FDynamicMeshAttributeSet* AttributeSet, const FDynamicMeshSculptLayers& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes)
{
	if (bClearExisting || (bDiscardExtraAttributes && !ToMatch.IsEnabled()))
	{
		Discard(AttributeSet);
	}
	if (ToMatch.IsEnabled())
	{
		FDynamicMesh3* ParentMesh = AttributeSet->GetParentMesh();
		Layers.Reserve(ToMatch.NumLayers());
		LayerWeights.Reserve(ToMatch.NumLayers());
		while (Layers.Num() < ToMatch.NumLayers())
		{
			FDynamicMeshSculptLayerAttribute* LayerToAdd = new FDynamicMeshSculptLayerAttribute(ParentMesh);
			int32 LayerIdx = Layers.Add(LayerToAdd);
			AttributeSet->RegisterExternalAttribute(LayerToAdd);
			// Initialize weight to also match the other layer, for new layers
			LayerWeights.Add(ToMatch.LayerWeights[LayerIdx]);
		}
		if (bDiscardExtraAttributes && NumLayers() > ToMatch.NumLayers())
		{
			int32 RemoveStart = ToMatch.NumLayers(), RemoveCount = NumLayers() - ToMatch.NumLayers();
			Layers.RemoveAt(RemoveStart, RemoveCount);
			LayerWeights.RemoveAt(RemoveStart, RemoveCount);
		}
	}
	ValidateActiveLayer();
}

void FDynamicMeshSculptLayers::CopyThroughMapping(const FDynamicMeshSculptLayers& Other, const FMeshIndexMappings& Mapping)
{
	const int32 NumLayers = FMath::Min(Layers.Num(), Other.Layers.Num());
	for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		Layers[LayerIdx].CopyThroughMapping(&Other.Layers[LayerIdx], Mapping);
	}
}

bool FDynamicMeshSculptLayers::CheckValidity(const FDynamicMeshAttributeSet* AttributeSet, bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const
{
	bool bValid = (LayerWeights.Num() == Layers.Num());
	for (const FDynamicMeshSculptLayerAttribute& Layer : Layers)
	{
		// validate layer is registered
		bValid = AttributeSet->RegisteredAttributes.Contains(&Layer) && bValid;
		// Note: We don't need to call Layer.CheckValidity here, since it is already called on registered attributes

		// validate parent mesh is set consistent with attribute set
		bValid = (AttributeSet->GetParentMesh() == Layer.GetParent()) && bValid;
	}

	if (!bValid)
	{
		switch (FailMode)
		{
		case EValidityCheckFailMode::Check:
			check(false);
		case EValidityCheckFailMode::Ensure:
			ensure(false);
		default:
			return false;
		}
	}

	return true;
}

bool FDynamicMeshSculptLayers::UpdateLayersFromMesh()
{
	if (Layers.IsEmpty() || !ensure(HasValidLayers()))
	{
		return false;
	}

	double ActiveWeight = LayerWeights[ActiveLayer];
	if (ActiveWeight == 0)
	{
		return false;
	}

	FDynamicMesh3* Mesh = Layers[0].GetParent();
	check(Mesh);
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		FVector3d Sum(0, 0, 0);
		for (int32 LayerIdx = 0; LayerIdx < Layers.Num(); ++LayerIdx)
		{
			FVector3d Offset;
			Layers[LayerIdx].GetValue(VID, Offset);
			Sum += Offset * LayerWeights[LayerIdx];
		}
		FVector3d Pos = Mesh->GetVertex(VID);

		FVector3d Delta = (Pos - Sum) / ActiveWeight;
		FVector3d OrigActiveOffset;
		Layers[ActiveLayer].GetValue(VID, OrigActiveOffset);
		Layers[ActiveLayer].SetValue(VID, OrigActiveOffset + Delta);
	}

	return true;
}

bool FDynamicMeshSculptLayers::UpdateMeshFromLayers()
{
	if (Layers.IsEmpty() || !ensure(Layers.Num() == LayerWeights.Num()))
	{
		return false;
	}

	FDynamicMesh3* Mesh = Layers[0].GetParent();
	check(Mesh);
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		FVector3d Sum(0, 0, 0);
		for (int32 LayerIdx = 0; LayerIdx < Layers.Num(); ++LayerIdx)
		{
			FVector3d Offset;
			Layers[LayerIdx].GetValue(VID, Offset);
			Sum += Offset * LayerWeights[LayerIdx];
		}
		Mesh->SetVertex(VID, Sum);
	}

	return true;
}


bool FDynamicMeshSculptLayers::ValidateActiveLayer()
{
	if (!IsEnabled() || !ensure(Layers.Num() == LayerWeights.Num()))
	{
		ActiveLayer = INDEX_NONE;
		return false;
	}
	ActiveLayer = FMath::Clamp(ActiveLayer, 0, LayerWeights.Num() - 1);

	// If active layer has zero weight, try to find an active layer with non-zero weight instead
	if (LayerWeights[ActiveLayer] == 0)
	{
		for (int32 Idx = ActiveLayer + 1; Idx < LayerWeights.Num(); ++Idx)
		{
			if (LayerWeights[Idx] != 0)
			{
				ActiveLayer = Idx;
				return true;
			}
		}
		for (int32 Idx = ActiveLayer - 1; Idx >= 0; --Idx)
		{
			if (LayerWeights[Idx] != 0)
			{
				ActiveLayer = Idx;
				return true;
			}
		}
		return false;
	}
	else
	{
		return true;
	}
}
