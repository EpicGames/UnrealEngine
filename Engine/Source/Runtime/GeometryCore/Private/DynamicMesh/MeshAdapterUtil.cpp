// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/MeshAdapterUtil.h"

using namespace UE::Geometry;

FPointSetAdapterd UE::Geometry::MakePointsAdapter(const FDynamicPointSet3d* PointSet)
{
	FPointSetAdapterd Adapter;
	Adapter.MaxPointID = [PointSet]() { return PointSet->MaxVertexID(); };
	Adapter.PointCount = [PointSet]() { return PointSet->VertexCount(); };
	Adapter.IsPoint = [PointSet](int Idx) { return PointSet->IsVertex(Idx); };
	Adapter.GetPoint = [PointSet](int Idx) { return PointSet->GetVertex(Idx); };

	Adapter.HasNormals = [PointSet] { return false; };
	Adapter.GetPointNormal = [PointSet](int Idx) {return FVector3f(0,0,1); };

	return Adapter;
}



FPointSetAdapterd UE::Geometry::MakeVerticesAdapter(const FDynamicMesh3* Mesh)
{
	FPointSetAdapterd Adapter;
	Adapter.MaxPointID = [Mesh]() { return Mesh->MaxVertexID(); };
	Adapter.PointCount = [Mesh]() { return Mesh->VertexCount(); };
	Adapter.IsPoint = [Mesh](int Idx) { return Mesh->IsVertex(Idx); };
	Adapter.GetPoint = [Mesh](int Idx) { return Mesh->GetVertex(Idx); };

	Adapter.HasNormals = [Mesh] { return Mesh->HasVertexNormals(); };
	Adapter.GetPointNormal = [Mesh](int Idx) {return Mesh->GetVertexNormal(Idx); };

	return Adapter;
}


FPointSetAdapterd UE::Geometry::MakeTriCentroidsAdapter(const FDynamicMesh3* Mesh)
{
	FPointSetAdapterd Adapter;
	Adapter.MaxPointID = [Mesh]() { return Mesh->MaxTriangleID(); };
	Adapter.PointCount = [Mesh]() { return Mesh->TriangleCount(); };
	Adapter.IsPoint = [Mesh](int Idx) { return Mesh->IsTriangle(Idx); };
	Adapter.GetPoint = [Mesh](int Idx) { return Mesh->GetTriCentroid(Idx); };

	Adapter.HasNormals = [] { return true; };
	Adapter.GetPointNormal = [Mesh](int Idx) {return (FVector3f)Mesh->GetTriNormal(Idx); };

	return Adapter;
}




FPointSetAdapterd UE::Geometry::MakeEdgeMidpointsAdapter(const FDynamicMesh3* Mesh)
{
	FPointSetAdapterd Adapter;
	Adapter.MaxPointID = [Mesh]() { return Mesh->MaxEdgeID(); };
	Adapter.PointCount = [Mesh]() { return Mesh->EdgeCount(); };
	Adapter.IsPoint = [Mesh] (int Idx) { return Mesh->IsEdge(Idx); };
	Adapter.GetPoint = [Mesh](int Idx) { return Mesh->GetEdgePoint(Idx, 0.5); };

	Adapter.HasNormals = [] { return false; };
	Adapter.GetPointNormal = [](int Idx) { return FVector3f::UnitY();};

	return Adapter;
}


FPointSetAdapterd UE::Geometry::MakeBoundaryEdgeMidpointsAdapter(const FDynamicMesh3* Mesh)
{
	// may be possible to do this more quickly by directly iterating over Mesh.EdgesBuffer[eid*4+3]  (still need to check valid)
	int NumBoundaryEdges = 0;
	for (int eid : Mesh->BoundaryEdgeIndicesItr())
	{
		NumBoundaryEdges++;
	}

	FPointSetAdapterd Adapter;
	Adapter.MaxPointID = [Mesh]() { return Mesh->MaxEdgeID(); };
	Adapter.PointCount = [NumBoundaryEdges]() { return NumBoundaryEdges; };
	Adapter.IsPoint = [Mesh](int Idx) { return Mesh->IsEdge(Idx) && Mesh->IsBoundaryEdge(Idx); };
	Adapter.GetPoint = [Mesh](int Idx) { return Mesh->GetEdgePoint(Idx, 0.5); };

	Adapter.HasNormals = [] { return false; };
	Adapter.GetPointNormal = [](int Idx) { return FVector3f::UnitY(); };

	return Adapter;
}


FTriangleMeshAdapterd UE::Geometry::MakeTransformedDynamicMeshAdapter(const FDynamicMesh3* Mesh, FTransform3d Transform)
{
	FTriangleMeshAdapterd Adapter = MakeDynamicMeshAdapter(Mesh);
	Adapter.GetVertex = [Mesh, Transform](int32 Idx)
	{
		return FVector3d(Transform.TransformPosition(Mesh->GetVertex(Idx)));
	};
	return Adapter;
}


FTriangleMeshAdapterd UE::Geometry::MakeDynamicMeshAdapter(const FDynamicMesh3* Mesh)
{
	return
	FTriangleMeshAdapterd {
		[Mesh](int Idx) { return Mesh->IsTriangle(Idx); },
		[Mesh](int Idx) { return Mesh->IsVertex(Idx); },
		[Mesh]() { return Mesh->MaxTriangleID(); },
		[Mesh]() { return Mesh->MaxVertexID(); },
		[Mesh]() { return Mesh->TriangleCount(); },
		[Mesh]() { return Mesh->VertexCount(); },
		[Mesh]() { return Mesh->GetChangeStamp(); },
		[Mesh](int Idx) { return Mesh->GetTriangle(Idx); },
		[Mesh](int Idx) { return Mesh->GetVertex(Idx); }
	};
}