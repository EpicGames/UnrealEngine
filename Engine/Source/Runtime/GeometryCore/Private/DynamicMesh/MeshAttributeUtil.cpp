// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/MeshAttributeUtil.h"

using namespace UE::Geometry;


bool UE::Geometry::CompactAttributeValues(
	const FDynamicMesh3& Mesh,
	TDynamicMeshScalarTriangleAttribute<int32>& TriangleAttrib,
	FInterval1i& OldMaxAttributeRangeOut,
	int& NewMaxAttributeValueOut,
	TArray<int32>& OldToNewMap,
	TArray<int32>& NewToOldMap,
	bool& bWasCompact)
{
	bWasCompact = false;

	// compute range of values in attribute set
	FInterval1i IndexRange = FInterval1i::Empty();
	for (int32 TriangleID : Mesh.TriangleIndicesItr())
	{
		IndexRange.Contain(TriangleAttrib.GetValue(TriangleID));
	}
	OldMaxAttributeRangeOut = IndexRange;
	if (IndexRange.Min < 0)
	{
		return false;
	}

	int32 MaxValue = IndexRange.Max;
	OldToNewMap.Init(IndexConstants::InvalidID, MaxValue+1);

	// generate remapping and set new values
	int32 NewValueCount = 0;
	for (int32 TriangleID : Mesh.TriangleIndicesItr())
	{
		int32 Value = TriangleAttrib.GetValue(TriangleID);
		if (OldToNewMap[Value] == IndexConstants::InvalidID)
		{
			OldToNewMap[Value] = NewValueCount;
			NewValueCount++;
		}
		int32 NewValue = OldToNewMap[Value];
		if (NewValue != Value)
		{
			TriangleAttrib.SetValue(TriangleID, NewValue);
		}
	}
	NewMaxAttributeValueOut = NewValueCount - 1;

	// construct inverse mapping
	NewToOldMap.Init(IndexConstants::InvalidID, NewValueCount);
	for (int32 k = 0; k <= MaxValue; ++k)
	{
		if (OldToNewMap[k] >= 0)
		{
			NewToOldMap[ OldToNewMap[k] ] = k;
		}
	}

	bWasCompact = ( NewToOldMap.Num() == OldToNewMap.Num() );

	return true;
}

bool UE::Geometry::CopyVertexUVsToOverlay(
	const FDynamicMesh3& Mesh,
	FDynamicMeshUVOverlay& UVOverlayOut,
	bool bCompactElements)
{
	if (!Mesh.HasVertexUVs())
	{
		return false;
	}

	if (UVOverlayOut.ElementCount() > 0)
	{
		UVOverlayOut.ClearElements();
	}
	
	UVOverlayOut.BeginUnsafeElementsInsert();
	for (int32 Vid : Mesh.VertexIndicesItr())
	{
		FVector2f UV = Mesh.GetVertexUV(Vid);
		UVOverlayOut.InsertElement(Vid, &UV.X, true);
	}
	UVOverlayOut.EndUnsafeElementsInsert();

	for (int32 Tid : Mesh.TriangleIndicesItr())
	{
		UVOverlayOut.SetTriangle(Tid, Mesh.GetTriangle(Tid));
	}

	if (bCompactElements)
	{
		FCompactMaps CompactMaps;
		UVOverlayOut.CompactInPlace(CompactMaps);
	}

	return true;
}

bool UE::Geometry::CopyVertexNormalsToOverlay(
	const FDynamicMesh3& Mesh,
	FDynamicMeshNormalOverlay& NormalOverlayOut,
	bool bCompactElements)
{
	if (!Mesh.HasVertexNormals())
	{
		return false;
	}

	if (NormalOverlayOut.ElementCount() > 0)
	{
		NormalOverlayOut.ClearElements();
	}

	NormalOverlayOut.BeginUnsafeElementsInsert();
	for (int32 Vid : Mesh.VertexIndicesItr())
	{
		FVector3f Normal = Mesh.GetVertexNormal(Vid);
		NormalOverlayOut.InsertElement(Vid, &Normal.X, true);
	}
	NormalOverlayOut.EndUnsafeElementsInsert();

	for (int32 Tid : Mesh.TriangleIndicesItr())
	{
		NormalOverlayOut.SetTriangle(Tid, Mesh.GetTriangle(Tid));
	}

	if (bCompactElements)
	{
		FCompactMaps CompactMaps;
		NormalOverlayOut.CompactInPlace(CompactMaps);
	}

	return true;
}
