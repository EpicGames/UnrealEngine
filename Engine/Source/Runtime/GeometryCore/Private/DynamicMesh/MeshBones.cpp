// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/MeshBones.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicBoneAttribute.h"

#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

#include "UObject/NameTypes.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"

using namespace UE::Geometry;

bool FMeshBones::GetBoneChildren(const FDynamicMesh3& Mesh, int32 BoneIndex, TArray<int32>& ChildrenIndices, bool bRecursive)
{
    if (!Mesh.HasAttributes() || !Mesh.Attributes()->HasBones())
    {
        return false;
    }

    const int32 NumBones = Mesh.Attributes()->GetNumBones();
    const TArray<int32>& ParentsAttrib = Mesh.Attributes()->GetBoneParentIndices()->GetAttribValues();

    // add direct children only
	for (int32 Idx = 0; Idx < NumBones; ++Idx)
	{
		if (ParentsAttrib[Idx] == BoneIndex)
		{
			ChildrenIndices.Add(Idx);
		}
	}
	
	// add all grandchildren as well
	if (bRecursive) 
	{
		// Map each bone index to its direct children bone indices
		TMap<int32, TArray<int32>> ChildrenMap;
		for (int32 Idx = 0; Idx < NumBones; ++Idx)
		{
			const int32 ParentIdx = ParentsAttrib[Idx];
			if (ChildrenMap.Contains(ParentsAttrib[Idx]))
			{
				ChildrenMap[ParentIdx].Add(Idx);
			}
			else 
			{
				ChildrenMap.Add(ParentIdx, {Idx});
			}
		}

		// Do breadth first search
		TQueue<int32> BFS;
		for (const int32 ChildIdx : ChildrenIndices)
		{
			BFS.Enqueue(ChildIdx);
		}

		while (!BFS.IsEmpty())
		{
			int32 ChildIdx; 
			BFS.Dequeue(ChildIdx);

			if (ChildrenMap.Contains(ChildIdx))
			{
				for (const int32 GrandChildIdx : ChildrenMap[ChildIdx])
				{
					BFS.Enqueue(GrandChildIdx);
					checkSlow(ChildrenIndices.Find(GrandChildIdx) == INDEX_NONE);
					ChildrenIndices.Add(GrandChildIdx);
				}
			}
		}
	}

    return true;
}

bool FMeshBones::GetBonesInIncreasingOrder(const FDynamicMesh3& Mesh, 
										   TArray<FName>& BoneNames, 
										   TArray<int32>& BoneParentIdx, 
										   TArray<FTransform>& BonePose, 
										   bool& bOrderChanged)
{
    if (!Mesh.HasAttributes() || !Mesh.Attributes()->HasBones())
    {
        return false;
    }

    const TArray<FName>& NamesAttrib = Mesh.Attributes()->GetBoneNames()->GetAttribValues();
    const TArray<int32>& ParentsAttrib = Mesh.Attributes()->GetBoneParentIndices()->GetAttribValues();
    const TArray<FTransform>& TransformsAttrib = Mesh.Attributes()->GetBonePoses()->GetAttribValues();

	const int32 NumBones = NamesAttrib.Num();
	if (NumBones == 0)
	{ 
		return true; // empty skeleton, nothing to do
	}

	// Do a scan to see if the order of the bones in the mesh bone attribute is in strictly increasing order.
	// So child must have an index greater than its parent (unless its a root bone)
	bool bIsIncreasingOrder = true;
	for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	{
		const int32 ParentBoneIndex = ParentsAttrib[BoneIdx];
		if (ParentBoneIndex == INDEX_NONE && BoneIdx != 0)
		{
			// root bone must be at index 0. Multi root is not supported in general Skeleton/SkeletalMesh and hence not supported in DynamicMesh.
			bIsIncreasingOrder = false;
			break;
		}
		else if (BoneIdx <= ParentBoneIndex)
		{
			bIsIncreasingOrder = false; // child index must be greater than it's parent
			break;
		}
	}

	if (!bIsIncreasingOrder)
	{
		// If bones in the bone attribute are out of order, traverse the hiearchy in breadth first search way to get the 
		// correct order

		// Root bone's parent will be INDEX_NONE
		const int32 RootBoneIdx = ParentsAttrib.Find(INDEX_NONE);
		if (!ensure(RootBoneIdx != INDEX_NONE))
		{
			return false; // couldn't find the root
		}

		// Get skeleton hiearchy by getting all the children and grandchildren of the root bone
		TArray<int32> ChildrenIndices;
		ChildrenIndices.Reserve(NumBones);
		FMeshBones::GetBoneChildren(Mesh, RootBoneIdx, ChildrenIndices, true);
		
		// map bone name to its index in the array.
		TMap<FName, int32> NameToIndex;

		// preallocate space
		NameToIndex.Reserve(NumBones);
		BoneNames.Reserve(NumBones); 
		BoneParentIdx.Reserve(NumBones);
		BonePose.Reserve(NumBones); 

		// add root first
		NameToIndex.Add(NamesAttrib[RootBoneIdx], 0);
		BoneNames.Add(NamesAttrib[RootBoneIdx]); 
		BoneParentIdx.Add(INDEX_NONE);
		BonePose.Add(TransformsAttrib[RootBoneIdx]);
		
		// add all children/grandchildren in the correct order
		for (const int32 BoneIdx : ChildrenIndices)
		{
			NameToIndex.Add(NamesAttrib[BoneIdx], NameToIndex.Num());
			BoneNames.Add(NamesAttrib[BoneIdx]); 
			BonePose.Add(TransformsAttrib[BoneIdx]);
		}

		// reindex the parent indices
		for (const int32 BoneIdx : ChildrenIndices)
		{
			const int32 OldParentIdx = ParentsAttrib[BoneIdx];
			const FName OldParentName = NamesAttrib[OldParentIdx];
			const int32 NewParentIdx = NameToIndex[OldParentName];
			BoneParentIdx.Add(NewParentIdx);
		}
	}
	else
	{
		BoneNames = NamesAttrib;
		BoneParentIdx = ParentsAttrib;
		BonePose = TransformsAttrib;
	}

	bOrderChanged = !bIsIncreasingOrder;

	return true;
}

bool FMeshBones::CombineLodBonesToReferenceSkeleton(const TArray<FDynamicMesh3>& Meshes, 
													TArray<FName>& BoneNames,
													TArray<int32>& BoneParentIdx,
													TArray<FTransform>& BonePose,  
													bool& bOrderChanged)
{
    bOrderChanged = false;

    if (Meshes.IsEmpty())
    {
        return true;
    }

    // all meshes must have bone attributes
    for (const FDynamicMesh3& Mesh : Meshes)
    {
        if (Mesh.HasAttributes() == false || !Mesh.Attributes()->HasBones())
        {
            return false;
        }
    }

    // check if all meshes have identical bone attributes which should be a common case when the array represents Lods
    for (int32 LodIdx = 1; LodIdx < Meshes.Num(); ++LodIdx)
    {
        if (!Meshes[0].Attributes()->IsSameBoneAttributesAs(*Meshes[LodIdx].Attributes()))
        {
            bOrderChanged = true;
            break;
        }
    }

    if (!bOrderChanged)
    {
        // if all meshes have the same bone attributes then simply copy over the attributes of the first mesh
        return GetBonesInIncreasingOrder(Meshes[0], BoneNames, BoneParentIdx, BonePose, bOrderChanged);
    }
    else 
    {
        // meshes don't have the same attributes so we find the mesh with the largest number of bones
        int32 MaxBoneNum = -1;
        int32 MaxBoneNumLod = -1;
        for (int32 Idx = 0; Idx < Meshes.Num(); ++Idx)
        {
            const int32 BoneNum = Meshes[Idx].Attributes()->GetNumBones();
            if (MaxBoneNum < BoneNum)
            {
                MaxBoneNum = BoneNum;
                MaxBoneNumLod = Idx;
            } 
        }

        // we check that it's a superset of the bone attributes in all other meshes
        TSet<FName> HashSet;
       	HashSet.Append(Meshes[MaxBoneNumLod].Attributes()->GetBoneNames()->GetAttribValues());
        for (int32 Idx = 0; Idx < Meshes.Num(); ++Idx)
        {
            if (Idx == MaxBoneNumLod)
            {
                continue;
            }

            const TArray<FName>& NamesAttrib = Meshes[Idx].Attributes()->GetBoneNames()->GetAttribValues();
            for (int32 BoneIdx = 0; BoneIdx < NamesAttrib.Num(); ++BoneIdx)
            {
                if (!ensure(HashSet.Contains(NamesAttrib[BoneIdx])))
                {
                    // The mesh has a bone that is not contained in the mesh with the largest number of bones.
                    // Hence, the lods are not compatible.
                    return false;
                }
            } 
        }

        return GetBonesInIncreasingOrder(Meshes[MaxBoneNumLod], BoneNames, BoneParentIdx, BonePose, bOrderChanged);
    }
}
