// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMesh/MeshIndexMappings.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

void FMeshIndexMappings::Initialize(FDynamicMesh3* Mesh)
{
	if (Mesh->HasAttributes())
	{
		FDynamicMeshAttributeSet* Attribs = Mesh->Attributes();
		UVMaps.SetNum(Attribs->NumUVLayers());
		NormalMaps.SetNum(Attribs->NumNormalLayers());
	}
}
