// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMesh/MeshIndexUtil.h"

using namespace UE::Geometry;

void UE::Geometry::TriangleToVertexIDs(const FDynamicMesh3* Mesh, TConstArrayView<int> TriangleIDs, TArray<int>& VertexIDsOut)
{
	// if we are getting close to full mesh it is probably more efficient to use a bitmap...

	int NumTris = TriangleIDs.Num();

	// @todo profile this constant
	if (NumTris < 25)
	{
		for (int k = 0; k < NumTris; ++k)
		{
			if (Mesh->IsTriangle(TriangleIDs[k]))
			{
				FIndex3i Tri = Mesh->GetTriangle(TriangleIDs[k]);
				VertexIDsOut.AddUnique(Tri[0]);
				VertexIDsOut.AddUnique(Tri[1]);
				VertexIDsOut.AddUnique(Tri[2]);
			}
		}
	}
	else
	{
		TSet<int> VertexSet;
		VertexSet.Reserve(TriangleIDs.Num()*3);
		for (int k = 0; k < NumTris; ++k)
		{
			if (Mesh->IsTriangle(TriangleIDs[k]))
			{
				FIndex3i Tri = Mesh->GetTriangle(TriangleIDs[k]);
				VertexSet.Add(Tri[0]);
				VertexSet.Add(Tri[1]);
				VertexSet.Add(Tri[2]);
			}
		}

		VertexIDsOut.Reserve(VertexSet.Num());
		for (int VertexID : VertexSet)
		{
			VertexIDsOut.Add(VertexID);
		}
	}
}



void UE::Geometry::VertexToTriangleOneRing(const FDynamicMesh3* Mesh, TConstArrayView<int> VertexIDs, TSet<int>& TriangleIDsOut)
{
	// for a TSet it is more efficient to just try to add each triangle twice, than it is to
	// try to avoid duplicate adds with more complex mesh queries
	int32 NumVerts = VertexIDs.Num();
	TriangleIDsOut.Reserve( (NumVerts < 5) ? NumVerts*6 : NumVerts*4);
	for (int32 vid : VertexIDs)
	{
		Mesh->EnumerateVertexEdges(vid, [&](int32 eid) 
		{
			FIndex2i EdgeT = Mesh->GetEdgeT(eid);
			TriangleIDsOut.Add(EdgeT.A);
			if (EdgeT.B != IndexConstants::InvalidID) TriangleIDsOut.Add(EdgeT.B);
		});
	}
}


FIndex2i UE::Geometry::FindVertexEdgesInTriangle(const FDynamicMesh3& Mesh, int32 TriangleID, int32 VertexID)
{
	FIndex2i Result(IndexConstants::InvalidID, IndexConstants::InvalidID);
	if (Mesh.IsTriangle(TriangleID))
	{
		FIndex3i TriV = Mesh.GetTriangle(TriangleID);
		FIndex3i TriEdges = Mesh.GetTriEdges(TriangleID);
		for (int32 j = 0; j < 3; ++j)
		{
			if (TriV[j] == VertexID)
			{
				return FIndex2i( TriEdges[(j==0)?2:j-1], TriEdges[j] );
			}
		}
	}
	return FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID);;
};


int32 UE::Geometry::FindSharedEdgeInTriangles(const FDynamicMesh3& Mesh, int32 Triangle0, int32 Triangle1)
{
	if (Mesh.IsTriangle(Triangle0) && Mesh.IsTriangle(Triangle1))
	{
		FIndex3i Edges0 = Mesh.GetTriEdges(Triangle0);
		FIndex3i Edges1 = Mesh.GetTriEdges(Triangle1);
		for (int32 j = 0; j < 3; ++j)
		{
			if (Edges1.Contains(Edges0[j]))
			{
				return Edges0[j];
			}
		}
	}
	return IndexConstants::InvalidID;
}



bool UE::Geometry::SplitBoundaryVertexTrianglesIntoSubsets(
	const FDynamicMesh3* Mesh,
	int32 VertexID,
	int32 SplitEdgeID,
	TArray<int32>& TriangleSet0, TArray<int32>& TriangleSet1)
{
	if (Mesh->IsVertex(VertexID) == false || Mesh->IsEdge(SplitEdgeID) == false)
	{
		return false;
	}
	FIndex2i StartTris = Mesh->GetEdgeT(SplitEdgeID);
	if (StartTris.B == IndexConstants::InvalidID)
	{
		return false;
	}

	for (int32 si = 0; si < 2; ++si)
	{
		TArray<int32>& CurrentTriangleSet = (si == 0) ? TriangleSet0 : TriangleSet1;
		int32 StartTri = StartTris[si];
		CurrentTriangleSet.Add(StartTri);

		int32 EdgeOtherTri = StartTris[si == 0 ? 1 : 0];
		int32 CurTri = StartTri;
		int32 PrevTri = EdgeOtherTri;

		bool bDone = false;
		while (!bDone)
		{
			FIndex3i NextTri = UE::Geometry::FindNextAdjacentTriangleAroundVtx(Mesh, VertexID, CurTri, PrevTri,
				[&](int32 Tri0, int32 Tri1, int32 Edge) { return Edge != SplitEdgeID; }
			);
			if (NextTri.A != IndexConstants::InvalidID)
			{
				if (NextTri.A == EdgeOtherTri)		// if we somehow looped around, then the arguments were bad and we need to abort
				{
					return false;
				}

				CurrentTriangleSet.Add(NextTri.A);
				PrevTri = CurTri;
				CurTri = NextTri.A;
			}
			else
			{
				bDone = true;
			}
		}
	}

	return (TriangleSet0.Num() > 0 && TriangleSet1.Num() > 0);
}



bool UE::Geometry::SplitInteriorVertexTrianglesIntoSubsets(
	const FDynamicMesh3* Mesh,
	int32 VertexID,
	int32 SplitEdgeID0, int32 SplitEdgeID1,
	TArray<int32>& TriangleSet0, TArray<int32>& TriangleSet1)
{
	if (Mesh->IsVertex(VertexID) == false || Mesh->IsEdge(SplitEdgeID0) == false || Mesh->IsEdge(SplitEdgeID1) == false)
	{
		return false;
	}

	FIndex2i StartTris = Mesh->GetEdgeT(SplitEdgeID1);
	if (StartTris.B < 0)
	{
		return false;
	}

	for (int32 si = 0; si < 2; ++si)
	{
		TArray<int32>& CurrentTriangleSet = (si == 0) ? TriangleSet0 : TriangleSet1;
		int32 StartTri = StartTris[si];
		CurrentTriangleSet.Add(StartTri);

		int32 EdgeOtherTri = StartTris[si == 0 ? 1 : 0];
		int32 CurTri = StartTri;
		int32 PrevTri = EdgeOtherTri;

		bool bDone = false;
		while (!bDone)
		{
			FIndex3i NextTri = UE::Geometry::FindNextAdjacentTriangleAroundVtx(Mesh, VertexID, CurTri, PrevTri,
				[&](int32 Tri0, int32 Tri1, int32 Edge) { return Edge != SplitEdgeID0 && Edge != SplitEdgeID1; }
			);
			if (NextTri.A != IndexConstants::InvalidID)
			{
				if (NextTri.A == EdgeOtherTri)		// if we somehow looped around, then the arguments were bad and we need to abort
				{
					return false;
				}

				CurrentTriangleSet.Add(NextTri.A);
				PrevTri = CurTri;
				CurTri = NextTri.A;
			}
			else
			{
				bDone = true;
			}
		}
	}

	return (TriangleSet0.Num() > 0 && TriangleSet1.Num() > 0);
}