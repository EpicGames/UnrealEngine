// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshNormals

#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "MeshQueries.h"
#include "HAL/UnrealMemory.h"

#include <atomic>

using namespace UE::Geometry;

void FMeshNormals::SetCount(int Count, bool bClearToZero)
{
	if (Normals.Num() < Count) 
	{
		Normals.SetNumUninitialized(Count);
	}
	if (bClearToZero)
	{
		FMemory::Memzero(Normals.GetData(), Normals.NumBytes());
	}
}



void FMeshNormals::CopyToVertexNormals(FDynamicMesh3* SetMesh, bool bInvert) const
{
	if (SetMesh->HasVertexNormals() == false)
	{
		SetMesh->EnableVertexNormals(FVector3f::UnitX());
	}

	float sign = (bInvert) ? -1.0f : 1.0f;
	int N = FMath::Min(Normals.Num(), SetMesh->MaxVertexID());
	for (int vi = 0; vi < N; ++vi)
	{
		if (Mesh->IsVertex(vi) && SetMesh->IsVertex(vi))
		{
			SetMesh->SetVertexNormal(vi, sign * (FVector3f)Normals[vi]);
		}
	}
}


void FMeshNormals::GetVertexNormalsFromOverlayNormals(ECombineSplitNormalsMethod CombineSplitNormals)
{
	check(Mesh);

	// no overlay to copy from
	if (!Mesh->HasAttributes() || !Mesh->Attributes()->PrimaryNormals())
	{
		Normals.Init(FVector::UnitZ(), Mesh->MaxVertexID());
		return;
	}

	const FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->PrimaryNormals();
	if (CombineSplitNormals == ECombineSplitNormalsMethod::Average)
	{
		Normals.Init(FVector::Zero(), Mesh->MaxVertexID());
		for (int32 TID : Mesh->TriangleIndicesItr())
		{
			if (NormalOverlay->IsSetTriangle(TID))
			{
				FIndex3i TriV = Mesh->GetTriangle(TID);
				FVector3f A, B, C;
				NormalOverlay->GetTriElements(TID, A, B, C);
				Normals[TriV.A] += (FVector)A;
				Normals[TriV.B] += (FVector)B;
				Normals[TriV.C] += (FVector)C;
			}
		}

		for (int32 k = 0; k < Normals.Num(); ++k)
		{
			Normalize(Normals[k]);
		}
	}
	else // ECombineSplitNormalsMethod::CopyAny
	{
		Normals.Init(FVector::UnitZ(), Mesh->MaxVertexID());
		for (int32 TID : Mesh->TriangleIndicesItr())
		{
			if (NormalOverlay->IsSetTriangle(TID))
			{
				FIndex3i TriV = Mesh->GetTriangle(TID);
				FVector3f A, B, C;
				NormalOverlay->GetTriElements(TID, A, B, C);
				Normals[TriV.A] = (FVector)A;
				Normals[TriV.B] = (FVector)B;
				Normals[TriV.C] = (FVector)C;
			}
		}
	}
}


void FMeshNormals::CopyToOverlay(FDynamicMeshNormalOverlay* NormalOverlay, bool bInvert) const
{
	float sign = (bInvert) ? -1.0f : 1.0f;
	for (int ElemIdx : NormalOverlay->ElementIndicesItr())
	{
		NormalOverlay->SetElement(ElemIdx, sign * (FVector3f)Normals[ElemIdx]);
	}
}


void FMeshNormals::Compute_FaceAvg_AreaWeighted()
{
	SetCount(Mesh->MaxVertexID(), true);

	for (int TriIdx : Mesh->TriangleIndicesItr())
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh->GetTriInfo(TriIdx, TriNormal, TriArea, TriCentroid);
		TriNormal *= TriArea;

		FIndex3i Triangle = Mesh->GetTriangle(TriIdx);
		Normals[Triangle.A] += TriNormal;
		Normals[Triangle.B] += TriNormal;
		Normals[Triangle.C] += TriNormal;
	}

	for (int VertIdx : Mesh->VertexIndicesItr())
	{
		Normalize(Normals[VertIdx]);
	}
}

void FMeshNormals::Compute_FaceAvg(bool bWeightByArea, bool bWeightByAngle)
{
	if (!bWeightByAngle && bWeightByArea)
	{
		Compute_FaceAvg_AreaWeighted(); // faster case
		return;
	}

	// most general case
	SetCount(Mesh->MaxVertexID(), true);

	for (int TriIdx : Mesh->TriangleIndicesItr())
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh->GetTriInfo(TriIdx, TriNormal, TriArea, TriCentroid);
		FVector3d TriNormalWeights = GetVertexWeightsOnTriangle(Mesh, TriIdx, TriArea, bWeightByArea, bWeightByAngle);

		FIndex3i Triangle = Mesh->GetTriangle(TriIdx);
		Normals[Triangle.A] += TriNormal * TriNormalWeights[0];
		Normals[Triangle.B] += TriNormal * TriNormalWeights[1];
		Normals[Triangle.C] += TriNormal * TriNormalWeights[2];
	}

	for (int VertIdx : Mesh->VertexIndicesItr())
	{
		Normalize(Normals[VertIdx]);
	}
}

void FMeshNormals::Compute_Triangle()
{
	int NumTriangles = Mesh->MaxTriangleID();
	SetCount(NumTriangles, false);
	ParallelFor(NumTriangles, [&](int32 Index)
	{
		if (Mesh->IsTriangle(Index))
		{
			Normals[Index] = Mesh->GetTriNormal(Index);
		}
	});
}

void FMeshNormals::SetDegenerateTriangleNormalsToNeighborNormal()
{
	check(Normals.Num() >= Mesh->MaxTriangleID());

	// We're going to look through the triangles and set any zero normals
	// to the normal of their neighbor, preferring to go toward the neighbor
	// with the longer side when possible. Since we could have multiple degenerate
	// triangles linked together, this may require a little neighborhood walk.

	// Our walking function
	auto GetNeighborThatHasNormal = [this](int32 StartTid, TSet<int32>& WalkedTidsOut, int32& NonDegenerateNeighborTidOut)
	{
		WalkedTidsOut.Reset();
		NonDegenerateNeighborTidOut = FDynamicMesh3::InvalidID;

		check(StartTid != FDynamicMesh3::InvalidID && Normals[StartTid] == FVector3d::Zero());

		// We don't like recursion, so we use a little stack instead to help us prioritize 
		// the longer-side neighbors in our walk.
		TArray<int32> TidsToSearch;
		TidsToSearch.Push(StartTid);

		while (!TidsToSearch.IsEmpty())
		{
			int32 CurrentTid = TidsToSearch.Pop();

			if (WalkedTidsOut.Contains(CurrentTid))
			{
				continue;
			}

			// See if we've reached a non-degenerate triangle
			if (Normals[CurrentTid] != FVector3d::Zero())
			{
				NonDegenerateNeighborTidOut = CurrentTid;
				return;
			}

			WalkedTidsOut.Add(CurrentTid);

			// Sanity check so we don't go forever
			if (WalkedTidsOut.Num() > Mesh->MaxTriangleID())
			{
				check(false);
				return;
			}

			// Otherwise, get neighbors and corresponding squared edge lengths.
			int32 NeighborTids[3];
			double SquaredEdgeLengths[3];
			FIndex3i TriEdges = Mesh->GetTriEdges(CurrentTid);
			for (int i = 0; i < 3; ++i)
			{
				FIndex2i Tids = Mesh->GetEdgeT(TriEdges[i]);
				int32 OtherTid = (Tids.A == CurrentTid) ? Tids.B : Tids.A;
				NeighborTids[i] = OtherTid;

				if (OtherTid == FDynamicMesh3::InvalidID)
				{
					SquaredEdgeLengths[i] = 0;
				}
				else
				{
					FVector3d Vert1, Vert2;
					Mesh->GetEdgeV(TriEdges[i], Vert1, Vert2);
					SquaredEdgeLengths[i] = FVector3d::DistSquared(Vert1, Vert2);
				}
			}

			// Order neighbors by ascending length
			if (SquaredEdgeLengths[0] > SquaredEdgeLengths[1])
			{
				Swap(NeighborTids[0], NeighborTids[1]);
				Swap(SquaredEdgeLengths[0], SquaredEdgeLengths[1]);
			}
			if (SquaredEdgeLengths[1] > SquaredEdgeLengths[2])
			{
				Swap(NeighborTids[1], NeighborTids[2]);
				Swap(SquaredEdgeLengths[1], SquaredEdgeLengths[2]);
			}
			if (SquaredEdgeLengths[0] > SquaredEdgeLengths[1])
			{
				Swap(NeighborTids[0], NeighborTids[1]);
				Swap(SquaredEdgeLengths[0], SquaredEdgeLengths[1]);
			}

			// Add onto stack. Longest length neighbor is at top of stack
			for (int32 i = 0; i < 3; ++i)
			{
				if (NeighborTids[i] != FDynamicMesh3::InvalidID)
				{
					TidsToSearch.Push(NeighborTids[i]);
				}
			}
		}
	};

	// It's possible that we could have an island of degenerates, ie no normal neighbor.
	// In that case we might as well not waste time starting the same walk from each one.
	TSet<int32> IslandDegenerates;

	TSet<int32> CurrentWalkedTids;

	for (int32 Tid : Mesh->TriangleIndicesItr())
	{
		if (Normals[Tid] == FVector3d::Zero() && !IslandDegenerates.Contains(Tid))
		{
			// Find a normal to use
			CurrentWalkedTids.Reset();
			int32 NonDegenerateNeighborTid = FDynamicMesh3::InvalidID;
			GetNeighborThatHasNormal(Tid, CurrentWalkedTids, NonDegenerateNeighborTid);

			// Make sure there was a non-degenerate neighbor.
			if (NonDegenerateNeighborTid == FDynamicMesh3::InvalidID)
			{
				ensureMsgf(false, TEXT("FMeshNormals::SetDegenerateTriangleNormalsToNeighborNormal: "
					"Had a component entirely composed of degenerate triangle normals."));
				IslandDegenerates.Append(CurrentWalkedTids);
			}
			else
			{
				// Apply the neighbor normal.
				FVector3d NormalToUse = Normals[NonDegenerateNeighborTid];
				check(NormalToUse != FVector3d::Zero());

				for (int32 WalkedTid : CurrentWalkedTids)
				{
					Normals[WalkedTid] = NormalToUse;
				}
			}
		}//end if normal is zero
	}//end for all triangles
}




void FMeshNormals::Compute_Overlay_FaceAvg(const FDynamicMeshNormalOverlay* NormalOverlay, bool bWeightByArea, bool bWeightByAngle)
{
	if ((!bWeightByAngle) && bWeightByArea)
	{
		Compute_Overlay_FaceAvg_AreaWeighted(NormalOverlay); // faster case
		return;
	}

	// most general case
	SetCount(NormalOverlay->MaxElementID(), true);

	for (int TriIdx : Mesh->TriangleIndicesItr())
	{
		FIndex3i Tri = NormalOverlay->GetTriangle(TriIdx);
		if (Tri.A == INDEX_NONE)
		{
			continue;
		}

		FVector3d V0, V1, V2;
		Mesh->GetTriVertices(TriIdx, V0, V1, V2);

		FVector3d TriNormal;
		double TriArea;
		TriNormal = VectorUtil::NormalArea(V0, V1, V2, TriArea);
		FVector3d TriNormalWeights = GetVertexWeightsOnTriangle(Mesh, TriIdx, TriArea, bWeightByArea, bWeightByAngle);

		for (int j = 0; j < 3; ++j) 
		{
			Normals[Tri[j]] += TriNormal * TriNormalWeights[j];
		}
	}

	for (int ElemIdx : NormalOverlay->ElementIndicesItr())
	{
		Normalize(Normals[ElemIdx]);
	}
}

void FMeshNormals::Compute_Overlay_FaceAvg_AreaWeighted(const FDynamicMeshNormalOverlay* NormalOverlay)
{
	SetCount(NormalOverlay->MaxElementID(), true);

	for (int TriIdx : Mesh->TriangleIndicesItr())
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh->GetTriInfo(TriIdx, TriNormal, TriArea, TriCentroid);
		TriNormal *= TriArea;

		FIndex3i Tri = NormalOverlay->GetTriangle(TriIdx);
		for (int j = 0; j < 3; ++j) 
		{
			if (Tri[j] != FDynamicMesh3::InvalidID)
			{
				Normals[Tri[j]] += TriNormal;
			}
		}
	}

	for (int ElemIdx : NormalOverlay->ElementIndicesItr())
	{
		Normalize(Normals[ElemIdx]);
	}
}


void FMeshNormals::QuickComputeVertexNormals(FDynamicMesh3& Mesh, bool bInvert)
{
	FMeshNormals normals(&Mesh);
	normals.ComputeVertexNormals();
	normals.CopyToVertexNormals(&Mesh, bInvert);
}


void FMeshNormals::SmoothVertexNormals(FDynamicMesh3& Mesh, int32 SmoothingRounds, double SmoothingAlpha)
{
	SmoothingRounds = FMath::Clamp(SmoothingRounds, 0, 500);
	SmoothingAlpha = FMathd::Clamp(SmoothingAlpha, 0.0, 1.0);
	if (SmoothingRounds > 0 && SmoothingAlpha > 0)
	{
		int32 NumV = Mesh.MaxVertexID();
		TArray<FVector3d> SmoothedNormals;
		SmoothedNormals.SetNum(NumV);
		for (int32 ri = 0; ri < SmoothingRounds; ++ri)
		{
			SmoothedNormals.Init(FVector3d::Zero(), NumV);

			// compute
			ParallelFor(NumV, [&](int32 vid)
			{
				if (Mesh.IsVertex(vid))
				{
					FVector3d SmoothedNormal = FVector3d::Zero();
					Mesh.EnumerateVertexVertices(vid, [&](int32 nbrvid)
					{
						SmoothedNormal += (FVector3d)Mesh.GetVertexNormal(nbrvid);
					});
					Normalize(SmoothedNormal);
					SmoothedNormals[vid] = Lerp((FVector3d)Mesh.GetVertexNormal(vid), SmoothedNormal, SmoothingAlpha);
					Normalize(SmoothedNormals[vid]);
				}
			});

			// update
			for (int32 vid : Mesh.VertexIndicesItr())
			{
				Mesh.SetVertexNormal(vid, (FVector3f)SmoothedNormals[vid]);
			}
		}
	}
}




void FMeshNormals::QuickComputeVertexNormalsForTriangles(FDynamicMesh3& Mesh, const TArray<int32>& Triangles, bool bWeightByArea, bool bWeightByAngle, bool bInvert)
{
	if (Mesh.HasVertexNormals() == false)
	{
		Mesh.EnableVertexNormals(FVector3f::UnitX());
	}

	TArray<int32> VertexIDs;
	UE::Geometry::TriangleToVertexIDs(&Mesh, Triangles, VertexIDs);
	ParallelFor(VertexIDs.Num(), [&](int32 i)
	{
		int32 vid = VertexIDs[i];
		FVector3d VtxNormal = ComputeVertexNormal(Mesh, vid, bWeightByArea, bWeightByAngle);
		Mesh.SetVertexNormal(vid, (FVector3f)VtxNormal);
	});
}


namespace UE::MeshNormalsLocals
{
	// This is a workaround for some platforms not supporting C++20's fetch_add for floats
	// UE platforms should be on C++20, so hopefully we can remove this method in the future and just use .fetch_add directly
	template<typename AtomicFloatType = std::atomic<float>>
	static inline void AtomicFloatFetchAdd(AtomicFloatType& Value, float ToAdd)
	{
		constexpr bool bHasAtomicFloatFetchAdd = requires(AtomicFloatType& AtomicFloat, float Param)
		{
			AtomicFloat.fetch_add(Param);
		};
		if constexpr (bHasAtomicFloatFetchAdd)
		{
			Value.fetch_add(ToAdd);
		}
		else
		{
			// Manual implementation of fetch_add
			float Old = Value.load();
			while (!Value.compare_exchange_weak(Old, Old + ToAdd)) {}
		}
	}
}

bool FMeshNormals::QuickRecomputeOverlayNormals(FDynamicMesh3& Mesh, bool bInvert, bool bWeightByArea, bool bWeightByAngle, bool bParallelCompute)
{
	if (!Mesh.HasAttributes() || Mesh.Attributes()->PrimaryNormals() == nullptr)
	{
		return false;
	}
	FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
	if (!bParallelCompute)
	{
		FMeshNormals Normals(&Mesh);
		Normals.RecomputeOverlayNormals(NormalOverlay, bWeightByArea, bWeightByAngle);
		Normals.CopyToOverlay(NormalOverlay, bInvert);
		return true;
	}
	else
	{
		// for the parallel case we want to compute once per triangle normal, and accumulate results in element normals
		// there is some overhead to using the atomic float buffer, so if not threading it is better to not do this
		TArray<std::atomic<float>> Normals;
		Normals.SetNumZeroed(NormalOverlay->MaxElementID() * 3);

		constexpr bool bForceSingleThreaded = false;

		ParallelFor(Mesh.MaxTriangleID(), [&Normals, &Mesh, bWeightByArea, bWeightByAngle, NormalOverlay](int32 TID)
		{
			if (!Mesh.IsTriangle(TID))
			{
				return;
			}

			FIndex3i ElTri = NormalOverlay->GetTriangle(TID);
			if (ElTri.A == INDEX_NONE)
			{
				// triangle was not set / has no elements
				return;
			}

			FVector3d V0, V1, V2;
			Mesh.GetTriVertices(TID, V0, V1, V2);

			FVector3d TriNormal;
			double TriArea;
			TriNormal = VectorUtil::NormalArea(V0, V1, V2, TriArea);
			FVector3f TriNormalWeights = (FVector3f)TMeshQueries<FDynamicMesh3>::GetVertexWeightsOnTriangle(Mesh, TID, TriArea, bWeightByArea, bWeightByAngle);
			FVector3f TriNormalf = (FVector3f)TriNormal;

			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				int32 ElID = ElTri[SubIdx];
				FVector3f AddNormal = TriNormalf * TriNormalWeights[SubIdx];
				int32 NormalArrayIdx = ElID * 3;
				for (int32 VecIdx = 0; VecIdx < 3; ++VecIdx)
				{
					UE::MeshNormalsLocals::AtomicFloatFetchAdd(Normals[NormalArrayIdx + VecIdx], AddNormal[VecIdx]);
				}
			}
		}, bForceSingleThreaded);

		float Sign = (bInvert) ? -1.0f : 1.0f;
		ParallelFor(NormalOverlay->MaxElementID(), [&Normals, Sign, NormalOverlay](int32 ElID)
		{
			if (NormalOverlay->IsElement(ElID))
			{
				int32 NormalsIdx = ElID * 3;
				// Note: Normalization intentionally computed in double precision for accuracy
				FVector3d Normal(Normals[NormalsIdx], Normals[NormalsIdx + 1], Normals[NormalsIdx + 2]);
				NormalOverlay->SetElement(ElID, static_cast<FVector3f>(Sign * Normalized(Normal)));
			}
		}, bForceSingleThreaded);
	}
	return true;
}

bool FMeshNormals::RecomputeOverlayTriNormals(FDynamicMesh3& Mesh, const TArray<int32>& Triangles, bool bWeightByArea, bool bWeightByAngle)
{
	if (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals() != nullptr )
	{
		FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
		TSet<int32> UniqueElementIDs;
		for ( int32 tid : Triangles)
		{
			if (NormalOverlay->IsSetTriangle(tid))
			{
				FIndex3i NormalTri = NormalOverlay->GetTriangle(tid);
				UniqueElementIDs.Add(NormalTri.A);
				UniqueElementIDs.Add(NormalTri.B);
				UniqueElementIDs.Add(NormalTri.C);
			}
		}

		return RecomputeOverlayElementNormals(Mesh, UniqueElementIDs.Array(), bWeightByArea, bWeightByAngle);
	}
	return false;
}

bool FMeshNormals::RecomputeOverlayElementNormals(FDynamicMesh3& Mesh, const TArray<int32>& ElementIDs, bool bWeightByArea, bool bWeightByAngle)
{
	if (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals() != nullptr )
	{
		FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
		ParallelFor(ElementIDs.Num(), [&](int32 k) 
		{
			int32 ElementID = ElementIDs[k];
			if ( NormalOverlay->IsElement(ElementID) )
			{
				FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(Mesh, NormalOverlay, ElementID, bWeightByArea, bWeightByAngle);
				NormalOverlay->SetElement(ElementID, (FVector3f)NewNormal);
			}
		});
		return true;
	}
	return false;
}


FVector3d FMeshNormals::ComputeVertexNormal(const FDynamicMesh3& Mesh, int VertIdx, bool bWeightByArea, bool bWeightByAngle)
{
	FVector3d SumNormal = FVector3d::Zero();
	Mesh.EnumerateVertexTriangles(VertIdx, [&](int32 TriIdx)
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh.GetTriInfo(TriIdx, TriNormal, TriArea, TriCentroid);
		FVector3d TriNormalWeights = GetVertexWeightsOnTriangle(&Mesh, TriIdx, TriArea, bWeightByArea, bWeightByAngle);

		FIndex3i Triangle = Mesh.GetTriangle(TriIdx);
		int32 j = IndexUtil::FindTriIndex(VertIdx, Triangle);
		SumNormal += TriNormal * TriNormalWeights[j];
	});
	return Normalized(SumNormal);
}


FVector3d FMeshNormals::ComputeVertexNormal(const FDynamicMesh3& Mesh, int32 VertIdx, TFunctionRef<bool(int32)> TriangleFilterFunc, bool bWeightByArea, bool bWeightByAngle)
{
	FVector3d NormalSum(0, 0, 0);
	Mesh.EnumerateVertexTriangles(VertIdx, [&](int32 TriIdx)
	{
		if (TriangleFilterFunc(TriIdx))
		{
			FVector3d TriNormal, TriCentroid; double TriArea;
			Mesh.GetTriInfo(TriIdx, TriNormal, TriArea, TriCentroid);
			FVector3d TriNormalWeights = GetVertexWeightsOnTriangle(&Mesh, TriIdx, TriArea, bWeightByArea, bWeightByAngle);

			FIndex3i Triangle = Mesh.GetTriangle(TriIdx);
			int32 j = IndexUtil::FindTriIndex(VertIdx, Triangle);
			NormalSum += TriNormal * TriNormalWeights[j];
		}
	});
	return Normalized(NormalSum);
}



FVector3d FMeshNormals::ComputeOverlayNormal(const FDynamicMesh3& Mesh, const FDynamicMeshNormalOverlay* NormalOverlay, int ElemIdx, bool bWeightByArea, bool bWeightByAngle)
{
	int ParentVertexID = NormalOverlay->GetParentVertex(ElemIdx);
	FVector3d SumNormal = FVector3d::Zero();
	int Count = 0;
	Mesh.EnumerateVertexTriangles(ParentVertexID, [&](int32 TriIdx)
	{
		if (NormalOverlay->TriangleHasElement(TriIdx, ElemIdx))
		{
			FVector3d Normal, Centroid; double Area;
			Mesh.GetTriInfo(TriIdx, Normal, Area, Centroid);
			FVector3d TriNormalWeights = GetVertexWeightsOnTriangle(&Mesh, TriIdx, Area, bWeightByArea, bWeightByAngle);
			FIndex3i Triangle = NormalOverlay->GetTriangle(TriIdx);
			int32 j = IndexUtil::FindTriIndex(ElemIdx, Triangle);	// todo: we computed already in TriangleHasElement...
			SumNormal += Normal * TriNormalWeights[j];
			Count++;
		}
	});

	return (Count > 0) ? Normalized(SumNormal) : FVector3d::Zero();
}


void FMeshNormals::InitializeOverlayToPerVertexNormals(FDynamicMeshNormalOverlay* NormalOverlay, bool bUseMeshVertexNormalsIfAvailable)
{
	const FDynamicMesh3* Mesh = NormalOverlay->GetParentMesh();
	bool bUseMeshNormals = bUseMeshVertexNormalsIfAvailable && Mesh->HasVertexNormals();
	FMeshNormals Normals(Mesh);
	if (bUseMeshNormals == false)
	{
		Normals.ComputeVertexNormals();
	}

	NormalOverlay->ClearElements();

	TArray<int> VertToNormalMap;
	VertToNormalMap.SetNumUninitialized(Mesh->MaxVertexID());
	for (int vid : Mesh->VertexIndicesItr())
	{
		FVector3f Normal = (bUseMeshNormals) ? Mesh->GetVertexNormal(vid) : (FVector3f)Normals[vid];
		int nid = NormalOverlay->AppendElement(Normal);
		VertToNormalMap[vid] = nid;
	}

	for (int tid : Mesh->TriangleIndicesItr())
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		Tri.A = VertToNormalMap[Tri.A];
		Tri.B = VertToNormalMap[Tri.B];
		Tri.C = VertToNormalMap[Tri.C];
		NormalOverlay->SetTriangle(tid, Tri);
	}
}



void FMeshNormals::InitializeOverlayToPerTriangleNormals(FDynamicMeshNormalOverlay* NormalOverlay)
{
	const FDynamicMesh3* Mesh = NormalOverlay->GetParentMesh();

	NormalOverlay->ClearElements();

	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		FVector3d Normal = Mesh->GetTriNormal(tid);
		int32 e0 = NormalOverlay->AppendElement((FVector3f)Normal);
		int32 e1 = NormalOverlay->AppendElement((FVector3f)Normal);
		int32 e2 = NormalOverlay->AppendElement((FVector3f)Normal);
		NormalOverlay->SetTriangle(tid, FIndex3i(e0, e1, e2));
	}
}


void FMeshNormals::InitializeOverlayTopologyFromOpeningAngle(const FDynamicMesh3* Mesh, FDynamicMeshNormalOverlay* NormalOverlay, 
	double AngleThresholdDeg)
{
	double NormalDotProdThreshold = FMathd::Cos(AngleThresholdDeg * FMathd::DegToRad);

	FMeshNormals FaceNormals(Mesh);
	FaceNormals.ComputeTriangleNormals();
	const TArray<FVector3d>& Normals = FaceNormals.GetNormals();
	NormalOverlay->CreateFromPredicate([&Normals, &NormalDotProdThreshold](int VID, int TA, int TB) {
		return Normals[TA].Dot(Normals[TB]) > NormalDotProdThreshold;
	}, 0);
}

void FMeshNormals::InitializeOverlayTopologyFromFaceGroups(const FDynamicMesh3* Mesh, FDynamicMeshNormalOverlay* NormalOverlay)
{
	ensure(Mesh->HasTriangleGroups());
	NormalOverlay->CreateFromPredicate([Mesh](int VID, int TA, int TB) {
		return Mesh->GetTriangleGroup(TA) == Mesh->GetTriangleGroup(TB);
	}, 0);
}


void FMeshNormals::InitializeMeshToPerTriangleNormals(FDynamicMesh3* Mesh)
{
	if (Mesh->HasAttributes() == false)
	{
		Mesh->EnableAttributes();
	}
	FDynamicMeshNormalOverlay* Overlay = Mesh->Attributes()->PrimaryNormals();
	InitializeOverlayToPerTriangleNormals(Overlay);
}



void FMeshNormals::InitializeOverlayRegionToPerVertexNormals(FDynamicMeshNormalOverlay* NormalOverlay, const TArray<int32>& Triangles)
{
	const FDynamicMesh3* Mesh = NormalOverlay->GetParentMesh();

	// should we remove existing elements that may become unreferenced?

	TSet<int32> TriangleSet(Triangles);
	TArray<int32> Vertices;
	UE::Geometry::TriangleToVertexIDs(Mesh, Triangles, Vertices);
	auto TriangleSetFunc = [&](int32 tid) { return TriangleSet.Contains(tid); };
	int32 NumVertices = Vertices.Num();
	TMap<int32, int32> TriangleMap;
	TriangleMap.Reserve(NumVertices);

	TArray<int32> VertNormals;
	VertNormals.SetNum(NumVertices);
	for ( int32 i = 0; i < NumVertices; ++i)
	{
		int32 vid = Vertices[i];
		FVector3d Normal = FMeshNormals::ComputeVertexNormal(*Mesh, vid, TFunctionRef<bool(int32)>(TriangleSetFunc), true, true);
		int32 nid = NormalOverlay->AppendElement((FVector3f)Normal);
		VertNormals[i] = nid;

		TriangleMap.Add(vid, i);
	}

	for (int32 tid : Triangles)
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		Tri.A = VertNormals[ TriangleMap[Tri.A] ];
		Tri.B = VertNormals[ TriangleMap[Tri.B] ];
		Tri.C = VertNormals[ TriangleMap[Tri.C] ];
		NormalOverlay->SetTriangle(tid, Tri);
	}
}

FVector3d FMeshNormals::GetVertexWeightsOnTriangle(const FDynamicMesh3* Mesh, int TriID, double TriArea, bool bWeightByArea, bool bWeightByAngle)
{
	return TMeshQueries<FDynamicMesh3>::GetVertexWeightsOnTriangle(*Mesh, TriID, TriArea, bWeightByArea, bWeightByAngle);
}
