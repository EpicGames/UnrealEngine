// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/MeshTangents.h"
#include "VectorUtil.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;



FDynamicMeshTangents::FDynamicMeshTangents(const FDynamicMesh3* MeshIn)
{
	Mesh = MeshIn;
	if (ensure(Mesh) && ensure(Mesh->HasAttributes()))
	{
		Normals = Mesh->Attributes()->PrimaryNormals();
		if (ensure(Normals) && Mesh->Attributes()->HasTangentSpace() )
		{
			Tangents = Mesh->Attributes()->PrimaryTangents();
			Bitangents = Mesh->Attributes()->PrimaryBiTangents();
		}
	}
}


bool FDynamicMeshTangents::HasValidTangents(const bool bCheckValues /*=false*/) const
{
	if (!Mesh || !Tangents || !Bitangents)
	{
		return false;
	}

	if (!bCheckValues)
	{
		// If not checking values, then we are done. Overlays are valid. 
		return true;
	}

	for (const int TriId : Mesh->TriangleIndicesItr())
	{
		if (!Tangents->IsSetTriangle(TriId))
		{
			return false;
		}
		
		for (int TriVtxId = 0; TriVtxId < 2; ++TriVtxId)
		{
			FVector3f T, B;
			Tangents->GetTriElement(TriId, TriVtxId, T);
			Bitangents->GetTriElement(TriId, TriVtxId, B);
			if (T.IsNearlyZero() || T.ContainsNaN() || B.IsNearlyZero() || B.ContainsNaN())
			{
				return false;
			}
		}
	}
	return true;
}



void FDynamicMeshTangents::GetTangentFrame(int32 TriangleID, int32 TriVertexIndex, FVector3f& Normal, FVector3f& Tangent, FVector3f& Bitangent) const
{
	if (Normals && Normals->IsSetTriangle(TriangleID))
	{
		Normals->GetTriElement(TriangleID, TriVertexIndex, Normal);
		if (Tangents && Tangents->IsSetTriangle(TriangleID))
		{
			Tangents->GetTriElement(TriangleID, TriVertexIndex, Tangent);
			Bitangents->GetTriElement(TriangleID, TriVertexIndex, Bitangent);
		}
		else
		{
			VectorUtil::MakePerpVectors(Normal, Tangent, Bitangent);
		}
	}
	else
	{
		Normal = FVector3f::UnitZ();
		Tangent = FVector3f::UnitX();
		Bitangent = FVector3f::UnitY();
	}
}


void FDynamicMeshTangents::GetTangentVectors(int32 TriangleID, int32 TriVertexIndex, const FVector3f& Normal, FVector3f& Tangent, FVector3f& Bitangent) const
{
	if (Tangents && Tangents->IsSetTriangle(TriangleID) )
	{
		Tangents->GetTriElement(TriangleID, TriVertexIndex, Tangent);
		Bitangents->GetTriElement(TriangleID, TriVertexIndex, Bitangent);
	}
	else
	{
		VectorUtil::MakePerpVectors(Normal, Tangent, Bitangent);
	}
}




void FDynamicMeshTangents::GetTangentVectors(int32 TriangleID, int32 TriVertexIndex, FVector3f& Tangent, FVector3f& Bitangent) const
{
	if (Tangents && Tangents->IsSetTriangle(TriangleID))
	{
		Tangents->GetTriElement(TriangleID, TriVertexIndex, Tangent);
		Bitangents->GetTriElement(TriangleID, TriVertexIndex, Bitangent);
	}
	else
	{
		Tangent = FVector3f::UnitX();
		Bitangent = FVector3f::UnitY();
	}
}



template<typename RealType>
void TMeshTangents<RealType>::SetTangentCount(int Count, bool bClearToZero)
{
	if (Tangents.Num() < Count)
	{
		Tangents.SetNum(Count);
	}
	if (Bitangents.Num() < Count)
	{
		Bitangents.SetNum(Count);
	}
	if (bClearToZero)
	{
		for (int i = 0; i < Count; ++i)
		{
			Tangents[i] = TVector<RealType>::Zero();
			Bitangents[i] = TVector<RealType>::Zero();
		}
	}
}


template<typename RealType>
bool TMeshTangents<RealType>::CopyTriVertexTangents(const FDynamicMesh3& SourceMesh)
{
	if (SourceMesh.HasAttributes() == false || SourceMesh.Attributes()->HasTangentSpace() == false)
	{
		return false;
	}
	const FDynamicMeshNormalOverlay* TangentsAttrib = SourceMesh.Attributes()->PrimaryTangents();
	const FDynamicMeshNormalOverlay* BiTangentsAttrib = SourceMesh.Attributes()->PrimaryBiTangents();

	//InitializeTriVertexTangents(false);    // this requires Mesh to be initialized which is not strictly necessary...
	SetTangentCount(SourceMesh.MaxTriangleID() * 3, false);

	for (int32 tid : SourceMesh.TriangleIndicesItr())
	{
		if ( TangentsAttrib->IsSetTriangle(tid) && BiTangentsAttrib->IsSetTriangle(tid) )
		{
			FVector3f TriTangents[3];
			TangentsAttrib->GetTriElements(tid, TriTangents[0], TriTangents[1], TriTangents[2]);
			FVector3f TriBiTangents[3];
			BiTangentsAttrib->GetTriElements(tid, TriBiTangents[0], TriBiTangents[1], TriBiTangents[2]);

			int32 k = 3 * tid;
			for (int32 j = 0; j < 3; ++j )
			{
				Tangents[k+j] = TVector<RealType>(TriTangents[j]);
				Bitangents[k+j] = TVector<RealType>(TriBiTangents[j]);
			}
		}
	}

	return true;
}



template<typename RealType>
void TMeshTangents<RealType>::ComputeTriVertexTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options)
{
	if (Options.bAveraged)
	{
		ComputeMikkTStyleTangents(NormalOverlay, UVOverlay, Options);
	}
	else
	{
		ComputeSeparatePerTriangleTangents(NormalOverlay, UVOverlay, Options);
	}
}


/**
 * Utility function to compute tangent/bitangent on a triangle. Possibly should be exposed somewher?
 */
static void ComputeFaceTangent(
	FVector3d TriVertices[3], FVector2f TriUVs[3],
	FVector3d& TangentOut, FVector3d& BitangentOut,
	FVector2d& MagnitudesOut,
	double& OrientationSignOut,
	bool& bIsDegenerateOut)
{
	FVector2d UVEdge1 = (FVector2d)TriUVs[1] - (FVector2d)TriUVs[0];
	FVector2d UVEdge2 = (FVector2d)TriUVs[2] - (FVector2d)TriUVs[0];
	FVector3d TriEdge1 = TriVertices[1] - TriVertices[0];
	FVector3d TriEdge2 = TriVertices[2] - TriVertices[0];

	FVector3d TriTangent = (UVEdge2.Y * TriEdge1) - (UVEdge1.Y * TriEdge2);
	FVector3d TriBitangent = (-UVEdge2.X * TriEdge1) + (UVEdge1.X * TriEdge2);

	double UVArea = (UVEdge1.X * UVEdge2.Y) - (UVEdge1.Y * UVEdge2.X);
	bool bPreserveOrientation = (UVArea >= 0);

	UVArea = FMathd::Abs(UVArea);

	// if a triangle is zero-UV-area due to one edge being collapsed, we still have a 
	// valid direction on the other edge. We are going to keep those
	double TriTangentLength = TriTangent.Length();
	double TriBitangentLength = TriBitangent.Length();
	TangentOut = (TriTangentLength > 0) ? (TriTangent / TriTangentLength) : FVector3d::Zero();
	BitangentOut = (TriBitangentLength > 0) ? (TriBitangent / TriBitangentLength) : FVector3d::Zero();
	MagnitudesOut = FVector2d(TriTangentLength / UVArea, TriBitangentLength / UVArea);

	if (bPreserveOrientation)
	{
		OrientationSignOut = 1.0;
	}
	else
	{
		OrientationSignOut = -1.0;
		TangentOut = -TangentOut;
		BitangentOut = -BitangentOut;
	}

	bIsDegenerateOut = (UVArea < FMathd::ZeroTolerance);
}



template<typename RealType>
void TMeshTangents<RealType>::ComputeTriangleTangents(const FDynamicMeshUVOverlay* UVOverlay, bool bOrthogonalize)
{
	int32 MaxTriangleID = Mesh->MaxTriangleID();
	InitializeTriangleTangents(false);

	AllDegenerateTris.Reset();
	FCriticalSection DegenerateTriLock;

	// compute per-triangle tangent and bitangent
	ParallelFor(MaxTriangleID, [&](int32 TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID) == false)
		{
			return;
		}
		if (UVOverlay->IsSetTriangle(TriangleID) == false)
		{
			DegenerateTriLock.Lock();
			AllDegenerateTris.Add(TriangleID);
			DegenerateTriLock.Unlock();
			return;
		}

		FVector3d TriVertices[3];
		Mesh->GetTriVertices(TriangleID, TriVertices[0], TriVertices[1], TriVertices[2]);
		FVector2f TriUVs[3];
		UVOverlay->GetTriElements(TriangleID, TriUVs[0], TriUVs[1], TriUVs[2]);
		FVector3d TriNormal = Mesh->GetTriNormal(TriangleID);

		FVector3d Tangent, Bitangent;
		FVector2d Magnitudes;
		double OrientationSign;
		bool bIsDegenerate;
		ComputeFaceTangent(TriVertices, TriUVs, Tangent, Bitangent, Magnitudes, OrientationSign, bIsDegenerate);

		if (bIsDegenerate)
		{
			DegenerateTriLock.Lock();
			AllDegenerateTris.Add(TriangleID);
			DegenerateTriLock.Unlock();
			Tangents[TriangleID] = (TVector<RealType>)Tangent;
			Bitangents[TriangleID] = (TVector<RealType>)Bitangent;
			return;
		}

		if (bOrthogonalize)
		{
			double BitangentSign = VectorUtil::BitangentSign(TriNormal, Tangent, Bitangent);
			Bitangent = VectorUtil::Bitangent(TriNormal, Tangent, BitangentSign);
			Normalize(Bitangent);
		}

		Tangents[TriangleID] = (TVector<RealType>)Tangent;
		Bitangents[TriangleID] = (TVector<RealType>)Bitangent;
	});
}



template<typename RealType>
bool TMeshTangents<RealType>::CopyToOverlays(FDynamicMesh3& MeshToSet) const
{
	if (!MeshToSet.HasAttributes() || MeshToSet.Attributes()->NumNormalLayers() != 3)
	{
		return false;
	}

	// Set aliases to make iterating over tangents and bitangents easier
	FDynamicMeshNormalOverlay* TangentOverlays[2] = { MeshToSet.Attributes()->PrimaryTangents(), MeshToSet.Attributes()->PrimaryBiTangents() };
	const TArray<TVector<RealType>>* TangentValues[2] = { &Tangents, &Bitangents };
	
	for (int Idx = 0; Idx < 2; Idx++)
	{
		// Create overlay topology
		const TArray<TVector<RealType>>& TV = *TangentValues[Idx];
		TangentOverlays[Idx]->CreateFromPredicate([&MeshToSet, &TV](int ParentVertexIdx, int TriIDA, int TriIDB) -> bool
		{
			FIndex3i TriA = MeshToSet.GetTriangle(TriIDA);
			FIndex3i TriB = MeshToSet.GetTriangle(TriIDB);
			int SubA = TriA.IndexOf(ParentVertexIdx);
			int SubB = TriB.IndexOf(ParentVertexIdx);
			checkSlow(SubA > -1 && SubB > -1);
			const TVector<RealType>& A = TV[TriIDA * 3 + SubA];
			const TVector<RealType>& B = TV[TriIDB * 3 + SubB];
			return DistanceSquared(A,B) < TMathUtil<RealType>::ZeroTolerance;
		}, 0);

		// Write tangent values out for each wedge value
		// Note: shared elements will be written to multiple times, and the last value written will be used
		for (int TID : MeshToSet.TriangleIndicesItr())
		{
			FIndex3i ElTri = TangentOverlays[Idx]->GetTriangle(TID);
			for (int SubIdx = 0; SubIdx < 3; SubIdx++)
			{
				TangentOverlays[Idx]->SetElement(ElTri[SubIdx], (FVector3f)TV[TID * 3 + SubIdx]);
			}
		}
	}
	return true;
}



static FVector3d PlaneProjectionNormalized(const FVector3d& Vector, const FVector3d& PlaneNormal)
{
	return Normalized(Vector - Vector.Dot(PlaneNormal) * PlaneNormal);
}



template<typename RealType>
void TMeshTangents<RealType>::ComputeSeparatePerTriangleTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options)
{
	int32 MaxTriangleID = Mesh->MaxTriangleID();
	InitializeTriVertexTangents(false);

	const EParallelForFlags ParallelForFlags = Options.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

	// compute per-triangle tangent and bitangent
	ParallelFor(MaxTriangleID, [&](int32 TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID) == false || UVOverlay->IsSetTriangle(TriangleID) == false)
		{
			return;
		}

		FVector3d TriVertices[3];
		Mesh->GetTriVertices(TriangleID, TriVertices[0], TriVertices[1], TriVertices[2]);
		FVector2f TriUVs[3];
		UVOverlay->GetTriElements(TriangleID, TriUVs[0], TriUVs[1], TriUVs[2]);
		FVector3f TriNormals[3];
		NormalOverlay->GetTriElements(TriangleID, TriNormals[0], TriNormals[1], TriNormals[2]);

		FVector3d Tangent, Bitangent;
		FVector2d Magnitudes;
		double OrientationSign;
		bool bIsDegenerate;
		ComputeFaceTangent(TriVertices, TriUVs, Tangent, Bitangent, Magnitudes, OrientationSign, bIsDegenerate);

		for (int32 j = 0; j < 3; ++j)
		{
			FVector3d VtxNormal = (FVector3d)TriNormals[j];
			FVector3d ProjectedTangent = PlaneProjectionNormalized(Tangent, VtxNormal);

			double BitangentSign = VectorUtil::BitangentSign(VtxNormal, ProjectedTangent, Bitangent);
			FVector3d ReconsBitangent = VectorUtil::Bitangent(VtxNormal, ProjectedTangent, BitangentSign);

			SetPerTriangleTangent(TriangleID, j, 
				Normalized((TVector<RealType>)ProjectedTangent),
				Normalized((TVector<RealType>)ReconsBitangent) );
		}

	}, ParallelForFlags);
}



template<typename RealType>
void TMeshTangents<RealType>::ComputeMikkTStyleTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options)
{
	checkSlow(NormalOverlay != nullptr);
	checkSlow(UVOverlay != nullptr);

	double HardcodedSplitDotThresh = -0.99;

	int32 MaxTriangleID = Mesh->MaxTriangleID();
	InitializeTriVertexTangents(true);

	TArray<FVector3d> TriTangents;
	TriTangents.SetNum(MaxTriangleID);
	TArray<FVector3d> TriBitangents;
	TriBitangents.SetNum(MaxTriangleID);
	TArray<FVector3d> TriMagnitudes;
	TriMagnitudes.SetNum(MaxTriangleID);

	TArray<int32> DegenerateTris;
	FCriticalSection DegenerateTrisLock;

	const EParallelForFlags ParallelForFlags = Options.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

	// similar process to MikkTSpace:
	// http://image.diku.dk/projects/media/morten.mikkelsen.08.pdf
	//  - todo: currently splitting any triangle with large-angle from accumulated group. Should combine these split triangles into a new group.
	//  - todo: degenerate handling. Currently propagating from arbitrary nbr edge. Should propagate from matching edges (ie shared-uv, shared-normal, etc, same as group)

	// compute per-triangle tangent and bitangent
	ParallelFor(MaxTriangleID, [&](int32 TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID) == false || UVOverlay->IsSetTriangle(TriangleID) == false)
		{
			return;
		}

		FVector3d TriVertices[3];
		Mesh->GetTriVertices(TriangleID, TriVertices[0], TriVertices[1], TriVertices[2]);
		FVector2f TriUVs[3];
		UVOverlay->GetTriElements(TriangleID, TriUVs[0], TriUVs[1], TriUVs[2]);

		FVector3d Tangent, Bitangent;
		FVector2d Magnitudes;
		double OrientationSign;
		bool bIsDegenerate;
		ComputeFaceTangent(TriVertices, TriUVs, Tangent, Bitangent, Magnitudes, OrientationSign, bIsDegenerate);

		TriTangents[TriangleID] = Tangent;
		TriBitangents[TriangleID] = Bitangent;
		TriMagnitudes[TriangleID] = (bIsDegenerate) ? FVector3d::Zero() : FVector3d(Magnitudes.X, Magnitudes.Y, 0);

		if (bIsDegenerate)
		{
			DegenerateTrisLock.Lock();
			DegenerateTris.Add(TriangleID);
			DegenerateTrisLock.Unlock();
			return;
		}

	}, ParallelForFlags);

	// Average triangles tangents across vertex "wedges", ie combine one-ring
	// triangles that share same UV and Normal, for each triangle-vertex
	ParallelFor(Mesh->MaxVertexID(), [&](int32 VertexID)
	{
		if (Mesh->IsVertex(VertexID) == false)
		{
			return;
		}

		// iterate through triangle one-ring at vertex and find "unique" vertices
		// ie that have shared UV and Normal
		TArray<FIndex3i,TInlineAllocator<8>> Uniques;
		TArray<FVector3d,TInlineAllocator<8>> AccumTangent;
		TArray<FVector3d,TInlineAllocator<8>> AccumBitangent;
		TArray<FVector3d, TInlineAllocator<8>> AccumMagnitudeAngle;
		TArray<FIndex3i> TriSet;
		Mesh->EnumerateVertexTriangles(VertexID, [&](int32 TriangleID)
		{
			// if zero-magnitude, this was a degenerate triangle and so ignore it
			FVector3d Magnitudes = TriMagnitudes[TriangleID];
			if (Magnitudes.SquaredLength() < 0)
			{
				return;
			}

			// Find the index of the vertex in the triangle, and look up UV and Normal elements.
			// (If Normal is not set we fall back to triangle normal)
			FIndex3i Triangle = Mesh->GetTriangle(TriangleID);
			int32 Idx = IndexUtil::FindTriIndex(VertexID, Triangle);
			FIndex3i UVTriangle = UVOverlay->GetTriangle(TriangleID);
			int32 UVElementID = UVTriangle[Idx];
			FIndex3i NormTriangle = NormalOverlay->GetTriangle(TriangleID);
			int32 NormalElementID = NormTriangle[Idx];
			FVector3d Normal = NormalOverlay->IsSetTriangle(TriangleID) ?
				(FVector3d)NormalOverlay->GetElement(NormalElementID) : Mesh->GetTriNormal(TriangleID);

			// project vectors to plane and re-normalize
			FVector3d ProjTangent = PlaneProjectionNormalized(TriTangents[TriangleID], Normal);
			FVector3d ProjBitangent = PlaneProjectionNormalized(TriBitangents[TriangleID], Normal);

			// compute angle between outgoing edges
			double Angle = 1.0;
			if (Options.bAngleWeighted)
			{
				FVector3d TriVerts[3];
				Mesh->GetTriVertices(TriangleID, TriVerts[0], TriVerts[1], TriVerts[2]);
				FVector3d EdgeV1 = TriVerts[(Idx+1)%3] - TriVerts[Idx];
				FVector3d EdgeV2 = TriVerts[(Idx+2)%3] - TriVerts[Idx];
				Angle = AngleR(Normalized(EdgeV1), Normalized(EdgeV2));

				// weight by angle
				ProjTangent *= Angle;
				ProjBitangent *= Angle;
				Magnitudes.X *= Angle;
				Magnitudes.Y *= Angle;
			}
			FVector3d MagnitudeAngle(Magnitudes.X, Magnitudes.Y, Angle);

			FIndex3i Key(UVElementID, NormalElementID, (Magnitudes.Z < 0) ? -1 : 1 );
			//FIndex3i Key(UVElementID, NormalElementID, 0);
			int32 FoundIndex = Uniques.IndexOfByKey(Key);
			if (FoundIndex == INDEX_NONE)
			{
				FoundIndex = Uniques.Num();
				Uniques.Add(Key);
				AccumTangent.Add(ProjTangent);
				AccumBitangent.Add(ProjBitangent);
				AccumMagnitudeAngle.Add(MagnitudeAngle);
			}
			else
			{
				double DotTangent = Normalized(AccumTangent[FoundIndex]).Dot(ProjTangent);
				double DotBitangent = Normalized(AccumBitangent[FoundIndex]).Dot(ProjBitangent);
				if (DotTangent <= HardcodedSplitDotThresh || DotBitangent <= HardcodedSplitDotThresh )
				{
					// If angle between new and existing tangents is too large, we split into a new tangent group.
					// Currently this group will be unique, ie a single triangle, because we do not search for a
					// compatible existing group
					FoundIndex = Uniques.Num();
					Uniques.Add(Key);
					AccumTangent.Add(ProjTangent);
					AccumBitangent.Add(ProjBitangent);
					AccumMagnitudeAngle.Add(MagnitudeAngle);
				}
				else
				{
					check(Uniques[FoundIndex] == Key);
					AccumTangent[FoundIndex] += ProjTangent;
					AccumBitangent[FoundIndex] += ProjBitangent;
					AccumMagnitudeAngle[FoundIndex] += MagnitudeAngle;
				}
			}
			TriSet.Add(FIndex3i(TriangleID, Idx, FoundIndex));
		});


		// normalize for each group
		int32 NumTangents = AccumTangent.Num();
		for (int32 k = 0; k < NumTangents; ++k)
		{
			Normalize(AccumTangent[k]);
			Normalize(AccumBitangent[k]);
			if (AccumMagnitudeAngle[k].Z > 0)
			{
				AccumMagnitudeAngle[k].X /= AccumMagnitudeAngle[k].Z;
				AccumMagnitudeAngle[k].Y /= AccumMagnitudeAngle[k].Z;
			}
		}

		// initialize per-triangle-vertex in final tangent/bitangent lists
		for (const FIndex3i& TriEntry : TriSet)
		{
			int32 TriangleID = TriEntry.A;
			int32 TriIndex = TriEntry.B;
			int32 UniqueIndex = TriEntry.C;
			const FVector3d& Tangent = AccumTangent[UniqueIndex];
			const FVector3d& Bitangent = AccumBitangent[UniqueIndex];
			int32 TriVertIndex = 3*TriangleID + TriIndex;

			// todo: catch failures here?

			Tangents[TriVertIndex] = Normalized((TVector<RealType>)Tangent);

			FIndex3i NormTriangle = NormalOverlay->GetTriangle(TriangleID);
			FVector3d TriVertNormal = NormalOverlay->IsSetTriangle(TriangleID) ?
				(FVector3d)NormalOverlay->GetElement(NormTriangle[TriIndex]) : Mesh->GetTriNormal(TriangleID);

			// convert bitangent
			double BinormalSign = VectorUtil::BitangentSign(TriVertNormal, Tangent, Bitangent);
			FVector3d ReconsBitangent = VectorUtil::Bitangent(TriVertNormal, Tangent, BinormalSign);
			Bitangents[TriVertIndex] = Normalized((TVector<RealType>)ReconsBitangent);

			//FVector3d OrigBitangent = AccumBitangent[UniqueIndex];
			//TVector<RealType> BTResult = Bitangents[3 * TriangleID + TriIndex];
			//if (BTResult.SquaredLength() < 0.99)
			//{
			//	UE_LOGF(LogTemp, Warning, "TriVertNormal is %f %f %f", TriVertNormal.X, TriVertNormal.Y, TriVertNormal.Z);
			//	UE_LOGF(LogTemp, Warning, "GroupTangent is %f %f %f", GroupTangent.X, GroupTangent.Y, GroupTangent.Z);
			//	UE_LOGF(LogTemp, Warning, "Orig Bitangent is %f %f %f", OrigBitangent.X, OrigBitangent.Y, OrigBitangent.Z);
			//	UE_LOGF(LogTemp, Warning, "Dots are %f %f %f", TriVertNormal.Dot(GroupTangent), TriVertNormal.Dot(OrigBitangent), GroupTangent.Dot(OrigBitangent));
			//	UE_LOGF(LogTemp, Warning, "BinormalSign is %f", BinormalSign);
			//	UE_LOGF(LogTemp, Warning, "Recons Bitangent is %f %f %f", BTResult.X, BTResult.Y, BTResult.Z);
			//}
		}
	}, ParallelForFlags);


	AllDegenerateTris = DegenerateTris;

	// propagate to degenerates
	// note: this currently just takes the first paired edge, but really for each wedge-vertex we
	// should be using the criteria above if possible... (same UV/normal island)
	if (DegenerateTris.Num() > 0)
	{
		TSet<int32> DegenSet(DegenerateTris);
		TArray<int32> NewDegenerates;
		while (DegenerateTris.Num() > 0)
		{
			NewDegenerates.Reset();
			int32 NumDegenerates = DegenerateTris.Num();
			for (int32 k = 0; k < NumDegenerates; ++k)
			{
				bool bHandled = false;
				int32 TriangleID = DegenerateTris[k];
				FIndex3i TriEdges = Mesh->GetTriEdges(TriangleID);
				for (int j = 0; j < 3; ++j)
				{
					FDynamicMesh3::FEdge Edge = Mesh->GetEdge(TriEdges[j]);
					int32 OtherTriangleID = (Edge.Tri.A == TriangleID) ? Edge.Tri.B : Edge.Tri.A;
					if (DegenSet.Contains(OtherTriangleID) || OtherTriangleID == IndexConstants::InvalidID)
					{
						continue;
					}

					FIndex3i Triangle = Mesh->GetTriangle(TriangleID);
					int32 VertAIdx = IndexUtil::FindTriIndex(Edge.Vert.A, Triangle);
					int32 VertBIdx = IndexUtil::FindTriIndex(Edge.Vert.B, Triangle);
					int32 VertCIdx = IndexUtil::FindTriOtherIndex(Edge.Vert.A, Edge.Vert.B, Triangle);

					FIndex3i OtherTri = Mesh->GetTriangle(OtherTriangleID);
					int32 OtherVertAIdx = IndexUtil::FindTriIndex(Edge.Vert.A, OtherTri);
					int32 OtherVertBIdx = IndexUtil::FindTriIndex(Edge.Vert.B, OtherTri);

					Tangents[3*TriangleID + VertAIdx] = Tangents[3*OtherTriangleID + OtherVertAIdx];
					Bitangents[3*TriangleID + VertAIdx] = Bitangents[3*OtherTriangleID + OtherVertAIdx];
					Tangents[3*TriangleID + VertBIdx] = Tangents[3*OtherTriangleID + OtherVertBIdx];
					Bitangents[3*TriangleID + VertBIdx] = Bitangents[3*OtherTriangleID + OtherVertBIdx];

					// this does not seem correct...
					Tangents[3*TriangleID + VertCIdx] = Tangents[3* TriangleID + VertAIdx];
					Bitangents[3*TriangleID + VertCIdx] = Bitangents[3*TriangleID + VertAIdx];

					bHandled = true;
					break;
				}

				if (bHandled)
				{
					DegenSet.Remove(TriangleID);
				}
				else
				{
					NewDegenerates.Add(TriangleID);
				}
			}

			if (NewDegenerates.Num() == DegenerateTris.Num())
			{
				DegenerateTris.Reset();		// no progress, abort loop
			}
			else
			{
				Swap(DegenerateTris, NewDegenerates);		// hopefully this is doing MoveTemp ?
			}
		}
	}
}

namespace UE
{
namespace Geometry
{

#if PLATFORM_COMPILER_CLANG
#define UE_EXTERN_TEMPLATE_API GEOMETRYCORE_API
#else
#define UE_EXTERN_TEMPLATE_API
#endif

template class UE_EXTERN_TEMPLATE_API TMeshTangents<float>;
template class UE_EXTERN_TEMPLATE_API TMeshTangents<double>;

#undef UE_EXTERN_TEMPLATE_API

} // end namespace UE::Geometry
} // end namespace UE


