// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

namespace MeshTransformsLocal
{
	enum class ESkipOffset
	{
		SkipOffset
	};

	template<typename TransformPosF, typename TransformVecF>
	void TransformSculptLayers(FDynamicMesh3& Mesh, TransformPosF TransformPosition, TransformVecF TransformOffset, EParallelForFlags ParallelForFlags = EParallelForFlags::None)
	{
		if (Mesh.HasAttributes() && Mesh.Attributes()->NumSculptLayers() > 0)
		{
			FDynamicMeshSculptLayers* SculptLayers = Mesh.Attributes()->GetSculptLayers();
			ParallelFor(Mesh.MaxVertexID(), [&Mesh, SculptLayers, &TransformPosition, &TransformOffset](int VID)
				{
					if (Mesh.IsVertex(VID))
					{
						FVector3d BaseLayerPos;
						SculptLayers->GetLayer(0)->GetValue(VID, BaseLayerPos);
						SculptLayers->GetLayer(0)->SetValue(VID, TransformPosition(BaseLayerPos));
						if constexpr (!std::is_same<TransformVecF, MeshTransformsLocal::ESkipOffset>())
						{
							for (int32 LayerIdx = 1, NumLayers = Mesh.Attributes()->NumSculptLayers(); LayerIdx < NumLayers; ++LayerIdx)
							{
								FVector3d Offset;
								SculptLayers->GetLayer(LayerIdx)->GetValue(VID, Offset);
								SculptLayers->GetLayer(LayerIdx)->SetValue(VID, TransformOffset(Offset));
							}
						}
					}
				},
				ParallelForFlags
			);
		}
	}

	template<typename TransformPosF, typename TransformNormalF, typename TransformTangentF, typename TransformOffsetF>
	void TransformMeshHelper(FDynamicMesh3& Mesh, TransformPosF TransformPosition, TransformNormalF TransformNormal, TransformTangentF TransformTangent, TransformOffsetF TransformOffset, MeshTransforms::ETransformAttributes TransformAttributes)
	{
		using namespace MeshTransforms;

		const bool bVertexNormals = bool(TransformAttributes & ETransformAttributes::VertexNormals) && Mesh.HasVertexNormals();
		const bool bPositions = bool(TransformAttributes & ETransformAttributes::Positions);

		const int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [bPositions, bVertexNormals, &Mesh, &TransformPosition, &TransformNormal](int vid)
		{
			if (Mesh.IsVertex(vid)) 
			{
				if (bPositions)
				{
					FVector3d Position = Mesh.GetVertex(vid);
					Position = TransformPosition(Position);
					Mesh.SetVertex(vid, Position);
				}

				if (bVertexNormals)
				{
					FVector3f Normal = Mesh.GetVertexNormal(vid);
					Normal = TransformNormal(Normal);
					Mesh.SetVertexNormal(vid, Normal);
				}
			}
		});

		if (Mesh.HasAttributes())
		{
			FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
			if (Normals && bool(TransformAttributes & ETransformAttributes::Normals))
			{
				int NumNormals = Normals->MaxElementID();
				ParallelFor(NumNormals, [&TransformNormal, &Normals](int ElemID)
				{
					if (Normals->IsElement(ElemID))
					{
						FVector3f Normal = Normals->GetElement(ElemID);
						Normal = TransformNormal(Normal);
						Normals->SetElement(ElemID, Normal);
					}
				});
			}
			if (Mesh.Attributes()->HasTangentSpace() && bool(TransformAttributes & ETransformAttributes::Tangents))
			{
				for (int32 TangentLayerIdx = 1; TangentLayerIdx < 3; ++TangentLayerIdx)
				{
					FDynamicMeshNormalOverlay* TangentLayer = Mesh.Attributes()->GetNormalLayer(TangentLayerIdx);
					int NumTangents = TangentLayer->MaxElementID();
					ParallelFor(NumTangents, [&TransformTangent, &TangentLayer](int ElemID)
					{
						if (TangentLayer->IsElement(ElemID))
						{
							FVector3f Tangent = TangentLayer->GetElement(ElemID);
							Tangent = TransformTangent(Tangent);
							TangentLayer->SetElement(ElemID, Normalized(Tangent));
						}
					});
				}
			}
		
			if (bool(TransformAttributes & ETransformAttributes::SculptLayers))
			{
				TransformSculptLayers(Mesh,
					TransformPosition,
					TransformOffset);
			}
		}
	}
}


void MeshTransforms::Translate(FDynamicMesh3& Mesh, const FVector3d& Translation, ETransformAttributes TransformAttributes)
{
	int NumVertices = Mesh.MaxVertexID();
	if (bool(TransformAttributes & ETransformAttributes::Positions))
	{
		ParallelFor(NumVertices, [&](int vid) 
		{
			if (Mesh.IsVertex(vid))
			{
				Mesh.SetVertex(vid, Mesh.GetVertex(vid) + Translation);
			}
		});
	}
	if (bool(TransformAttributes & ETransformAttributes::SculptLayers))
	{
		MeshTransformsLocal::TransformSculptLayers(Mesh,
			[&Translation](const FVector3d& Pos) {return Pos + Translation;},
			MeshTransformsLocal::ESkipOffset::SkipOffset);
	}
}

void MeshTransforms::Scale(FDynamicMesh3& Mesh, const FVector3d& Scale, const FVector3d& Origin, bool bReverseOrientationIfNeeded, bool bOnlyTransformPositions)
{
	ETransformAttributes TransformAttributes = bOnlyTransformPositions ? ETransformAttributes::Positions : ETransformAttributes::All;
	MeshTransforms::Scale(Mesh, Scale, Origin, bReverseOrientationIfNeeded, TransformAttributes);
}

void MeshTransforms::Scale(FDynamicMesh3& Mesh, const FVector3d& Scale, const FVector3d& Origin, bool bReverseOrientationIfNeeded, ETransformAttributes TransformAttributes)
{
	const ETransformAttributes NormalsOrTangents = ETransformAttributes::VertexNormals | ETransformAttributes::Normals | ETransformAttributes::Tangents;
	bool bNeedNormalTangentScaling = bool(TransformAttributes & NormalsOrTangents) && !Scale.IsUniform();
	int NumVertices = Mesh.MaxVertexID();
	FVector3f TangentScale = (FVector3f)Scale;
	FVector3f NormalScale = TangentScale;
	if (bNeedNormalTangentScaling)
	{
		// compute a safe inverse-scale to apply to normal vectors
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			if (!FMath::IsNearlyZero(NormalScale[Idx]))
			{
				NormalScale[Idx] = 1.0f / NormalScale[Idx];
			}
		}
	}
	else
	{
		// we don't need normal/tangent scaling, so disable them
		TransformAttributes = TransformAttributes & (~NormalsOrTangents);
	}
	
	MeshTransformsLocal::TransformMeshHelper(Mesh, 
		[&Origin, &Scale](const FVector3d& Pos) -> FVector3d {return (Pos - Origin) * Scale + Origin;},
		[NormalScale](const FVector3f& Normal) -> FVector3f { return Normalized(Normal * NormalScale); },
		[TangentScale](const FVector3f& Tangent) -> FVector3f { return Normalized(Tangent * TangentScale); },
		[&Scale](const FVector3d& Offset) -> FVector3d { return Offset * Scale;},
		TransformAttributes);

	if (bReverseOrientationIfNeeded && Scale.X * Scale.Y * Scale.Z < 0)
	{
		Mesh.ReverseOrientation(false);
	}
}



void MeshTransforms::WorldToFrameCoords(FDynamicMesh3& Mesh, const FFrame3d& Frame, ETransformAttributes TransformAttributes)
{
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		[&Frame](const FVector3d& Pos) {return Frame.ToFramePoint(Pos); },
		[&Frame](const FVector3f& Normal) -> FVector3f { return (FVector3f)Frame.ToFrameVector((FVector3d)Normal); },
		[&Frame](const FVector3f& Tangent) -> FVector3f { return (FVector3f)Frame.ToFrameVector((FVector3d)Tangent); },
		[&Frame](const FVector3d& Offset) { return Frame.ToFrameVector(Offset); },
		TransformAttributes);
}





void MeshTransforms::FrameCoordsToWorld(FDynamicMesh3& Mesh, const FFrame3d& Frame, ETransformAttributes TransformAttributes)
{
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		[&Frame](const FVector3d& Pos) {return Frame.FromFramePoint(Pos); },
		[&Frame](const FVector3f& Normal) -> FVector3f { return (FVector3f)Frame.FromFrameVector((FVector3d)Normal); },
		[&Frame](const FVector3f& Tangent) -> FVector3f { return (FVector3f)Frame.FromFrameVector((FVector3d)Tangent); },
		[&Frame](const FVector3d& Offset) { return Frame.FromFrameVector(Offset); },
		TransformAttributes);
}


void MeshTransforms::Rotate(FDynamicMesh3& Mesh, const FRotator& Rotation, const FVector3d& RotationOrigin, ETransformAttributes TransformAttributes)
{
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		[&Rotation, &RotationOrigin](const FVector3d& Pos) {return Rotation.RotateVector((Pos - RotationOrigin)) + RotationOrigin; },
		[&Rotation](const FVector3f& Normal) -> FVector3f { return (FVector3f)Rotation.RotateVector((FVector3d)Normal); },
		[&Rotation](const FVector3f& Tangent) -> FVector3f { return (FVector3f)Rotation.RotateVector((FVector3d)Tangent); },
		[&Rotation](const FVector3d& Offset) { return Rotation.RotateVector(Offset); },
		TransformAttributes);
}

void MeshTransforms::ApplyTransform(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform, bool bReverseOrientationIfNeeded, ETransformAttributes TransformAttributes)
{
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		[&Transform](const FVector3d& Pos) {return Transform.TransformPosition(Pos); },
		[&Transform](const FVector3f& Normal) -> FVector3f { return (FVector3f)Transform.TransformNormal((FVector3d)Normal); },
		[&Transform](const FVector3f& Tangent) -> FVector3f { return Normalized((FVector3f)Transform.TransformVector((FVector3d)Tangent)); },
		[&Transform](const FVector3d& Offset) { return Transform.TransformVector(Offset); },
		TransformAttributes);

	if (bReverseOrientationIfNeeded && Transform.GetDeterminant() < 0)
	{
		Mesh.ReverseOrientation(false);
	}
}



void MeshTransforms::ApplyTransformInverse(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform, bool bReverseOrientationIfNeeded, ETransformAttributes TransformAttributes)
{
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		[&Transform](const FVector3d& Pos) {return Transform.InverseTransformPosition(Pos); },
		[&Transform](const FVector3f& Normal) -> FVector3f { return (FVector3f)Transform.InverseTransformNormal((FVector3d)Normal); },
		[&Transform](const FVector3f& Tangent) -> FVector3f { return Normalized((FVector3f)Transform.InverseTransformVector((FVector3d)Tangent)); },
		[&Transform](const FVector3d& Offset) { return Transform.InverseTransformVector(Offset); },
		TransformAttributes);

	if (bReverseOrientationIfNeeded && Transform.GetDeterminant() < 0)
	{
		Mesh.ReverseOrientation(false);
	}
}


void MeshTransforms::ReverseOrientationIfNeeded(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform)
{
	if (Transform.GetDeterminant() < 0)
	{
		Mesh.ReverseOrientation(false);
	}
}



void MeshTransforms::ApplyTransform(FDynamicMesh3& Mesh,
	TFunctionRef<FVector3d(const FVector3d&)> PositionTransform,
	TFunctionRef<FVector3f(const FVector3f&)> NormalTransform)
{
	ApplyTransform(Mesh, PositionTransform, NormalTransform, 
		[](const FVector3f& T)->FVector3f 
		{
			// we un-set the transform tangent flag, so this method should not be called
			checkNoEntry();
			return T;
		},
		// transform everything except tangents
		~ETransformAttributes::Tangents);
}

void MeshTransforms::ApplyTransform(FDynamicMesh3& Mesh,
	TFunctionRef<FVector3d(const FVector3d&)> PositionTransform,
	TFunctionRef<FVector3f(const FVector3f&)> NormalTransform,
	TFunctionRef<FVector3f(const FVector3f&)> TangentTransform,
	ETransformAttributes TransformAttributes)
{
	FVector3d ZeroTransformed = PositionTransform(FVector::ZeroVector);
	MeshTransformsLocal::TransformMeshHelper(Mesh,
		PositionTransform,
		NormalTransform,
		TangentTransform,
		[&PositionTransform, ZeroTransformed](const FVector3d& Offset) { return PositionTransform(Offset) - ZeroTransformed; },
		TransformAttributes);
}

