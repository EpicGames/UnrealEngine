// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMesh/NonManifoldMappingSupport.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

namespace UE::Geometry
{

class FNonManifoldSourceVertexIdAttributeChange final : 
	public FDynamicMeshAttributeChangeBase
{
	friend class FNonManifoldSourceVertexIdAttribute;
	TArray<TPair<int32, int32>> OldSourceVertexIds, NewSourceVertexIds;

	bool bOldInvalidState = false;
	bool bNewInvalidState = false;

public:
	FNonManifoldSourceVertexIdAttributeChange() = default;

	virtual ~FNonManifoldSourceVertexIdAttributeChange() override = default;

	void SaveInitialVertex(const FDynamicMeshAttributeBase* Attribute, int VertexID) override;

	void StoreAllFinalVertices(const FDynamicMeshAttributeBase* Attribute, const TSet<int>& VertexIDs) override;

	bool Apply(FDynamicMeshAttributeBase* Attribute, bool bRevert) const override;
};

class FNonManifoldSourceVertexIdAttribute final :
	public FDynamicMeshAttributeBase
{
	friend class FNonManifoldSourceVertexIdAttributeChange;
	
	FDynamicMesh3* Parent = nullptr;
	TDynamicVector<int32> SourceVertexIds;
	bool bValid = false;
	
public:
	FNonManifoldSourceVertexIdAttribute() = default;

	FNonManifoldSourceVertexIdAttribute(FDynamicMesh3* InParent) : Parent(InParent)
	{
	}

	virtual ~FNonManifoldSourceVertexIdAttribute() override = default;
	
	const FDynamicMesh3* GetParent() const
	{
		return Parent;
	}
	FDynamicMesh3* GetParent()
	{
		return Parent;
	}
	
	bool IsValid() const
	{
		return bValid;
	}

	FDynamicMeshAttributeBase* MakeCopy(FDynamicMesh3* ParentIn) const override
	{
		FNonManifoldSourceVertexIdAttribute* Attribute = new FNonManifoldSourceVertexIdAttribute(ParentIn);
		Attribute->Copy(*this);
		return Attribute;
	}
	
	FDynamicMeshAttributeBase* MakeNew(FDynamicMesh3* ParentIn) const override
	{
		FNonManifoldSourceVertexIdAttribute* Attribute = new FNonManifoldSourceVertexIdAttribute(ParentIn);
		Attribute->Initialize();
		return Attribute;
	}
	
	void CompactInPlace(const FCompactMaps& CompactMaps) override
	{
		for (int32 FromVID = 0, NumVID = CompactMaps.NumVertexMappings(); FromVID < NumVID; FromVID++)
		{
			const int32 ToVID = CompactMaps.GetVertexMapping(FromVID);
			if (ToVID == FCompactMaps::InvalidID)
			{
				continue;
			}
			if (ensure(ToVID <= FromVID))
			{
				SourceVertexIds[ToVID] = SourceVertexIds[FromVID];
			}
		}
		SourceVertexIds.Resize(Parent->MaxVertexID());
	}
	
	void Reparent(FDynamicMesh3* NewParent) override
	{
		Parent = NewParent;
	}
	
	bool CopyThroughMapping(const TDynamicAttributeBase<FDynamicMesh3>* Source, const FMeshIndexMappings& Mapping) override
	{
		for (const TPair<int32, int32>& MapVID : Mapping.GetVertexMap().GetForwardMap())
		{
			int32 SourceVertexId;
			if (!ensure(Source->CopyOut(MapVID.Key, &SourceVertexId, sizeof(SourceVertexId))))
			{
				return false;
			}
			SetValue(MapVID.Value, SourceVertexId);
		}
		return true;
	}
	
	bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		if (BufferSize != sizeof(int32) || !Parent->IsVertex(RawID))
		{
			return false;
		}

		*static_cast<int32 *>(Buffer) = SourceVertexIds[RawID];
		return true;
	}
	
	bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		if (BufferSize != sizeof(int32) || !Parent->IsVertex(RawID))
		{
			return false;
		}

		SourceVertexIds[RawID] = *static_cast<int32 *>(Buffer);
		return true;
	}
	
	TUniquePtr<TDynamicAttributeChangeBase<FDynamicMesh3>> NewBlankChange() const override
	{
		return MakeUnique<FNonManifoldSourceVertexIdAttributeChange>();
	}
	
	void Initialize()
	{
		SourceVertexIds.Resize(Parent->MaxVertexID());
		SourceVertexIds.Fill(INDEX_NONE);
	}

	void InitializeFromArray(TConstArrayView<int32> VertexToNonManifoldVertexIDMap)
	{
		check(VertexToNonManifoldVertexIDMap.Num() == Parent->MaxVertexID());
		SourceVertexIds.Resize(Parent->MaxVertexID());
		
		for (int32 VertexID : Parent->VertexIndicesItr())
		{
			SetValue(VertexID, VertexToNonManifoldVertexIDMap[VertexID]);
		}
		bValid = true;
	}

	void Copy(const FNonManifoldSourceVertexIdAttribute& Copy)
	{
		FDynamicMeshAttributeBase::CopyParentClassData(Copy);
		SourceVertexIds = Copy.SourceVertexIds;
		bValid = Copy.bValid;
	}
	
	int32 GetValue(int32 InVertexId) const
	{
		return SourceVertexIds[InVertexId];
	}
	
	void SetValue(int32 InVertexId, int32 InSourceVertexId)
	{
		SourceVertexIds[InVertexId] = InSourceVertexId;
	}

	virtual bool Append(const TDynamicAttributeBase<FDynamicMesh3>& Source, const FDynamicMesh3::FAppendInfo& Mapping) override
	{
		// Non-manifold vertex mapping only supports a single source mesh, so appended vertices cannot preserve sources ...
		// Instead, we always append defaulted elements
		AppendDefaulted(Mapping);
		return true;
	}
	virtual void AppendDefaulted(const FDynamicMesh3::FAppendInfo& Mapping) override
	{
		const int32 NewNum = Mapping.VertexOffset + Mapping.NumVertex;
		if (NewNum > SourceVertexIds.Num())
		{
			SourceVertexIds.Resize(NewNum, INDEX_NONE);
		}
	}

	// Any topo operation on the mesh will invalidate the non-manifold information.
	virtual void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo& ) override
	{
		bValid = false;
	}

	virtual void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo) override
	{
		bValid = false;
	}

	virtual void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo& ) override
	{
		bValid = false;
	}

	virtual void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo& ) override
	{
		bValid = false;
	}

	virtual void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo& ) override
	{
		bValid = false;
	}

	virtual void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& ) override
	{
		bValid = false;
	}

	virtual void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& , const TArrayView<const int>& ) override
	{
		bValid = false;
	}

	virtual void OnNewVertex(int VertexID, bool bInserted) override
	{
		SourceVertexIds.InsertAt(VertexID, VertexID);
	}

	virtual void OnRemoveVertex(int VertexID) override
	{
		bValid = false;
	}

	virtual SIZE_T GetByteCount() const override
	{
		return SourceVertexIds.GetByteCount();
	}
};

void FNonManifoldSourceVertexIdAttributeChange::SaveInitialVertex(const FDynamicMeshAttributeBase* Attribute, int VertexID)
{
	const FNonManifoldSourceVertexIdAttribute* NonManifoldAttribute = static_cast<const FNonManifoldSourceVertexIdAttribute*>(Attribute);
	if (OldSourceVertexIds.IsEmpty())
	{
		bOldInvalidState = NonManifoldAttribute->IsValid();
	}
	
	OldSourceVertexIds.Emplace(VertexID, NonManifoldAttribute->GetValue(VertexID));
}

void FNonManifoldSourceVertexIdAttributeChange::StoreAllFinalVertices(const FDynamicMeshAttributeBase* Attribute, const TSet<int>& VertexIDs)
{
	NewSourceVertexIds.Reserve(NewSourceVertexIds.Num() + VertexIDs.Num());
	const FNonManifoldSourceVertexIdAttribute* NonManifoldAttribute = static_cast<const FNonManifoldSourceVertexIdAttribute*>(Attribute);
	for (int32 VertexID: VertexIDs)
	{
		NewSourceVertexIds.Emplace(VertexID, NonManifoldAttribute->GetValue(VertexID));
	}

	// Store the last known valid state.  
	bNewInvalidState = NonManifoldAttribute->IsValid();
}

bool FNonManifoldSourceVertexIdAttributeChange::Apply(FDynamicMeshAttributeBase* Attribute, bool bRevert) const
{
	const TArray<TPair<int32, int32>>& Changes = bRevert ? OldSourceVertexIds : NewSourceVertexIds;
	FNonManifoldSourceVertexIdAttribute* NonManifoldAttribute = static_cast<FNonManifoldSourceVertexIdAttribute*>(Attribute);
	for (const TPair<int32, int32>& Item: Changes)
	{
		if (ensure(NonManifoldAttribute->GetParent()->IsVertex(Item.Key)))
		{
			NonManifoldAttribute->SetValue(Item.Key, Item.Value);
		}
	}

	// Restore the valid state as well.
	NonManifoldAttribute->bValid = bRevert ? bOldInvalidState : bNewInvalidState;
	
	return true;
}


FName FNonManifoldMappingSupport::NonManifoldMeshVIDsAttrName = FName("NonManifoldVIDAttr");


FNonManifoldMappingSupport::FNonManifoldMappingSupport(const FDynamicMesh3& MeshIn)
{
	Reset(MeshIn);
}

void FNonManifoldMappingSupport::Reset(const FDynamicMesh3& MeshIn)
{
	NonManifoldSrcVIDsAttribute = nullptr;
	DynamicMesh = &MeshIn;

	const FDynamicMeshAttributeSet* Attributes = DynamicMesh->Attributes();

	if (Attributes && Attributes->HasAttachedAttribute(NonManifoldMeshVIDsAttrName))
	{
		NonManifoldSrcVIDsAttribute = static_cast<const FNonManifoldSourceVertexIdAttribute* >(Attributes->GetAttachedAttribute(NonManifoldMeshVIDsAttrName));
	}
}

bool  FNonManifoldMappingSupport::IsNonManifoldVertexInSource() const
{
	return NonManifoldSrcVIDsAttribute != nullptr && NonManifoldSrcVIDsAttribute->IsValid();
}

int32 FNonManifoldMappingSupport::GetOriginalNonManifoldVertexID(const int32 vid) const
{
	checkSlow(DynamicMesh->IsVertex(vid));

	int32 Result = vid;
	if (NonManifoldSrcVIDsAttribute)
	{
		Result = NonManifoldSrcVIDsAttribute->GetValue(vid);
	}
	return Result;
}

bool FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(const TArray<int32>& VertexToNonManifoldVertexIDMap, FDynamicMesh3& MeshInOut)
{
	const int32 MaxVID = MeshInOut.MaxVertexID();
	FDynamicMeshAttributeSet* Attributes = MeshInOut.Attributes();

	if (VertexToNonManifoldVertexIDMap.Num() < MaxVID || !Attributes)
	{
		return false;
	}

	// create and attach a vertex ID buffer, removing any pre-exising one.
	FNonManifoldSourceVertexIdAttribute* VIDAttr = [&] {
		// replace existing buffer.  we *should* be able to re-use it
		if (Attributes->HasAttachedAttribute(NonManifoldMeshVIDsAttrName))
		{
			Attributes->RemoveAttribute(NonManifoldMeshVIDsAttrName);
		}
		FNonManifoldSourceVertexIdAttribute* SrcMeshVIDAttr = new FNonManifoldSourceVertexIdAttribute(&MeshInOut);
		SrcMeshVIDAttr->SetName(NonManifoldMeshVIDsAttrName);
		MeshInOut.Attributes()->AttachAttribute(NonManifoldMeshVIDsAttrName, SrcMeshVIDAttr);
		return SrcMeshVIDAttr;
	}();
	
	VIDAttr->InitializeFromArray(VertexToNonManifoldVertexIDMap);
	return true;
}

void FNonManifoldMappingSupport::RemoveNonManifoldVertexMappingData(FDynamicMesh3& MeshInOut)
{

	FDynamicMeshAttributeSet* Attributes = MeshInOut.Attributes();

	if (Attributes && Attributes->HasAttachedAttribute(NonManifoldMeshVIDsAttrName))
	{
		Attributes->RemoveAttribute(NonManifoldMeshVIDsAttrName);
	}

}
}