// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMesh/Operations/SplitAttributeWelder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshAdapterUtil.h"
#include "Spatial/PointHashGrid3.h"
#include "Util/IndexPriorityQueue.h"
#include "Util/IndexUtil.h"

using namespace UE::Geometry;

const double FMergeCoincidentMeshEdges::DEFAULT_TOLERANCE = FMathf::ZeroTolerance;


bool FMergeCoincidentMeshEdges::Apply()
{
	MergeVtxDistSqr = MergeVertexTolerance * MergeVertexTolerance;
	double UseMergeSearchTol = (MergeSearchTolerance > 0) ? MergeSearchTolerance : 2*MergeVertexTolerance;

	//
	// construct hash table for edge midpoints
	//

	TArray<FVector3d> BoundaryMidPoints;
	TArray<int32> ToMidPt;
	ToMidPt.Init(-1, Mesh->MaxEdgeID());
	for (int32 EID : Mesh->BoundaryEdgeIndicesItr())
	{
		ToMidPt[EID] = BoundaryMidPoints.Add(Mesh->GetEdgePoint(EID, 0.5));
	}
	InitialNumBoundaryEdges = BoundaryMidPoints.Num();

	// use denser grid as number of boundary edges increases
	int hashN = 64;
	if (InitialNumBoundaryEdges > 1000)   hashN = 128;
	if (InitialNumBoundaryEdges > 10000)  hashN = 256;
	if (InitialNumBoundaryEdges > 100000)  hashN = 512;
	FAxisAlignedBox3d Bounds = Mesh->GetBounds(true);
	double CellSize = FMath::Max(FMathd::ZeroTolerance, Bounds.MaxDim() / (double)hashN);
	TPointHashGrid3<int32, double> MidpointsHash(CellSize, -1);

	UseMergeSearchTol = FMathd::Min(CellSize, UseMergeSearchTol);

	// temp values and buffers
	FVector3d A, B, C, D;
	TArray<int> equivBuffer;
	TArray<int> SearchMatches;
	SearchMatches.Reserve(1024);  // allocate buffer

	//
	// construct edge equivalence sets. First we find all other edges with same
	// midpoint, and then we form equivalence set for edge from subset that also
	// has same endpoints
	//

	typedef TArray<int> EdgesList;
	TArray<EdgesList*> EquivalenceSets;
	EquivalenceSets.Init(nullptr, Mesh->MaxEdgeID());
	TSet<int> RemainingEdges;

	for (int eid : Mesh->BoundaryEdgeIndicesItr()) 
	{
		const int32 MidPtIdx = ToMidPt[eid];
		FVector3d midpt = BoundaryMidPoints[MidPtIdx];

		// find all other edges with same midpoint in query sphere
		SearchMatches.Reset();
		MidpointsHash.FindPointsInBall(midpt, UseMergeSearchTol, [&](const int32& PtIdx)
			{
				return FVector3d::DistSquared(midpt, BoundaryMidPoints[ToMidPt[PtIdx]]);
			}, SearchMatches);
		// add each point after querying for neighbors, so we only find edges with earlier IDs
		MidpointsHash.InsertPointUnsafe(eid, midpt);

		int N = SearchMatches.Num();
		if (N == 0)
		{
			continue;		// edge has no matches
		}

		Mesh->GetEdgeV(eid, A, B);

		// if same endpoints, add to equivalence set for this edge (and matching reverse equivalence)
		equivBuffer.Reset();
		for (int i = 0; i < N; ++i) 
		{
			int32 MatchEID = SearchMatches[i];
			Mesh->GetEdgeV(SearchMatches[i], C, D);
			if ( IsSameEdge(A, B, C, D) ) 
			{
				equivBuffer.Add(SearchMatches[i]);
				if (!EquivalenceSets[MatchEID])
				{
					EquivalenceSets[MatchEID] = new EdgesList();
					RemainingEdges.Add(MatchEID);
				}
				EquivalenceSets[MatchEID]->Add(eid);
			}
		}
		if (equivBuffer.Num() > 0)
		{
			EquivalenceSets[eid] = new EdgesList(equivBuffer);
			RemainingEdges.Add(eid);
		}
	}


	//
	// add potential duplicate edges to priority queue, sorted by number of possible matches. 
	//

	// [TODO] could replace remaining hashset w/ PQ, and use conservative count?
	// [TODO] Does this need to be a PQ? Not updating PQ below anyway...
	FIndexPriorityQueue DuplicatesQueue;
	DuplicatesQueue.Initialize(Mesh->MaxEdgeID());
	for (int eid : RemainingEdges) 
	{
		if (OnlyUniquePairs) 
		{
			if (EquivalenceSets[eid]->Num() != 1)
			{
				continue;
			}

			// check that reverse match is the same and unique
			int other_eid = (*EquivalenceSets[eid])[0];
			if (EquivalenceSets[other_eid]->Num() != 1 || (*EquivalenceSets[other_eid])[0] != eid)
			{
				continue;
			}
		}
		const float Priority = (float)EquivalenceSets[eid]->Num();
		DuplicatesQueue.Insert(eid, Priority);
	}

	//
	// process all potential matches, merging edges as we go in a greedy fashion.
	//

	while (DuplicatesQueue.GetCount() > 0) 
	{
		int eid = DuplicatesQueue.Dequeue();
		
		if (Mesh->IsEdge(eid) == false || EquivalenceSets[eid] == nullptr || RemainingEdges.Contains(eid) == false)
		{
			continue;               // dealt with this edge already
		}
		if (Mesh->IsBoundaryEdge(eid) == false)
		{
			continue;				// this edge got merged already
		}

		EdgesList& Matches = *EquivalenceSets[eid];

		// select best viable match (currently just "first"...)
		// @todo could we make better decisions here? prefer planarity?
		bool bMerged = false;
		int FailedCount = 0;
		for (int i = 0; i < Matches.Num() && bMerged == false; ++i) 
		{
			int other_eid = Matches[i];
			if (Mesh->IsEdge(other_eid) == false || Mesh->IsBoundaryEdge(other_eid) == false)
			{
				continue;
			}

			// When there is no geometry selection, EdgesToMerge is never initialized
			bool bWeldingAcrossEntireMesh = (EdgesToMerge == nullptr);

			// Edges only considered for merging if we're welding the entire mesh or, when using a geometry selection,
			// if EITHER edge in the Match are a part of the selection
			if (bWeldingAcrossEntireMesh || EdgesToMerge->Contains(eid) || EdgesToMerge->Contains(other_eid))
			{
				FDynamicMesh3::FMergeEdgesInfo MergeInfo;
				EMeshResult Result = Mesh->MergeEdges(eid, other_eid, MergeInfo);
				if (Result != EMeshResult::Ok) 
				{
					// if the operation failed we remove this edge from the equivalence set
					Matches.RemoveAt(i);
					i--;

					EquivalenceSets[other_eid]->Remove(eid);
					//DuplicatesQueue.UpdatePriority(...);  // should we do this?

					FailedCount++;
				}
				else 
				{
					// ok we merged, other edge is no longer free
					bMerged = true;
					delete EquivalenceSets[other_eid];
					EquivalenceSets[other_eid] = nullptr;
					RemainingEdges.Remove(other_eid);

					// weld attributes 
					if (bWeldAttrsOnMergedEdges)
					{ 
						SplitAttributeWelder.WeldSplitElements(*Mesh, MergeInfo.KeptVerts[0]);
						SplitAttributeWelder.WeldSplitElements(*Mesh, MergeInfo.KeptVerts[1]);
					}
				}
			}
		}

		// Removing branch with two identical cases to fix static analysis warning.
		// However, these two branches are *not* the same...we're just not sure 
		// what the right thing to do is in the else case
		//if (bMerged) 
		//{
			delete EquivalenceSets[eid];
			EquivalenceSets[eid] = nullptr;
			RemainingEdges.Remove(eid);
		//}
		//else 
		//{
		//	// should we do something else here? doesn't make sense to put
		//	// back into Q, as it should be at the top, right?
		//	delete EquivalenceSets[eid];
		//	EquivalenceSets[eid] = nullptr;
		//	RemainingEdges.Remove(eid);
		//}

	}

	FinalNumBoundaryEdges = 0;
	for (int eid : Mesh->BoundaryEdgeIndicesItr())
	{
		FinalNumBoundaryEdges++;
	}

	return true;
}

