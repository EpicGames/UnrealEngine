// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMesh/Operations/SplitAttributeWelder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"


using namespace UE::Geometry;

namespace
{


template <typename OverlayType, typename ShouldWeldFunctorType>
void WeldSplits(const UE::Geometry::FDynamicMesh3* ParentMesh, const int32 ParentVID, OverlayType& Overlay,  ShouldWeldFunctorType& ShouldWeld)
{
	if (!ParentMesh || !ParentMesh->IsVertex(ParentVID))
	{
		return;
	}

	TArray<int> ElementIDs;
	Overlay.GetVertexElements(ParentVID, ElementIDs);

	// assume the number of elements at given vertex is small and do simple O(n^2) 
	const int32 NumElements = ElementIDs.Num();
	TArray<int> ConsumedMask;  ConsumedMask.SetNumZeroed(NumElements);
	for (int32 i = 0; i < NumElements; ++i)
	{
		if (ConsumedMask[i] == 1) 
		{
			continue;
		}

		const int32 eid = ElementIDs[i];

		for (int32 j = i + 1; j < NumElements; ++j)
		{
			if(ConsumedMask[j] == 1)
			{
				continue;
			}
			const int32 oeid = ElementIDs[j];

			if (ShouldWeld(eid, oeid))
			{
				// consumed by weld.
				ConsumedMask[j] = 1;

				for (int TID : ParentMesh->VtxTrianglesItr(ParentVID)) // only care about triangles connected to the *new* vertex; these are updated
				{
					if (!Overlay.IsSetTriangle(TID))
					{
						continue;
					}

					FIndex3i TriElements = Overlay.GetTriangle(TID);
					bool bUpdateTriangle = false;
					for (int c = 0; c < 3; ++c)
					{
						if (TriElements[c] == oeid)
						{
							TriElements[c] = eid;
							bUpdateTriangle = true;
						}
					}
					if (bUpdateTriangle)
					{
						Overlay.SetTriangle(TID, TriElements, true /* allow element freeing*/);
					}
				}
			}
		}

	}

}

}

void  FSplitAttributeWelder::WeldSplitElements(FDynamicMesh3& ParentMesh, const int32 ParentVID)
{

	

	FDynamicMeshAttributeSet* Attributes = ParentMesh.Attributes();

	if (!Attributes)
	{
		return;
	}

	if (!ParentMesh.IsVertex(ParentVID))
	{
		return;
	}

	// uvs 
	for (int32 i = 0, I = Attributes->NumUVLayers(); i < I; ++i)
	{
		FDynamicMeshUVOverlay* Overlay = Attributes->GetUVLayer(i);
		WeldSplitUVs(ParentVID, *Overlay, UVDistSqrdThreshold);
	}

	// tangent space
	for (int32 i = 0, I = Attributes->NumNormalLayers(); i < I; ++i )
	{
		const float DotThreshold = (i == 0) ? NormalVecDotThreshold : TangentVecDotThreshold;
		FDynamicMeshNormalOverlay* Overlay = Attributes->GetNormalLayer(i);
		WeldSplitUnitVectors(ParentVID, *Overlay, DotThreshold);
	}

	// colors
	if (FDynamicMeshColorOverlay* Overlay = Attributes->PrimaryColors())
	{
		WeldSplitColors(ParentVID, *Overlay, ColorDistSqrdThreshold);
	}
}


void FSplitAttributeWelder::WeldSplitElements(FDynamicMesh3& ParentMesh)
{
	for (int vid : ParentMesh.VertexIndicesItr())
	{
		WeldSplitElements(ParentMesh, vid);
	}
}

void FSplitAttributeWelder::WeldSplitUVs(const int32 ParentVID, FDynamicMeshUVOverlay& Overlay, float UVDistSqrdThreshold)
{

	const FDynamicMesh3* ParentMesh = Overlay.GetParentMesh();

	const float Threshold = FMath::Max(UVDistSqrdThreshold, 0.f);

	auto ShouldWeld = [&Overlay, Threshold](const int32 eid, const int32 oeid)->bool
	{
		const FVector2f UV            = Overlay.GetElement(eid);
		const FVector2f otherUV       = Overlay.GetElement(oeid);
		const float UVDistanceSquared = FVector2f::DistSquared(UV, otherUV);

		return (UVDistanceSquared <= Threshold);
	};

	WeldSplits(ParentMesh, ParentVID, Overlay, ShouldWeld);
}

// note, this only compares vector orientation - not length.
void FSplitAttributeWelder::WeldSplitUnitVectors(const int32 ParentVID, FDynamicMeshNormalOverlay& Overlay, float DotThreshold, bool bMergeZeroVectors)
{
	const FDynamicMesh3* ParentMesh = Overlay.GetParentMesh();

	auto ShouldWeld = [&Overlay, DotThreshold, bMergeZeroVectors](const int32 eid, const int32 oeid)->bool
	{
		FVector3f Vec      = Overlay.GetElement(eid);
		FVector3f otherVec = Overlay.GetElement(oeid);

		const bool bVecNormalized = Vec.Normalize();
		const bool bOtherVecNormalized = otherVec.Normalize();
	
		if (bVecNormalized && bOtherVecNormalized)
		{
			const float  CosAngle = FVector3f::DotProduct(Vec, otherVec);
			return ( FMath::Abs(1.f - CosAngle) <= DotThreshold);
		}

		const bool bBothZero = (!bVecNormalized && !bOtherVecNormalized);
		if (bMergeZeroVectors && bBothZero)
		{
			return true;
		}

		return false;
	};

	WeldSplits(ParentMesh, ParentVID, Overlay, ShouldWeld);
}

void FSplitAttributeWelder::WeldSplitColors(const int32 ParentVID, FDynamicMeshColorOverlay& Overlay, float ColorDistSqrdThreshold)
{
	const FDynamicMesh3* ParentMesh = Overlay.GetParentMesh();

	const float Threshold = FMath::Max(ColorDistSqrdThreshold, 0.f);

	auto ShouldWeld = [&Overlay, Threshold](const int32 eid, const int32 oeid)->bool
	{
		const FVector4f Color = Overlay.GetElement(eid);
		const FVector4f otherColor = Overlay.GetElement(oeid);

		const float ColorDistSqrd = (Color - otherColor).SizeSquared();

		return (ColorDistSqrd <= Threshold);
	};

	WeldSplits(ParentMesh, ParentVID, Overlay, ShouldWeld);
}
