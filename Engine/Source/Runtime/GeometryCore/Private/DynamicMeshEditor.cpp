// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Util/BufferUtil.h"
#include "MeshRegionBoundaryLoops.h"
#include "DynamicSubmesh3.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshQueries.h"
#include "Selections/MeshConnectedComponents.h"

using namespace UE::Geometry;


// local static helper functions for seam handling
namespace UE::Private::OverlaySeamHelpers
{
	template<typename RealType, int OverlayDim>
	static bool CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, FDynamicMesh3* Mesh, TDynamicMeshOverlay<RealType, OverlayDim>* Overlay, TArray<int32>* AddedElementIDs)
	{
		// Note: The mesh boundary edge version of this method is the DisconnectTrianglesAlongEdges method --
		//  the methods are very similar, so if you change this method consider changing that one as well

		auto FindElementForVertex = [Mesh, Overlay](int32 MeshVertexID, int32 MeshEdgeID) -> int32
		{
			FIndex2i Tris = Mesh->GetEdgeT(MeshEdgeID);
			FIndex3i Tri0 = Mesh->GetTriangle(Tris.A);
			FIndex3i UVTri0 = Overlay->GetTriangle(Tris.A);
			for (int32 j = 0; j < 3; ++j)
			{
				if (Tri0[j] == MeshVertexID)
				{
					return UVTri0[j];
				}
			}
			return IndexConstants::InvalidID;
		};

		/**
		 * @param TidsUntilNextSeam Tids in a fan away from Eid around Vid until the next seam. Will
		 *  be the entire one-ring if the given Eid is the only seam attached to Vid.
		 */
		auto DoesVertHaveAnotherSeamAttached = [&EidsToMakeIntoSeams, Mesh, Overlay, AddedElementIDs](int32 Eid, int32 Vid, TArray<int32>& TidsUntilNextSeam)
		{
			TidsUntilNextSeam.Reset();

			FIndex2i EdgeTids = Mesh->GetEdgeT(Eid);
			int32 CurrentTid = EdgeTids.A;
			int32 PreviousTid = EdgeTids.B;

			int32 MaxNumTids = Mesh->GetVtxEdgeCount(Vid);
			while (true) 
			{
				TidsUntilNextSeam.Add(CurrentTid);
				if (TidsUntilNextSeam.Num() > MaxNumTids) // sanity check
				{
					ensure(false);
					return false;
				}

				FIndex3i NextTriResult = UE::Geometry::FindNextAdjacentTriangleAroundVtx(Mesh, Vid, CurrentTid, PreviousTid,
					[&EidsToMakeIntoSeams, Overlay, Eid](int32 Tri0, int32 Tri1, int32 EidBetween) {
						// We stop at seam or at future seam not including our starting edge (if we walk across the starting
						// edge we will stop at a later check- we just don't want to think that we reached a true seam).
						return Overlay->AreTrianglesConnected(Tri0, Tri1) && !(EidBetween != Eid && EidsToMakeIntoSeams.Contains(EidBetween)); }
				);
				int32 NextTid = NextTriResult.A;

				// See if we've come to a (current or future) seam or wrapped around
				if (NextTid == IndexConstants::InvalidID)
				{
					return true;
				}
				else if (NextTid == EdgeTids.A)
				{
					return false;
				}
				PreviousTid = CurrentTid;
				CurrentTid = NextTid;
			}
		};

		auto SplitEdgeVertElement = [Mesh, Overlay, AddedElementIDs, &FindElementForVertex](int32 Eid, int32 Vid, const TArray<int32>& TidsToModify) {
			int32 ElementID = FindElementForVertex(Vid, Eid);
			if (ensure(ElementID != IndexConstants::InvalidID))
			{
				int32 NewElementID = Overlay->SplitElement(ElementID, TidsToModify);
				if (AddedElementIDs)
				{
					AddedElementIDs->Add(NewElementID);
				}
			}
		};

		for (int32 Eid : EidsToMakeIntoSeams)
		{
			FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);
			bool VertNeedsSplitting[2] = { false, false };
			TArray<int32> VertTidsToUseForSplit[2];

			if (Overlay->IsSeamEdge(Eid))
			{
				// If this is already a seam edge, make sure its endpoints are not bowties. We create such edges 
				// sometimes as we split adjacent edges, and it is also likely to be what the user wants when
				// selecting edges adjacent to a bowtie.
				for (int i = 0; i < 2; ++i)
				{
					if (Overlay->IsBowtieInOverlay(EdgeVids[i]))
					{
						Overlay->SplitBowtiesAtVertex(EdgeVids[i], AddedElementIDs);
					}
				}
				continue;
			}

			// If we're not already a seam, then it's going to become one. A vert needs to get split if it
			// has a present or future seam attached, or else it will become a bowtie if we just split the
			// other vert.
			for (int i = 0; i < 2; ++i)
			{
				VertNeedsSplitting[i] = DoesVertHaveAnotherSeamAttached(Eid, EdgeVids[i], VertTidsToUseForSplit[i]);
			}
			
			// If neither absolutely has to get split, then one needs to get split anyway so that we
			// make the edge into a seam.
			if (!VertNeedsSplitting[0] && !VertNeedsSplitting[1])
			{
				// We'll go halfway around the triangle one-ring, rounding up
				int32 NumTrisToDisconnect = (VertTidsToUseForSplit[0].Num() + 1) / 2;

				// In doing this, we are splitting an adjacent edge and therefore might inadvertantly
				// introduce a bowtie on the other vert of that edge, which we would like to avoid. 
				// So keep track of that potential bowtie.
				int32 OtherSplitEdge = Mesh->FindEdgeFromTriPair(
					VertTidsToUseForSplit[0][NumTrisToDisconnect - 1],
					VertTidsToUseForSplit[0][NumTrisToDisconnect]);
				int32 PotentialBowtieVid = ensure(OtherSplitEdge != IndexConstants::InvalidID) ?
					IndexUtil::FindEdgeOtherVertex(Mesh->GetEdgeV(OtherSplitEdge), EdgeVids[0])
					: IndexConstants::InvalidID;

				// Perform the split
				VertTidsToUseForSplit[0].SetNum(NumTrisToDisconnect);
				SplitEdgeVertElement(Eid, EdgeVids[0], VertTidsToUseForSplit[0]);

				// Deal with the bowtie if we created one
				if (ensure(PotentialBowtieVid != IndexConstants::InvalidID) 
					&& Overlay->IsBowtieInOverlay(PotentialBowtieVid))
				{
					Overlay->SplitBowtiesAtVertex(PotentialBowtieVid, AddedElementIDs);
				}
			}
			else
			{
				for (int i = 0; i < 2; ++i)
				{
					if (VertNeedsSplitting[i])
					{
						SplitEdgeVertElement(Eid, EdgeVids[i], VertTidsToUseForSplit[i]);
					}
				}
			}//end if at least one side needed splitting
		}//end for each edge

		return true;
	}

	template<typename RealType, int OverlayDim>
	bool RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams, FDynamicMesh3* Mesh, TDynamicMeshOverlay<RealType, OverlayDim>* Overlay)
	{
		for (int32 Eid : EidsToRemoveAsSeams)
		{
			FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);

			// The following logic closely resemebles the logic within FDynamicMeshOverlay::IsSeamEdge,
			// but since we want to know both if it's a seam and use many of the intermediate values
			// computed by IsSeamEdge, we replicate it here.

			FIndex2i Tris = Mesh->GetEdgeT(Eid);
			if (Tris.B == FDynamicMesh3::InvalidID)
			{
				continue; // Technically a seam, but we don't want this one because there's no opposite edge to merge.
			}

			bool bASet = Overlay->IsSetTriangle(Tris.A), bBSet = Overlay->IsSetTriangle(Tris.B);
			if (!bASet || !bBSet)
			{
				continue; // Similar problem as the above case - could be a seam here, but if one triangle
						  // in the Overlay isn't set, there's nothing to merge.
			}

			FIndex3i Triangle0 = Overlay->GetTriangle(Tris.A);
			FIndex3i BaseTriangle0 = Mesh->GetTriangle(Tris.A);
			int idx_base_a0 = BaseTriangle0.IndexOf(EdgeVids.A);
			int idx_base_b0 = BaseTriangle0.IndexOf(EdgeVids.B);

			FIndex3i Triangle1 = Overlay->GetTriangle(Tris.B);
			FIndex3i BaseTriangle1 = Mesh->GetTriangle(Tris.B);
			int idx_base_a1 = BaseTriangle1.IndexOf(EdgeVids.A);
			int idx_base_b1 = BaseTriangle1.IndexOf(EdgeVids.B);

			int el_a_tri0 = Triangle0[idx_base_a0];
			int el_b_tri0 = Triangle0[idx_base_b0];
			int el_a_tri1 = Triangle1[idx_base_a1];
			int el_b_tri1 = Triangle1[idx_base_b1];

			// This shouldn't ever be the case, but lets just check. If true,
			// it would indicate that there's a joined seam but somehow our pairwise
			// vertex matching didn't work above for some reason.
			ensure(!(el_a_tri0 == el_b_tri1 && el_b_tri0 == el_a_tri1));

			if (el_a_tri0 != el_a_tri1)
			{
				Overlay->MergeElement(el_a_tri0, el_a_tri1);
			}
			if (el_b_tri0 != el_b_tri1)
			{
				Overlay->MergeElement(el_b_tri0, el_b_tri1);

			}
		}
		return true;
	}
}

void FDynamicMeshEditResult::GetAllTriangles(TArray<int>& TrianglesOut) const
{
	BufferUtil::AppendElements(TrianglesOut, NewTriangles);

	int NumQuads = NewQuads.Num();
	for (int k = 0; k < NumQuads; ++k)
	{
		TrianglesOut.Add(NewQuads[k].A);
		TrianglesOut.Add(NewQuads[k].B);
	}
	int NumPolys = NewPolygons.Num();
	for (int k = 0; k < NumPolys; ++k)
	{
		BufferUtil::AppendElements(TrianglesOut, NewPolygons[k]);
	}
}



bool FDynamicMeshEditor::RemoveIsolatedVertices()
{
	bool bSuccess = true;
	for (int VID : Mesh->VertexIndicesItr())
	{
		if (!Mesh->IsReferencedVertex(VID))
		{
			constexpr bool bPreserveManifold = false;
			bSuccess = Mesh->RemoveVertex(VID, bPreserveManifold) == EMeshResult::Ok && bSuccess;
		}
	}
	return bSuccess;
}

namespace DynamicMeshEditorLocals
{
// Does all the work of StitchVertexLoopsMinimal and StitchVertexLoopToTriVidPairSequence since they only differ in the way
// that they pick verts per quad.
template <typename FuncType>
bool StitchLoopsInternal(FDynamicMeshEditor& Editor, int32 NumQuads, FuncType&& GetQuadVidsForIndex, FDynamicMeshEditResult& ResultOut)
{
	ResultOut.NewQuads.Reserve(NumQuads);
	ResultOut.NewGroups.Reserve(NumQuads);

	for (int i = 0; i < NumQuads; ++i)
	{
		int32 a, b, c, d;
		GetQuadVidsForIndex(i, a, b, c, d);

		int NewGroupID = Editor.Mesh->AllocateTriangleGroup();
		ResultOut.NewGroups.Add(NewGroupID);

		FIndex3i t1(b, a, d);
		int tid1 = Editor.Mesh->AppendTriangle(t1, NewGroupID);

		FIndex3i t2(a, c, d);
		int tid2 = Editor.Mesh->AppendTriangle(t2, NewGroupID);

		ResultOut.NewQuads.Add(FIndex2i(tid1, tid2));

		if (tid1 < 0 || tid2 < 0)
		{
			goto operation_failed;
		}
	}

	return true;

operation_failed:
	// remove what we added so far
	if (ResultOut.NewQuads.Num())
	{
		TArray<int> Triangles; Triangles.Reserve(2*ResultOut.NewQuads.Num());
		for (const FIndex2i& QuadTriIndices : ResultOut.NewQuads)
		{
			Triangles.Add(QuadTriIndices.A);
			Triangles.Add(QuadTriIndices.B);
		}
		if (!Editor.RemoveTriangles(Triangles, false))
		{
			ensureMsgf(false, TEXT("FDynamicMeshEditor::StitchVertexLoopsMinimal: failed to add all triangles, and also failed to back out changes."));
		}
	}
	return false;
}
}

bool FDynamicMeshEditor::StitchVertexLoopsMinimal(const TArray<int>& Loop1, const TArray<int>& Loop2, FDynamicMeshEditResult& ResultOut)
{
	int N = Loop1.Num();
	if (!ensureMsgf(N == Loop2.Num(), TEXT("FDynamicMeshEditor::StitchVertexLoopsMinimal: loops are not the same length!")))
	{
		return false;
	}
	return DynamicMeshEditorLocals::StitchLoopsInternal(*this, N, [N, &Loop1, &Loop2]
	(int32 Index, int32& VertA, int32& VertB, int32& VertC, int32& VertD) {
		VertA = Loop1[Index];
		VertB = Loop1[(Index + 1) % N];
		VertC = Loop2[Index];
		VertD = Loop2[(Index + 1) % N];
		}, ResultOut);
}

bool FDynamicMeshEditor::StitchVertexLoopToTriVidPairSequence(
	const TArray<TPair<int32, TPair<int8, int8>>>& TriVidPairs,
	const TArray<int>& VertexLoop, FDynamicMeshEditResult& ResultOut)
{
	int N = TriVidPairs.Num();
	if (!ensureMsgf(N == VertexLoop.Num(), TEXT("FDynamicMeshEditor::StitchVertexLoopToEdgeSequence: sequences are not the same length!")))
	{
		return false;
	}
	return DynamicMeshEditorLocals::StitchLoopsInternal(*this, N, [this, N, &TriVidPairs, &VertexLoop]
	(int32 Index, int32& VertA, int32& VertB, int32& VertC, int32& VertD) {
		FIndex3i TriVids1 = Mesh->GetTriangle(TriVidPairs[Index].Key);
		VertA = TriVids1[TriVidPairs[Index].Value.Key];
		VertB = TriVids1[TriVidPairs[Index].Value.Value];

		VertC = VertexLoop[Index];
		VertD = VertexLoop[(Index + 1) % N];
		}, ResultOut);
}

bool FDynamicMeshEditor::ConvertLoopToTriVidPairSequence(const FDynamicMesh3& Mesh, const TArray<int32>& VidLoop,
	const TArray<int32>& EdgeLoop, TArray<TPair<int32, TPair<int8, int8>>>& TriVertPairsOut)
{
	if (!ensure(EdgeLoop.Num() == VidLoop.Num()))
	{
		return false;
	}

	for (int32 QuadIndex = 0; QuadIndex < EdgeLoop.Num(); ++QuadIndex)
	{
		int32 Tid = Mesh.GetEdgeT(EdgeLoop[QuadIndex]).A;
		int32 FirstVid = VidLoop[QuadIndex];
		int32 SecondVid = VidLoop[(QuadIndex + 1) % VidLoop.Num()];

		FIndex3i TriVids = Mesh.GetTriangle(Tid);
		int8 SubIdx1 = (int8)IndexUtil::FindTriIndex(FirstVid, TriVids);
		int8 SubIdx2 = (int8)IndexUtil::FindTriIndex(SecondVid, TriVids);
		if (ensure(SubIdx1 >= 0 && SubIdx2 >= 0))
		{
			TriVertPairsOut.Emplace(Tid,
				TPair<int8, int8>(SubIdx1, SubIdx2));
		}
		else
		{
			return false;
		}

	}
	return true;
}



bool FDynamicMeshEditor::WeldVertexLoops(const TArray<int32>& Loop1, const TArray<int32>& Loop2)
{
	int32 N = Loop1.Num();
	checkf(N == Loop2.Num(), TEXT("FDynamicMeshEditor::WeldVertexLoops: loops are not the same length!"));
	if (N != Loop2.Num())
	{
		return false;
	}

	int32 FailureCount = 0;

	// collect set of edges
	TArray<int32> Edges1, Edges2;
	Edges1.SetNum(N);
	Edges2.SetNum(N);
	for (int32 i = 0; i < N; ++i)
	{
		int32 a = Loop1[i];
		int32 b = Loop1[(i + 1) % N];
		Edges1[i] = Mesh->FindEdge(a, b);
		if (Edges1[i] == FDynamicMesh3::InvalidID)
		{
			return false;
		}

		int32 c = Loop2[i];
		int32 d = Loop2[(i + 1) % N];
		Edges2[i] = Mesh->FindEdge(c, d);
		if (Edges2[i] == FDynamicMesh3::InvalidID)
		{
			return false;
		}
	}

	// merge edges. Some merges may merge multiple edges, in which case we want to 
	// skip those when we encounter them later.
	TArray<int32> SkipEdges;
	for (int32 i = 0; i < N; ++i)
	{
		int32 Edge1 = Edges1[i];
		int32 Edge2 = Edges2[i];
		if (SkipEdges.Contains(Edge2))		// occurs at loop closures
		{
			continue;
		}

		FDynamicMesh3::FMergeEdgesInfo MergeInfo;
		EMeshResult Result = Mesh->MergeEdges(Edge1, Edge2, MergeInfo);
		if (Result != EMeshResult::Ok)
		{
			FailureCount++;
		}
		else
		{
			if (MergeInfo.ExtraRemovedEdges.A != FDynamicMesh3::InvalidID)
			{
				SkipEdges.Add(MergeInfo.ExtraRemovedEdges.A);
			}
			if (MergeInfo.ExtraRemovedEdges.B != FDynamicMesh3::InvalidID)
			{
				SkipEdges.Add(MergeInfo.ExtraRemovedEdges.B);
			}
			for (int RemovedEID : MergeInfo.BowtiesRemovedEdges)
			{
				SkipEdges.Add(RemovedEID);
			}
		}
	}

	return (FailureCount > 0);
}






bool FDynamicMeshEditor::StitchSparselyCorrespondedVertexLoops(const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2, FDynamicMeshEditResult& ResultOut, bool bReverseOrientation)
{
	int CorrespondN = MatchedIndices1.Num();
	if (!ensureMsgf(CorrespondN == MatchedIndices2.Num(), TEXT("FDynamicMeshEditor::StitchSparselyCorrespondedVertices: correspondence arrays are not the same length!")))
	{
		return false;
	}
	// TODO: support case of only one corresponded vertex & a connecting a full loop around?
	// this requires allowing start==end to not immediately stop the walk ...
	if (!ensureMsgf(CorrespondN >= 2, TEXT("Must have at least two corresponded vertices")))
	{
		return false;
	}
	ResultOut.NewGroups.Reserve(CorrespondN);

	int i = 0;
	for (; i < CorrespondN; ++i) 
	{
		int Starts[2] { MatchedIndices1[i], MatchedIndices2[i] };
		int Ends[2] { MatchedIndices1[(i + 1) % CorrespondN], MatchedIndices2[(i + 1) % CorrespondN] };

		auto GetWrappedSpanLen = [](const FDynamicMesh3* M, const TArray<int>& VertexIDs, int StartInd, int EndInd)->float
		{
			double LenTotal = 0;
			FVector3d V = M->GetVertex(VertexIDs[StartInd]);
			for (int Ind = StartInd, IndNext; Ind != EndInd;)
			{
				IndNext = (Ind + 1) % VertexIDs.Num();
				FVector3d VNext = M->GetVertex(VertexIDs[IndNext]);
				LenTotal += Distance(V, VNext);
				Ind = IndNext;
				V = VNext;
			}
			return (float)LenTotal;
		};
		float LenTotal[2] { GetWrappedSpanLen(Mesh, VertexIDs1, Starts[0], Ends[0]), GetWrappedSpanLen(Mesh, VertexIDs2, Starts[1], Ends[1]) };
		float LenAlong[2] { FMathf::Epsilon, FMathf::Epsilon };
		LenTotal[0] += FMathf::Epsilon;
		LenTotal[1] += FMathf::Epsilon;


		int NewGroupID = Mesh->AllocateTriangleGroup();
		ResultOut.NewGroups.Add(NewGroupID);
		
		int Walks[2]{ Starts[0], Starts[1] };
		FVector3d Vertex[2]{ Mesh->GetVertex(VertexIDs1[Starts[0]]), Mesh->GetVertex(VertexIDs2[Starts[1]]) };
		while (Walks[0] != Ends[0] || Walks[1] != Ends[1])
		{
			float PctAlong[2]{ LenAlong[0] / LenTotal[0], LenAlong[1] / LenTotal[1] };
			bool bAdvanceSecond = (Walks[0] == Ends[0] || (Walks[1] != Ends[1] && PctAlong[0] > PctAlong[1]));
			FIndex3i Tri(VertexIDs1[Walks[0]], VertexIDs2[Walks[1]], -1);
			if (!bAdvanceSecond)
			{
				Walks[0] = (Walks[0] + 1) % VertexIDs1.Num();
				
				Tri.C = VertexIDs1[Walks[0]];
				FVector3d NextV = Mesh->GetVertex(Tri.C);
				LenAlong[0] += (float)Distance(NextV, Vertex[0]);
				Vertex[0] = NextV;
			}
			else
			{
				Walks[1] = (Walks[1] + 1) % VertexIDs2.Num();
				Tri.C = VertexIDs2[Walks[1]];
				FVector3d NextV = Mesh->GetVertex(Tri.C);
				LenAlong[1] += (float)Distance(NextV, Vertex[1]);
				Vertex[1] = NextV;
			}
			if (bReverseOrientation)
			{
				Swap(Tri.B, Tri.C);
			}
			int Tid = Mesh->AppendTriangle(Tri, NewGroupID);
			if (Tid < 0)
			{
				goto operation_failed;
			}
			ResultOut.NewTriangles.Add(Tid);
		}
	}

	return true;

operation_failed:
	// remove what we added so far
	if (ResultOut.NewTriangles.Num())
	{
		ensureMsgf(RemoveTriangles(ResultOut.NewTriangles, false), TEXT("FDynamicMeshEditor::StitchSparselyCorrespondedVertexLoops: failed to add all triangles, and also failed to back out changes."));
	}
	return false;
}

bool FDynamicMeshEditor::AddTriangleFan_OrderedVertexLoop(int CenterVertex, const TArray<int>& VertexLoop, int GroupID, FDynamicMeshEditResult& ResultOut)
{
	if (GroupID == -1)
	{
		GroupID = Mesh->AllocateTriangleGroup();
		ResultOut.NewGroups.Add(GroupID);
	}

	int N = VertexLoop.Num();
	ResultOut.NewTriangles.Reserve(N);

	int i = 0;
	for (i = 0; i < N; ++i)
	{
		int A = VertexLoop[i];
		int B = VertexLoop[(i + 1) % N];

		FIndex3i NewT(CenterVertex, B, A);
		int NewTID = Mesh->AppendTriangle(NewT, GroupID);
		if (NewTID < 0)
		{
			goto operation_failed;
		}

		ResultOut.NewTriangles.Add(NewTID);
	}

	return true;

operation_failed:
	// remove what we added so far
	if (!RemoveTriangles(ResultOut.NewTriangles, false))
	{
		checkf(false, TEXT("FDynamicMeshEditor::AddTriangleFan: failed to add all triangles, and also failed to back out changes."));
	}
	return false;
}



bool FDynamicMeshEditor::RemoveTriangles(const TArray<int>& Triangles, bool bRemoveIsolatedVerts)
{
	return RemoveTriangles(Triangles, bRemoveIsolatedVerts, [](int) {});
}


bool FDynamicMeshEditor::RemoveTriangles(const TArray<int>& Triangles, bool bRemoveIsolatedVerts, TFunctionRef<void(int)> OnRemoveTriFunc)
{
	bool bAllOK = true;
	int NumTriangles = Triangles.Num();
	for (int i = 0; i < NumTriangles; ++i)
	{
		if (Mesh->IsTriangle(Triangles[i]) == false)
		{
			continue;
		}

		OnRemoveTriFunc(Triangles[i]);

		EMeshResult result = Mesh->RemoveTriangle(Triangles[i], bRemoveIsolatedVerts, false);
		if (result != EMeshResult::Ok)
		{
			bAllOK = false;
		}
	}
	return bAllOK;
}



int FDynamicMeshEditor::RemoveSmallComponents(double MinVolume, double MinArea, int MinTriangleCount)
{
	FMeshConnectedComponents C(Mesh);
	C.FindConnectedTriangles();
	if (C.Num() == 1)
	{
		return 0;
	}
	int Removed = 0;
	for (FMeshConnectedComponents::FComponent& Comp : C) {
		bool bRemove = Comp.Indices.Num() < MinTriangleCount;
		if (!bRemove)
		{
			FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*Mesh, Comp.Indices);
			bRemove = VolArea.X < MinVolume || VolArea.Y < MinArea;
		}
		if (bRemove) {
			RemoveTriangles(Comp.Indices, true);
			Removed++;
		}
	}
	return Removed;
}



/**
 * Make a copy of provided triangles, with new vertices. You provide IndexMaps because
 * you know if you are doing a small subset or a full-mesh-copy.
 */
void FDynamicMeshEditor::DuplicateTriangles(const TArray<int>& Triangles, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut)
{
	ResultOut.Reset();
	IndexMaps.Initialize(Mesh);

	for (int TriangleID : Triangles) 
	{
		FIndex3i Tri = Mesh->GetTriangle(TriangleID);

		int NewGroupID = Mesh->HasTriangleGroups() ? FindOrCreateDuplicateGroup(TriangleID, IndexMaps, ResultOut) : -1;

		FIndex3i NewTri;
		NewTri[0] = FindOrCreateDuplicateVertex(Tri[0], IndexMaps, ResultOut);
		NewTri[1] = FindOrCreateDuplicateVertex(Tri[1], IndexMaps, ResultOut);
		NewTri[2] = FindOrCreateDuplicateVertex(Tri[2], IndexMaps, ResultOut);

		int NewTriangleID = Mesh->AppendTriangle(NewTri, NewGroupID);
		IndexMaps.SetTriangle(TriangleID, NewTriangleID);
		ResultOut.NewTriangles.Add(NewTriangleID);

		CopyAttributes(TriangleID, NewTriangleID, IndexMaps, ResultOut);

		//Mesh->CheckValidity(true);
	}

}

bool FDynamicMeshEditor::DisconnectTriangles(const TArray<int>& Triangles, TArray<FLoopPairSet>& LoopSetOut, bool bHandleBoundaryVertices)
{
	// find the region boundary loops
	FMeshRegionBoundaryLoops RegionLoops(Mesh, Triangles, false);
	bool bOK = RegionLoops.Compute();
	if (!ensure(bOK))
	{
		return false;
	}
	TArray<FEdgeLoop>& Loops = RegionLoops.Loops;

	TSet<int> TriangleSet(Triangles);

	return DisconnectTriangles(TriangleSet, Loops, LoopSetOut, bHandleBoundaryVertices);
}

bool FDynamicMeshEditor::DisconnectTriangles(const TSet<int>& TriangleSet, const TArray<FEdgeLoop>& Loops, TArray<FLoopPairSet>& LoopSetOut, bool bHandleBoundaryVertices)
{
	// process each loop island
	int NumLoops = Loops.Num();
	LoopSetOut.SetNum(NumLoops);
	TArray<int> FilteredTriangles;
	TMap<int32, int32> OldVidsToNewVids; 
	for ( int li = 0; li < NumLoops; ++li)
	{
		const FEdgeLoop& Loop = Loops[li];
		FLoopPairSet& LoopPair = LoopSetOut[li];
		LoopPair.OuterVertices = Loop.Vertices;
		LoopPair.OuterEdges = Loop.Edges;

		bool bSawBoundaryInLoop = false;

		// duplicate the vertices
		int NumVertices = Loop.Vertices.Num();
		TArray<int> NewVertexLoop; NewVertexLoop.SetNum(NumVertices);
		for (int vi = 0; vi < NumVertices; ++vi)
		{
			int VertID = Loop.Vertices[vi];

			// See if we've processed this vertex already as part of a bowtie
			int32* ExistingNewVertID = OldVidsToNewVids.Find(VertID);
			if (ExistingNewVertID)
			{
				// We've already done the split, but we still need to update the loop pairs. The way we
				// do this depends on whether the vert was originally a boundary vert or not, because
				// we treat these cases differently when we perform splits.
				// If it was originally a boundary vert, the new vertex ended up as an isolated
				// vertex.
				if (!Mesh->IsReferencedVertex(*ExistingNewVertID))
				{
					// The isolated vertex should be in the outer loop
					LoopPair.OuterVertices[vi] = *ExistingNewVertID;
					LoopPair.OuterEdges[vi] = FDynamicMesh3::InvalidID;
					LoopPair.OuterEdges[(vi == 0) ? NumVertices - 1 : vi - 1] = FDynamicMesh3::InvalidID;
					NewVertexLoop[vi] = VertID;
				}
				else
				{
					// Otherwise, the new vertex should end up in the inner loop (and not isolated, because
					// it is attached to the selected triangles).
					NewVertexLoop[vi] = *ExistingNewVertID;
				}

				continue;
			}
			// If we get here, we still need to split the vertex.

			// See how many of the adjacent triangles are in our selection.
			FilteredTriangles.Reset();
			int TriRingCount = 0;
			for (int RingTID : Mesh->VtxTrianglesItr(VertID))
			{
				if (TriangleSet.Contains(RingTID))
				{
					FilteredTriangles.Add(RingTID);
				}
				TriRingCount++;
			}

			if (FilteredTriangles.Num() < TriRingCount)
			{
				// This is a non-mesh-boundary vert
				checkSlow(!Mesh->SplitVertexWouldLeaveIsolated(VertID, FilteredTriangles));
				DynamicMeshInfo::FVertexSplitInfo SplitInfo;
				const EMeshResult MeshResult = Mesh->SplitVertex(VertID, FilteredTriangles, SplitInfo);
				ensure(MeshResult == EMeshResult::Ok);

				int NewVertID = SplitInfo.NewVertex;
				OldVidsToNewVids.Add(VertID, NewVertID);
				NewVertexLoop[vi] = NewVertID;
			}
			else if (bHandleBoundaryVertices)
			{
				// if we have a boundary vertex, we are going to duplicate it and use the duplicated
				// vertex as the "old" one, and just keep the existing one on the "inner" loop.
				// This means we have to rewrite vertex in the "outer" loop, and that loop will no longer actually be an EdgeLoop, so we set those edges to invalid
				int32 NewVertID = Mesh->AppendVertex(*Mesh, VertID);
				OldVidsToNewVids.Add(VertID, NewVertID);
				LoopPair.OuterVertices[vi] = NewVertID;
				LoopPair.OuterEdges[vi] = FDynamicMesh3::InvalidID;
				LoopPair.OuterEdges[(vi == 0) ? NumVertices-1 : vi-1] = FDynamicMesh3::InvalidID;
				NewVertexLoop[vi] = VertID;
				bSawBoundaryInLoop = true;
			}
			else
			{
				ensure(false);
				return false;   // cannot proceed
			}
		}

		FEdgeLoop InnerLoop;
		if (!ensure(InnerLoop.InitializeFromVertices(Mesh, NewVertexLoop, false)))
		{
			return false;
		}
		LoopPair.InnerVertices = MoveTemp(InnerLoop.Vertices);
		LoopPair.InnerEdges = MoveTemp(InnerLoop.Edges);
		LoopPair.bOuterIncludesIsolatedVertices = bSawBoundaryInLoop;
	}

	return true;
}


void FDynamicMeshEditor::DisconnectTriangles(const TArray<int>& Triangles, bool bPreventBowties)
{
	TSet<int> TriSet, BoundaryVerts;
	TArray<int> NewVerts, OldVertsThatSplit;
	TArray<int> FilteredTriangles;
	DynamicMeshInfo::FVertexSplitInfo SplitInfo;

	TriSet.Append(Triangles);
	for (int TID : Triangles)
	{
		FIndex3i Nbrs = Mesh->GetTriNeighbourTris(TID);
		FIndex3i Tri = Mesh->GetTriangle(TID);
		for (int SubIdx = 0; SubIdx < 3; SubIdx++)
		{
			int NeighborTID = Nbrs[SubIdx];
			if (!TriSet.Contains(NeighborTID))
			{
				BoundaryVerts.Add(Tri[SubIdx]);
				BoundaryVerts.Add(Tri[(SubIdx + 1) % 3]);
			}
		}
	}
	for (int VID : BoundaryVerts)
	{
		FilteredTriangles.Reset();
		int TriRingCount = 0;
		for (int RingTID : Mesh->VtxTrianglesItr(VID))
		{
			if (TriSet.Contains(RingTID))
			{
				FilteredTriangles.Add(RingTID);
			}
			TriRingCount++;
		}

		if (FilteredTriangles.Num() < TriRingCount)
		{
			checkSlow(!Mesh->SplitVertexWouldLeaveIsolated(VID, FilteredTriangles));
			ensure(EMeshResult::Ok == Mesh->SplitVertex(VID, FilteredTriangles, SplitInfo));
			NewVerts.Add(SplitInfo.NewVertex);
			OldVertsThatSplit.Add(SplitInfo.OriginalVertex);
		}
	}
	if (bPreventBowties)
	{
		// reusable arrays for SplitBowtiesInternal
		TArray<int> BowtieTriangles, BowtieContiguousGroupLengths;
		TArray<bool> BowtieGroupIsLoop;
		for (int VID : OldVertsThatSplit)
		{
			SplitBowtiesInternal(VID, BowtieTriangles, BowtieContiguousGroupLengths, BowtieGroupIsLoop, nullptr);
		}
		for (int VID : NewVerts)
		{
			SplitBowtiesInternal(VID, BowtieTriangles, BowtieContiguousGroupLengths, BowtieGroupIsLoop, nullptr);
		}
	}
}

bool FDynamicMeshEditor::DisconnectTrianglesAlongEdges(const TSet<int32>& EdgeIDs, TArray<int32>* AddedVertexIDs)
{
	// Note: The overlay seams version of this method is the CreateSeamsAlongEdges method --
	//  the methods are very similar, so if you change this method consider changing that one as well

	/**
	 * @param TIDsUntilNextBoundary TIDs in a fan away from EID around VID until the next boundary. Will
	 *  be the entire one-ring if the given EID is the only boundary attached to VID.
	 */
	auto DoesVertHaveAnotherBoundaryEdgeAttached = 
		[this, &EdgeIDs, AddedVertexIDs](int32 EID, int32 VID, TArray<int32>& TIDsUntilNextBoundary) -> bool
	{
		TIDsUntilNextBoundary.Reset();

		FIndex2i EdgeTIDs = Mesh->GetEdgeT(EID);
		int32 CurrentTID = EdgeTIDs.A;
		int32 PreviousTID = EdgeTIDs.B;

		int32 MaxNumTIDs = Mesh->GetVtxEdgeCount(VID);
		while (true)
		{
			TIDsUntilNextBoundary.Add(CurrentTID);
			if (TIDsUntilNextBoundary.Num() > MaxNumTIDs) // sanity check
			{
				ensure(false);
				return false;
			}

			FIndex3i NextTriResult = UE::Geometry::FindNextAdjacentTriangleAroundVtx(Mesh, VID, CurrentTID, PreviousTID,
				[&EdgeIDs, EID](int32 Tri0, int32 Tri1, int32 EIDBetween)
				{
					// Treat 'future' boundary edges that we haven't disconnected yet as boundary edges here
					// (but not the current EID, since we're looking for *another* boundary edge)
					return !(EIDBetween != EID && EdgeIDs.Contains(EIDBetween)); 
				}
			);
			int32 NextTID = NextTriResult.A;

			// Stop if we've come to a (current or future) new boundary ...
			if (NextTID == IndexConstants::InvalidID)
			{
				return true;
			}
			// ... or if we've wrapped around to the start
			else if (NextTID == EdgeTIDs.A)
			{
				return false;
			}
			PreviousTID = CurrentTID;
			CurrentTID = NextTID;
		}
	};

	auto SplitEdgeVert = [this, AddedVertexIDs](int32 VID, const TArray<int32>& TIDsToModify) 
	{
		DynamicMeshInfo::FVertexSplitInfo SplitInfo;
		if (Mesh->SplitVertex(VID, TIDsToModify, SplitInfo) == EMeshResult::Ok && AddedVertexIDs != nullptr)
		{
			AddedVertexIDs->Add(SplitInfo.NewVertex);
		}
	};
	// reusable arrays for bowtie and edge disconnecting logic below
	TArray<int> BowtieTriangles, BowtieContiguousGroupLengths;
	TArray<bool> BowtieGroupIsLoop;
	TArray<int32> VertTIDsToUseForSplit[2];
	for (int32 EID : EdgeIDs)
	{
		FIndex2i EdgeVIDs = Mesh->GetEdgeV(EID);
		bool VertNeedsSplitting[2] = { false, false };
		VertTIDsToUseForSplit[0].Reset();
		VertTIDsToUseForSplit[1].Reset();

		if (Mesh->IsBoundaryEdge(EID))
		{
			// If this is already a boundary edge, make sure its endpoints are not bowties. We create such edges 
			// sometimes as we split adjacent edges, and it is also likely to be what the user wants when
			// selecting edges adjacent to a bowtie.
			for (int i = 0; i < 2; ++i)
			{
				SplitBowtiesInternal(EdgeVIDs[i], BowtieTriangles, BowtieContiguousGroupLengths, BowtieGroupIsLoop, AddedVertexIDs);
			}
			continue;
		}

		// If we're not already a boundary, then it's going to become one. A vert needs to get split if it
		// has a present or future boundary edge attached, or else it will become a bowtie if we just split the
		// other vert.
		for (int i = 0; i < 2; ++i)
		{
			VertNeedsSplitting[i] = DoesVertHaveAnotherBoundaryEdgeAttached(EID, EdgeVIDs[i], VertTIDsToUseForSplit[i]);
		}

		// If neither absolutely has to get split, then one needs to get split anyway so that we
		// make the edge into a boundary.
		if (!VertNeedsSplitting[0] && !VertNeedsSplitting[1])
		{
			// We'll go halfway around the triangle one-ring, rounding up
			int32 NumTrisToDisconnect = (VertTIDsToUseForSplit[0].Num() + 1) / 2;

			// In doing this, we are splitting an adjacent edge and therefore might inadvertantly
			// introduce a bowtie on the other vert of that edge, which we would like to avoid. 
			// So keep track of that potential bowtie.
			int32 OtherSplitEdge = Mesh->FindEdgeFromTriPair(
				VertTIDsToUseForSplit[0][NumTrisToDisconnect - 1],
				VertTIDsToUseForSplit[0][NumTrisToDisconnect]);
			int32 PotentialBowtieVID = ensure(OtherSplitEdge != IndexConstants::InvalidID) ?
				IndexUtil::FindEdgeOtherVertex(Mesh->GetEdgeV(OtherSplitEdge), EdgeVIDs[0])
				: IndexConstants::InvalidID;

			// Perform the split
			VertTIDsToUseForSplit[0].SetNum(NumTrisToDisconnect);
			SplitEdgeVert(EdgeVIDs[0], VertTIDsToUseForSplit[0]);

			// Deal with the bowtie if we created one
			if (ensure(PotentialBowtieVID != IndexConstants::InvalidID)
				&& Mesh->IsBowtieVertex(PotentialBowtieVID))
			{
				SplitBowtiesInternal(PotentialBowtieVID, BowtieTriangles, BowtieContiguousGroupLengths, BowtieGroupIsLoop, AddedVertexIDs);
			}
		}
		else
		{
			for (int i = 0; i < 2; ++i)
			{
				if (VertNeedsSplitting[i])
				{
					SplitEdgeVert(EdgeVIDs[i], VertTIDsToUseForSplit[i]);
				}
			}
		}//end if at least one side needed splitting
	}//end for each edge

	return true;
}


void FDynamicMeshEditor::SplitBowties(FDynamicMeshEditResult& ResultOut)
{
	TArray<int> Triangles, ContiguousGroupLengths;
	TArray<bool> GroupIsLoop;
	ResultOut.Reset();
	TSet<int> AddedVerticesWithIDLessThanMax; // added vertices that we can't filter just by checking against original max id; this will be empty for compact meshes
	for (int VertexID = 0, OriginalMaxID = Mesh->MaxVertexID(); VertexID < OriginalMaxID; VertexID++)
	{
		if (!Mesh->IsVertex(VertexID) || AddedVerticesWithIDLessThanMax.Contains(VertexID))
		{
			continue;
		}
		int32 NumVertsBefore = ResultOut.NewVertices.Num();
		SplitBowtiesInternal(VertexID, Triangles, ContiguousGroupLengths, GroupIsLoop, &ResultOut.NewVertices);
		for (int Idx = NumVertsBefore; Idx < ResultOut.NewVertices.Num(); Idx++)
		{
			if (ResultOut.NewVertices[Idx] < OriginalMaxID)
			{
				AddedVerticesWithIDLessThanMax.Add(ResultOut.NewVertices[Idx]);
			}
		}
	}
}



void FDynamicMeshEditor::SplitBowtiesInternal(int VertexID, TArray<int>& TrianglesOut, TArray<int>& ContiguousGroupLengths, TArray<bool>& GroupIsLoop, TArray<int32>* NewVertices)
{
	DynamicMeshInfo::FVertexSplitInfo SplitInfo;
	if (ensure(EMeshResult::Ok == Mesh->GetVtxContiguousTriangles(VertexID, TrianglesOut, ContiguousGroupLengths, GroupIsLoop)))
	{
		if (ContiguousGroupLengths.Num() > 1)
		{
			// is bowtie
			for (int GroupIdx = 1, GroupStartIdx = ContiguousGroupLengths[0]; GroupIdx < ContiguousGroupLengths.Num(); GroupStartIdx += ContiguousGroupLengths[GroupIdx++])
			{
				ensure(EMeshResult::Ok == Mesh->SplitVertex(VertexID, TArrayView<const int>(TrianglesOut.GetData() + GroupStartIdx, ContiguousGroupLengths[GroupIdx]), SplitInfo));
				if (NewVertices)
				{
					NewVertices->Add(SplitInfo.NewVertex);
				}
			}
		}
	}
}

void FDynamicMeshEditor::SplitBowtiesAtTriangles(const TArray<int32>& TriangleIDs, FDynamicMeshEditResult& ResultOut)
{
	TArray<int> Triangles, ContiguousGroupLengths;
	TArray<bool> GroupIsLoop;
	TSet<int32> VidsProcessed;
	for (int32 Tid : TriangleIDs)
	{
		FIndex3i TriVids = Mesh->GetTriangle(Tid);
		for (int i = 0; i < 3; ++i)
		{
			int32 Vid = TriVids[i];
			bool bWasAlreadyProcessed = false;
			VidsProcessed.Add(Vid, &bWasAlreadyProcessed);
			if (!bWasAlreadyProcessed)
			{
				SplitBowtiesInternal(Vid, Triangles, ContiguousGroupLengths, GroupIsLoop, &ResultOut.NewVertices);
			}
		}
	}
}


bool FDynamicMeshEditor::ReinsertSubmesh(const FDynamicSubmesh3& Region, FOptionallySparseIndexMap& SubToNewV, TArray<int>* new_tris, EDuplicateTriBehavior DuplicateBehavior)
{
	check(Region.GetBaseMesh() == Mesh);
	const FDynamicMesh3& Sub = Region.GetSubmesh();
	bool bAllOK = true;

	FIndexFlagSet done_v(Sub.MaxVertexID(), Sub.TriangleCount()/2);
	SubToNewV.Initialize(Sub.MaxVertexID(), Sub.VertexCount());

	int NT = Sub.MaxTriangleID();
	for (int ti = 0; ti < NT; ++ti )
	{
		if (Sub.IsTriangle(ti) == false)
		{
			continue;
		}

		FIndex3i sub_t = Sub.GetTriangle(ti);
		int gid = Sub.GetTriangleGroup(ti);

		FIndex3i new_t = FIndex3i::Zero();
		for ( int j = 0; j < 3; ++j )
		{
			int sub_v = sub_t[j];
			int new_v = -1;
			if (done_v[sub_v] == false)
			{
				// first check if this is a boundary vtx on submesh and maps to a bdry vtx on base mesh
				if (Sub.IsBoundaryVertex(sub_v))
				{
					int base_v = Region.MapVertexToBaseMesh(sub_v);
					if (base_v >= 0 && Mesh->IsVertex(base_v) && Region.InBaseBorderVertices(base_v) == true)
					{
						// this should always be true
						if (ensure(Mesh->IsBoundaryVertex(base_v)))
						{
							new_v = base_v;
						}
					}
				}

				// if that didn't happen, append new vtx
				if (new_v == -1)
				{
					new_v = Mesh->AppendVertex(Sub, sub_v);
				}

				SubToNewV.Set(sub_v, new_v);
				done_v.Add(sub_v);

			}
			else
			{
				new_v = SubToNewV[sub_v];
			}

			new_t[j] = new_v;
		}

		// try to handle duplicate-tri case
		if (DuplicateBehavior == EDuplicateTriBehavior::EnsureContinue)
		{
			ensure(Mesh->FindTriangle(new_t.A, new_t.B, new_t.C) == FDynamicMesh3::InvalidID);
		}
		else
		{
			int existing_tid = Mesh->FindTriangle(new_t.A, new_t.B, new_t.C);
			if (existing_tid != FDynamicMesh3::InvalidID)
			{
				if (DuplicateBehavior == EDuplicateTriBehavior::EnsureAbort)
				{
					ensure(false);
					return false;
				}
				else if (DuplicateBehavior == EDuplicateTriBehavior::UseExisting)
				{
					if (new_tris)
					{
						new_tris->Add(existing_tid);
					}
					continue;
				}
				else if (DuplicateBehavior == EDuplicateTriBehavior::Replace)
				{
					Mesh->RemoveTriangle(existing_tid, false);
				}
			}
		}


		int new_tid = Mesh->AppendTriangle(new_t, gid);
		ensure(new_tid >= 0);
		if (!Mesh->IsTriangle(new_tid))
		{
			bAllOK = false;
		}

		if (new_tris)
		{
			new_tris->Add(new_tid);
		}
	}

	return bAllOK;
}



FVector3f FDynamicMeshEditor::ComputeAndSetQuadNormal(const FIndex2i& QuadTris, bool bIsPlanar)
{
	FVector3f Normal(0, 0, 1);
	if (bIsPlanar)
	{
		Normal = (FVector3f)Mesh->GetTriNormal(QuadTris.A);
	}
	else
	{
		Normal = (FVector3f)Mesh->GetTriNormal(QuadTris.A);
		Normal += (FVector3f)Mesh->GetTriNormal(QuadTris.B);
		Normalize(Normal);
	}
	SetQuadNormals(QuadTris, Normal);
	return Normal;
}




void FDynamicMeshEditor::SetQuadNormals(const FIndex2i& QuadTris, const FVector3f& Normal)
{
	check(Mesh->HasAttributes());
	FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();

	FIndex3i Triangle1 = Mesh->GetTriangle(QuadTris.A);

	FIndex3i NormalTriangle1;
	NormalTriangle1[0] = Normals->AppendElement(Normal);
	NormalTriangle1[1] = Normals->AppendElement(Normal);
	NormalTriangle1[2] = Normals->AppendElement(Normal);
	Normals->SetTriangle(QuadTris.A, NormalTriangle1);

	if (Mesh->IsTriangle(QuadTris.B))
	{
		FIndex3i Triangle2 = Mesh->GetTriangle(QuadTris.B);
		FIndex3i NormalTriangle2;
		for (int j = 0; j < 3; ++j)
		{
			int i = Triangle1.IndexOf(Triangle2[j]);
			if (i == -1)
			{
				NormalTriangle2[j] = Normals->AppendElement(Normal);
			}
			else
			{
				NormalTriangle2[j] = NormalTriangle1[i];
			}
		}
		Normals->SetTriangle(QuadTris.B, NormalTriangle2);
	}

}


void FDynamicMeshEditor::SetTriangleNormals(const TArray<int>& Triangles, const FVector3f& Normal)
{
	check(Mesh->HasAttributes());
	FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();

	TMap<int, int> Vertices;

	for (int tid : Triangles)
	{
		if (Normals->IsSetTriangle(tid))
		{
			Normals->UnsetTriangle(tid);
		}

		FIndex3i BaseTri = Mesh->GetTriangle(tid);
		FIndex3i ElemTri;
		for (int j = 0; j < 3; ++j)
		{
			const int* FoundElementID = Vertices.Find(BaseTri[j]);
			if (FoundElementID == nullptr)
			{
				ElemTri[j] = Normals->AppendElement(Normal);
				Vertices.Add(BaseTri[j], ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}
		Normals->SetTriangle(tid, ElemTri);
	}
}


void FDynamicMeshEditor::SetTriangleNormals(const TArray<int>& Triangles)
{
	check(Mesh->HasAttributes());
	FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();

	TSet<int32> TriangleSet(Triangles);
	TUniqueFunction<bool(int32)> TrianglePredicate = [&](int32 TriangleID) { return TriangleSet.Contains(TriangleID); };

	TMap<int, int> Vertices;

	for (int tid : Triangles)
	{
		if (Normals->IsSetTriangle(tid))
		{
			Normals->UnsetTriangle(tid);
		}

		FIndex3i BaseTri = Mesh->GetTriangle(tid);
		FIndex3i ElemTri;
		for (int j = 0; j < 3; ++j)
		{
			const int* FoundElementID = Vertices.Find(BaseTri[j]);
			if (FoundElementID == nullptr)
			{
				FVector3d VtxROINormal = FMeshNormals::ComputeVertexNormal(*Mesh, BaseTri[j], TrianglePredicate);
				ElemTri[j] = Normals->AppendElement( (FVector3f)VtxROINormal);
				Vertices.Add(BaseTri[j], ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}
		Normals->SetTriangle(tid, ElemTri);
	}
}



void FDynamicMeshEditor::SetTubeNormals(const TArray<int>& Triangles, const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2, bool bReverseNormals)
{
	check(Mesh->HasAttributes());
	check(MatchedIndices1.Num() == MatchedIndices2.Num());
	int NumMatched = MatchedIndices1.Num();
	FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();

	TArray<FVector3f> MatchedEdgeNormals[2]; // matched edge normals for the two sides
	MatchedEdgeNormals[0].SetNum(NumMatched);
	MatchedEdgeNormals[1].SetNum(NumMatched);
	for (int LastMatchedIdx = NumMatched - 1, Idx = 0; Idx < NumMatched; LastMatchedIdx = Idx++)
	{
		// get edge indices
		int M1[2]{ MatchedIndices1[LastMatchedIdx], MatchedIndices1[Idx] };
		int M2[2]{ MatchedIndices2[LastMatchedIdx], MatchedIndices2[Idx] };
		// make sure they're not the same index
		if (M1[0] == M1[1])
		{
			M1[1] = (M1[1] + 1) % VertexIDs1.Num();
		}
		if (M2[0] == M2[1])
		{
			M2[1] = (M2[1] + 1) % VertexIDs2.Num();
		}
		FVector3f Corners[4]{ (FVector3f)Mesh->GetVertex(VertexIDs1[M1[0]]), (FVector3f)Mesh->GetVertex(VertexIDs1[M1[1]]), (FVector3f)Mesh->GetVertex(VertexIDs2[M2[0]]), (FVector3f)Mesh->GetVertex(VertexIDs2[M2[1]]) };
		FVector3f Edges[2]{ Corners[1] - Corners[0], Corners[3] - Corners[2] };
		FVector3f Across = Corners[2] - Corners[0];
		MatchedEdgeNormals[0][LastMatchedIdx] = Normalized(Edges[0].Cross(Across));
		MatchedEdgeNormals[1][LastMatchedIdx] = Normalized(Edges[1].Cross(Across));
	}

	TArray<FVector3f> MatchedVertNormals[2];
	MatchedVertNormals[0].SetNum(NumMatched);
	MatchedVertNormals[1].SetNum(NumMatched);
	TArray<FVector3f> VertNormals[2];
	VertNormals[0].SetNum(VertexIDs1.Num());
	VertNormals[1].SetNum(VertexIDs2.Num());
	for (int LastMatchedIdx = NumMatched - 1, Idx = 0; Idx < NumMatched; LastMatchedIdx = Idx++)
	{
		MatchedVertNormals[0][Idx] = Normalized(MatchedEdgeNormals[0][LastMatchedIdx] + MatchedEdgeNormals[0][Idx]);
		MatchedVertNormals[1][Idx] = Normalized(MatchedEdgeNormals[1][LastMatchedIdx] + MatchedEdgeNormals[1][Idx]);
	}

	float NormalScale = bReverseNormals ? -1.0f : 1.0f;
	TMap<int, int> VertToElID;
	for (int Side = 0; Side < 2; Side++)
	{
		const TArray<int>& MatchedIndices = Side == 0 ? MatchedIndices1 : MatchedIndices2;
		const TArray<int>& VertexIDs = Side == 0 ? VertexIDs1 : VertexIDs2;
		int NumVertices = VertNormals[Side].Num();
		for (int LastMatchedIdx = NumMatched - 1, Idx = 0; Idx < NumMatched; LastMatchedIdx = Idx++)
		{
			int Start = MatchedIndices[LastMatchedIdx], End = MatchedIndices[Idx];

			VertNormals[Side][End] = MatchedVertNormals[Side][Idx];
			if (Start != End)
			{
				VertNormals[Side][Start] = MatchedVertNormals[Side][LastMatchedIdx];

				FVector3d StartPos = Mesh->GetVertex(VertexIDs[Start]);
				FVector3d Along =  Mesh->GetVertex(VertexIDs[End]) - StartPos;
				double SepSq = Along.SquaredLength();
				if (SepSq < KINDA_SMALL_NUMBER)
				{
					for (int InsideIdx = (Start + 1) % NumVertices; InsideIdx != End; InsideIdx = (InsideIdx + 1) % NumVertices)
					{
						VertNormals[Side][InsideIdx] = VertNormals[Side][End]; // just copy the end normal in since all the vertices are almost in the same position
					}
				}
				for (int InsideIdx = (Start + 1) % NumVertices; InsideIdx != End; InsideIdx = (InsideIdx + 1) % NumVertices)
				{
					double InterpT = (Mesh->GetVertex(VertexIDs[InsideIdx]) - StartPos).Dot(Along) / SepSq;
					VertNormals[Side][InsideIdx] = InterpT * VertNormals[Side][End] + (1 - InterpT)* VertNormals[Side][Start];
				}
			}
		}

		for (int Idx = 0; Idx < NumVertices; Idx++)
		{
			int VID = VertexIDs[Idx];
			VertToElID.Add(VID, Normals->AppendElement(NormalScale * VertNormals[Side][Idx]));
		}
	}
	for (int TID : Triangles)
	{
		FIndex3i Tri = Mesh->GetTriangle(TID);
		FIndex3i ElTri(VertToElID[Tri.A], VertToElID[Tri.B], VertToElID[Tri.C]);
		Normals->SetTriangle(TID, ElTri);
	}
}


void FDynamicMeshEditor::SetGeneralTubeUVs(const TArray<int>& Triangles,
	const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2,
	const TArray<float>& UValues, const FVector3f& VDir,
	float UVScaleFactor, const FVector2f& UVTranslation, int UVLayerIndex)
{
	// not really a valid tube if only two vertices on either side
	if (!ensure(VertexIDs1.Num() >= 3 && VertexIDs2.Num() >= 3))
	{
		return;
	}

	check(Mesh->HasAttributes());
	check(MatchedIndices1.Num() == MatchedIndices2.Num());
	check(UValues.Num() == MatchedIndices1.Num() + 1);
	int NumMatched = MatchedIndices1.Num();

	FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

	FVector3d RefPos = Mesh->GetVertex(VertexIDs1[0]);
	auto GetUV = [this, &VDir, &UVScaleFactor, &UVTranslation, &RefPos](int MeshIdx, float UStart, float UEnd, float Param)
	{
		return FVector2f(float( (Mesh->GetVertex(MeshIdx) - RefPos).Dot((FVector3d)VDir) ), FMath::Lerp(UStart, UEnd, Param)) * UVScaleFactor + UVTranslation;
	};

	TArray<FVector2f> VertUVs[2];
	VertUVs[0].SetNum(VertexIDs1.Num()+1);
	VertUVs[1].SetNum(VertexIDs2.Num()+1);
	
	TMap<int, int> VertToElID;
	int DuplicateMappingForLastVert[2]{ -1, -1 }; // second element ids for the first vertices, to handle the seam at the loop
	for (int Side = 0; Side < 2; Side++)
	{
		const TArray<int>& MatchedIndices = Side == 0 ? MatchedIndices1 : MatchedIndices2;
		const TArray<int>& VertexIDs = Side == 0 ? VertexIDs1 : VertexIDs2;
		int NumVertices = VertexIDs.Num();
		for (int Idx = 0; Idx < NumMatched; Idx++)
		{
			int NextIdx = Idx + 1;
			int NextIdxLooped = NextIdx % NumMatched;
			bool bOnLast = NextIdx == NumMatched;

			int Start = MatchedIndices[Idx], End = MatchedIndices[NextIdxLooped];
			int EndUnlooped = bOnLast ? NumVertices : End;

			VertUVs[Side][EndUnlooped] = GetUV(VertexIDs[End], UValues[Idx], UValues[NextIdx], 1.0f);
			if (Start != End)
			{
				VertUVs[Side][Start] = GetUV(VertexIDs[Start], UValues[Idx], UValues[NextIdx], 0.0f);

				FVector3d StartPos = Mesh->GetVertex(VertexIDs[Start]);
				FVector3d Along =  Mesh->GetVertex(VertexIDs[End]) - StartPos;
				double SepSq = Along.SquaredLength();
				if (SepSq < KINDA_SMALL_NUMBER)
				{
					for (int InsideIdx = (Start + 1) % NumVertices; InsideIdx != End; InsideIdx = (InsideIdx + 1) % NumVertices)
					{
						VertUVs[Side][InsideIdx] = VertUVs[Side][EndUnlooped]; // just copy the end normal in since all the vertices are almost in the same position
					}
				}
				for (int InsideIdx = (Start + 1) % NumVertices; InsideIdx != End; InsideIdx = (InsideIdx + 1) % NumVertices)
				{
					float InterpT = float( (Mesh->GetVertex(VertexIDs[InsideIdx]) - StartPos).Dot(Along) / SepSq );
					VertUVs[Side][InsideIdx] = GetUV(VertexIDs[InsideIdx], UValues[Idx], UValues[NextIdx], InterpT);
				}
			}
		}

		for (int Idx = 0; Idx < NumVertices; Idx++)
		{
			int VID = VertexIDs[Idx];
			VertToElID.Add(VID, UVs->AppendElement(VertUVs[Side][Idx]));
		}
		DuplicateMappingForLastVert[Side] = UVs->AppendElement(VertUVs[Side].Last());
	}
	
	bool bPastInitialVertices[2]{ false, false };
	int FirstVID[2] = { VertexIDs1[0], VertexIDs2[0] };
	for (int TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
	{
		int TID = Triangles[TriIdx];
		FIndex3i Tri = Mesh->GetTriangle(TID);
		FIndex3i ElTri(VertToElID[Tri.A], VertToElID[Tri.B], VertToElID[Tri.C]);

		// hacky special handling for the seam at the end of the loop -- the second time we see the start vertices, switch to the end seam elements
		for (int Side = 0; Side < 2; Side++)
		{
			int FirstVIDSubIdx = Tri.IndexOf(FirstVID[Side]);
			bPastInitialVertices[Side] = bPastInitialVertices[Side] || FirstVIDSubIdx == -1;
			if (bPastInitialVertices[Side] && FirstVIDSubIdx >= 0)
			{
				ElTri[FirstVIDSubIdx] = DuplicateMappingForLastVert[Side];
			}
		}
		UVs->SetTriangle(TID, ElTri);
	}
}


void FDynamicMeshEditor::SetTriangleUVsFromProjection(const TArray<int>& Triangles, const FFrame3d& ProjectionFrame, float UVScaleFactor, const FVector2f& UVTranslation, bool bShiftToOrigin, int UVLayerIndex)
{
	SetTriangleUVsFromProjection(Triangles, ProjectionFrame, FVector2f(UVScaleFactor, UVScaleFactor), UVTranslation, UVLayerIndex, bShiftToOrigin, false);
}

void FDynamicMeshEditor::SetTriangleUVsFromProjection(const TArray<int>& Triangles, const FFrame3d& ProjectionFrame, const FVector2f& UVScale, 
	const FVector2f& UVTranslation, int UVLayerIndex, bool bShiftToOrigin, bool bNormalizeBeforeScaling)
{
	if (!Triangles.Num())
	{
		return;
	}

	check(Mesh->HasAttributes() && Mesh->Attributes()->NumUVLayers() > UVLayerIndex);
	FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

	TMap<int, int> BaseToOverlayVIDMap;
	TArray<int> AllUVIndices;

	FAxisAlignedBox2f UVBounds(FAxisAlignedBox2f::Empty());

	for (int TID : Triangles)
	{
		if (UVs->IsSetTriangle(TID))
		{
			UVs->UnsetTriangle(TID);
		}

		FIndex3i BaseTri = Mesh->GetTriangle(TID);
		FIndex3i ElemTri;
		for (int j = 0; j < 3; ++j)
		{
			const int* FoundElementID = BaseToOverlayVIDMap.Find(BaseTri[j]);
			if (FoundElementID == nullptr)
			{
				FVector2f UV = (FVector2f)ProjectionFrame.ToPlaneUV(Mesh->GetVertex(BaseTri[j]), 2);
				UVBounds.Contain(UV);
				ElemTri[j] = UVs->AppendElement(UV);
				AllUVIndices.Add(ElemTri[j]);
				BaseToOverlayVIDMap.Add(BaseTri[j], ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}
		UVs->SetTriangle(TID, ElemTri);
	}

	FVector2f UvScaleToUse = bNormalizeBeforeScaling ? FVector2f(UVScale[0] / UVBounds.Width(), UVScale[1] / UVBounds.Height())
		: UVScale;

	// shift UVs so that their bbox min-corner is at origin and scaled by external scale factor
	for (int UVID : AllUVIndices)
	{
		FVector2f UV = UVs->GetElement(UVID);
		FVector2f TransformedUV = (bShiftToOrigin) ? ((UV - UVBounds.Min) * UvScaleToUse) : (UV * UvScaleToUse);
		TransformedUV += UVTranslation;
		UVs->SetElement(UVID, TransformedUV);
	}
}


void FDynamicMeshEditor::SetQuadUVsFromProjection(const FIndex2i& QuadTris, const FFrame3d& ProjectionFrame, float UVScaleFactor, const FVector2f& UVTranslation, int UVLayerIndex)
{
	check(Mesh->HasAttributes() && Mesh->Attributes()->NumUVLayers() > UVLayerIndex );
	FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

	FIndex4i AllUVIndices(-1, -1, -1, -1);
	FVector2f AllUVs[4];

	// project first triangle
	FIndex3i Triangle1 = Mesh->GetTriangle(QuadTris.A);
	FIndex3i UVTriangle1;
	for (int j = 0; j < 3; ++j)
	{
		FVector2f UV = (FVector2f)ProjectionFrame.ToPlaneUV(Mesh->GetVertex(Triangle1[j]), 2);
		UVTriangle1[j] = UVs->AppendElement(UV);
		AllUVs[j] = UV;
		AllUVIndices[j] = UVTriangle1[j];
	}
	UVs->SetTriangle(QuadTris.A, UVTriangle1);

	// project second triangle
	if (Mesh->IsTriangle(QuadTris.B))
	{
		FIndex3i Triangle2 = Mesh->GetTriangle(QuadTris.B);
		FIndex3i UVTriangle2;
		for (int j = 0; j < 3; ++j)
		{
			int i = Triangle1.IndexOf(Triangle2[j]);
			if (i == -1)
			{
				FVector2f UV = (FVector2f)ProjectionFrame.ToPlaneUV(Mesh->GetVertex(Triangle2[j]), 2);
				UVTriangle2[j] = UVs->AppendElement(UV);
				AllUVs[3] = UV;
				AllUVIndices[3] = UVTriangle2[j];
			}
			else
			{
				UVTriangle2[j] = UVTriangle1[i];
			}
		}
		UVs->SetTriangle(QuadTris.B, UVTriangle2);
	}

	// shift UVs so that their bbox min-corner is at origin and scaled by external scale factor
	FAxisAlignedBox2f UVBounds(FAxisAlignedBox2f::Empty());
	UVBounds.Contain(AllUVs[0]);  UVBounds.Contain(AllUVs[1]);  UVBounds.Contain(AllUVs[2]);
	if (AllUVIndices[3] != -1)
	{
		UVBounds.Contain(AllUVs[3]);
	}
	for (int j = 0; j < 4; ++j)
	{
		if (AllUVIndices[j] != -1)
		{
			FVector2f TransformedUV = (AllUVs[j] - UVBounds.Min) * UVScaleFactor;
			TransformedUV += UVTranslation;
			UVs->SetElement(AllUVIndices[j], TransformedUV);
		}
	}
}


void FDynamicMeshEditor::RescaleAttributeUVs(float UVScale, bool bWorldSpace, int UVLayerIndex, TOptional<FTransformSRT3d> ToWorld)
{
	check(Mesh->HasAttributes() && Mesh->Attributes()->NumUVLayers() > UVLayerIndex );
	FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

	if (bWorldSpace)
	{
		FVector2f TriUVs[3];
		FVector3d TriVs[3];
		float TotalEdgeUVLen = 0;
		double TotalEdgeLen = 0;
		for (int TID : Mesh->TriangleIndicesItr())
		{
			if (!UVs->IsSetTriangle(TID))
			{
				continue;
			}
			UVs->GetTriElements(TID, TriUVs[0], TriUVs[1], TriUVs[2]);
			Mesh->GetTriVertices(TID, TriVs[0], TriVs[1], TriVs[2]);
			if (ToWorld.IsSet())
			{
				for (int i = 0; i < 3; i++)
				{
					TriVs[i] = ToWorld->TransformPosition(TriVs[i]);
				}
			}
			for (int j = 2, i = 0; i < 3; j = i++)
			{
				TotalEdgeUVLen += Distance(TriUVs[j], TriUVs[i]);
				TotalEdgeLen += Distance(TriVs[j], TriVs[i]);
			}
		}
		if (TotalEdgeUVLen > KINDA_SMALL_NUMBER)
		{
			float AvgUVScale = float (TotalEdgeLen / TotalEdgeUVLen);
			UVScale *= AvgUVScale;
		}
	}

	for (int UVID : UVs->ElementIndicesItr())
	{
		FVector2f UV;
		UVs->GetElement(UVID, UV);
		UVs->SetElement(UVID, UV*UVScale);
	}
}






void FDynamicMeshEditor::ReverseTriangleOrientations(const TArray<int>& Triangles, bool bInvertNormals)
{
	for (int tid : Triangles)
	{
		Mesh->ReverseTriOrientation(tid);
	}
	if (bInvertNormals)
	{
		InvertTriangleNormals(Triangles);
	}
}


void FDynamicMeshEditor::InvertTriangleNormals(const TArray<int>& Triangles)
{
	// @todo re-use the TBitA

	if (Mesh->HasVertexNormals())
	{
		TBitArray<FDefaultBitArrayAllocator> DoneVertices(false, Mesh->MaxVertexID());
		for (int TriangleID : Triangles)
		{
			FIndex3i Tri = Mesh->GetTriangle(TriangleID);
			for (int j = 0; j < 3; ++j)
			{
				if (DoneVertices[Tri[j]] == false)
				{
					Mesh->SetVertexNormal(Tri[j], -Mesh->GetVertexNormal(Tri[j]));
					DoneVertices[Tri[j]] = true;
				}
			}
		}
	}


	if (Mesh->HasAttributes())
	{
		for (int NormalLayerIndex = 0; NormalLayerIndex < Mesh->Attributes()->NumNormalLayers(); NormalLayerIndex++)
		{
			FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->GetNormalLayer(NormalLayerIndex);
			TBitArray<FDefaultBitArrayAllocator> DoneNormals(false, NormalOverlay->MaxElementID());
			for (int TriangleID : Triangles)
			{
				FIndex3i ElemTri = NormalOverlay->GetTriangle(TriangleID);
				for (int j = 0; j < 3; ++j)
				{
					if (NormalOverlay->IsElement(ElemTri[j]) && DoneNormals[ElemTri[j]] == false)
					{
						NormalOverlay->SetElement(ElemTri[j], -NormalOverlay->GetElement(ElemTri[j]));
						DoneNormals[ElemTri[j]] = true;
					}
				}
			}
		}
	}
}



void FDynamicMeshEditor::CopyAttributes(int FromTriangleID, int ToTriangleID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut)
{
	if (Mesh->HasAttributes() == false)
	{
		return;
	}

	
	for (int UVLayerIndex = 0; UVLayerIndex < Mesh->Attributes()->NumUVLayers(); UVLayerIndex++)
	{
		FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->GetUVLayer(UVLayerIndex);
		if (UVOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = UVOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = UVOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				int NewElemID = FindOrCreateDuplicateUV(FromElemTri[j], UVLayerIndex, IndexMaps);
				ToElemTri[j] = NewElemID;
			}
			UVOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}

	// Make sure the storage in NewNormalOverlayElements has a slot for each normal layer.
	if (ResultOut.NewNormalOverlayElements.Num() < Mesh->Attributes()->NumNormalLayers())
	{
		ResultOut.NewNormalOverlayElements.AddDefaulted(Mesh->Attributes()->NumNormalLayers() - ResultOut.NewNormalOverlayElements.Num());
	}

	for (int NormalLayerIndex = 0; NormalLayerIndex < Mesh->Attributes()->NumNormalLayers(); NormalLayerIndex++)
	{
		FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->GetNormalLayer(NormalLayerIndex);

		if (NormalOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = NormalOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = NormalOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				int NewElemID = FindOrCreateDuplicateNormal(FromElemTri[j], NormalLayerIndex, IndexMaps, &ResultOut);
				ToElemTri[j] = NewElemID;
			}
			NormalOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}
	
	if (Mesh->Attributes()->HasPrimaryColors())
	{
		FDynamicMeshColorOverlay* ColorOverlay = Mesh->Attributes()->PrimaryColors();
		if (ColorOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = ColorOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = ColorOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				int NewElemID = FindOrCreateDuplicateColor(FromElemTri[j], IndexMaps, &ResultOut);
				ToElemTri[j] = NewElemID;
			}
			ColorOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}

	if (Mesh->Attributes()->HasMaterialID())
	{
		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh->Attributes()->GetMaterialID();
		MaterialIDs->SetValue(ToTriangleID, MaterialIDs->GetValue(FromTriangleID));
	}

	for (int PolygroupLayerIndex = 0; PolygroupLayerIndex < Mesh->Attributes()->NumPolygroupLayers(); PolygroupLayerIndex++)
	{
		FDynamicMeshPolygroupAttribute* Polygroup = Mesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
		Polygroup->SetValue(ToTriangleID, Polygroup->GetValue(FromTriangleID));
	}

}



int FDynamicMeshEditor::FindOrCreateDuplicateUV(int ElementID, int UVLayerIndex, FMeshIndexMappings& IndexMaps)
{
	int NewElementID = IndexMaps.GetNewUV(UVLayerIndex, ElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->GetUVLayer(UVLayerIndex);
		NewElementID = UVOverlay->AppendElement(UVOverlay->GetElement(ElementID));
		IndexMaps.SetUV(UVLayerIndex, ElementID, NewElementID);
	}
	return NewElementID;
}



int FDynamicMeshEditor::FindOrCreateDuplicateNormal(int ElementID, int NormalLayerIndex, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult* ResultOut)
{
	int NewElementID = IndexMaps.GetNewNormal(NormalLayerIndex, ElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->GetNormalLayer(NormalLayerIndex);
		NewElementID = NormalOverlay->AppendElement(NormalOverlay->GetElement(ElementID));
		IndexMaps.SetNormal(NormalLayerIndex, ElementID, NewElementID);
		if (ResultOut)
		{
			check(ResultOut->NewNormalOverlayElements.Num() > NormalLayerIndex);
			ResultOut->NewNormalOverlayElements[NormalLayerIndex].Add(NewElementID);
		}
	}
	return NewElementID;
}


int FDynamicMeshEditor::FindOrCreateDuplicateColor(int ElementID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult* ResultOut)
{
	int NewElementID = IndexMaps.GetNewColor(ElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		FDynamicMeshColorOverlay* ColorOverlay = Mesh->Attributes()->PrimaryColors();
		NewElementID = ColorOverlay->AppendElement(ColorOverlay->GetElement(ElementID));
		IndexMaps.SetColor(ElementID, NewElementID);
		if (ResultOut)
		{
			ResultOut->NewColorOverlayElements.Add(NewElementID);
		}
	}
	return NewElementID;
}


int FDynamicMeshEditor::FindOrCreateDuplicateVertex(int VertexID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut)
{
	int NewVertexID = IndexMaps.GetNewVertex(VertexID);
	if (NewVertexID == IndexMaps.InvalidID())
	{
		NewVertexID = Mesh->AppendVertex(*Mesh, VertexID);
		IndexMaps.SetVertex(VertexID, NewVertexID);
		ResultOut.NewVertices.Add(NewVertexID);

		if (Mesh->HasAttributes())
		{
			for (int WeightLayerIndex = 0; WeightLayerIndex < Mesh->Attributes()->NumWeightLayers(); ++WeightLayerIndex)
			{
				FDynamicMeshWeightAttribute* WeightAttr = Mesh->Attributes()->GetWeightLayer(WeightLayerIndex);
				float Val;
				WeightAttr->GetValue(VertexID, &Val);
				WeightAttr->SetNewValue(NewVertexID, &Val);
			}
		}
	}
	return NewVertexID;
}



int FDynamicMeshEditor::FindOrCreateDuplicateGroup(int TriangleID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut)
{
	int GroupID = Mesh->GetTriangleGroup(TriangleID);
	int NewGroupID = IndexMaps.GetNewGroup(GroupID);
	if (NewGroupID == IndexMaps.InvalidID())
	{
		NewGroupID = Mesh->AllocateTriangleGroup();
		IndexMaps.SetGroup(GroupID, NewGroupID);
		ResultOut.NewGroups.Add(NewGroupID);
	}
	return NewGroupID;
}




void FDynamicMeshEditor::AppendMesh(const FDynamicMesh3* AppendMesh,
	FMeshIndexMappings& IndexMapsOut, 
	TFunction<FVector3d(int, const FVector3d&)> PositionTransform,
	TFunction<FVector3d(int, const FVector3d&)> NormalTransform,
	bool bReverseOrientation)
{
	// TODO: add a 'fast path' w/ the new Mesh->AppendWithOffsets, and build IndexMapsOut from the offsets
	// (this could always be used w/ identical results if both meshes are compact, and perhaps via a flag in the non-compact case)

	// todo: handle this case by making a copy?
	check(AppendMesh != Mesh);

	IndexMapsOut.Reset();
	IndexMapsOut.Initialize(Mesh);

	FIndexMapi& VertexMap = IndexMapsOut.GetVertexMap();
	VertexMap.Reserve(AppendMesh->VertexCount());
	for (int VertID : AppendMesh->VertexIndicesItr())
	{
		FVector3d Position = AppendMesh->GetVertex(VertID);
		if (PositionTransform != nullptr)
		{
			Position = PositionTransform(VertID, Position);
		}
		int NewVertID = Mesh->AppendVertex(Position);
		VertexMap.Add(VertID, NewVertID);

		if (AppendMesh->HasVertexNormals() && Mesh->HasVertexNormals())
		{
			FVector3f Normal = AppendMesh->GetVertexNormal(VertID);
			if (NormalTransform != nullptr)
			{
				Normal = (FVector3f)NormalTransform(VertID, (FVector3d)Normal);
			}
			Mesh->SetVertexNormal(NewVertID, Normal);
		}

		if (AppendMesh->HasVertexUVs() && Mesh->HasVertexUVs())
		{
			FVector2f UV = AppendMesh->GetVertexUV(VertID);
			Mesh->SetVertexUV(NewVertID, UV);
		}

		if (AppendMesh->HasVertexColors() && Mesh->HasVertexColors())
		{
			FVector3f Color = AppendMesh->GetVertexColor(VertID);
			Mesh->SetVertexColor(NewVertID, Color);
		}
	}

	FIndexMapi& TriangleMap = IndexMapsOut.GetTriangleMap();
	bool bAppendGroups = AppendMesh->HasTriangleGroups() && Mesh->HasTriangleGroups();
	FIndexMapi& GroupsMap = IndexMapsOut.GetGroupMap();
	for (int TriID : AppendMesh->TriangleIndicesItr())
	{
		// append trigroup
		int GroupID = FDynamicMesh3::InvalidID;
		if (bAppendGroups)
		{
			GroupID = AppendMesh->GetTriangleGroup(TriID);
			if (GroupID != FDynamicMesh3::InvalidID)
			{
				const int* FoundNewGroupID = GroupsMap.FindTo(GroupID);
				if (FoundNewGroupID == nullptr)
				{
					int NewGroupID = Mesh->AllocateTriangleGroup();
					GroupsMap.Add(GroupID, NewGroupID);
					GroupID = NewGroupID;
				}
				else
				{
					GroupID = *FoundNewGroupID;
				}
			}
		}

		FIndex3i Tri = AppendMesh->GetTriangle(TriID);
		int NewTriID = Mesh->AppendTriangle(VertexMap.GetTo(Tri.A), VertexMap.GetTo(Tri.B), VertexMap.GetTo(Tri.C), GroupID);
		if (ensure(NewTriID >= 0))
		{
			TriangleMap.Add(TriID, NewTriID);
		}
	}


	// @todo can we have a template fn that does this?

	if (AppendMesh->HasAttributes() && Mesh->HasAttributes())
	{
		int NumNormalLayers = FMath::Min(Mesh->Attributes()->NumNormalLayers(), AppendMesh->Attributes()->NumNormalLayers());
		for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers; NormalLayerIndex++)
		{ 
			const FDynamicMeshNormalOverlay* FromNormals = AppendMesh->Attributes()->GetNormalLayer(NormalLayerIndex);
			FDynamicMeshNormalOverlay* ToNormals = Mesh->Attributes()->GetNormalLayer(NormalLayerIndex);
			if (FromNormals != nullptr && ToNormals != nullptr)
			{
				FIndexMapi& NormalMap = IndexMapsOut.GetNormalMap(NormalLayerIndex);
				NormalMap.Reserve(FromNormals->ElementCount());
				AppendNormals(AppendMesh, FromNormals, ToNormals,
					VertexMap, TriangleMap, NormalTransform, NormalMap);
			}
		}

		int NumUVLayers = FMath::Min(Mesh->Attributes()->NumUVLayers(), AppendMesh->Attributes()->NumUVLayers());
		for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
		{
			const FDynamicMeshUVOverlay* FromUVs = AppendMesh->Attributes()->GetUVLayer(UVLayerIndex);
			FDynamicMeshUVOverlay* ToUVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);
			if (FromUVs != nullptr && ToUVs != nullptr)
			{
				FIndexMapi& UVMap = IndexMapsOut.GetUVMap(UVLayerIndex);
				UVMap.Reserve(FromUVs->ElementCount());
				AppendUVs(AppendMesh, FromUVs, ToUVs,
					VertexMap, TriangleMap, UVMap);
			}
		}

		if (AppendMesh->Attributes()->HasPrimaryColors() && Mesh->Attributes()->HasPrimaryColors())
		{
			const FDynamicMeshColorOverlay* FromColors = AppendMesh->Attributes()->PrimaryColors();
			FDynamicMeshColorOverlay* ToColors = Mesh->Attributes()->PrimaryColors();
			if (FromColors != nullptr && ToColors != nullptr)
			{
				FIndexMapi& ColorMap = IndexMapsOut.GetColorMap();
				ColorMap.Reserve(FromColors->ElementCount());
				AppendColors(AppendMesh, FromColors, ToColors,
					VertexMap, TriangleMap, ColorMap);
			}
		}

		if (AppendMesh->Attributes()->HasMaterialID() && Mesh->Attributes()->HasMaterialID())
		{
			const FDynamicMeshMaterialAttribute* FromMaterialIDs = AppendMesh->Attributes()->GetMaterialID();
			FDynamicMeshMaterialAttribute* ToMaterialIDs = Mesh->Attributes()->GetMaterialID();
			for (const TPair<int32, int32>& MapTID : TriangleMap.GetForwardMap())
			{
				ToMaterialIDs->SetValue(MapTID.Value, FromMaterialIDs->GetValue(MapTID.Key));
			}
		}

		int NumPolygroupLayers = FMath::Min(Mesh->Attributes()->NumPolygroupLayers(), AppendMesh->Attributes()->NumPolygroupLayers());
		for (int PolygroupLayerIndex = 0; PolygroupLayerIndex < NumPolygroupLayers; PolygroupLayerIndex++)
		{
			// TODO: remap groups? this will be somewhat expensive...
			const FDynamicMeshPolygroupAttribute* FromPolygroups = AppendMesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
			FDynamicMeshPolygroupAttribute* ToPolygroups = Mesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
			for (const TPair<int32, int32>& MapTID : TriangleMap.GetForwardMap())
			{
				ToPolygroups->SetValue(MapTID.Value, FromPolygroups->GetValue(MapTID.Key));
			}
		}

		int NumWeightLayers = FMath::Min(Mesh->Attributes()->NumWeightLayers(), AppendMesh->Attributes()->NumWeightLayers());
		for (int WeightLayerIndex = 0; WeightLayerIndex < NumWeightLayers; WeightLayerIndex++)
		{
			const FDynamicMeshWeightAttribute* FromWeights = AppendMesh->Attributes()->GetWeightLayer(WeightLayerIndex);
			FDynamicMeshWeightAttribute* ToWeights = Mesh->Attributes()->GetWeightLayer(WeightLayerIndex);
			for (const TPair<int32, int32>& MapVID : VertexMap.GetForwardMap())
			{
				float Weight;
				FromWeights->GetValue(MapVID.Key, &Weight);
				ToWeights->SetValue(MapVID.Value, &Weight);
			}
		}

		if (AppendMesh->Attributes()->HasBones() && Mesh->Attributes()->HasBones())
		{
			const bool bSameSkeletons = AppendMesh->Attributes()->GetBoneNames()->IsSameAs(*Mesh->Attributes()->GetBoneNames());

			if (!bSameSkeletons)
			{
				Mesh->Attributes()->AppendBonesUnique(*AppendMesh->Attributes());
			}
			
			for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : AppendMesh->Attributes()->GetSkinWeightsAttributes())
			{
				FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = Mesh->Attributes()->GetSkinWeightsAttribute(AttribPair.Key);
				if (ToAttrib)
				{	
					if (bSameSkeletons)
					{
						// If the skeletons are the same then we do not need to re-index and we simply copy
						ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMapsOut);
					}
					else
					{
						// Make a copy of the append mesh skinning weights and reindex them with respect to the new skeleton
						FDynamicMeshVertexSkinWeightsAttribute CopyAppendMeshAttrib;
						CopyAppendMeshAttrib.Copy(*AttribPair.Value.Get());
						CopyAppendMeshAttrib.ReindexBoneIndicesToSkeleton(AppendMesh->Attributes()->GetBoneNames()->GetAttribValues(),
						 											  	  Mesh->Attributes()->GetBoneNames()->GetAttribValues());

						// Now copy re-indexed weights to the mesh we are appending to
						ToAttrib->CopyThroughMapping(&CopyAppendMeshAttrib, IndexMapsOut);
					}
				}
			}
		}
		else
		{
			for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : AppendMesh->Attributes()->GetSkinWeightsAttributes())
			{
				FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = Mesh->Attributes()->GetSkinWeightsAttribute(AttribPair.Key);
				if (ToAttrib)
				{
					ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMapsOut);
				}
			}
		}

		for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : AppendMesh->Attributes()->GetMorphTargetAttributes())
		{
			FDynamicMeshMorphTargetAttribute* ToAttrib = Mesh->Attributes()->GetMorphTargetAttribute(AttribPair.Key);
			if (ToAttrib)
			{
				ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMapsOut);	
			}
		}
		
		for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : AppendMesh->Attributes()->GetAttachedAttributes())
		{
			if (Mesh->Attributes()->HasAttachedAttribute(AttribPair.Key))
			{
				FDynamicMeshAttributeBase* ToAttrib = Mesh->Attributes()->GetAttachedAttribute(AttribPair.Key);
				ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMapsOut);
			}
		}

		if (Mesh->Attributes()->NumSculptLayers() > 0 && AppendMesh->Attributes()->NumSculptLayers() > 0)
		{
			Mesh->Attributes()->GetSculptLayers()->CopyThroughMapping(*AppendMesh->Attributes()->GetSculptLayers(), IndexMapsOut);
		}
	}

	// Flip tri orientations if requested, after all appends -- note we do this after mapping across all the attributes, as that logic assumes triangle vertices have matching vertex order
	if (bReverseOrientation)
	{
		const TMap<int32, int32>& ReverseTriMap = TriangleMap.GetReverseMap();
		for (const TPair<int32, int32> KV : ReverseTriMap)
		{
			int32 NewTID = KV.Key;
			Mesh->ReverseTriOrientation(NewTID);
		}
	}
}




void FDynamicMeshEditor::AppendMesh(const TTriangleMeshAdapter<double>* AppendMesh,
	FMeshIndexMappings& IndexMapsOut, 
	TFunction<FVector3d(int, const FVector3d&)> PositionTransform)
{
	IndexMapsOut.Reset();
	//IndexMapsOut.Initialize(Mesh);		// not supported

	FIndexMapi& VertexMap = IndexMapsOut.GetVertexMap();
	VertexMap.Reserve(AppendMesh->VertexCount());
	int32 MaxVertexID = AppendMesh->MaxVertexID();
	for ( int32 VertID = 0; VertID < MaxVertexID; ++VertID )
	{
		if (AppendMesh->IsVertex(VertID) == false) continue;

		FVector3d Position = AppendMesh->GetVertex(VertID);
		if (PositionTransform != nullptr)
		{
			Position = PositionTransform(VertID, Position);
		}
		int NewVertID = Mesh->AppendVertex(Position);
		VertexMap.Add(VertID, NewVertID);
	}

	// set face normals if they exist
	FDynamicMeshNormalOverlay* SetNormals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;

	FIndexMapi& TriangleMap = IndexMapsOut.GetTriangleMap();
	int32 MaxTriangleID = AppendMesh->MaxTriangleID();
	for (int TriID = 0; TriID < MaxTriangleID; ++TriID )
	{
		if (AppendMesh->IsTriangle(TriID) == false) continue;

		int GroupID = FDynamicMesh3::InvalidID;
		FIndex3i Tri = AppendMesh->GetTriangle(TriID);
		FIndex3i NewTri = FIndex3i(VertexMap.GetTo(Tri.A), VertexMap.GetTo(Tri.B), VertexMap.GetTo(Tri.C));
		int NewTriID = Mesh->AppendTriangle(NewTri.A, NewTri.B, NewTri.C, GroupID);
		TriangleMap.Add(TriID, NewTriID);

		if (SetNormals)
		{
			FVector3f TriNormal(Mesh->GetTriNormal(NewTriID));	//LWC_TODO: Precision loss
			FIndex3i NormalTri;
			for (int32 j = 0; j < 3; ++j)
			{
				NormalTri[j] = SetNormals->AppendElement(TriNormal);
				SetNormals->SetParentVertex(NormalTri[j], NewTri[j]);
			}
			SetNormals->SetTriangle(NewTriID, NormalTri);
		}
	}
}






void FDynamicMeshEditor::AppendNormals(const FDynamicMesh3* AppendMesh, 
	const FDynamicMeshNormalOverlay* FromNormals, FDynamicMeshNormalOverlay* ToNormals,
	const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
	TFunction<FVector3d(int, const FVector3d&)> NormalTransform,
	FIndexMapi& NormalMapOut)
{
	// copy over normals
	for (int ElemID : FromNormals->ElementIndicesItr())
	{
		int ParentVertID = FromNormals->GetParentVertex(ElemID);
		FVector3f Normal = FromNormals->GetElement(ElemID);
		if (NormalTransform != nullptr)
		{
			Normal = (FVector3f)NormalTransform(ParentVertID, (FVector3d)Normal);
		}
		int NewElemID = ToNormals->AppendElement(Normal);
		NormalMapOut.Add(ElemID, NewElemID);
	}

	// now set new triangles
	for (const TPair<int32, int32>& MapTID : TriangleMap.GetForwardMap())
	{
		if (FromNormals->IsSetTriangle(MapTID.Key))
		{
			FIndex3i ElemTri = FromNormals->GetTriangle(MapTID.Key);
			for (int j = 0; j < 3; ++j)
			{
				ElemTri[j] = FromNormals->IsElement(ElemTri[j]) ? NormalMapOut.GetTo(ElemTri[j]) : FDynamicMesh3::InvalidID;
			}
			ToNormals->SetTriangle(MapTID.Value, ElemTri);
		}
	}
}


void FDynamicMeshEditor::AppendUVs(const FDynamicMesh3* AppendMesh,
	const FDynamicMeshUVOverlay* FromUVs, FDynamicMeshUVOverlay* ToUVs,
	const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
	FIndexMapi& UVMapOut)
{
	// copy over uv elements
	for (int ElemID : FromUVs->ElementIndicesItr())
	{
		FVector2f UV = FromUVs->GetElement(ElemID);
		int NewElemID = ToUVs->AppendElement(UV);
		UVMapOut.Add(ElemID, NewElemID);
	}

	// now set new triangles
	for (const TPair<int32, int32>& MapTID : TriangleMap.GetForwardMap())
	{
		if (FromUVs->IsSetTriangle(MapTID.Key))
		{
			FIndex3i ElemTri = FromUVs->GetTriangle(MapTID.Key);
			for (int j = 0; j < 3; ++j)
			{
				ElemTri[j] = FromUVs->IsElement(ElemTri[j]) ? UVMapOut.GetTo(ElemTri[j]) : FDynamicMesh3::InvalidID;
			}
			ToUVs->SetTriangle(MapTID.Value, ElemTri);
		}
	}
}

void FDynamicMeshEditor::AppendColors(const FDynamicMesh3* AppendMesh,
	const FDynamicMeshColorOverlay* FromOverlay, FDynamicMeshColorOverlay* ToOverlay,
	const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
	FIndexMapi& MapOut)
{
	// copy over color elements
	for (int ElemID : FromOverlay->ElementIndicesItr())
	{
		int NewElemID = ToOverlay->AppendElement(FromOverlay->GetElement(ElemID));
		MapOut.Add(ElemID, NewElemID);
	}

	// now set new triangles
	for (const TPair<int32, int32>& MapTID : TriangleMap.GetForwardMap())
	{
		if (FromOverlay->IsSetTriangle(MapTID.Key))
		{
			FIndex3i ElemTri = FromOverlay->GetTriangle(MapTID.Key);
			for (int j = 0; j < 3; ++j)
			{
				ElemTri[j] = FromOverlay->IsElement(ElemTri[j]) ? MapOut.GetTo(ElemTri[j]) : FDynamicMesh3::InvalidID;
			}
			ToOverlay->SetTriangle(MapTID.Value, ElemTri);
		}
	}
}












// can these be replaced w/ template function?


namespace UE
{
namespace DynamicMeshEditorInternals
{


// Utility function for ::AppendTriangles()
static int AppendTriangleUVAttribute(const FDynamicMesh3* FromMesh, int FromElementID, FDynamicMesh3* ToMesh, int UVLayerIndex, FMeshIndexMappings& IndexMaps)
{
	int NewElementID = IndexMaps.GetNewUV(UVLayerIndex, FromElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		const FDynamicMeshUVOverlay* FromUVOverlay = FromMesh->Attributes()->GetUVLayer(UVLayerIndex);
		FDynamicMeshUVOverlay* ToUVOverlay = ToMesh->Attributes()->GetUVLayer(UVLayerIndex);
		NewElementID = ToUVOverlay->AppendElement(FromUVOverlay->GetElement(FromElementID));
		IndexMaps.SetUV(UVLayerIndex, FromElementID, NewElementID);
	}
	return NewElementID;
}


// Utility function for ::AppendTriangles()
static int AppendTriangleNormalAttribute(const FDynamicMesh3* FromMesh, int FromElementID, FDynamicMesh3* ToMesh, int NormalLayerIndex, FMeshIndexMappings& IndexMaps)
{
	int NewElementID = IndexMaps.GetNewNormal(NormalLayerIndex, FromElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		const FDynamicMeshNormalOverlay* FromNormalOverlay = FromMesh->Attributes()->GetNormalLayer(NormalLayerIndex);
		FDynamicMeshNormalOverlay* ToNormalOverlay = ToMesh->Attributes()->GetNormalLayer(NormalLayerIndex);
		NewElementID = ToNormalOverlay->AppendElement(FromNormalOverlay->GetElement(FromElementID));
		IndexMaps.SetNormal(NormalLayerIndex, FromElementID, NewElementID);
	}
	return NewElementID;
}

// Utility function for ::AppendTriangles()
static int AppendTriangleColorAttribute(const FDynamicMesh3* FromMesh, int FromElementID, FDynamicMesh3* ToMesh, FMeshIndexMappings& IndexMaps)
{
	int NewElementID = IndexMaps.GetNewColor(FromElementID);
	if (NewElementID == IndexMaps.InvalidID())
	{
		const FDynamicMeshColorOverlay* FromOverlay = FromMesh->Attributes()->PrimaryColors();
		FDynamicMeshColorOverlay* ToOverlay = ToMesh->Attributes()->PrimaryColors();
		NewElementID = ToOverlay->AppendElement(FromOverlay->GetElement(FromElementID));
		IndexMaps.SetColor(FromElementID, NewElementID);
	}
	return NewElementID;
}



// Utility function for ::AppendTriangles()
static void AppendTriangleAttributes(const FDynamicMesh3* FromMesh, int FromTriangleID, FDynamicMesh3* ToMesh, int ToTriangleID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut)
{
	if (FromMesh->HasAttributes() == false || ToMesh->HasAttributes() == false)
	{
		return;
	}


	for (int UVLayerIndex = 0; UVLayerIndex < FMath::Min(FromMesh->Attributes()->NumUVLayers(), ToMesh->Attributes()->NumUVLayers()); UVLayerIndex++)
	{
		const FDynamicMeshUVOverlay* FromUVOverlay = FromMesh->Attributes()->GetUVLayer(UVLayerIndex);
		FDynamicMeshUVOverlay* ToUVOverlay = ToMesh->Attributes()->GetUVLayer(UVLayerIndex);
		if (FromUVOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = FromUVOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = ToUVOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				check(FromElemTri[j] != FDynamicMesh3::InvalidID);
				int NewElemID = AppendTriangleUVAttribute(FromMesh, FromElemTri[j], ToMesh, UVLayerIndex, IndexMaps);
				ToElemTri[j] = NewElemID;
			}
			ToUVOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}


	for (int NormalLayerIndex = 0; NormalLayerIndex < FMath::Min(FromMesh->Attributes()->NumNormalLayers(), ToMesh->Attributes()->NumNormalLayers()); NormalLayerIndex++)
	{
		const FDynamicMeshNormalOverlay* FromNormalOverlay = FromMesh->Attributes()->GetNormalLayer(NormalLayerIndex);
		FDynamicMeshNormalOverlay* ToNormalOverlay = ToMesh->Attributes()->GetNormalLayer(NormalLayerIndex);
		if (FromNormalOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = FromNormalOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = ToNormalOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				check(FromElemTri[j] != FDynamicMesh3::InvalidID);
				int NewElemID = AppendTriangleNormalAttribute(FromMesh, FromElemTri[j], ToMesh, NormalLayerIndex, IndexMaps);
				ToElemTri[j] = NewElemID;
			}
			ToNormalOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}

	if (FromMesh->Attributes()->HasPrimaryColors() && ToMesh->Attributes()->HasPrimaryColors())
	{
		const FDynamicMeshColorOverlay* FromOverlay = FromMesh->Attributes()->PrimaryColors();
		FDynamicMeshColorOverlay* ToOverlay = ToMesh->Attributes()->PrimaryColors();
		if (FromOverlay->IsSetTriangle(FromTriangleID))
		{
			FIndex3i FromElemTri = FromOverlay->GetTriangle(FromTriangleID);
			FIndex3i ToElemTri = ToOverlay->GetTriangle(ToTriangleID);
			for (int j = 0; j < 3; ++j)
			{
				check(FromElemTri[j] != FDynamicMesh3::InvalidID);
				int NewElemID = AppendTriangleColorAttribute(FromMesh, FromElemTri[j], ToMesh, IndexMaps);
				ToElemTri[j] = NewElemID;
			}
			ToOverlay->SetTriangle(ToTriangleID, ToElemTri);
		}
	}

	if (FromMesh->Attributes()->HasMaterialID() && ToMesh->Attributes()->HasMaterialID())
	{
		const FDynamicMeshMaterialAttribute* FromMaterialIDs = FromMesh->Attributes()->GetMaterialID();
		FDynamicMeshMaterialAttribute* ToMaterialIDs = ToMesh->Attributes()->GetMaterialID();
		ToMaterialIDs->SetValue(ToTriangleID, FromMaterialIDs->GetValue(FromTriangleID));
	}

	int NumPolygroupLayers = FMath::Min(FromMesh->Attributes()->NumPolygroupLayers(), ToMesh->Attributes()->NumPolygroupLayers());
	for (int PolygroupLayerIndex = 0; PolygroupLayerIndex < NumPolygroupLayers; PolygroupLayerIndex++)
	{
		// TODO: remap groups? this will be somewhat expensive...
		const FDynamicMeshPolygroupAttribute* FromPolygroups = FromMesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
		FDynamicMeshPolygroupAttribute* ToPolygroups = ToMesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
		ToPolygroups->SetValue(ToTriangleID, FromPolygroups->GetValue(FromTriangleID));
	}
}



// Utility function for ::AppendTriangles()
static void AppendVertexAttributes(const FDynamicMesh3* FromMesh, FDynamicMesh3* ToMesh, FMeshIndexMappings& IndexMaps)
{

	if (FromMesh->HasAttributes() == false || ToMesh->HasAttributes() == false)
	{
		return;
	}

	int NumWeightLayers = FMath::Min(FromMesh->Attributes()->NumWeightLayers(), ToMesh->Attributes()->NumWeightLayers());
	for (int WeightLayerIndex = 0; WeightLayerIndex < NumWeightLayers; WeightLayerIndex++)
	{
		const FDynamicMeshWeightAttribute* FromWeights = FromMesh->Attributes()->GetWeightLayer(WeightLayerIndex);
		FDynamicMeshWeightAttribute* ToWeights = ToMesh->Attributes()->GetWeightLayer(WeightLayerIndex);
		for (const TPair<int32, int32>& MapVID : IndexMaps.GetVertexMap().GetForwardMap())
		{
			float Weight;
			FromWeights->GetValue(MapVID.Key, &Weight);
			ToWeights->SetValue(MapVID.Value, &Weight);
		}
	}

	// Copy skin weight and generic attributes after full IndexMaps have been created. 	
	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttribPair : FromMesh->Attributes()->GetSkinWeightsAttributes())
	{
		FDynamicMeshVertexSkinWeightsAttribute* ToAttrib = ToMesh->Attributes()->GetSkinWeightsAttribute(AttribPair.Key);
		if (ToAttrib)
		{
			ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMaps);
		}
	}

	for (const TPair<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttribPair : FromMesh->Attributes()->GetMorphTargetAttributes())
	{
		FDynamicMeshMorphTargetAttribute* ToAttrib = ToMesh->Attributes()->GetMorphTargetAttribute(AttribPair.Key);
		if (ToAttrib)
		{
			ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMaps);
		}
	}
	
	for (const TPair<FName, TUniquePtr<FDynamicMeshAttributeBase>>& AttribPair : FromMesh->Attributes()->GetAttachedAttributes())
	{
		if (ToMesh->Attributes()->HasAttachedAttribute(AttribPair.Key))
		{
			FDynamicMeshAttributeBase* ToAttrib = ToMesh->Attributes()->GetAttachedAttribute(AttribPair.Key);
			ToAttrib->CopyThroughMapping(AttribPair.Value.Get(), IndexMaps);
		}
	}

	if (FromMesh->Attributes()->NumSculptLayers() > 0 && ToMesh->Attributes()->NumSculptLayers() > 0)
	{
		ToMesh->Attributes()->GetSculptLayers()->CopyThroughMapping(*FromMesh->Attributes()->GetSculptLayers(), IndexMaps);
	}
}



}} // namespace UE::DynamicMeshEditorInternals

void FDynamicMeshEditor::AppendTriangles(const FDynamicMesh3* SourceMesh, const TArrayView<const int>& SourceTriangles, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut, bool bComputeTriangleMap)
{
	using namespace UE::DynamicMeshEditorInternals;

	ResultOut.Reset();
	IndexMaps.Initialize(Mesh);

	int32 NumTriangles = 0;
	for (int SourceTriangleID : SourceTriangles)
	{
		if (SourceMesh->IsTriangle(SourceTriangleID))
		{
			NumTriangles++;
		}
	}
	check(NumTriangles == SourceTriangles.Num());
	
	if (NumTriangles != SourceTriangles.Num())
	{
		return;
	}

	// reserve data
	{
		if (SourceMesh->HasAttributes() && Mesh->HasAttributes())
		{
			const int32 MaxNumTriangles = FMath::Max(Mesh->TriangleCount() + NumTriangles, Mesh->MaxTriangleID());
			
			const FDynamicMeshAttributeSet* SrcAttributes = SourceMesh->Attributes();
			FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
		
			IndexMaps.GetTriangleMap().Reserve(NumTriangles);
			ResultOut.NewTriangles.Reserve(NumTriangles);
			
			const int NumUVLayers = FMath::Min(SrcAttributes->NumUVLayers(), Attributes->NumUVLayers());
			for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
			{
				IndexMaps.GetUVMap(UVLayerIndex).Reserve(NumTriangles);
				Attributes->GetUVLayer(UVLayerIndex)->InitializeTriangles(MaxNumTriangles);
			}

			const int NumNormalLayers = FMath::Min(SrcAttributes->NumNormalLayers(), Attributes->NumNormalLayers());
			for (int NormalLayerIndex = 0; NormalLayerIndex < NumNormalLayers; NormalLayerIndex++)
			{
				IndexMaps.GetNormalMap(NormalLayerIndex).Reserve(NumTriangles);
				Attributes->GetNormalLayer(NormalLayerIndex)->InitializeTriangles(MaxNumTriangles);
			}

			if (Attributes->HasPrimaryColors())
			{
				IndexMaps.GetColorMap().Reserve(NumTriangles);
				Attributes->PrimaryColors()->InitializeTriangles(MaxNumTriangles);
			}
		}
	}

	const bool bHasTriangleGroups = Mesh->HasTriangleGroups();
	const bool bSourceHasTriangleGroups = SourceMesh->HasTriangleGroups();
	
	int DefaultGroupID = FDynamicMesh3::InvalidID;
	for (int SourceTriangleID : SourceTriangles)
	{
		// FindOrCreateDuplicateGroup
		int NewGroupID = FDynamicMesh3::InvalidID;
		if (bHasTriangleGroups)
		{
			if (bSourceHasTriangleGroups)
			{
				int SourceGroupID = SourceMesh->GetTriangleGroup(SourceTriangleID);
				if (SourceGroupID >= 0)
				{
					NewGroupID = IndexMaps.GetNewGroup(SourceGroupID);
					if (NewGroupID == IndexMaps.InvalidID())
					{
						NewGroupID = Mesh->AllocateTriangleGroup();
						IndexMaps.SetGroup(SourceGroupID, NewGroupID);
						ResultOut.NewGroups.Add(NewGroupID);
					}
				}
			}
			else
			{
				// If the source mesh does not have triangle groups, but the destination
				// mesh does, create a default group for all triangles.
				if (DefaultGroupID == FDynamicMesh3::InvalidID)
				{
					DefaultGroupID = Mesh->AllocateTriangleGroup();
					ResultOut.NewGroups.Add(DefaultGroupID);
				}
				NewGroupID = DefaultGroupID;
			}
		}

		// FindOrCreateDuplicateVertex
		FIndex3i NewTri;
		const FIndex3i& Tri = SourceMesh->GetTriangleRef(SourceTriangleID);
		for (int j = 0; j < 3; ++j)
		{
			int SourceVertexID = Tri[j];
			int NewVertexID = IndexMaps.GetNewVertex(SourceVertexID);
			if (NewVertexID == IndexMaps.InvalidID())
			{
				NewVertexID = Mesh->AppendVertex(*SourceMesh, SourceVertexID);
				IndexMaps.SetVertex(SourceVertexID, NewVertexID);
				ResultOut.NewVertices.Add(NewVertexID);
			}
			NewTri[j] = NewVertexID;
		}

		int NewTriangleID = Mesh->AppendTriangle(NewTri, NewGroupID);
		if (bComputeTriangleMap)
		{
			IndexMaps.SetTriangle(SourceTriangleID, NewTriangleID);
		}
		ResultOut.NewTriangles.Add(NewTriangleID);

		AppendTriangleAttributes(SourceMesh, SourceTriangleID, Mesh, NewTriangleID, IndexMaps, ResultOut);

		//Mesh->CheckValidity(true);
	}
 
	AppendVertexAttributes(SourceMesh, Mesh, IndexMaps);
}

// Note: This is the legacy / direct TArray<FDynamicMesh3> version of SplitMesh
bool FDynamicMeshEditor::SplitMesh(const FDynamicMesh3* SourceMesh, TArray<FDynamicMesh3>& SplitMeshes, TFunctionRef<int(int)> TriIDToMeshID, int DeleteMeshID, TArray<int32>* MeshIDsOut, bool bSortByID)
{
	using namespace UE::DynamicMeshEditorInternals;

	TMap<int, int> MeshIDToIndex;
	int NumMeshes = 0;
	bool bAlsoDelete = false;
	for (int TID : SourceMesh->TriangleIndicesItr())
	{
		int MeshID = TriIDToMeshID(TID);
		if (MeshID == DeleteMeshID)
		{
			bAlsoDelete = true;
			continue;
		}
		if (!MeshIDToIndex.Contains(MeshID))
		{
			MeshIDToIndex.Add(MeshID, NumMeshes++);
		}
	}

	if (!bAlsoDelete && NumMeshes < 2)
	{
		return false; // nothing to do, so don't bother filling the split meshes array
	}

	if (bSortByID)
	{
		TArray<int32> SortedMeshIDs;
		MeshIDToIndex.GenerateKeyArray(SortedMeshIDs);
		SortedMeshIDs.Sort();
		for (int32 Idx = 0; Idx < SortedMeshIDs.Num(); ++Idx)
		{
			MeshIDToIndex[SortedMeshIDs[Idx]] = Idx;
		}
	}

	if (MeshIDsOut)
	{
		MeshIDsOut->SetNum(NumMeshes);
		for (TPair<int32, int32> MeshIDIndexPair : MeshIDToIndex)
		{
			(*MeshIDsOut)[MeshIDIndexPair.Value] = MeshIDIndexPair.Key;
		}
	}

	SplitMeshes.Reset();
	SplitMeshes.SetNum(NumMeshes);
	// enable matching attributes
	for (FDynamicMesh3& M : SplitMeshes)
	{
		M.EnableMeshComponents(SourceMesh->GetComponentsFlags());
		if (SourceMesh->HasAttributes())
		{
			M.EnableAttributes();
			M.Attributes()->EnableMatchingAttributes(*SourceMesh->Attributes());
		}
	}

	if (NumMeshes == 0) // full delete case, just leave the empty SplitMeshes array
	{
		return true;
	}

	TArray<FMeshIndexMappings> Mappings; Mappings.Reserve(NumMeshes);
	FDynamicMeshEditResult UnusedInvalidResultAccumulator; // only here because some functions require it
	for (int Idx = 0; Idx < NumMeshes; Idx++)
	{
		FMeshIndexMappings& Map = Mappings.Emplace_GetRef();
		Map.Initialize(&SplitMeshes[Idx]);
	}

	for (int SourceTID : SourceMesh->TriangleIndicesItr())
	{
		int MeshID = TriIDToMeshID(SourceTID);
		if (MeshID == DeleteMeshID)
		{
			continue; // just skip triangles w/ the Delete Mesh ID
		}
		int MeshIndex = MeshIDToIndex[MeshID];
		FDynamicMesh3& Mesh = SplitMeshes[MeshIndex];
		FMeshIndexMappings& IndexMaps = Mappings[MeshIndex];

		FIndex3i Tri = SourceMesh->GetTriangle(SourceTID);

		// Find or create corresponding triangle group
		int NewGID = FDynamicMesh3::InvalidID;
		if (SourceMesh->HasTriangleGroups())
		{
			int SourceGroupID = SourceMesh->GetTriangleGroup(SourceTID);
			if (SourceGroupID >= 0)
			{
				NewGID = IndexMaps.GetNewGroup(SourceGroupID);
				if (NewGID == IndexMaps.InvalidID())
				{
					NewGID = Mesh.AllocateTriangleGroup();
					IndexMaps.SetGroup(SourceGroupID, NewGID);
				}
			}
		}

		bool bCreatedNewVertex[3] = {false, false, false};
		FIndex3i NewTri;
		for (int j = 0; j < 3; ++j)
		{
			int SourceVID = Tri[j];
			int NewVID = IndexMaps.GetNewVertex(SourceVID);
			if (NewVID == IndexMaps.InvalidID())
			{
				bCreatedNewVertex[j] = true;
				NewVID = Mesh.AppendVertex(*SourceMesh, SourceVID);
				IndexMaps.SetVertex(SourceVID, NewVID);
			}
			NewTri[j] = NewVID;
		}

		int NewTID = Mesh.AppendTriangle(NewTri, NewGID);

		// conceivably this should never happen, but it did occur due to other mesh issues,
		// and it can be handled here without much effort
		if (NewTID < 0)
		{
			// append failed, try creating separate new vertices
			for (int j = 0; j < 3; ++j)
			{
				if ( bCreatedNewVertex[j] == false )
				{
					int SourceVID = Tri[j];
					NewTri[j] = Mesh.AppendVertex(*SourceMesh, SourceVID);
				}
			}
			NewTID = Mesh.AppendTriangle(NewTri, NewGID);
		}

		if ( NewTID >= 0 )
		{
			IndexMaps.SetTriangle(SourceTID, NewTID);
			AppendTriangleAttributes(SourceMesh, SourceTID, &Mesh, NewTID, IndexMaps, UnusedInvalidResultAccumulator);
		}
		else
		{
			checkSlow(false);
			// something has gone very wrong, skip this triangle
		}
	}

	for (int Idx = 0; Idx < NumMeshes; Idx++)
	{
		AppendVertexAttributes(SourceMesh, &SplitMeshes[Idx], Mappings[Idx]);
	}
	
	return true;
}

bool FDynamicMeshEditor::SplitMesh(const FDynamicMesh3* SourceMesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes, bool bSplitIfSingle, TFunctionRef<int(int)> TriIDToMeshID, int DeleteMeshID, TArray<int32>* MeshIDsOut, bool bSortByID)
{
	using namespace UE::DynamicMeshEditorInternals;

	TMap<int, int> MeshIDToIndex;
	int NumMeshes = 0;
	bool bAlsoDelete = false;
	for (int TID : SourceMesh->TriangleIndicesItr())
	{
		int MeshID = TriIDToMeshID(TID);
		if (MeshID == DeleteMeshID)
		{
			bAlsoDelete = true;
			continue;
		}
		if (!MeshIDToIndex.Contains(MeshID))
		{
			MeshIDToIndex.Add(MeshID, NumMeshes++);
		}
	}

	if (!bAlsoDelete && NumMeshes < 2 && !bSplitIfSingle)
	{
		return false; // nothing to do, so don't bother filling the split meshes array
	}

	if (bSortByID)
	{
		TArray<int32> SortedMeshIDs;
		MeshIDToIndex.GenerateKeyArray(SortedMeshIDs);
		SortedMeshIDs.Sort();
		for (int32 Idx = 0; Idx < SortedMeshIDs.Num(); ++Idx)
		{
			MeshIDToIndex[SortedMeshIDs[Idx]] = Idx;
		}
	}

	if (MeshIDsOut)
	{
		MeshIDsOut->SetNum(NumMeshes);
		for (TPair<int32, int32> MeshIDIndexPair : MeshIDToIndex)
		{
			(*MeshIDsOut)[MeshIDIndexPair.Value] = MeshIDIndexPair.Key;
		}
	}

	SplitMeshes.Reset();
	SplitMeshes.SetNum(NumMeshes);

	if (!bAlsoDelete && NumMeshes == 1)
	{
		SplitMeshes[0] = MakeUnique<FDynamicMesh3>(*SourceMesh);
		return true;
	}

	// enable matching attributes
	for (int32 Idx = 0; Idx < SplitMeshes.Num(); ++Idx)
	{
		SplitMeshes[Idx] = MakeUnique<FDynamicMesh3>();
		FDynamicMesh3& M = *SplitMeshes[Idx];
		M.EnableMeshComponents(SourceMesh->GetComponentsFlags());
		if (SourceMesh->HasAttributes())
		{
			M.EnableAttributes();
			M.Attributes()->EnableMatchingAttributes(*SourceMesh->Attributes());
		}
	}

	if (NumMeshes == 0) // full delete case, just leave the empty SplitMeshes array
	{
		return true;
	}

	TArray<FMeshIndexMappings> Mappings; Mappings.Reserve(NumMeshes);
	FDynamicMeshEditResult UnusedInvalidResultAccumulator; // only here because some functions require it
	for (int Idx = 0; Idx < NumMeshes; Idx++)
	{
		FMeshIndexMappings& Map = Mappings.Emplace_GetRef();
		Map.Initialize(SplitMeshes[Idx].Get());
	}

	for (int SourceTID : SourceMesh->TriangleIndicesItr())
	{
		int MeshID = TriIDToMeshID(SourceTID);
		if (MeshID == DeleteMeshID)
		{
			continue; // just skip triangles w/ the Delete Mesh ID
		}
		int MeshIndex = MeshIDToIndex[MeshID];
		FDynamicMesh3& Mesh = *SplitMeshes[MeshIndex];
		FMeshIndexMappings& IndexMaps = Mappings[MeshIndex];

		FIndex3i Tri = SourceMesh->GetTriangle(SourceTID);

		// Find or create corresponding triangle group
		int NewGID = FDynamicMesh3::InvalidID;
		if (SourceMesh->HasTriangleGroups())
		{
			int SourceGroupID = SourceMesh->GetTriangleGroup(SourceTID);
			if (SourceGroupID >= 0)
			{
				NewGID = IndexMaps.GetNewGroup(SourceGroupID);
				if (NewGID == IndexMaps.InvalidID())
				{
					NewGID = Mesh.AllocateTriangleGroup();
					IndexMaps.SetGroup(SourceGroupID, NewGID);
				}
			}
		}

		bool bCreatedNewVertex[3] = { false, false, false };
		FIndex3i NewTri;
		for (int j = 0; j < 3; ++j)
		{
			int SourceVID = Tri[j];
			int NewVID = IndexMaps.GetNewVertex(SourceVID);
			if (NewVID == IndexMaps.InvalidID())
			{
				bCreatedNewVertex[j] = true;
				NewVID = Mesh.AppendVertex(*SourceMesh, SourceVID);
				IndexMaps.SetVertex(SourceVID, NewVID);
			}
			NewTri[j] = NewVID;
		}

		int NewTID = Mesh.AppendTriangle(NewTri, NewGID);

		// conceivably this should never happen, but it did occur due to other mesh issues,
		// and it can be handled here without much effort
		if (NewTID < 0)
		{
			// append failed, try creating separate new vertices
			for (int j = 0; j < 3; ++j)
			{
				if (bCreatedNewVertex[j] == false)
				{
					int SourceVID = Tri[j];
					NewTri[j] = Mesh.AppendVertex(*SourceMesh, SourceVID);
				}
			}
			NewTID = Mesh.AppendTriangle(NewTri, NewGID);
		}

		if (NewTID >= 0)
		{
			IndexMaps.SetTriangle(SourceTID, NewTID);
			AppendTriangleAttributes(SourceMesh, SourceTID, &Mesh, NewTID, IndexMaps, UnusedInvalidResultAccumulator);
		}
		else
		{
			checkSlow(false);
			// something has gone very wrong, skip this triangle
		}
	}

	for (int Idx = 0; Idx < NumMeshes; Idx++)
	{
		AppendVertexAttributes(SourceMesh, SplitMeshes[Idx].Get(), Mappings[Idx]);
	}

	return true;
}

template <typename RealType, int ElementSize>
void FDynamicMeshEditor::AppendElementSubset(
	const FDynamicMesh3* FromMesh,
	const TSet<int>& TriangleROI,
	const TSet<int>& VertexROI,
	const TDynamicMeshOverlay<RealType, ElementSize>* FromOverlay,
	TDynamicMeshOverlay<RealType, ElementSize>* ToOverlay)
{
	TMap<int32, int32> FromElementIDToElementID;
	for (int32 Tid : TriangleROI) {

		const FIndex3i TriVids = Mesh->GetTriangle(Tid);
		ensureMsgf(TriVids == FromMesh->GetTriangle(Tid),
			TEXT("Expected FromOverlay and ToOverlay to have a matching parent meshes"));

		const FIndex3i FromElementIDs = FromOverlay->GetTriangle(Tid);
		FIndex3i ToElementIDs = ToOverlay->GetTriangle(Tid);

		for (int SubIdx = 0; SubIdx < 3; ++SubIdx){

			const int FromElementID = FromElementIDs[SubIdx];

			if (VertexROI.Contains(TriVids[SubIdx])){

				int* ToElementID = FromElementIDToElementID.Find(FromElementID);
				if (!ToElementID)
				{
					for (int Index = 0; Index < ElementSize; Index++)
					{
						RealType FromData[ElementSize];
						FromOverlay->GetElement(FromElementID, FromData);
						const int ElementID = ToOverlay->AppendElement(FromData);

						ToElementID = &FromElementIDToElementID.Add(FromElementID, ElementID);
					}
				}

				ToElementIDs[SubIdx] = *ToElementID;
			}
		}

		ToOverlay->SetTriangle(Tid, ToElementIDs);
	}
}

bool FDynamicMeshEditor::RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams, TDynamicMeshOverlay<float, 2>* Overlay)
{
	return UE::Private::OverlaySeamHelpers::RemoveSeamsAtEdges<float, 2>(EidsToRemoveAsSeams, Overlay->GetParentMesh(), Overlay);
}
bool FDynamicMeshEditor::RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams, TDynamicMeshOverlay<float, 3>* Overlay)
{
	return UE::Private::OverlaySeamHelpers::RemoveSeamsAtEdges<float, 3>(EidsToRemoveAsSeams, Overlay->GetParentMesh(), Overlay);
}
bool FDynamicMeshEditor::CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, TDynamicMeshOverlay<float, 2>* Overlay, TArray<int32>* AddedElementIDs)
{
	return UE::Private::OverlaySeamHelpers::CreateSeamsAtEdges<float, 2>(EidsToMakeIntoSeams, Overlay->GetParentMesh(), Overlay, AddedElementIDs);
}
bool FDynamicMeshEditor::CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, TDynamicMeshOverlay<float, 3>* Overlay, TArray<int32>* AddedElementIDs)
{
	return UE::Private::OverlaySeamHelpers::CreateSeamsAtEdges<float, 3>(EidsToMakeIntoSeams, Overlay->GetParentMesh(), Overlay, AddedElementIDs);
}

template GEOMETRYCORE_API void FDynamicMeshEditor::AppendElementSubset(
	const FDynamicMesh3*,
	const TSet<int>&,
	const TSet<int>&,
	const TDynamicMeshOverlay<float, 3>*,
	TDynamicMeshOverlay<float, 3>*);
