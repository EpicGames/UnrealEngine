// Copyright Epic Games, Inc. All Rights Reserved.


#include "EdgeLoop.h"

using namespace UE::Geometry;

void FEdgeLoop::Initialize(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges, const TArray<int>* BowtieVerticesIn)
{
	Mesh = mesh;
	Vertices = vertices;
	Edges = edges;
	if (BowtieVerticesIn != nullptr)
	{
		BowtieVertices = *BowtieVerticesIn;
		bBowtiesCalculated = true;
	}
}


void FEdgeLoop::InitializeFromEdges(const TArray<int>& EdgesIn)
{
	check(Mesh != nullptr);
	Edges = EdgesIn;

	int NumEdges = Edges.Num();
	Vertices.SetNum(NumEdges);

	FIndex2i start_ev = Mesh->GetEdgeV(Edges[0]);
	FIndex2i prev_ev = start_ev;
	for (int i = 1; i < NumEdges; ++i)
	{
		FIndex2i next_ev = Mesh->GetEdgeV(Edges[i]);
		Vertices[i] = IndexUtil::FindSharedEdgeVertex(prev_ev, next_ev);
		prev_ev = next_ev;
	}
	Vertices[0] = IndexUtil::FindEdgeOtherVertex(start_ev, Vertices[1]);
}


bool FEdgeLoop::InitializeFromVertices(const TArray<int>& VerticesIn, bool bAutoOrient)
{
	check(Mesh != nullptr);
	Vertices = VerticesIn;

	int NumVertices = Vertices.Num();
	Edges.SetNum(NumVertices);
	for (int i = 0; i < NumVertices; ++i)
	{
		int a = Vertices[i], b = Vertices[(i + 1) % NumVertices];
		Edges[i] = Mesh->FindEdge(a, b);
		if (Edges[i] == FDynamicMesh3::InvalidID)
		{
			ensureMsgf(false, TEXT("EdgeLoop.FromVertices: invalid edge [%d,%d]"), a, b);
			return false;
		}
	}

	if (bAutoOrient)
	{
		SetCorrectOrientation();
	}

	return true;
}


void FEdgeLoop::CalculateBowtieVertices()
{
	BowtieVertices.Reset();
	int NumVertices = Vertices.Num();
	for (int i = 0; i < NumVertices; ++i)
	{
		if (Mesh->IsBowtieVertex(Vertices[i]))
		{
			BowtieVertices.Add(Vertices[i]);
		}
	}
	bBowtiesCalculated = true;
}


FAxisAlignedBox3d FEdgeLoop::GetBounds() const
{
	FAxisAlignedBox3d box = FAxisAlignedBox3d::Empty();
	int NumV = Vertices.Num();
	for (int i = 0; i < NumV; ++i)
	{
		box.Contain(Mesh->GetVertex(Vertices[i]));
	}
	return box;
}


bool FEdgeLoop::SetCorrectOrientation()
{
	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		int eid = Edges[i];
		if (Mesh->IsBoundaryEdge(eid))
		{
			int a = Vertices[i], b = Vertices[(i + 1) % NumEdges];
			FIndex2i ev = Mesh->GetOrientedBoundaryEdgeV(eid);
			if (ev.A == b && ev.B == a)
			{
				Reverse();
				return true;
			}
			else
			{
				return false;
			}
		}
	}
	return false;
}


bool FEdgeLoop::IsInternalLoop() const
{
	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		if (Mesh->IsBoundaryEdge(Edges[i]))
		{
			return false;
		}
	}
	return true;
}


bool FEdgeLoop::IsBoundaryLoop(const FDynamicMesh3* TestMesh) const
{
	const FDynamicMesh3* UseMesh = (TestMesh != nullptr) ? TestMesh : Mesh;

	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		if (UseMesh->IsBoundaryEdge(Edges[i]) == false)
		{
			return false;
		}
	}
	return true;
}


int FEdgeLoop::FindVertexIndex(int VertexID) const
{
	int N = Vertices.Num();
	for (int i = 0; i < N; ++i)
	{
		if (Vertices[i] == VertexID)
		{
			return i;
		}
	}
	return -1;
}


int FEdgeLoop::FindNearestVertexIndex(const FVector3d& QueryPoint) const
{
	int iNear = -1;
	double fNearSqr = TNumericLimits<double>::Max();
	int N = Vertices.Num();
	for (int i = 0; i < N; ++i)
	{
		FVector3d lv = Mesh->GetVertex(Vertices[i]);
		double d2 = DistanceSquared(QueryPoint, lv);
		if (d2 < fNearSqr)
		{
			fNearSqr = d2;
			iNear = i;
		}
	}
	return iNear;
}




bool FEdgeLoop::CheckValidity(EValidityCheckFailMode FailMode) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FEdgeLoop::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FEdgeLoop::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}


	CheckOrFailF(Vertices.Num() == Edges.Num());
	for (int ei = 0; ei < Edges.Num(); ++ei)
	{
		FIndex2i ev = Mesh->GetEdgeV(Edges[ei]);
		CheckOrFailF(Mesh->IsVertex(ev.A));
		CheckOrFailF(Mesh->IsVertex(ev.B));
		CheckOrFailF(Mesh->FindEdge(ev.A, ev.B) != FDynamicMesh3::InvalidID);
		CheckOrFailF(Vertices[ei] == ev.A || Vertices[ei] == ev.B);
		CheckOrFailF(Vertices[(ei + 1) % Edges.Num()] == ev.A || Vertices[(ei + 1) % Edges.Num()] == ev.B);
	}
	for (int vi = 0; vi < Vertices.Num(); ++vi)
	{
		int a = Vertices[vi], b = Vertices[(vi + 1) % Vertices.Num()];
		CheckOrFailF(Mesh->IsVertex(a));
		CheckOrFailF(Mesh->IsVertex(b));
		CheckOrFailF(Mesh->FindEdge(a, b) != FDynamicMesh3::InvalidID);
		int n = 0, edge_before_b = Edges[vi], edge_after_b = Edges[(vi + 1) % Vertices.Num()];
		for (int nbr_e : Mesh->VtxEdgesItr(b))
		{
			if (nbr_e == edge_before_b || nbr_e == edge_after_b)
			{
				n++;
			}
		}
		CheckOrFailF(n == 2);
	}
	return is_ok;
}



void FEdgeLoop::VertexLoopToEdgeLoop(const FDynamicMesh3* Mesh, const TArray<int>& VertexLoop, TArray<int>& OutEdgeLoop)
{
	// @todo this function should be in a utility class?

	int NV = VertexLoop.Num();
	OutEdgeLoop.SetNum(NV);
	for (int i = 0; i < NV; ++i)
	{
		int v0 = VertexLoop[i];
		int v1 = VertexLoop[(i + 1) % NV];
		OutEdgeLoop[i] = Mesh->FindEdge(v0, v1);
	}
}


bool UE::Geometry::ConvertLoopToTriOrderedEdgeLoop(const FDynamicMesh3& Mesh, const TArray<int32>& VertexLoop,
	const TArray<int32>& EdgeLoop, TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut)
{
	return ConvertLoopToTriOrderedEdgeLoop(Mesh, VertexLoop, EdgeLoop, [](int, int32 A, int32 B) { return A;}, TriOrderedEdgesLoopOut);
}


bool UE::Geometry::ConvertLoopToTriOrderedEdgeLoop(const FDynamicMesh3& Mesh, const TArray<int32>& VertexLoop,
	const TArray<int32>& EdgeLoop, 
	TFunctionRef<int(int, int,int)> SelectEdgeTriangleFunc,
	TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut )
{
	if (!ensure(EdgeLoop.Num() == VertexLoop.Num()))
	{
		return false;
	}

	TriOrderedEdgesLoopOut.Reset();
	TriOrderedEdgesLoopOut.Reserve(EdgeLoop.Num());
	for (int32 LoopIndex = 0; LoopIndex < EdgeLoop.Num(); ++LoopIndex)
	{
		FIndex2i EdgeT = Mesh.GetEdgeT(EdgeLoop[LoopIndex]);
		int32 TriangleID = SelectEdgeTriangleFunc(EdgeLoop[LoopIndex], EdgeT.A, EdgeT.B);
		int32 FirstVertexID = VertexLoop[LoopIndex];
		int32 SecondVertexID = VertexLoop[(LoopIndex + 1) % VertexLoop.Num()];

		FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
		int32 FirstVertexIndex = (int32)IndexUtil::FindTriIndex(FirstVertexID, Triangle);
		int32 SecondVertexIndex = (int32)IndexUtil::FindTriIndex(SecondVertexID, Triangle);
		if (ensure(FirstVertexIndex >= 0 && SecondVertexIndex >= 0))
		{
			TriOrderedEdgesLoopOut.Emplace( FMeshTriOrderedEdgeID(TriangleID, FirstVertexIndex, SecondVertexIndex) );
		}
		else
		{
			return false;
		}
	}
	return true;
}


bool UE::Geometry::ConvertTriOrderedEdgeLoopToLoop(const FDynamicMesh3& Mesh,
	const TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut,
	TArray<int32>& VertexLoop, TArray<int32>* EdgeLoop)
{
	int32 N = TriOrderedEdgesLoopOut.Num();
	VertexLoop.Reset();
	VertexLoop.Reserve(N-1);

	FIndex3i FirstTri = Mesh.GetTriangle(TriOrderedEdgesLoopOut[0].TriangleID);
	VertexLoop.Add(FirstTri[TriOrderedEdgesLoopOut[0].VertIndexA]);
	VertexLoop.Add(FirstTri[TriOrderedEdgesLoopOut[0].VertIndexB]);
	for (int32 k = 1; k < N-1; ++k)
	{
		FIndex3i Tri = Mesh.GetTriangle(TriOrderedEdgesLoopOut[k].TriangleID);
		check( Tri[TriOrderedEdgesLoopOut[k].VertIndexA] == VertexLoop.Last() );
		VertexLoop.Add( Tri[TriOrderedEdgesLoopOut[k].VertIndexB] );
	}
	if ( EdgeLoop != nullptr )
	{
		EdgeLoop->Reset();
		EdgeLoop->Reserve(N);
		for (int32 k = 0; k < N; ++k)
		{
			int32 EdgeID = Mesh.FindEdgeFromTri(VertexLoop[k], VertexLoop[(k+1)%N], TriOrderedEdgesLoopOut[k].TriangleID);
			EdgeLoop->Add(EdgeID);
		}
	}
	return true;
}

