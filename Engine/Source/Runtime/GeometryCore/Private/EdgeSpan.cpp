// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdgeSpan.h"

using namespace UE::Geometry;

void FEdgeSpan::Initialize(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges, const TArray<int>* BowtieVerticesIn)
{
	Mesh = mesh;
	Vertices = vertices;
	Edges = edges;
	if (BowtieVerticesIn != nullptr)
	{
		BowtieVertices = *BowtieVerticesIn;
		bBowtiesCalculated = true;
	}
}



void FEdgeSpan::InitializeFromEdges(const TArray<int>& EdgesIn)
{
	check(Mesh != nullptr);
	Edges = EdgesIn;

	int NumEdges = Edges.Num();
	Vertices.SetNum(NumEdges + 1);

	FIndex2i start_ev = Mesh->GetEdgeV(Edges[0]);
	FIndex2i prev_ev = start_ev;
	if (NumEdges > 1)
	{
		for (int i = 1; i < Edges.Num(); ++i)
		{
			FIndex2i next_ev = Mesh->GetEdgeV(Edges[i]);
			Vertices[i] = IndexUtil::FindSharedEdgeVertex(prev_ev, next_ev);
			prev_ev = next_ev;
		}
		Vertices[0] = IndexUtil::FindEdgeOtherVertex(start_ev, Vertices[1]);
		Vertices[Vertices.Num() - 1] = IndexUtil::FindEdgeOtherVertex(prev_ev, Vertices[Vertices.Num() - 2]);
	}
	else
	{
		Vertices[0] = start_ev[0];
		Vertices[1] = start_ev[1];
	}
}




bool FEdgeSpan::InitializeFromVertices(const TArray<int>& VerticesIn, bool bAutoOrient)
{
	check(Mesh != nullptr);
	Vertices = VerticesIn;

	int NumVertices = Vertices.Num();
	Edges.SetNum(NumVertices - 1);
	for (int i = 0; i < NumVertices - 1; ++i)
	{
		int a = Vertices[i], b = Vertices[i + 1];
		Edges[i] = Mesh->FindEdge(a, b);
		if (Edges[i] == FDynamicMesh3::InvalidID)
		{
			checkf(false, TEXT("EdgeSpan.FromVertices: invalid edge [%d,%d]"), a, b);
			return false;
		}
	}

	if (bAutoOrient)
	{
		SetCorrectOrientation();
	}

	return true;
}


void FEdgeSpan::CalculateBowtieVertices()
{
	BowtieVertices.Reset();
	int NumVertices = Vertices.Num();
	for (int i = 0; i < NumVertices; ++i)
	{
		if (Mesh->IsBowtieVertex(Vertices[i]))
		{
			BowtieVertices.Add(Vertices[i]);
		}
	}
	bBowtiesCalculated = true;
}



FAxisAlignedBox3d FEdgeSpan::GetBounds() const
{
	FAxisAlignedBox3d box = FAxisAlignedBox3d::Empty();
	for (int i = 0; i < Vertices.Num(); ++i)
	{
		box.Contain(Mesh->GetVertex(Vertices[i]));
	}
	return box;
}



void FEdgeSpan::GetPolyline(FPolyline3d& PolylineOut) const
{
	PolylineOut.Clear();
	for (int i = 0; i < Vertices.Num(); ++i)
	{
		PolylineOut.AppendVertex(Mesh->GetVertex(Vertices[i]));
	}
}


bool FEdgeSpan::SetCorrectOrientation()
{
	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		int eid = Edges[i];
		if (Mesh->IsBoundaryEdge(eid))
		{
			int a = Vertices[i], b = Vertices[i + 1];
			FIndex2i ev = Mesh->GetOrientedBoundaryEdgeV(eid);
			if (ev.A == b && ev.B == a)
			{
				Reverse();
				return true;
			}
			else
			{
				return false;
			}
		}
	}
	return false;
}



bool FEdgeSpan::IsInternalspan() const
{
	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		if (Mesh->IsBoundaryEdge(Edges[i]))
		{
			return false;
		}
	}
	return true;
}


bool FEdgeSpan::IsBoundaryspan(const FDynamicMesh3* TestMesh) const
{
	const FDynamicMesh3* UseMesh = (TestMesh != nullptr) ? TestMesh : Mesh;

	int NumEdges = Edges.Num();
	for (int i = 0; i < NumEdges; ++i)
	{
		if (UseMesh->IsBoundaryEdge(Edges[i]) == false)
		{
			return false;
		}
	}
	return true;
}



int FEdgeSpan::FindVertexIndex(int VertexID) const
{
	int N = Vertices.Num();
	for (int i = 0; i < N; ++i)
	{
		if (Vertices[i] == VertexID)
		{
			return i;
		}
	}
	return -1;
}



int FEdgeSpan::FindNearestVertexIndex(const FVector3d& QueryPoint) const
{
	int iNear = -1;
	double fNearSqr = TNumericLimits<double>::Max();
	int N = Vertices.Num();
	for (int i = 0; i < N; ++i)
	{
		FVector3d lv = Mesh->GetVertex(Vertices[i]);
		double d2 = DistanceSquared(QueryPoint, lv);
		if (d2 < fNearSqr)
		{
			fNearSqr = d2;
			iNear = i;
		}
	}
	return iNear;
}



bool FEdgeSpan::CheckValidity(EValidityCheckFailMode FailMode) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FEdgeSpan::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FEdgeSpan::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}


	CheckOrFailF(Vertices.Num() == (Edges.Num() + 1));
	for (int ei = 0; ei < Edges.Num(); ++ei)
	{
		FIndex2i ev = Mesh->GetEdgeV(Edges[ei]);
		CheckOrFailF(Mesh->IsVertex(ev.A));
		CheckOrFailF(Mesh->IsVertex(ev.B));
		CheckOrFailF(Mesh->FindEdge(ev.A, ev.B) != FDynamicMesh3::InvalidID);
		CheckOrFailF(Vertices[ei] == ev.A || Vertices[ei] == ev.B);
		CheckOrFailF(Vertices[ei + 1] == ev.A || Vertices[ei + 1] == ev.B);
	}
	for (int vi = 0; vi < Vertices.Num() - 1; ++vi)
	{
		int a = Vertices[vi], b = Vertices[vi + 1];
		CheckOrFailF(Mesh->IsVertex(a));
		CheckOrFailF(Mesh->IsVertex(b));
		CheckOrFailF(Mesh->FindEdge(a, b) != FDynamicMesh3::InvalidID);

		// @todo rewrite this test for span, has to handle endpoint vertices that only have one nbr
		if (vi < Vertices.Num() - 2) {
			int n = 0, edge_before_b = Edges[vi], edge_after_b = Edges[(vi + 1) % Vertices.Num()];
			for (int nbr_e : Mesh->VtxEdgesItr(b))
			{
				if (nbr_e == edge_before_b || nbr_e == edge_after_b)
				{
					n++;
				}
			}
			CheckOrFailF(n == 2);
		}
	}
	return is_ok;
}



void FEdgeSpan::VertexSpanToEdgeSpan(const FDynamicMesh3* Mesh, const TArray<int>& VertexSpan, TArray<int>& OutEdgeSpan)
{
	// @todo this function should be in a utility class?

	int NV = VertexSpan.Num();
	OutEdgeSpan.SetNum(NV - 1);
	for (int i = 0; i < NV - 1; ++i)
	{
		int v0 = VertexSpan[i];
		int v1 = VertexSpan[i + 1];
		OutEdgeSpan[i] = Mesh->FindEdge(v0, v1);
	}
}


void FEdgeSpan::GetSubspansByAngle(double AngleThresholdDeg, int32 MinSpanSize, TArray<FEdgeSpan>& OutSpans) const
{
	auto IsEdgeAngleSharp = [this](int32 SharedVid, const FIndex2i& AttachedEids, double DotThreshold) -> bool
	{
		if (!ensure(Mesh && Mesh->IsEdge(AttachedEids.A) && Mesh->IsEdge(AttachedEids.B)))
		{
			return false;
		}

		// Gets vector pointing from the Vid along the edge.
		auto GetEdgeUnitVector = [this, SharedVid](int32 Eid, FVector3d& VectorOut)->bool
		{
			FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);
			// Make sure that the Vid is at EdgeVids.A
			if (EdgeVids.B == SharedVid)
			{
				Swap(EdgeVids.A, EdgeVids.B);
			}
			VectorOut = Mesh->GetVertex(EdgeVids.B) - Mesh->GetVertex(EdgeVids.A);
			return VectorOut.Normalize(KINDA_SMALL_NUMBER);
		};

		FVector Edge1, Edge2;
		if (!GetEdgeUnitVector(AttachedEids.A, Edge1) || !GetEdgeUnitVector(AttachedEids.B, Edge2))
		{
			// If either edge was degenerate, we don't want to consider the angle "sharp"
			return false;
		}

		return Edge1.Dot(Edge2) >= DotThreshold;
	};

	const double CosAngleThresholdDeg = FMath::Cos(FMath::DegreesToRadians(AngleThresholdDeg));

	// First just identify corners

	TArray<int32> SpanCorners;		// These are indices of Vertices, not vertex indices
	for (int32 SpanVertexIndex = 1; SpanVertexIndex < Vertices.Num() - 1; ++SpanVertexIndex)
	{
		const int32 PreviousEdgeID = Edges[SpanVertexIndex - 1];
		const int32 VertexID = Vertices[SpanVertexIndex];
		check(SpanVertexIndex < Edges.Num());
		const int32 NextEdgeID = Edges[SpanVertexIndex];

		const bool bIsSharp = IsEdgeAngleSharp(VertexID, FIndex2i{ PreviousEdgeID, NextEdgeID }, CosAngleThresholdDeg);

		if (bIsSharp)
		{
			SpanCorners.Add(SpanVertexIndex);
		}
	}

	// Filter corners too close together

	if (SpanCorners.Num() > 1)
	{
		TArray<int32> FilteredSpanCorners;
		FilteredSpanCorners.Add(SpanCorners[0]);

		for (int32 CornerIndex = 1; CornerIndex < SpanCorners.Num(); ++CornerIndex)
		{
			int32 Distance = SpanCorners[CornerIndex] - FilteredSpanCorners.Last();
			Distance = FMath::Min(Distance, Vertices.Num() - Distance);	 // check wraparound distance as well	

			if (Distance >= MinSpanSize)
			{
				FilteredSpanCorners.Add(SpanCorners[CornerIndex]);
			}
		}

		SpanCorners = MoveTemp(FilteredSpanCorners);
	}

	// Now assemble output spans

	if (SpanCorners.Num() == 0)
	{
		OutSpans.Add(*this);
		return;
	}

	int32 NewSpanStart = 0;
	for (int32 NewSpanEnd : SpanCorners)
	{
		TArray<int32> NewSpanVertices;
		TArray<int32> NewSpanEdges;

		for (int32 NewSpanVertexIndex = NewSpanStart; NewSpanVertexIndex < NewSpanEnd; ++NewSpanVertexIndex)
		{
			NewSpanVertices.Add(Vertices[NewSpanVertexIndex]);
			ensure(NewSpanVertexIndex < Edges.Num());
			NewSpanEdges.Add(Edges[NewSpanVertexIndex]);
		}
		NewSpanVertices.Add(Vertices[NewSpanEnd]);		// final vertex in this span

		if (ensure(NewSpanVertices.Num() > 0))
		{
			OutSpans.Add(FEdgeSpan(Mesh, NewSpanVertices, NewSpanEdges));
		}
		NewSpanStart = NewSpanEnd;
	}

	// Final span: prepend it to the first one if they are connected and there is no corner between them

	TArray<int32> FinalSpanVertices;
	TArray<int32> FinalSpanEdges;
	for (int32 FinalSpanVertexIndex = NewSpanStart; FinalSpanVertexIndex < Vertices.Num(); ++FinalSpanVertexIndex)
	{
		FinalSpanVertices.Add(Vertices[FinalSpanVertexIndex]);
		if (FinalSpanVertexIndex < Edges.Num())
		{
			FinalSpanEdges.Add(Edges[FinalSpanVertexIndex]);
		}
	}

	const bool bSpanIsLoop = (IndexUtil::FindSharedEdgeVertex(Mesh->GetEdgeV(Edges[0]), Mesh->GetEdgeV(Edges.Last())) != IndexConstants::InvalidID);
	const bool bSharpCorner = IsEdgeAngleSharp(Vertices[0], FIndex2i{ Edges[0], Edges.Last() }, CosAngleThresholdDeg);
	const bool bShouldPrepend = bSpanIsLoop && !bSharpCorner && (OutSpans.Num() > 0);

	if (bShouldPrepend)
	{
		if (ensure(FinalSpanVertices.Last() == OutSpans[0].Vertices[0]))
		{
			FinalSpanVertices.Pop();
		}

		OutSpans[0].Vertices.Insert(FinalSpanVertices, 0);
		OutSpans[0].Edges.Insert(FinalSpanEdges, 0);
	}
	else
	{
		// Don't add to the first span, create a new final output span
		OutSpans.Add(FEdgeSpan(Mesh, FinalSpanVertices, FinalSpanEdges));
	}
}

