// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/DiscMeshGenerator.h"
#include "MathUtil.h"

using namespace UE::Geometry;

FDiscMeshGenerator::FDiscMeshGenerator()
{
	Normal = FVector3f::UnitZ();
	IndicesMap = FIndex2i(0, 1);
	AngleSamples = 10;
	RadialSamples = 2;
	StartAngle = 0;
	EndAngle = 360;
	Radius = 1;
}


FPuncturedDiscMeshGenerator::FPuncturedDiscMeshGenerator()
{
	HoleRadius = .5;
}




FMeshShapeGenerator& FDiscMeshGenerator::Generate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DiscMeshGenerator_Generate);
	
	int AngleNV = AngleSamples > 3 ? AngleSamples : 3;
	int RadialNV = RadialSamples > 1 ? RadialSamples : 1;
	int NumVertices = AngleNV * RadialNV + 1;
	
	bool bFullDisc = (EndAngle - StartAngle) >= 359.999;
	int AngleNT = bFullDisc ? AngleNV : AngleNV - 1;

	int NumTriangles = AngleNT * (2 * RadialNV - 1);
	SetBufferSizes(NumVertices, NumTriangles, NumVertices, NumVertices);

	Vertices[0] = MakeVertex(0, 0); // center
	UVs[0] = FVector2f(.5f, .5f);
	double IdxToRadiansOffset = StartAngle * FMathd::DegToRad;
	double IdxToRadiansScale = (EndAngle - StartAngle) * FMathd::DegToRad / double(AngleNV - (bFullDisc ? 0 : 1));
	double IdxToRadiusScale = Radius / double(RadialNV);
	double IdxToUVRadiusScale = .5 / double(RadialNV);
	for (int AngleIdx = 0; AngleIdx < AngleNV; AngleIdx++)
	{
		double Angle = IdxToRadiansOffset + IdxToRadiansScale * AngleIdx;
		double CosA = FMathd::Cos(Angle);
		double SinA = FMathd::Sin(Angle);
		for (int RadIdx = 1; RadIdx <= RadialNV; RadIdx++)
		{
			int VertIdx = AngleIdx + 1 + (RadIdx-1) * AngleNV;
			double R = RadIdx * IdxToRadiusScale;
			Vertices[VertIdx] = MakeVertex(R*CosA, R*SinA);
			double Uvalue = 0.5 + CosA * RadIdx * IdxToUVRadiusScale;
			double Vvalue = 0.5 + SinA * RadIdx * IdxToUVRadiusScale;
			UVs[VertIdx] = FVector2f(float(Uvalue), float(Vvalue));
		}
	}

	for (int Idx = 0; Idx < Vertices.Num(); Idx++)
	{
		UVParentVertex[Idx] = Idx;
		NormalParentVertex[Idx] = Idx;
		Normals[Idx] = Normal;
	}

	int TriIdx = 0, PolyIdx = 0;
	for (int AngleIdx = 0; AngleIdx+1 < AngleNV; AngleIdx++)
	{
		SetTriangleWithMatchedUVNormal(TriIdx, 0, AngleIdx + 2, AngleIdx + 1);
		SetTrianglePolygon(TriIdx, PolyIdx);
		if (!bSinglePolygroup)
		{
			PolyIdx++;
		}
		TriIdx++;
	}
	if (bFullDisc)
	{
		SetTriangleWithMatchedUVNormal(TriIdx, 0, 1, AngleNV);
		SetTrianglePolygon(TriIdx, PolyIdx);
		TriIdx++;
		if (!bSinglePolygroup)
		{
			PolyIdx++;
		}
	}

	for (int RadiusIdx = 0; RadiusIdx+1 < RadialNV; RadiusIdx++)
	{
		int Inner = 1 + RadiusIdx * AngleNV;
		int Outer = Inner + AngleNV;
		for (int AngleIdx = 0; AngleIdx + 1 < AngleNV; AngleIdx++)
		{
			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleIdx, Outer + AngleIdx + 1, Outer + AngleIdx);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleIdx, Inner + AngleIdx + 1, Outer + AngleIdx + 1);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			if (!bSinglePolygroup)
			{
				PolyIdx++;
			}
		}
		if (bFullDisc)
		{
			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleNV - 1, Outer, Outer + AngleNV - 1);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleNV - 1, Inner, Outer);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			if (!bSinglePolygroup)
			{
				PolyIdx++;
			}
		}
	}


	return *this;
}

FMeshShapeGenerator& FPuncturedDiscMeshGenerator::Generate()
{
	int AngleNV = AngleSamples > 3 ? AngleSamples : 3;
	int RadialNV = RadialSamples > 2 ? RadialSamples : 2;
	int NumVertices = AngleNV * RadialNV;

	bool bFullDisc = (EndAngle - StartAngle) >= 359.999;
	int AngleNT = bFullDisc ? AngleNV : AngleNV - 1;

	int NumTriangles = AngleNT * 2 * (RadialNV-1);
	SetBufferSizes(NumVertices, NumTriangles, NumVertices, NumVertices);

	double IdxToRadiansOffset = StartAngle * FMathd::DegToRad;
	double IdxToRadiansScale = (EndAngle - StartAngle) * FMathd::DegToRad / double(AngleNV - (bFullDisc ? 0 : 1));
	double IdxToRadiusScale = (Radius - HoleRadius) / double(RadialNV-1);
	double IdxToRadiusOffset = HoleRadius;
	double RadiusToUVScale = .5 / Radius;
	for (int AngleIdx = 0; AngleIdx < AngleNV; AngleIdx++)
	{
		double Angle = IdxToRadiansOffset + IdxToRadiansScale * AngleIdx;
		double CosA = FMathd::Cos(Angle);
		double SinA = FMathd::Sin(Angle);
		for (int RadIdx = 0; RadIdx < RadialNV; RadIdx++)
		{
			int VertIdx = AngleIdx + RadIdx * AngleNV;
			double R = RadIdx * IdxToRadiusScale + IdxToRadiusOffset;
			Vertices[VertIdx] = MakeVertex(R*CosA, R*SinA);
			double Uvalue = 0.5 + CosA * R * RadiusToUVScale;
			double Vvalue = 0.5 + SinA * R * RadiusToUVScale;
			UVs[VertIdx] = FVector2f(float(Uvalue), float(Vvalue));
		}
	}

	for (int Idx = 0; Idx < Vertices.Num(); Idx++)
	{
		UVParentVertex[Idx] = Idx;
		NormalParentVertex[Idx] = Idx;
		Normals[Idx] = Normal;
	}

	int TriIdx = 0, PolyIdx = 0;
	for (int RadiusIdx = 0; RadiusIdx+1 < RadialNV; RadiusIdx++)
	{
		int Inner = RadiusIdx * AngleNV;
		int Outer = Inner + AngleNV;
		for (int AngleIdx = 0; AngleIdx + 1 < AngleNV; AngleIdx++)
		{
			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleIdx, Outer + AngleIdx + 1, Outer + AngleIdx);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleIdx, Inner + AngleIdx + 1, Outer + AngleIdx + 1);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			if (!bSinglePolygroup)
			{
				PolyIdx++;
			}
		}
		if (bFullDisc)
		{
			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleNV - 1, Outer, Outer + AngleNV - 1);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			SetTriangleWithMatchedUVNormal(TriIdx, Inner + AngleNV - 1, Inner, Outer);
			SetTrianglePolygon(TriIdx, PolyIdx);
			TriIdx++;

			if (!bSinglePolygroup)
			{
				PolyIdx++;
			}
		}
	}

	return *this;
}
