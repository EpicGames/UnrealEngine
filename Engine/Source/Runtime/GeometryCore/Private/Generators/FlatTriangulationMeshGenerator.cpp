// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/FlatTriangulationMeshGenerator.h"
#include "BoxTypes.h"

using namespace UE::Geometry;

FFlatTriangulationMeshGenerator::FFlatTriangulationMeshGenerator()
{
	Normal = FVector3f::UnitZ();
	IndicesMap = FIndex2i(0, 1);
}

FFlatTriangulationMeshGenerator::~FFlatTriangulationMeshGenerator() = default;

FMeshShapeGenerator& FFlatTriangulationMeshGenerator::Generate()
{
	int NumVertices = Vertices2D.Num();
	ensure(NumVertices >= 3);
	if (NumVertices < 3)
	{
		return *this;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FlatTriangulationMeshGenerator_Generate);

	int NumTriangles = Triangles2D.Num();
	SetBufferSizes(NumVertices, NumTriangles, NumVertices, NumVertices);

	FAxisAlignedBox2d BoundingBox(Vertices2D);
	double Width = BoundingBox.Width(), Height = BoundingBox.Height();
	double UVScale = FMath::Max(Width, Height);

	for (int VertIdx = 0; VertIdx < NumVertices; ++VertIdx)
	{
		const FVector2d& Pos = Vertices2D[VertIdx];

		UVs[VertIdx] = FVector2f(
			(float)((Pos.X - BoundingBox.Min.X) / UVScale),
			(float)((Pos.Y - BoundingBox.Min.Y) / UVScale));
		UVParentVertex[VertIdx] = VertIdx;

		Normals[VertIdx] = Normal;
		NormalParentVertex[VertIdx] = VertIdx;

		Vertices[VertIdx] = MakeVertex(Pos.X, Pos.Y);
	}


	for (int TriIdx = 0; TriIdx < NumTriangles; TriIdx++)
	{
		FIndex3i Tri = Triangles2D[TriIdx];
		SetTriangle(TriIdx, Tri);
		SetTriangleUVs(TriIdx, Tri);
		SetTriangleNormals(TriIdx, Tri);
		int PolygroupID = Triangles2DPolygroups.IsEmpty() ? 0 : Triangles2DPolygroups[TriIdx];
		SetTrianglePolygon(TriIdx, PolygroupID);
	}


	return *this;
}
