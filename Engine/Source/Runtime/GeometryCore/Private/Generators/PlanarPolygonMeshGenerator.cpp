// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/PlanarPolygonMeshGenerator.h"
#include "CompGeom/PolygonTriangulation.h"

using namespace UE::Geometry;

FPlanarPolygonMeshGenerator::FPlanarPolygonMeshGenerator()
{
	Normal = FVector3f::UnitZ();
	IndicesMap = FIndex2i(0, 1);
}

FPlanarPolygonMeshGenerator::~FPlanarPolygonMeshGenerator() = default;


void FPlanarPolygonMeshGenerator::SetPolygon(const TArray<FVector2D>& PolygonVerts)
{
	Polygon = FPolygon2d();
	int NumVerts = PolygonVerts.Num();
	for (int i = 0; i < NumVerts; ++i)
	{
		Polygon.AppendVertex(FVector2d(PolygonVerts[i]));
	}
}




FMeshShapeGenerator& FPlanarPolygonMeshGenerator::Generate()
{
	int NumVertices = Polygon.VertexCount();
	check(NumVertices >= 3);
	if (NumVertices < 3)
	{
		return *this;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(PlanarPolygonMeshGenerator_Generate);

	TArray<FIndex3i> TriangleList;
	PolygonTriangulation::TriangulateSimplePolygon(Polygon.GetVertices(), TriangleList, false);

	int NumTriangles = TriangleList.Num();
	SetBufferSizes(NumVertices, NumTriangles, NumVertices, NumVertices);

	FAxisAlignedBox2d BoundingBox = Polygon.Bounds();
	double Width = BoundingBox.Width(), Height = BoundingBox.Height();
	double UVScale = FMath::Max(Width, Height);

	for (int VertIdx = 0; VertIdx < NumVertices; ++VertIdx)
	{
		FVector2d Pos = Polygon[VertIdx];

		UVs[VertIdx] = FVector2f(
			(float)((Pos.X - BoundingBox.Min.X) / UVScale),
			(float)((Pos.Y - BoundingBox.Min.Y) / UVScale));
		UVParentVertex[VertIdx] = VertIdx;

		Normals[VertIdx] = Normal;
		NormalParentVertex[VertIdx] = VertIdx;

		Vertices[VertIdx] = MakeVertex(Pos.X, Pos.Y);
	}


	for (int TriIdx = 0; TriIdx < NumTriangles; TriIdx++)
	{
		FIndex3i Tri = TriangleList[TriIdx];
		SetTriangle(TriIdx, Tri);
		SetTriangleUVs(TriIdx, Tri);
		SetTriangleNormals(TriIdx, Tri);
		SetTrianglePolygon(TriIdx, 0);
	}


	return *this;
}