// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/PolygonEdgeMeshGenerator.h"

using namespace UE::Geometry;

namespace PolygonEdgeMeshGeneratorLocal
{
	// Given two non-collinear rays and a circle radius, there is a circle that both rays touch (tangentially) at the same distance along each ray. Return this ray distance.
	double TwoRayCircleTangentTest(const FVector2d& RayOrigin, const FVector2d& RayDirection0, const FVector2d& RayDirection1, double CircleRadius)
	{
		const double InteriorRayAngle = AngleR(RayDirection0, RayDirection1);
		const double Denom = FMath::Tan(0.5 * InteriorRayAngle);
		if (!ensure(!FMath::IsNearlyEqual(Denom, 0.0)))		// Should check for ray collinearity before this function
		{
			return 0.0;
		}
		return CircleRadius / Denom;
	}

	// Given a circle radius, two points on the circle, and a tangent to the first point, find the circle center.
	FVector2d FindCircleCenter(double CircleRadius, const FVector2d& PointOnCircle0, const FVector2d& PointOnCircle1, const FVector2d& Tangent0)
	{
		const FVector2d ZeroToMid = 0.5 * (PointOnCircle1 - PointOnCircle0);
		const double LengthToMid = ZeroToMid.Length();
		const double MidToCenterDistance = FMath::Sqrt(CircleRadius * CircleRadius - LengthToMid * LengthToMid);
		const double Scalar = MidToCenterDistance / LengthToMid;

		// Two possible centers given two points and a radius
		const FVector2d C0 = PointOnCircle0 + ZeroToMid + Scalar * PerpCW(ZeroToMid);
		const FVector2d C1 = PointOnCircle0 + ZeroToMid - Scalar * PerpCW(ZeroToMid);

		// Find the center that produces a diameter cord perpendicular to the input segment
		const double Dot0 = FMath::Abs((C0 - PointOnCircle0).Dot(Tangent0));
		const double Dot1 = FMath::Abs((C1 - PointOnCircle0).Dot(Tangent0));
		if (Dot0 < Dot1)
		{
			return C0;
		}
		else
		{
			return C1;
		}
	}


	// Given two points on a circle, discretize the shortest arc between them using the given number of points
	void GenerateArc(const FVector2d& PointOnCircle0, 
		const FVector2d& PointOnCircle1, 
		const FVector2d& CircleCenter, 
		double CircleRadius, 
		int NumArcPoints, 
		TArray<FVector2d>& OutArc)
	{
		double Param0 = 2.0 * FMath::Atan2(PointOnCircle0[1] - CircleCenter[1], PointOnCircle0[0] - CircleCenter[0] + CircleRadius);
		double Param1 = 2.0 * FMath::Atan2(PointOnCircle1[1] - CircleCenter[1], PointOnCircle1[0] - CircleCenter[0] + CircleRadius);

		// There are two arcs connecting the two parameters -- choose the shorter one. Param1 - Param0 can be negative.
		if (FMath::Abs(Param1 - Param0) > PI)
		{
			if (Param0 < Param1)
			{
				Param0 += TWO_PI;
			}
			else
			{
				Param1 += TWO_PI;
			}
		}

		const double DeltaParam = (Param1 - Param0) / (double)(NumArcPoints - 1);

		for (int I = 0; I < NumArcPoints; ++I)
		{
			const double Param = Param0 + I * DeltaParam;
			const FVector2d ArcPoint = CircleCenter + CircleRadius * FVector2d{ FMath::Cos(Param), FMath::Sin(Param) };
			OutArc.Add(ArcPoint);
		}
	}

	// Given a corner P0-P1-P2, generate an arc with given radius (discretized into NumArcPoints-1 segments)
	void RoundCorner(const FVector2d& P0, const FVector2d& P1, const FVector2d& P2, double CircleRadius, int NumArcPoints, TArray<FVector2d>& OutCorner)
	{
		if (!ensure(NumArcPoints > 1))
		{
			OutCorner.Add(P1);
			return;
		}

		const FVector2d& RayOrigin = P1;
		const FVector2d RayDirection0 = Normalized(P0 - RayOrigin);
		const FVector2d RayDirection1 = Normalized(P2 - RayOrigin);

		if (FMath::IsNearlyEqual(FMath::Abs(RayDirection0.Dot(RayDirection1)), 1.0))
		{
			// TODO: Make a proper triangle fan, not a bunch of vertices on top of each other
			for (int I = 0; I < NumArcPoints; ++I)
			{
				OutCorner.Add(P1);
			}
			return;
		}

		const double RayIntersectLength = TwoRayCircleTangentTest(RayOrigin, RayDirection0, RayDirection1, CircleRadius);

		const FVector2d PointOnCircle0 = RayOrigin + RayIntersectLength * RayDirection0;
		const FVector2d PointOnCircle1 = RayOrigin + RayIntersectLength * RayDirection1;
		const FVector2d Tangent0 = Normalized(P0 - PointOnCircle0);
		const FVector2d CircleCenter = FindCircleCenter(CircleRadius, PointOnCircle0, PointOnCircle1, Tangent0);

		GenerateArc(PointOnCircle0, PointOnCircle1, CircleCenter, CircleRadius, NumArcPoints, OutCorner);
	}

	FVector2d ProjectPathVertex(const FVector3d& InVertex, const FFrame3d& ProjectionFrame)
	{
		return ProjectionFrame.ToPlaneUV(InVertex);
	}

	FVector3d UnprojectPathVertex(const FVector2d& InVertex, const FFrame3d& ProjectionFrame)
	{
		return ProjectionFrame.FromPlaneUV(InVertex);
	}


	// For each corner in the path, compute the maximum radius of an arc that could be used to round out the corner. 
	// Arc radius is limited such that each "corner arc" would only extend to the midway point of each incident path edge.
	void ComputeMaxCornerRadii(const TArray<FVector3d>& Path, const FFrame3d& PolygonFrame, TArray<double>& MaxCircleRadii)
	{
		int N = Path.Num();
		MaxCircleRadii.Init(BIG_NUMBER, N);

		for (int CornerID = 0; CornerID < N; ++CornerID)
		{
			const int MinusOne = (CornerID + N - 1) % N;
			const int PlusOne = (CornerID + 1) % N;
			const FVector2d M1 = ProjectPathVertex(Path[MinusOne], PolygonFrame);
			const FVector2d C = ProjectPathVertex(Path[CornerID], PolygonFrame);
			const FVector2d P1 = ProjectPathVertex(Path[PlusOne], PolygonFrame);

			const double CornerAngle = AngleR(Normalized(M1 - C), Normalized(P1 - C));

			// Don't let an arc go more than half the edge length (conservative limit but easy to compute)
			const double L1 = Distance(M1, C);
			const double RMax1 = 0.5 * L1 * FMath::Tan(0.5 * CornerAngle);
			const double L2 = Distance(C, P1);
			const double RMax2 = 0.5 * L2 * FMath::Tan(0.5 * CornerAngle);

			MaxCircleRadii[CornerID] = FMath::Min(RMax1, RMax2);
		}
	}

	double ArcLength(const TArray<FVector3d>& Path, double MinSegmentArcLength = 0.1 )
	{
		double Length = 0;
		for (int Index = 1; Index < Path.Num(); ++Index)
		{
			Length += FMath::Max(Distance(Path[Index], Path[Index - 1]), MinSegmentArcLength);
		}

		// loop
		Length += FMath::Max(Distance(Path.Last(), Path[0]), MinSegmentArcLength);

		return Length;
	}

	double PartialArcLength(const TArray<FVector3d>& Path, int MaxIndex, double MinSegmentArcLength = 0.1)
	{
		if (MaxIndex == Path.Num())
		{
			return ArcLength(Path);
		}

		double Length = 0.0;
		for (int Index = 1; Index <= MaxIndex; ++Index)
		{
			Length += FMath::Max(Distance(Path[Index], Path[Index - 1]), MinSegmentArcLength);
		}
		return Length;
	}
}


FPolygonEdgeMeshGenerator::FPolygonEdgeMeshGenerator(const TArray<FFrame3d>& InPolygon,
	bool bInClosed,
	const TArray<double>& InOffsetScaleFactors,
	double InWidth,
	FVector3d InNormal,
	bool bInRoundedCorners,
	double InCornerRadius,
	bool bInLimitCornerRadius,
	int InNumArcVertices) :
	Polygon(InPolygon),
	bClosed(bInClosed),
	OffsetScaleFactors(InOffsetScaleFactors),
	Width(InWidth),
	Normal(InNormal),
	bRoundedCorners(bInRoundedCorners),
	CornerRadius(InCornerRadius),
	bLimitCornerRadius(bInLimitCornerRadius),
	NumArcVertices(InNumArcVertices)
{
	check(Polygon.Num() == OffsetScaleFactors.Num());
}

FPolygonEdgeMeshGenerator::~FPolygonEdgeMeshGenerator() = default;

// Given a piecewise linear path, replace each corner with a discretized arc
void FPolygonEdgeMeshGenerator::CurvePath(const TArray<FVector3d>& InPath,
	const TArray<bool>& InteriorAngleFlag,
	const TArray<double>& MaxCornerRadii,
	const TArray<double>& OtherSideMaxCornerRadii,
	const FFrame3d& ProjectionFrame,
	TArray<FVector3d>& OutPath) const
{
	using namespace PolygonEdgeMeshGeneratorLocal;

	const int N = InPath.Num();
	if (N < 3)
	{
		return;
	}

	int Begin = bClosed ? 0 : 1;
	int End = bClosed ? N : N - 1;

	if (!bClosed)
	{
		OutPath.Add(InPath[0]);
	}

	for (int CornerID = Begin; CornerID < End; ++CornerID)
	{
		const int Prev = bClosed ? (CornerID + N - 1) % N : CornerID - 1;
		const int Next = bClosed ? (CornerID + 1) % N : CornerID + 1;

		const FVector2d PrevVertex = ProjectPathVertex(InPath[Prev], ProjectionFrame);
		const FVector2d CurrVertex = ProjectPathVertex(InPath[CornerID], ProjectionFrame);
		const FVector2d NextVertex = ProjectPathVertex(InPath[Next], ProjectionFrame);

		TArray<FVector2d> ArcVertices;

		double EffectiveRadius = CornerRadius;
		if (bLimitCornerRadius)
		{
			// The input Radius is taken as the desired radius for exterior corners. So the desired interior radius is the input
			// radius minus width.
			// Each corner has a max interior and max exterior radius. We want to respect these limits but still have:
			//		interior radius = exterior radius - width

			if (InteriorAngleFlag[CornerID])
			{
				double ClampedExteriorRadius = FMath::Min(CornerRadius, OtherSideMaxCornerRadii[CornerID]);
				double InteriorMax = MaxCornerRadii[CornerID];
				EffectiveRadius = FMath::Clamp(ClampedExteriorRadius - Width, 0.0, InteriorMax);
			}
			else
			{
				double InteriorMax = OtherSideMaxCornerRadii[CornerID];
				double ExteriorMax = MaxCornerRadii[CornerID];
				EffectiveRadius = FMath::Min(CornerRadius, FMath::Min(ExteriorMax, InteriorMax + Width));
			}
		}
		else
		{
			if (InteriorAngleFlag[CornerID])
			{
				EffectiveRadius -= Width;
			}
		}

		if (EffectiveRadius > KINDA_SMALL_NUMBER)
		{
			RoundCorner(PrevVertex, CurrVertex, NextVertex, EffectiveRadius, NumArcVertices, ArcVertices);
		}
		else if (OtherSideMaxCornerRadii[CornerID] <= KINDA_SMALL_NUMBER && MaxCornerRadii[CornerID] <= KINDA_SMALL_NUMBER)
		{
			// Both interior and exterior radii are close to zero -- don't insert any arc here at all
			ArcVertices.Add(CurrVertex);
		}
		else
		{
			// TODO: Make a proper triangle fan, not a bunch of vertices on top of each other
			for (int I = 0; I < NumArcVertices; ++I)
			{
				ArcVertices.Add(CurrVertex);
			}
		}

		for (const FVector2d& V : ArcVertices)
		{
			OutPath.Add(UnprojectPathVertex(V, ProjectionFrame));
		}
	}

	if (!bClosed)
	{
		OutPath.Add(InPath.Last());
	}
}

// Generate triangulation
// TODO: Enable more subdivisions along the width and length dimensions if requested
FMeshShapeGenerator& FPolygonEdgeMeshGenerator::Generate()
{
	using namespace PolygonEdgeMeshGeneratorLocal;
	
	TRACE_CPUPROFILER_EVENT_SCOPE(PolygonEdgeMeshGenerator_Generate);

	const int32 NumInputVertices = Polygon.Num();

	TArray<FVector3d> CenterPath;
	TArray<FVector3d> LeftSidePath;
	TArray<FVector3d> RightSidePath;

	// Trace the input path, placing vertices on either side of each input vertex 
	const FVector3d LeftVertex{ 0, -0.5 * Width, 0 };
	const FVector3d RightVertex{ 0, 0.5 * Width, 0 };
	for (int32 CurrentInputVertex = 0; CurrentInputVertex < NumInputVertices; ++CurrentInputVertex)
	{
		const FFrame3d& CurrentFrame = Polygon[CurrentInputVertex];
		const int32 NewVertexAIndex = 2 * CurrentInputVertex;
		const int32 NewVertexBIndex = NewVertexAIndex + 1;
		const FVector3d NewVertexA = CurrentFrame.FromFramePoint(OffsetScaleFactors[CurrentInputVertex] * LeftVertex);
		const FVector3d NewVertexB = CurrentFrame.FromFramePoint(OffsetScaleFactors[CurrentInputVertex] * RightVertex);

		CenterPath.Add(CurrentFrame.Origin);
		LeftSidePath.Add(NewVertexA);
		RightSidePath.Add(NewVertexB);
	}

	TArray<FVector3d> NewLeftSidePath;
	TArray<FVector3d> NewRightSidePath;
	
	const FFrame3d PolygonFrame = Polygon[0];

	if (bRoundedCorners && NumInputVertices >= 3)
	{
		// Compute the maximum allowable radius for each corner, on each path side
		TArray<double> LeftMaxCornerRadii, RightMaxCornerRadii;
		ComputeMaxCornerRadii(LeftSidePath, PolygonFrame, LeftMaxCornerRadii);
		ComputeMaxCornerRadii(RightSidePath, PolygonFrame, RightMaxCornerRadii);

		// For each point on the input path, determine whether the corner turns "left" or "right". This is used to define
		// the interior and exterior radii at each path corner.
		// TODO: Support a "round outwards" mode where we replace the turning flags with an "interior to the polygon" flag
		TArray<bool> TurnsLeft;
		TArray<bool> TurnsRight;

		if (bClosed)
		{
			for (int32 I = 0; I < CenterPath.Num(); ++I)
			{
				const int32 Prev = I == 0 ? CenterPath.Num() - 1 : I - 1;
				const int32 Next = (I + 1) % CenterPath.Num();
				const FVector2d A = ProjectPathVertex(CenterPath[Prev], PolygonFrame);
				const FVector2d B = ProjectPathVertex(CenterPath[I], PolygonFrame);
				const FVector2d C = ProjectPathVertex(CenterPath[Next], PolygonFrame);
				TurnsLeft.Add(Orient(A, B, C) < 0);
				TurnsRight.Add(!TurnsLeft.Last());
			}
		}
		else
		{
			TurnsLeft.Add(false);	// First point
			TurnsRight.Add(false);

			for (int32 I = 1; I < CenterPath.Num()-1; ++I)
			{
				const int32 Prev = I - 1;
				const int32 Next = I + 1;
				const FVector2d A = ProjectPathVertex(CenterPath[Prev], PolygonFrame);
				const FVector2d B = ProjectPathVertex(CenterPath[I], PolygonFrame);
				const FVector2d C = ProjectPathVertex(CenterPath[Next], PolygonFrame);
				TurnsLeft.Add(Orient(A, B, C) < 0);
				TurnsRight.Add(!TurnsLeft.Last());
			}

			TurnsLeft.Add(false);	// Last point
			TurnsRight.Add(false);
		}

		CurvePath(LeftSidePath,  TurnsLeft,  LeftMaxCornerRadii,  RightMaxCornerRadii, PolygonFrame, NewLeftSidePath);
		CurvePath(RightSidePath, TurnsRight, RightMaxCornerRadii, LeftMaxCornerRadii,  PolygonFrame, NewRightSidePath);
	}
	else
	{
		NewLeftSidePath = LeftSidePath;
		NewRightSidePath = RightSidePath;
	}

	check(NewLeftSidePath.Num() == NewRightSidePath.Num());


	//
	// Set vertices, triangles, polygroups in the output mesh
	//

	const int32 FinalNumVertices = 2 * NewLeftSidePath.Num();
	const int32 FinalNumUVs = 2 * NewLeftSidePath.Num() + 2;		// Extra pair of UVs for the final loop polygon
	const int32 FinalNumTriangles = bClosed ?  2 * NewLeftSidePath.Num() : 2 * NewLeftSidePath.Num() - 2;
	SetBufferSizes(FinalNumVertices, FinalNumTriangles, FinalNumUVs, FinalNumVertices);

	Vertices.Empty();
	for (int32 CurrentVertex = 0; CurrentVertex < NewLeftSidePath.Num(); ++CurrentVertex)
	{
		Vertices.Add(NewLeftSidePath[CurrentVertex]);
		Vertices.Add(NewRightSidePath[CurrentVertex]);
	}

	int PolyIndex = 0;
	int EndVertex = bClosed ? NewLeftSidePath.Num() : NewLeftSidePath.Num() - 1;
	for (int32 CurrentVertex = 0; CurrentVertex < EndVertex; ++CurrentVertex)
	{
		const int32 NewVertexA = 2 * CurrentVertex;
		const int32 NewVertexB = NewVertexA + 1;
		const int32 NewVertexC = (NewVertexA + 2) % FinalNumVertices;
		const int32 NewVertexD = (NewVertexA + 3) % FinalNumVertices;

		const FIndex3i NewTriA{ NewVertexA, NewVertexB, NewVertexC };
		const int32 NewTriAIndex = 2 * CurrentVertex;

		SetTriangle(NewTriAIndex, NewTriA);
		SetTriangleUVs(NewTriAIndex, NewTriA);
		SetTriangleNormals(NewTriAIndex, NewTriA);
		SetTrianglePolygon(NewTriAIndex, PolyIndex);

		const int32 NewTriBIndex = NewTriAIndex + 1;
		const FIndex3i NewTriB{ NewVertexC, NewVertexB, NewVertexD };

		SetTriangle(NewTriBIndex, NewTriB);
		SetTriangleUVs(NewTriBIndex, NewTriB);
		SetTriangleNormals(NewTriBIndex, NewTriB);
		SetTrianglePolygon(NewTriBIndex, PolyIndex);

		if (!bSinglePolyGroup)
		{
			PolyIndex++;
		}
	}

	for (int32 NewVertexIndex = 0; NewVertexIndex < FinalNumVertices; ++NewVertexIndex)
	{
		Normals[NewVertexIndex] = FVector3f(Normal);
		NormalParentVertex[NewVertexIndex] = NewVertexIndex;
	}

	//
	// Generate UVs 
	//
	 
	// Create a UV strip for the path
	const float UVLeft = 0.0f, UVBottom = 0.0f;
	float UVRight = 1.0f, UVTop = 1.0f;
	if (bScaleUVByAspectRatio && UVWidth != UVHeight)
	{
		if (UVWidth > UVHeight)
		{
			UVTop = float( UVHeight / UVWidth );
		}
		else
		{
			UVRight = float( UVWidth / UVHeight );
		}
	}
	FVector2f UV00 = FVector2f(UVLeft, UVBottom);
	FVector2f UV01 = FVector2f(UVRight, UVBottom);
	FVector2f UV11 = FVector2f(UVRight, UVTop);
	FVector2f UV10 = FVector2f(UVLeft, UVTop);

	const int32 FinalNumVerticesOneSide = NewLeftSidePath.Num();

	double TotalArcLength = ArcLength(NewLeftSidePath);
	const double MinSegmentArcLength = 0.1 * TotalArcLength / (double)FinalNumVerticesOneSide;
	TotalArcLength = ArcLength(NewLeftSidePath, MinSegmentArcLength);

	for (int32 NewVertexIndex = 0; NewVertexIndex < FinalNumVerticesOneSide; ++NewVertexIndex)
	{
		const int32 NewVertexAIndex = 2 * NewVertexIndex;
		const int32 NewVertexBIndex = NewVertexAIndex + 1;

		double UParam = PartialArcLength(NewLeftSidePath, NewVertexIndex, MinSegmentArcLength) / TotalArcLength;

		UVs[NewVertexAIndex] = BilinearInterp(UV00, UV01, UV11, UV10, float(UParam), 0.0f);
		UVs[NewVertexBIndex] = BilinearInterp(UV00, UV01, UV11, UV10, float(UParam), 1.0f);
		UVParentVertex[NewVertexAIndex] = NewVertexAIndex;
		UVParentVertex[NewVertexBIndex] = NewVertexBIndex;
	}

	// UVs for the final quad	
	if (bClosed)
	{
		const int32 NewTriAIndex = 2 * (FinalNumVerticesOneSide - 1);
		const int32 NewTriBIndex = NewTriAIndex + 1;

		const int32 NewVertexA = 2 * (FinalNumVerticesOneSide - 1);
		const int32 NewVertexB = NewVertexA + 1;
		const int32 NewVertexC = NewVertexA + 2;
		const int32 NewVertexD = NewVertexA + 3;

		ensure(NewVertexD < UVs.Num());

		FIndex3i NewTriA{ NewVertexA, NewVertexB, NewVertexC };
		SetTriangleUVs(NewTriAIndex, NewTriA);

		FIndex3i NewTriB{ NewVertexC, NewVertexB, NewVertexD };
		SetTriangleUVs(NewTriBIndex, NewTriB);

		UVs[2 * FinalNumVerticesOneSide] = { UVRight, UVBottom }; 
		UVs[2 * FinalNumVerticesOneSide + 1] = { UVRight, UVTop };
		UVParentVertex[2 * FinalNumVerticesOneSide] = 0;
		UVParentVertex[2 * FinalNumVerticesOneSide + 1] = 1;
	}

	return *this;
}
