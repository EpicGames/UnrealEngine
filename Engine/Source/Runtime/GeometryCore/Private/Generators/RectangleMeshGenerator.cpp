// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/RectangleMeshGenerator.h"
#include "MathUtil.h"

using namespace UE::Geometry;

FRectangleMeshGenerator::FRectangleMeshGenerator()
{
	Origin = FVector3d::Zero();
	Width = 10.0f;
	Height = 10.0f;
	WidthVertexCount = HeightVertexCount = 8;
	Normal = FVector3f::UnitZ();
	IndicesMap = FIndex2i(0, 1);
	bScaleUVByAspectRatio = true;
}



FMeshShapeGenerator& FRectangleMeshGenerator::Generate()
{
	check(IndicesMap.A >= 0 && IndicesMap.A <= 2);
	check(IndicesMap.B >= 0 && IndicesMap.B <= 2);

	TRACE_CPUPROFILER_EVENT_SCOPE(RectangleMeshGenerator_Generate);

	int WidthNV = (WidthVertexCount > 1) ? WidthVertexCount : 2;
	int HeightNV = (HeightVertexCount > 1) ? HeightVertexCount : 2;

	int TotalNumVertices = WidthNV * HeightNV;
	int TotalNumTriangles = 2 * (WidthNV - 1) * (HeightNV - 1);
	SetBufferSizes(TotalNumVertices, TotalNumTriangles, TotalNumVertices, TotalNumVertices);

	// corner vertices
	FVector3d v00 = MakeVertex(0, -Width / 2.0f, -Height / 2.0f);
	FVector3d v01 = MakeVertex(1, Width / 2.0f, -Height / 2.0f);
	FVector3d v11 = MakeVertex(2, Width / 2.0f, Height / 2.0f);
	FVector3d v10 = MakeVertex(3, -Width / 2.0f, Height / 2.0f);

	// corner UVs
	float uvleft = 0.0f, uvright = 1.0f, uvbottom = 0.0f, uvtop = 1.0f;
	if (bScaleUVByAspectRatio && Width != Height)
	{
		if (Width > Height)
		{
			uvtop = float( Height / Width );
		}
		else
		{
			uvright = float( Width / Height );
		}
	}

	FVector2f uv00 = FVector2f(uvleft, uvbottom);
	FVector2f uv01 = FVector2f(uvright, uvbottom);
	FVector2f uv11 = FVector2f(uvright, uvtop);
	FVector2f uv10 = FVector2f(uvleft, uvtop);


	int vi = 0;
	int ti = 0;

	// add vertex rows
	int start_vi = vi;
	for (int yi = 0; yi < HeightNV; ++yi)
	{
		double ty = (double)yi / (double)(HeightNV - 1);
		for (int xi = 0; xi < WidthNV; ++xi)
		{
			double tx = (double)xi / (double)(WidthNV - 1);
			Normals[vi] = Normal;
			NormalParentVertex[vi] = vi;
			UVs[vi] = BilinearInterp(uv00, uv01, uv11, uv10, (float)tx, (float)ty);
			UVParentVertex[vi] = vi;
			Vertices[vi++] = BilinearInterp(v00, v01, v11, v10, tx, ty);
		}
	}

	// add triangulated quads
	int PolyIndex = 0;
	for (int y0 = 0; y0 < HeightNV - 1; ++y0)
	{
		for (int x0 = 0; x0 < WidthNV - 1; ++x0)
		{
			int i00 = start_vi + y0 * WidthNV + x0;
			int i10 = start_vi + (y0 + 1)*WidthNV + x0;
			int i01 = i00 + 1, i11 = i10 + 1;

			SetTriangle(ti, i00, i11, i01);
			SetTrianglePolygon(ti, PolyIndex);
			SetTriangleUVs(ti, i00, i11, i01);
			SetTriangleNormals(ti, i00, i11, i01);

			ti++;

			SetTriangle(ti, i00, i10, i11);
			SetTrianglePolygon(ti, PolyIndex);
			SetTriangleUVs(ti, i00, i10, i11);
			SetTriangleNormals(ti, i00, i10, i11);

			ti++;
			if (!bSinglePolyGroup)
			{
				PolyIndex++;
			}
		}
	}

	return *this;
}



FRoundedRectangleMeshGenerator::FRoundedRectangleMeshGenerator()
{
	Radius = 1.0;
	AngleSamples = 16;
	SharpCorners = ERoundedRectangleCorner::None;
}



FMeshShapeGenerator& FRoundedRectangleMeshGenerator::Generate()
{
	check(IndicesMap.A >= 0 && IndicesMap.A <= 2);
	check(IndicesMap.B >= 0 && IndicesMap.B <= 2);

	int WidthNV = (WidthVertexCount > 1) ? WidthVertexCount : 2;
	int HeightNV = (HeightVertexCount > 1) ? HeightVertexCount : 2;
	int RoundNV = (AngleSamples > 0) ? AngleSamples : 1;
	WidthNV += 2;
	HeightNV += 2;

	int NumRound = 4-NumSharpCorners(SharpCorners);

	int TotalNumVertices = WidthNV * HeightNV + NumRound * (RoundNV-1);
	int TotalNumTriangles = 2 * (WidthNV - 1) * (HeightNV - 1) + NumRound * (RoundNV - 1);
	SetBufferSizes(TotalNumVertices, TotalNumTriangles, TotalNumVertices, TotalNumVertices);

	// make UVs and normals self-parented, and assign all normals to constant
	for (int VertIdx = 0; VertIdx < TotalNumVertices; VertIdx++)
	{
		NormalParentVertex[VertIdx] = VertIdx;
		UVParentVertex[VertIdx] = VertIdx;
		Normals[VertIdx] = Normal;
	}

	float TotWidth = FMathf::Max(FMathf::ZeroTolerance, float(Radius * 2 + Width));
	float TotHeight = FMathf::Max(FMathf::ZeroTolerance, float(Radius * 2 + Height));
	

	// corner vertices
	FVector3d v00 = MakeVertex(0, -TotWidth / 2.0f, -TotHeight / 2.0f);
	FVector3d v01 = MakeVertex(1, TotWidth / 2.0f, -TotHeight / 2.0f);
	FVector3d v11 = MakeVertex(2, TotWidth / 2.0f, TotHeight / 2.0f);
	FVector3d v10 = MakeVertex(3, -TotWidth / 2.0f, TotHeight / 2.0f);

	// corner UVs
	float uvleft = 0.0f, uvright = 1.0f, uvbottom = 0.0f, uvtop = 1.0f;
	if (bScaleUVByAspectRatio && TotWidth != TotHeight)
	{
		if (TotWidth > TotHeight)
		{
			uvtop = TotHeight / TotWidth;
		}
		else
		{
			uvright = TotWidth / TotHeight;
		}
	}

	FVector2f uv00 = FVector2f(uvleft, uvbottom);
	FVector2f uv01 = FVector2f(uvright, uvbottom);
	FVector2f uv11 = FVector2f(uvright, uvtop);
	FVector2f uv10 = FVector2f(uvleft, uvtop);


	int vi = 0;
	int ti = 0;

	TArray<double> XFrac, YFrac; XFrac.SetNum(WidthNV); YFrac.SetNum(HeightNV);
	XFrac[0] = 0; XFrac.Last() = 1;
	YFrac[0] = 0; YFrac.Last() = 1;
	for (int XIdx = 1; XIdx + 1 < XFrac.Num(); XIdx++)
	{
		XFrac[XIdx] = (Radius + Width * (XIdx - 1) / (WidthNV - 3)) / TotWidth;
	}
	for (int YIdx = 1; YIdx + 1 < YFrac.Num(); YIdx++)
	{
		YFrac[YIdx] = (Radius + Height * (YIdx - 1) / (HeightNV - 3)) / TotHeight;
	}

	// add vertex rows
	int start_vi = vi;
	for (int yi = 0; yi < HeightNV; ++yi)
	{
		double ty = YFrac[yi];
		for (int xi = 0; xi < WidthNV; ++xi)
		{
			double tx = XFrac[xi];
			UVs[vi] = BilinearInterp(uv00, uv01, uv11, uv10, (float)tx, (float)ty);
			Vertices[vi++] = BilinearInterp(v00, v01, v11, v10, tx, ty);
		}
	}

	// add triangulated quads
	int PolyIndex = 0;
	for (int y0 = 0; y0 < HeightNV - 1; ++y0)
	{
		bool bIsYEdge = y0 == 0 || y0 == HeightNV - 2;
		for (int x0 = 0; x0 < WidthNV - 1; ++x0)
		{
			if (bIsYEdge)
			{
				bool bIsXEdge = x0 == 0 || x0 == WidthNV - 2;
				if (bIsXEdge && !SideInCorners(x0 == 0 ? 0 : 1, y0 == 0 ? 0 : 1, SharpCorners))
				{
					// rounded corner; skip triangulation; we'll get it in a second pass
					continue;
				}
			}
			int i00 = start_vi + y0 * WidthNV + x0;
			int i10 = start_vi + (y0 + 1)*WidthNV + x0;
			int i01 = i00 + 1, i11 = i10 + 1;

			SetTriangle(ti, i00, i11, i01);
			SetTrianglePolygon(ti, PolyIndex);
			SetTriangleUVs(ti, i00, i11, i01);
			SetTriangleNormals(ti, i00, i11, i01);

			ti++;

			SetTriangle(ti, i00, i10, i11);
			SetTrianglePolygon(ti, PolyIndex);
			SetTriangleUVs(ti, i00, i10, i11);
			SetTriangleNormals(ti, i00, i10, i11);

			ti++;
			if (!bSinglePolyGroup)
			{
				PolyIndex++;
			}
		}
	}

	for (int SideX = 0; SideX < 2; SideX++)
	{
		for (int SideY = 0; SideY < 2; SideY++)
		{
			if (!SideInCorners(SideX, SideY, SharpCorners))
			{
				// rounded corner
				int CornerY = SideY * (HeightNV - 1);
				int CornerX = SideX * (WidthNV - 1);
				int InCornerY = SideY ? HeightNV - 2 : 1;
				int InCornerX = SideX ? WidthNV - 2 : 1;

				// vertex to re-purpose
				int UseVIdx = start_vi + CornerY * WidthNV + CornerX;

				int VCenterIdx = start_vi + InCornerY * WidthNV + InCornerX;
				int OffXIdx = start_vi + InCornerY * WidthNV + CornerX;
				int OffYIdx = start_vi + CornerY * WidthNV + InCornerX;
				int ActingCosIdx = SideY == SideX ? OffYIdx : OffXIdx;
				int ActingSinIdx = SideY == SideX ? OffXIdx : OffYIdx;
				FVector3d CenterV = Vertices[VCenterIdx];
				FVector3d CosV = Vertices[ActingCosIdx] - CenterV;
				FVector3d SinV = Vertices[ActingSinIdx] - CenterV;
				FVector2f CenterUV = UVs[VCenterIdx];
				FVector2f CosUV = UVs[ActingCosIdx] - CenterUV;
				FVector2f SinUV = UVs[ActingSinIdx] - CenterUV;
				
				int LastUsedIdx = ActingCosIdx;

				for (int AngleSampleIdx = 1; AngleSampleIdx < RoundNV+1; AngleSampleIdx++)
				{
					double Angle = FMathd::HalfPi * AngleSampleIdx / float(RoundNV+1);
					double CosAngle = FMathd::Cos(Angle);
					double SinAngle = FMathd::Sin(Angle);
					Vertices[UseVIdx] = CenterV + CosAngle * CosV + SinAngle * SinV;
					UVs[UseVIdx] = CenterUV + (float)CosAngle * CosUV + (float)SinAngle * SinUV;
					SetTriangle(ti, VCenterIdx, LastUsedIdx, UseVIdx);
					SetTrianglePolygon(ti, PolyIndex);
					SetTriangleUVs(ti, VCenterIdx, LastUsedIdx, UseVIdx);
					SetTriangleNormals(ti, VCenterIdx, LastUsedIdx, UseVIdx);

					LastUsedIdx = UseVIdx;
					UseVIdx = vi++;
					ti++;
				}
				vi--;

				// add last triangle in fan
				SetTriangle(ti, VCenterIdx, LastUsedIdx, ActingSinIdx);
				SetTrianglePolygon(ti, PolyIndex);
				SetTriangleUVs(ti, VCenterIdx, LastUsedIdx, ActingSinIdx);
				SetTriangleNormals(ti, VCenterIdx, LastUsedIdx, ActingSinIdx);
				ti++;

				if (!bSinglePolyGroup)
				{
					PolyIndex++;
				}
			}
		}
	}

	return *this;
}