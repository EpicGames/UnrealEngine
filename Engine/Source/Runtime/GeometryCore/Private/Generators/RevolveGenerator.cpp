// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/RevolveGenerator.h"
#include "Util/RevolveUtil.h"
#include "Async/ParallelFor.h"
#include "CompGeom/PolygonTriangulation.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"

using namespace UE::Geometry;


namespace UELocal
{
	//
	// These are local copies of FDynamicMeshEditor functions, which is currenlty not available in GeometryCore.
	// However simpler implementations would be sufficient for the sweep code below, which only uses them
	// to handle planar caps.
	//

	static void SetTriangleNormals(FDynamicMesh3* Mesh, const TArray<int>& Triangles)
	{
		check(Mesh->HasAttributes());
		FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();

		TSet<int32> TriangleSet(Triangles);
		TUniqueFunction<bool(int32)> TrianglePredicate = [&](int32 TriangleID) { return TriangleSet.Contains(TriangleID); };

		TMap<int, int> Vertices;

		for (int tid : Triangles)
		{
			if (Normals->IsSetTriangle(tid))
			{
				Normals->UnsetTriangle(tid);
			}

			FIndex3i BaseTri = Mesh->GetTriangle(tid);
			FIndex3i ElemTri;
			for (int j = 0; j < 3; ++j)
			{
				const int* FoundElementID = Vertices.Find(BaseTri[j]);
				if (FoundElementID == nullptr)
				{
					FVector3d VtxROINormal = FMeshNormals::ComputeVertexNormal(*Mesh, BaseTri[j], TrianglePredicate);
					ElemTri[j] = Normals->AppendElement( (FVector3f)VtxROINormal);
					Vertices.Add(BaseTri[j], ElemTri[j]);
				}
				else
				{
					ElemTri[j] = *FoundElementID;
				}
			}
			Normals->SetTriangle(tid, ElemTri);
		}
	}



	static void SetTriangleUVsFromProjection(FDynamicMesh3* Mesh, const TArray<int>& Triangles, const FFrame3d& ProjectionFrame, const FVector2f& UVScale, 
		const FVector2f& UVTranslation, int UVLayerIndex, bool bShiftToOrigin, bool bNormalizeBeforeScaling)
	{
		if (!Triangles.Num())
		{
			return;
		}

		check(Mesh->HasAttributes() && Mesh->Attributes()->NumUVLayers() > UVLayerIndex);
		FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

		TMap<int, int> BaseToOverlayVIDMap;
		TArray<int> AllUVIndices;

		FAxisAlignedBox2f UVBounds(FAxisAlignedBox2f::Empty());

		for (int TID : Triangles)
		{
			if (UVs->IsSetTriangle(TID))
			{
				UVs->UnsetTriangle(TID);
			}

			FIndex3i BaseTri = Mesh->GetTriangle(TID);
			FIndex3i ElemTri;
			for (int j = 0; j < 3; ++j)
			{
				const int* FoundElementID = BaseToOverlayVIDMap.Find(BaseTri[j]);
				if (FoundElementID == nullptr)
				{
					FVector2f UV = (FVector2f)ProjectionFrame.ToPlaneUV(Mesh->GetVertex(BaseTri[j]), 2);
					UVBounds.Contain(UV);
					ElemTri[j] = UVs->AppendElement(UV);
					AllUVIndices.Add(ElemTri[j]);
					BaseToOverlayVIDMap.Add(BaseTri[j], ElemTri[j]);
				}
				else
				{
					ElemTri[j] = *FoundElementID;
				}
			}
			UVs->SetTriangle(TID, ElemTri);
		}

		FVector2f UvScaleToUse = bNormalizeBeforeScaling ? FVector2f(UVScale[0] / UVBounds.Width(), UVScale[1] / UVBounds.Height())
			: UVScale;

		// shift UVs so that their bbox min-corner is at origin and scaled by external scale factor
		for (int UVID : AllUVIndices)
		{
			FVector2f UV = UVs->GetElement(UVID);
			FVector2f TransformedUV = (bShiftToOrigin) ? ((UV - UVBounds.Min) * UvScaleToUse) : (UV * UvScaleToUse);
			TransformedUV += UVTranslation;
			UVs->SetElement(UVID, TransformedUV);
		}
	}

}



FDynamicMesh3 FRevolvePlanarPathGenerator::GenerateMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RevolvePlanarPathGenerator_GenerateMesh);
	
	// revolve around +Z axis, around origin
	FVector3d AxisDirection(0, 0, 1);
	FVector3d AxisOrigin(0, 0, 0);

	TArray<FVector3d> ProfileCurve;
	TSet<int32> ProfileVerticesToWeld;
	bool bProfileCurveIsClosed = false;

	for (FVector2d Point : PathVertices)
	{
		ProfileCurve.Add(FVector3d(Point.X, 0, Point.Y));
	}
	// unclear why but the sweep code seems to be written for a clockwise ordering...
	Algo::Reverse(ProfileCurve);

	// Project first and last points onto the revolution axis to cap it
	if (bCapped)
	{
		FVector3d FirstPoint = ProfileCurve[0];
		FVector3d LastPoint = ProfileCurve.Last();

		double DistanceAlongAxis = AxisDirection.Dot(LastPoint - AxisOrigin);
		FVector3d ProjectedPoint = AxisOrigin + (AxisDirection * DistanceAlongAxis);
		ProfileCurve.Add(ProjectedPoint);

		DistanceAlongAxis = AxisDirection.Dot(FirstPoint - AxisOrigin);
		ProjectedPoint = AxisOrigin + (AxisDirection * DistanceAlongAxis);
		ProfileCurve.Add(ProjectedPoint);

		bProfileCurveIsClosed = true;
	}
	else
	{
		bProfileCurveIsClosed = false;
	}

	//double TotalRevolutionDegrees = RevolveDegrees;// (AlongAxisOffsetPerDegree == 0) ? ClampedRevolutionDegrees : RevolutionDegrees;
	double TotalRevolutionDegrees = FMath::Clamp(RevolveDegrees, 0.1f, 360.0f);

	double DegreesPerStep = TotalRevolutionDegrees / (double)Steps;
	double DegreesOffset = DegreeOffset;
	if (bReverseDirection)
	{
		DegreesPerStep *= -1;
		DegreesOffset *= -1;
	}
	//double DownAxisOffsetPerStep = TotalRevolutionDegrees * AlongAxisOffsetPerDegree / Steps;

	if (bProfileAtMidpoint && DegreesPerStep != 0 && FMathd::Abs(DegreesPerStep) < 180)
	{
		RevolveUtil::MakeProfileCurveMidpointOfFirstStep(ProfileCurve, DegreesPerStep, AxisOrigin, AxisDirection);
	}

	// Generate the sweep curve
	
	TArray<FFrame3d> SweepCurve;
	bool bSweepCurveIsClosed = (TotalRevolutionDegrees == 360);

	//CurveSweepOpOut.bSweepCurveIsClosed = bWeldFullRevolution && AlongAxisOffsetPerDegree == 0 && TotalRevolutionDegrees == 360;
	int32 NumSweepFrames = bSweepCurveIsClosed ? Steps : Steps + 1; // If closed, last sweep frame is also first
	SweepCurve.Reserve(NumSweepFrames);
	RevolveUtil::GenerateSweepCurve(AxisOrigin, AxisDirection, DegreesOffset,
		DegreesPerStep, 0.0, NumSweepFrames, SweepCurve);

	// Weld any vertices that are on the axis
	RevolveUtil::WeldPointsOnAxis(ProfileCurve, AxisOrigin,
		AxisDirection, 0.1, ProfileVerticesToWeld);

	//CurveSweepOpOut.DiagonalTolerance = DiagonalProportionTolerance;
	//double UVScale = MaterialProperties.UVScale;

	//double UVScale = 1.0;
	//CurveSweepOp.UVScale = FVector2d(UVScale, UVScale);

	// what for?
	//if (bReverseProfileCurve ^ bFlipVs)
	//{
	//	CurveSweepOpOut.UVScale[1] *= -1;
	//	CurveSweepOpOut.UVOffset = FVector2d(0, UVScale);
	//}

	bool bUVsSkipFullyWeldedEdges = true;
	ECapFillMode CapFillMode = (bFillPartialRevolveEndcaps) ?
		ECapFillMode::EarClipping : ECapFillMode::None;		// delaunay? hits ensure...

	// Use a mesh generator to do the work.
	FProfileSweepGenerator CurveSweeper;
	CurveSweeper.ProfileCurve = ProfileCurve;
	CurveSweeper.SweepCurve = SweepCurve;
	CurveSweeper.WeldedVertices = ProfileVerticesToWeld;
	CurveSweeper.bProfileCurveIsClosed = bProfileCurveIsClosed;
	CurveSweeper.bSweepCurveIsClosed = bSweepCurveIsClosed;

	CurveSweeper.bSharpNormals = false; // bSharpNormals;
	CurveSweeper.UVScale = UVScale;
	CurveSweeper.bUVScaleRelativeWorld = bUVScaleRelativeWorld;
	CurveSweeper.UnitUVInWorldCoordinates = UnitUVInWorldCoordinates;
	CurveSweeper.UVOffset = UVOffset;
	CurveSweeper.bUVsSkipFullyWeldedEdges = bUVsSkipFullyWeldedEdges;
	CurveSweeper.PolygonGroupingMode = PolygonGroupingMode;
	CurveSweeper.QuadSplitMethod = QuadSplitMethod;
	CurveSweeper.DiagonalTolerance = DiagonalTolerance;
	
	FDynamicMesh3 ResultMesh(&CurveSweeper.Generate());

	// Cap the ends if needed
	if (bProfileCurveIsClosed && !bSweepCurveIsClosed && CapFillMode != ECapFillMode::None)
	{
		for (int CapIndex = 0; CapIndex < 2; ++CapIndex)
		{
			// Note that we rely on the sweep generator to tell us the start and end profile curves for
			// capping rather than looking for a boundary ourselves. This is because in cases where the
			// side of the profile curve is welded, there will actually be one open boundary rather than
			// two separate ones, yet we want to cap the two sides separately.
			// This does rely on the fact that the hole fillers we use only care about vertices, and don't
			// rely on there being edges between them. It also relies on the vertex ids in the DynamicMesh
			// matching the ones used in the generator. Without those two facts, the work of filling two
			// halves of an open boundary separately would be harder to do...

			// TODO: curve is planar we don't have to figure this out...

			// Put the profile curve into an edge loop
			TArray<int> VertexLoop = CurveSweeper.EndProfiles[CapIndex];
			int NV = VertexLoop.Num();
			TArray<int32> Edges;
			Edges.SetNum(NV);
			for (int i = 0; i < NV; ++i)
			{
				Edges[i] = ResultMesh.FindEdge(VertexLoop[i], VertexLoop[(i + 1) % NV]);
			}

			//FEdgeLoop::VertexLoopToEdgeLoop(ResultMesh.Get(), , Edges);
			//FEdgeLoop Loop(ResultMesh.Get(), CurveSweeper.EndProfiles[CapIndex], Edges);

			// The first cap actually needs to face in the opposite direction of the sweep to face outward.
			if (CapIndex == 0)
			{
				Algo::Reverse(VertexLoop);
				Algo::Reverse(Edges);
			}

			// Compute a best-fit plane of the vertices
			TArray<FVector3d> VertexPositions;
			for (int32 vid : VertexLoop)
			{
				VertexPositions.Add( ResultMesh.GetVertex(vid) );
			}

			FVector3d PlaneOrigin;
			FVector3d PlaneNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
			PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

			int32 NewGroupID = ResultMesh.AllocateTriangleGroup();

			TArray<FIndex3i> Triangles;
			TArray<int32> NewTriangles;
			PolygonTriangulation::TriangulateSimplePolygon(VertexPositions, Triangles);
			for (FIndex3i PolyTriangle : Triangles)
			{
				FIndex3i MeshTriangle( VertexLoop[PolyTriangle.A], VertexLoop[PolyTriangle.B], VertexLoop[PolyTriangle.C]);
				int32 NewTriangle = ResultMesh.AppendTriangle(MeshTriangle, NewGroupID);
				if (NewTriangle >= 0)
				{
					NewTriangles.Add(NewTriangle);
				}
			}

			// Compute normals and UVs
			if (ResultMesh.HasAttributes())
			{
				UELocal::SetTriangleNormals(&ResultMesh, NewTriangles);

				// Set UVs for the cap
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				if (bUVScaleRelativeWorld)
				{
					FVector2f ScaleToUse(UVScale);
					ScaleToUse /= CurveSweeper.UnitUVInWorldCoordinates;
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, ScaleToUse, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						false); // don't normalize
				}
				else
				{
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, (FVector2f)UVScale, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						true); // normalize first
				}

			}
		}// end for each loop
	}//end if capping


	return ResultMesh;
}







FDynamicMesh3 FRevolvePlanarPolygonGenerator::GenerateMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RevolvePlanarPolygonGenerator_GenerateMesh);
	
	// revolve around +Z axis, around origin
	FVector3d AxisDirection(0, 0, 1);
	FVector3d AxisOrigin(0, 0, 0);

	TArray<FVector3d> ProfileCurve;
	TSet<int32> ProfileVerticesToWeld;
	bool bProfileCurveIsClosed = true;

	for (FVector2d Point : PolygonVertices)
	{
		ProfileCurve.Add(FVector3d(Point.X, 0, Point.Y));
	}
	// unclear why but the sweep code seems to be written for a clockwise ordering...
	Algo::Reverse(ProfileCurve);

	//double TotalRevolutionDegrees = RevolveDegrees;// (AlongAxisOffsetPerDegree == 0) ? ClampedRevolutionDegrees : RevolutionDegrees;
	double TotalRevolutionDegrees = FMath::Clamp(RevolveDegrees, 0.1f, 360.0f);

	double DegreesPerStep = TotalRevolutionDegrees / (double)Steps;
	double DegreesOffset = DegreeOffset;
	if (bReverseDirection)
	{
		DegreesPerStep *= -1;
		DegreesOffset *= -1;
	}
	//double DownAxisOffsetPerStep = TotalRevolutionDegrees * AlongAxisOffsetPerDegree / Steps;

	if (bProfileAtMidpoint && DegreesPerStep != 0 && FMathd::Abs(DegreesPerStep) < 180)
	{
		RevolveUtil::MakeProfileCurveMidpointOfFirstStep(ProfileCurve, DegreesPerStep, AxisOrigin, AxisDirection);
	}

	// Generate the sweep curve

	TArray<FFrame3d> SweepCurve;
	bool bSweepCurveIsClosed = (TotalRevolutionDegrees == 360);

	//CurveSweepOpOut.bSweepCurveIsClosed = bWeldFullRevolution && AlongAxisOffsetPerDegree == 0 && TotalRevolutionDegrees == 360;
	int32 NumSweepFrames = bSweepCurveIsClosed ? Steps : Steps + 1; // If closed, last sweep frame is also first
	SweepCurve.Reserve(NumSweepFrames);
	RevolveUtil::GenerateSweepCurve(AxisOrigin, AxisDirection, DegreesOffset,
		DegreesPerStep, 0.0, NumSweepFrames, SweepCurve);

	// Weld any vertices that are on the axis
	if (bWeldVertsOnAxis)
	{
		RevolveUtil::WeldPointsOnAxis(ProfileCurve, AxisOrigin,
			AxisDirection, 0.1, ProfileVerticesToWeld);
	}

	//CurveSweepOpOut.DiagonalTolerance = DiagonalProportionTolerance;
	//double UVScale = MaterialProperties.UVScale;

	//double UVScale = 1.0;
	//CurveSweepOp.UVScale = FVector2d(UVScale, UVScale);

	// what for?
	//if (bReverseProfileCurve ^ bFlipVs)
	//{
	//	CurveSweepOpOut.UVScale[1] *= -1;
	//	CurveSweepOpOut.UVOffset = FVector2d(0, UVScale);
	//}

	bool bUVsSkipFullyWeldedEdges = true;
	ECapFillMode CapFillMode = (bFillPartialRevolveEndcaps) ?
		ECapFillMode::EarClipping : ECapFillMode::None;		// delaunay? hits ensure...

															// Use a mesh generator to do the work.
	FProfileSweepGenerator CurveSweeper;
	CurveSweeper.ProfileCurve = ProfileCurve;
	CurveSweeper.SweepCurve = SweepCurve;
	CurveSweeper.WeldedVertices = ProfileVerticesToWeld;
	CurveSweeper.bProfileCurveIsClosed = bProfileCurveIsClosed;
	CurveSweeper.bSweepCurveIsClosed = bSweepCurveIsClosed;

	CurveSweeper.bSharpNormals = false; // bSharpNormals;
	CurveSweeper.UVScale = UVScale;
	CurveSweeper.bUVScaleRelativeWorld = bUVScaleRelativeWorld;
	CurveSweeper.UnitUVInWorldCoordinates = UnitUVInWorldCoordinates;
	CurveSweeper.UVOffset = UVOffset;
	CurveSweeper.bUVsSkipFullyWeldedEdges = bUVsSkipFullyWeldedEdges;
	CurveSweeper.PolygonGroupingMode = PolygonGroupingMode;
	CurveSweeper.QuadSplitMethod = QuadSplitMethod;
	CurveSweeper.DiagonalTolerance = DiagonalTolerance;

	FDynamicMesh3 ResultMesh(&CurveSweeper.Generate());

	// Cap the ends if needed
	if (bProfileCurveIsClosed && !bSweepCurveIsClosed && CapFillMode != ECapFillMode::None)
	{
		for (int CapIndex = 0; CapIndex < 2; ++CapIndex)
		{
			// Note that we rely on the sweep generator to tell us the start and end profile curves for
			// capping rather than looking for a boundary ourselves. This is because in cases where the
			// side of the profile curve is welded, there will actually be one open boundary rather than
			// two separate ones, yet we want to cap the two sides separately.
			// This does rely on the fact that the hole fillers we use only care about vertices, and don't
			// rely on there being edges between them. It also relies on the vertex ids in the DynamicMesh
			// matching the ones used in the generator. Without those two facts, the work of filling two
			// halves of an open boundary separately would be harder to do...

			// TODO: curve is planar we don't have to figure this out...

			// Put the profile curve into an edge loop
			TArray<int> VertexLoop = CurveSweeper.EndProfiles[CapIndex];
			int NV = VertexLoop.Num();
			TArray<int32> Edges;
			Edges.SetNum(NV);
			for (int i = 0; i < NV; ++i)
			{
				Edges[i] = ResultMesh.FindEdge(VertexLoop[i], VertexLoop[(i + 1) % NV]);
			}

			//FEdgeLoop::VertexLoopToEdgeLoop(ResultMesh.Get(), , Edges);
			//FEdgeLoop Loop(ResultMesh.Get(), CurveSweeper.EndProfiles[CapIndex], Edges);

			// The first cap actually needs to face in the opposite direction of the sweep to face outward.
			if (CapIndex == 0)
			{
				Algo::Reverse(VertexLoop);
				Algo::Reverse(Edges);
			}

			// Compute a best-fit plane of the vertices
			TArray<FVector3d> VertexPositions;
			for (int32 vid : VertexLoop)
			{
				VertexPositions.Add( ResultMesh.GetVertex(vid) );
			}

			FVector3d PlaneOrigin;
			FVector3d PlaneNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
			PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

			int32 NewGroupID = ResultMesh.AllocateTriangleGroup();

			TArray<FIndex3i> Triangles;
			TArray<int32> NewTriangles;
			PolygonTriangulation::TriangulateSimplePolygon(VertexPositions, Triangles);
			for (FIndex3i PolyTriangle : Triangles)
			{
				FIndex3i MeshTriangle( VertexLoop[PolyTriangle.A], VertexLoop[PolyTriangle.B], VertexLoop[PolyTriangle.C]);
				int32 NewTriangle = ResultMesh.AppendTriangle(MeshTriangle, NewGroupID);
				if (NewTriangle >= 0)
				{
					NewTriangles.Add(NewTriangle);
				}
			}

			// Compute normals and UVs
			if (ResultMesh.HasAttributes())
			{
				UELocal::SetTriangleNormals(&ResultMesh, NewTriangles);

				// Set UVs for the cap
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				if (bUVScaleRelativeWorld)
				{
					FVector2f ScaleToUse(UVScale);
					ScaleToUse /= CurveSweeper.UnitUVInWorldCoordinates;
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, ScaleToUse, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						false); // don't normalize
				}
				else
				{
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, (FVector2f)UVScale, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						true); // normalize first
				}

			}
		}// end for each loop
	}//end if capping

	return ResultMesh;
}







FDynamicMesh3 FSpiralRevolvePlanarPolygonGenerator::GenerateMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SpiralRevolvePlanarPolygonGenerator_GenerateMesh);
	
	// revolve around +Z axis, around origin
	FVector3d AxisDirection(0, 0, 1);
	FVector3d AxisOrigin(0, 0, 0);

	TArray<FVector3d> ProfileCurve;
	TSet<int32> ProfileVerticesToWeld;
	bool bProfileCurveIsClosed = true;

	for (FVector2d Point : PolygonVertices)
	{
		ProfileCurve.Add(FVector3d(Point.X, 0, Point.Y));
	}
	// unclear why but the sweep code seems to be written for a clockwise ordering...
	Algo::Reverse(ProfileCurve);

	double RisePerDegree = RisePerFullRevolution / 360.0;
	double TotalRevolutionDegrees = FMath::Max(0.01, RevolveDegrees);
	double DegreesPerStep = TotalRevolutionDegrees / (double)Steps;
	double DegreesOffset = DegreeOffset;
	if (bReverseDirection)
	{
		DegreesPerStep *= -1;
		DegreesOffset *= -1;
	}
	double DownAxisOffsetPerStep = TotalRevolutionDegrees * RisePerDegree / Steps;

	if (bProfileAtMidpoint && DegreesPerStep != 0 && FMathd::Abs(DegreesPerStep) < 180)
	{
		RevolveUtil::MakeProfileCurveMidpointOfFirstStep(ProfileCurve, DegreesPerStep, AxisOrigin, AxisDirection);
	}

	// Generate the sweep curve

	TArray<FFrame3d> SweepCurve;
	bool bSweepCurveIsClosed = false;

	//CurveSweepOpOut.bSweepCurveIsClosed = bWeldFullRevolution && AlongAxisOffsetPerDegree == 0 && TotalRevolutionDegrees == 360;
	int32 NumSweepFrames = bSweepCurveIsClosed ? Steps : Steps + 1; // If closed, last sweep frame is also first
	SweepCurve.Reserve(NumSweepFrames);
	RevolveUtil::GenerateSweepCurve(AxisOrigin, AxisDirection, DegreesOffset,
		DegreesPerStep, DownAxisOffsetPerStep, NumSweepFrames, SweepCurve);

	// Weld any vertices that are on the axis
	if (bWeldVertsOnAxis && DownAxisOffsetPerStep == 0)
	{
		RevolveUtil::WeldPointsOnAxis(ProfileCurve, AxisOrigin,
			AxisDirection, 0.1, ProfileVerticesToWeld);
	}

	//CurveSweepOpOut.DiagonalTolerance = DiagonalProportionTolerance;
	//double UVScale = MaterialProperties.UVScale;

	//double UVScale = 1.0;
	//CurveSweepOp.UVScale = FVector2d(UVScale, UVScale);

	// what for?
	//if (bReverseProfileCurve ^ bFlipVs)
	//{
	//	CurveSweepOpOut.UVScale[1] *= -1;
	//	CurveSweepOpOut.UVOffset = FVector2d(0, UVScale);
	//}

	bool bUVsSkipFullyWeldedEdges = true;
	ECapFillMode CapFillMode = (bFillPartialRevolveEndcaps) ?
		ECapFillMode::EarClipping : ECapFillMode::None;		// delaunay? hits ensure...

															// Use a mesh generator to do the work.
	FProfileSweepGenerator CurveSweeper;
	CurveSweeper.ProfileCurve = ProfileCurve;
	CurveSweeper.SweepCurve = SweepCurve;
	CurveSweeper.WeldedVertices = ProfileVerticesToWeld;
	CurveSweeper.bProfileCurveIsClosed = bProfileCurveIsClosed;
	CurveSweeper.bSweepCurveIsClosed = bSweepCurveIsClosed;

	CurveSweeper.bSharpNormals = false; // bSharpNormals;
	CurveSweeper.UVScale = UVScale;
	CurveSweeper.bUVScaleRelativeWorld = bUVScaleRelativeWorld;
	CurveSweeper.UnitUVInWorldCoordinates = UnitUVInWorldCoordinates;
	CurveSweeper.UVOffset = UVOffset;
	CurveSweeper.bUVsSkipFullyWeldedEdges = bUVsSkipFullyWeldedEdges;
	CurveSweeper.PolygonGroupingMode = PolygonGroupingMode;
	CurveSweeper.QuadSplitMethod = QuadSplitMethod;
	CurveSweeper.DiagonalTolerance = DiagonalTolerance;

	FDynamicMesh3 ResultMesh(&CurveSweeper.Generate());

	// Cap the ends if needed
	if (bProfileCurveIsClosed && !bSweepCurveIsClosed && CapFillMode != ECapFillMode::None)
	{
		for (int CapIndex = 0; CapIndex < 2; ++CapIndex)
		{
			// Note that we rely on the sweep generator to tell us the start and end profile curves for
			// capping rather than looking for a boundary ourselves. This is because in cases where the
			// side of the profile curve is welded, there will actually be one open boundary rather than
			// two separate ones, yet we want to cap the two sides separately.
			// This does rely on the fact that the hole fillers we use only care about vertices, and don't
			// rely on there being edges between them. It also relies on the vertex ids in the DynamicMesh
			// matching the ones used in the generator. Without those two facts, the work of filling two
			// halves of an open boundary separately would be harder to do...

			// TODO: curve is planar we don't have to figure this out...

			// Put the profile curve into an edge loop
			TArray<int> VertexLoop = CurveSweeper.EndProfiles[CapIndex];
			int NV = VertexLoop.Num();
			TArray<int32> Edges;
			Edges.SetNum(NV);
			for (int i = 0; i < NV; ++i)
			{
				Edges[i] = ResultMesh.FindEdge(VertexLoop[i], VertexLoop[(i + 1) % NV]);
			}

			//FEdgeLoop::VertexLoopToEdgeLoop(ResultMesh.Get(), , Edges);
			//FEdgeLoop Loop(ResultMesh.Get(), CurveSweeper.EndProfiles[CapIndex], Edges);

			// The first cap actually needs to face in the opposite direction of the sweep to face outward.
			if (CapIndex == 0)
			{
				Algo::Reverse(VertexLoop);
				Algo::Reverse(Edges);
			}

			// Compute a best-fit plane of the vertices
			TArray<FVector3d> VertexPositions;
			for (int32 vid : VertexLoop)
			{
				VertexPositions.Add( ResultMesh.GetVertex(vid) );
			}

			FVector3d PlaneOrigin;
			FVector3d PlaneNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
			PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

			int32 NewGroupID = ResultMesh.AllocateTriangleGroup();

			TArray<FIndex3i> Triangles;
			TArray<int32> NewTriangles;
			PolygonTriangulation::TriangulateSimplePolygon(VertexPositions, Triangles);
			for (FIndex3i PolyTriangle : Triangles)
			{
				FIndex3i MeshTriangle( VertexLoop[PolyTriangle.A], VertexLoop[PolyTriangle.B], VertexLoop[PolyTriangle.C]);
				int32 NewTriangle = ResultMesh.AppendTriangle(MeshTriangle, NewGroupID);
				if (NewTriangle >= 0)
				{
					NewTriangles.Add(NewTriangle);
				}
			}

			// Compute normals and UVs
			if (ResultMesh.HasAttributes())
			{
				UELocal::SetTriangleNormals(&ResultMesh, NewTriangles);

				// Set UVs for the cap
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				if (bUVScaleRelativeWorld)
				{
					FVector2f ScaleToUse(UVScale);
					ScaleToUse /= CurveSweeper.UnitUVInWorldCoordinates;
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, ScaleToUse, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						false); // don't normalize
				}
				else
				{
					UELocal::SetTriangleUVsFromProjection(&ResultMesh, NewTriangles, ProjectionFrame, (FVector2f)UVScale, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						true); // normalize first
				}

			}
		}// end for each loop
	}//end if capping


	return ResultMesh;
}
