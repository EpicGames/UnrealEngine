// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/StairGenerator.h"
#include "MathUtil.h"

using namespace UE::Geometry;

/**
 * FStairGenerator
 */

FStairGenerator::~FStairGenerator() = default;

FMeshShapeGenerator& FStairGenerator::Generate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StairGenerator_Generate);
	
	ResetData();

	switch (StairStyle)
	{
	case EStairStyle::Floating:
		return GenerateFloatingStairs();
	case EStairStyle::Solid:
	default:
		return GenerateSolidStairs();
	}
}

/**
 * Generate a solid stair mesh.
 *
 * Stair topology composition. (side view cross section)
 *
 * (EStairStyle::Solid)
 *                  C___C
 *                  |   |          | z (height)
 *              C___C___I          |
 *              |   |   |          |_____ x (depth)
 *          C___C___|___I           \
 *          |   |   |   |            \ y (width)
 *  Row --> C___I___I___C
 *
 *          ^-- Column
 *
 *          C = corner vertex
 *          I = interior connect quad vertex
 *
 */
FMeshShapeGenerator& FStairGenerator::GenerateSolidStairs()
{
	auto TriangleNumber = [](int x)
	{
		return (x * (x + 1) / 2);
	};

	// Number of quads that connect the left & right sides.
	const int NumConnectQuads = 4 * NumSteps;
	// Number of verts whose adjacent edges on the left & right sides are not parallel.
	const int NumCornerVerts = 4 * (NumSteps + 1);
	NumVertsPerSide = TriangleNumber(NumSteps + 1) + NumSteps;
	NumVerts = 2 * NumVertsPerSide;
	NumQuadsPerSide = TriangleNumber(NumSteps);
	NumQuads = 2 * NumQuadsPerSide + NumConnectQuads;

	// Number of mesh attributes for the side quads assuming sharing.
	const int NumSideQuadAttrs = NumVerts;
	// Number of mesh attributes for the corner vertices.
	const int NumConnectQuadCornerAttrs = 2 * NumCornerVerts;
	// Number of mesh attributes for the interior vertices of the bottom and back connect quads.
	const int NumConnectQuadInteriorAttrs = 4 * (NumSteps - 1);
	NumAttrs = NumConnectQuadCornerAttrs + NumConnectQuadInteriorAttrs + NumSideQuadAttrs;
	SetBufferSizes(NumVerts, 2 * NumQuads, NumAttrs, NumAttrs);

	// Ordered Side lists for iteration.
	constexpr ESide RightLeftSides[2] = { ESide::Right, ESide::Left };
	constexpr ESide AllSides[6] = { ESide::Right, ESide::Left, ESide::Front, ESide::Top, ESide::Back, ESide::Bottom };

	// Generate vertices by vertical column per Right/Left side.
	LeftSideColumnId = (NumSteps + 1);
	VertexIds.SetNum(2 * LeftSideColumnId);
	VertexIdsToColumnRow.SetNum(NumVerts);
	int VertexId = 0;
	for (const ESide& Side : RightLeftSides)
	{
		int StartColumnId = (Side == ESide::Right ? 0 : LeftSideColumnId);
		for (int VertexColumn = 0; VertexColumn < NumSteps + 1; VertexColumn++)
		{
			int NumVertices = (VertexColumn == NumSteps ? VertexColumn + 1 : VertexColumn + 2);
			VertexIds[StartColumnId + VertexColumn].SetNum(NumVertices);
			for (int VertexRow = 0; VertexRow < NumVertices; VertexRow++)
			{
				Vertices[VertexId] = GenerateVertex(Side, VertexColumn, VertexRow);
				VertexIds[StartColumnId + VertexColumn][VertexRow] = VertexId;
				VertexIdsToColumnRow[VertexId] = FIndex2i(StartColumnId + VertexColumn, VertexRow);
				VertexId++;
			}
		}
	}

	// Generate quad representation for each side.
	//
	// Compute these in respective groups for later normal,
	// UV & polygroup assignment.
	FaceDesc.SetNum(4 * NumQuads);
	int FaceDescId = 0;
	for (const ESide& Side : AllSides)
	{
		switch (Side)
		{
		case ESide::Right:
		case ESide::Left:
		{
			const int StartColId = Side == ESide::Right ? 0 : LeftSideColumnId;
			if (Side == ESide::Right)
			{
				RightStartFaceId = FaceDescId;
			}
			else
			{
				LeftStartFaceId = FaceDescId;
			}
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int CurrentColId = StartColId + Step;
				for (int VertexRow = 0; VertexRow < VertexIds[CurrentColId].Num() - 1; VertexRow++)
				{
					FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow]);
					FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow + 1]);
					FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow + 1]);
					FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow]);
				}
			}
			break;
		}
		case ESide::Front:
		{
			FrontStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int LastStepRow = VertexIds[Step].Num() - 1;
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow - 1]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow - 1]);
			}
			break;
		}
		case ESide::Top:
		{
			TopStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int LastStepRow = VertexIds[Step].Num() - 1;
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId + 1][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow]);
			}
			break;
		}
		case ESide::Back:
		{
			BackStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				FaceDesc[FaceDescId++] = (VertexIds[NumSteps][Step]);
				FaceDesc[FaceDescId++] = (VertexIds[NumSteps][Step + 1]);
				FaceDesc[FaceDescId++] = (VertexIds[LeftSideColumnId + NumSteps][Step + 1]);
				FaceDesc[FaceDescId++] = (VertexIds[LeftSideColumnId + NumSteps][Step]);
			}
			break;
		}
		case ESide::Bottom:
		{
			BottomStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				FaceDesc[FaceDescId++] = (VertexIds[Step][0]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1][0]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1 + LeftSideColumnId][0]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][0]);
			}
			break;
		}
		}
	}
	LastFaceId = FaceDescId;

	// Compute Normals & UVs
	NormalDesc.SetNum(4 * NumQuads);
	int NormalId = 0;

	UVDesc.SetNum(4 * NumQuads);
	int UVId = 0;
	float MaxDimension = GetMaxDimension();
	float UVScale = (bScaleUVByAspectRatio) ? (1.0f / (float)MaxDimension) : 1.0f;

	const int NumFrontFaceVertex = TopStartFaceId - FrontStartFaceId;
	const int NumTopFaceVertex = BackStartFaceId - TopStartFaceId;
	const int NumBackFaceVertex = BottomStartFaceId - BackStartFaceId;
	const int NumBottomFaceVertex = LastFaceId - BottomStartFaceId;
	for (const ESide& Side : AllSides)
	{
		switch (Side)
		{
		case ESide::Right:
		case ESide::Left:
		{
			const int StartVertexId = (Side == ESide::Right ? 0 : NumVertsPerSide);
			for (int SideVertex = 0; SideVertex < NumVertsPerSide; SideVertex++)
			{
				int VId = StartVertexId + SideVertex;
				Normals[NormalId++] = GenerateNormal(Side, VId);
				UVs[UVId++] = GenerateUV(Side, SideVertex, VId, UVScale);
			}
			// For the Side Quads, the Normal/UV descriptor is identical to the Face descriptor.
			const int StartSideFaceVertex = (Side == ESide::Right ? 0 : 4 * NumQuadsPerSide);
			const int NumSideFaceVertex = 4 * NumQuadsPerSide;
			for (int SideFaceVertex = 0; SideFaceVertex < NumSideFaceVertex; SideFaceVertex++)
			{
				NormalDesc[StartSideFaceVertex + SideFaceVertex] = FaceDesc[StartSideFaceVertex + SideFaceVertex];
				UVDesc[StartSideFaceVertex + SideFaceVertex] = FaceDesc[StartSideFaceVertex + SideFaceVertex];
			}
			break;
		}
		case ESide::Front:
		case ESide::Top:
		{
			const int NumFaceVertex = (Side == ESide::Front ? NumFrontFaceVertex : NumTopFaceVertex);
			const int StartFaceId = (Side == ESide::Front ? FrontStartFaceId : TopStartFaceId);
			for (int FaceVertexId = 0; FaceVertexId < NumFaceVertex; FaceVertexId++)
			{
				int VId = FaceDesc[StartFaceId + FaceVertexId];
				Normals[NormalId] = GenerateNormal(Side, VId);
				NormalDesc[StartFaceId + FaceVertexId] = NormalId;
				NormalId++;

				UVs[UVId] = GenerateUV(Side, FaceVertexId / 4, VId, UVScale);
				UVDesc[StartFaceId + FaceVertexId] = UVId;
				UVId++;
			}
			break;
		}
		case ESide::Back:
		case ESide::Bottom:
		{
			const int NormalStartId = NormalId;
			const int UVStartId = UVId;
			const int NumFaceVertex = (Side == ESide::Back ? NumBackFaceVertex : NumBottomFaceVertex);
			const int StartFaceId = (Side == ESide::Back ? BackStartFaceId : BottomStartFaceId);
			for (int FaceVertexId = 0; FaceVertexId < NumFaceVertex; FaceVertexId += 4)
			{
				Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId]);
				Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 3]);

				const int SideFaceId = FaceVertexId / 4;
				UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId], UVScale);
				UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 3], UVScale);

				// For the last face, add the last/cap vertices.
				if (FaceVertexId + 4 >= NumFaceVertex)
				{
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 1]);
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 2]);

					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 1], UVScale);
					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 2], UVScale);
				}
			}
			// Assign the normals/UV.
			int FaceVertexId = StartFaceId;
			int FaceNormalId = NormalStartId;
			int FaceUVId = UVStartId;
			for (int FaceId = 0; FaceId < NumSteps; FaceId++)
			{
				// Face descriptor index order for back/bottom
				//
				//         1.___.2
				//          |   |
				//          |___|
				//         0     3
				//
				// The normals are generated along each vertex row
				// in order: (0, 3, 1, 2). Thus the respective normal/UV
				// IDs follow: (0, 2, 3, 1).
				//
				NormalDesc[FaceVertexId] = FaceNormalId;
				NormalDesc[FaceVertexId + 1] = FaceNormalId + 2;
				NormalDesc[FaceVertexId + 2] = FaceNormalId + 3;
				NormalDesc[FaceVertexId + 3] = FaceNormalId + 1;

				UVDesc[FaceVertexId] = FaceUVId;
				UVDesc[FaceVertexId + 1] = FaceUVId + 2;
				UVDesc[FaceVertexId + 2] = FaceUVId + 3;
				UVDesc[FaceVertexId + 3] = FaceUVId + 1;

				FaceVertexId += 4;
				FaceNormalId += 2;
				FaceUVId += 2;
			}
			break;
		}
		}
	}

	// Triangulate quad mesh into the output mesh.
	constexpr int Tris[2][3] = { {0, 3, 2}, {2, 1, 0} };
	int CurrentTriId = 0;
	int GroupId = 0;
	for (int FaceId = 0; FaceId < NumQuads; FaceId++)
	{
		ESide Side = FaceToSide(FaceId);
		
		const bool Flipped = (Side == ESide::Left);
		for (int TriId = 0; TriId < 2; TriId++)
		{
			const int AId = 4 * FaceId + Tris[TriId][0];
			const int BId = 4 * FaceId + Tris[TriId][1];
			const int CId = 4 * FaceId + Tris[TriId][2];
			const int FA = FaceDesc[AId];
			const int FB = FaceDesc[BId];
			const int FC = FaceDesc[CId];
			SetTriangle(CurrentTriId, FA, FB, FC, Flipped);
			const int NA = NormalDesc[AId];
			const int NB = NormalDesc[BId];
			const int NC = NormalDesc[CId];
			SetTriangleNormals(CurrentTriId, NA, NB, NC, Flipped);
			const int UVA = UVDesc[AId];
			const int UVB = UVDesc[BId];
			const int UVC = UVDesc[CId];
			SetTriangleUVs(CurrentTriId, UVA, UVB, UVC, Flipped);
			SetTrianglePolygon(CurrentTriId, GroupId);
			CurrentTriId++;
		}
		
		if (bPolygroupPerQuad)
		{
			GroupId += 1;
		}
		else
		{
			switch (Side)
			{
			// Right, Left, Back and Bottom sides are all in the same group. Since all faces within each side are
			// consecutive we increment GroupId only when the side is going to change in the next iteration
			case ESide::Right:
			case ESide::Left:
			case ESide::Back:
			case ESide::Bottom:
			{
				ESide NextSide = FaceToSide(FaceId + 1 < NumQuads ? FaceId + 1 : FaceId);
				GroupId += (NextSide != Side);
				break;
			}
				
			// Front and Top faces have incrementing GroupIds. Note: Previously faces on the same step were grouped
			// together but we changed that because having non-planar groups made edge loop insertion with
			// retriangulation screw up the step geometry
			case ESide::Front:
			case ESide::Top:
			{
				GroupId += 1;
				break;
			}
			}
		}

	}

	return *this;
}

/**
 * Generate a floating stair mesh.
 *
 * Stair topology composition. (side view cross section)
 *
 * (EStairStyle::Floating)
 *                  C___C
 *                  |   |          | z (height)
 *              C___C___I          |
 *              |   |   |          |_____ x (depth)
 *          C___C___C___C           \
 *          |   |   |                \ y (width)
 *  Row --> C___I___C
 *
 *          ^-- Column
 *
 *          C = corner vertex
 *          I = interior connect quad vertex.
 */
FMeshShapeGenerator& FStairGenerator::GenerateFloatingStairs()
{
	// Number of quads that connect the left & right sides.
	const int NumConnectQuads = 4 * NumSteps;
	// Number of verts whose adjacent edges on the left & right sides are not parallel.
	const int NumCornerVerts = 8 * NumSteps - 4;
	
	NumVertsPerSide = 4 * NumSteps;
	NumVerts = 2 * NumVertsPerSide;
	NumQuadsPerSide = 2 * NumSteps - 1;
	NumQuads = 2 * NumQuadsPerSide + NumConnectQuads;

	// Number of mesh attributes for the side quads assuming sharing.
	const int NumSideQuadAttrs = NumVerts;
	// Number of mesh attributes for the corner vertices.
	const int NumConnectQuadCornerAttrs = 2 * NumCornerVerts;
	// Number of mesh attributes for the interior vertices of the bottom and back connect quads.
	constexpr int NumConnectQuadInteriorAttrs = 4;
	NumAttrs = NumConnectQuadCornerAttrs + NumConnectQuadInteriorAttrs + NumSideQuadAttrs;
	SetBufferSizes(NumVerts, 2 * NumQuads, NumAttrs, NumAttrs);

	// Ordered Side lists for iteration.
	constexpr ESide RightLeftSides[2] = { ESide::Right, ESide::Left };
	constexpr ESide AllSides[6] = { ESide::Right, ESide::Left, ESide::Front, ESide::Top, ESide::Back, ESide::Bottom };

	// Generate vertices by vertical column per Right/Left side.
	LeftSideColumnId = (NumSteps + 1);
	VertexIds.SetNum(2 * LeftSideColumnId);
	VertexIdsToColumnRow.SetNum(NumVerts);
	int VertexId = 0;
	for (const ESide& Side : RightLeftSides)
	{
		int StartColumnId = (Side == ESide::Right ? 0 : LeftSideColumnId);
		for (int VertexColumn = 0; VertexColumn < NumSteps + 1; VertexColumn++)
		{
			int NumVertices = 4;
			if (VertexColumn == 0)
			{
				NumVertices = 2;
			}
			else if (VertexColumn == 1 || VertexColumn == NumSteps)
			{
				NumVertices = 3;
			}
			VertexIds[StartColumnId + VertexColumn].SetNum(NumVertices);
			for (int VertexRow = 0; VertexRow < NumVertices; VertexRow++)
			{
				Vertices[VertexId] = GenerateVertex(Side, VertexColumn, VertexRow);
				VertexIds[StartColumnId + VertexColumn][VertexRow] = VertexId;
				VertexIdsToColumnRow[VertexId] = FIndex2i(StartColumnId + VertexColumn, VertexRow);
				VertexId++;
			}
		}
	}

	// Generate quad representation for each side.
	//
	// Compute these in respective groups for later normal,
	// UV & polygroup assignment.
	FaceDesc.SetNum(4 * NumQuads);
	int FaceDescId = 0;
	for (const ESide& Side : AllSides)
	{
		switch (Side)
		{
		case ESide::Right:
		case ESide::Left:
		{
			const int StartColId = Side == ESide::Right ? 0 : LeftSideColumnId;
			if (Side == ESide::Right)
			{
				RightStartFaceId = FaceDescId;
			}
			else
			{
				LeftStartFaceId = FaceDescId;
			}
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int CurrentColId = StartColId + Step;
				if (Step < 2)
				{
					for (int VertexRow = 0; VertexRow < VertexIds[CurrentColId].Num() - 1; VertexRow++)
					{
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow + 1]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow + 1]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow]);
					}
				}
				else
				{
					for (int VertexRow = 1; VertexRow < VertexIds[CurrentColId].Num() - 1; VertexRow++)
					{
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId][VertexRow + 1]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow]);
						FaceDesc[FaceDescId++] = (VertexIds[CurrentColId + 1][VertexRow - 1]);
					}
				}
			}
			break;
		}
		case ESide::Front:
		{
			FrontStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int LastStepRow = VertexIds[Step].Num() - 1;
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow - 1]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow - 1]);
			}
			break;
		}
		case ESide::Top:
		{
			TopStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int LastStepRow = VertexIds[Step].Num() - 1;
				int NextStepRow = Step < NumSteps - 1 ? VertexIds[Step + 1].Num() - 2 : VertexIds[Step + 1].Num() - 1;
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId + 1][NextStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1][NextStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow]);
			}
			break;
		}
		case ESide::Back:
		{
			BackStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int VertexCol = Step + 2 < NumSteps ? Step + 2 : NumSteps;
				int VertexRow = Step < NumSteps - 1 ? 0 : 1;
				FaceDesc[FaceDescId++] = (VertexIds[VertexCol][VertexRow]);
				FaceDesc[FaceDescId++] = (VertexIds[VertexCol][VertexRow + 1]);
				FaceDesc[FaceDescId++] = (VertexIds[LeftSideColumnId + VertexCol][VertexRow + 1]);
				FaceDesc[FaceDescId++] = (VertexIds[LeftSideColumnId + VertexCol][VertexRow]);
			}
			break;
		}
		case ESide::Bottom:
		{
			BottomStartFaceId = FaceDescId;
			for (int Step = 0; Step < NumSteps; Step++)
			{
				int LastStepRow = Step < 2 ? 0 : 1;
				FaceDesc[FaceDescId++] = (VertexIds[Step][LastStepRow]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1][0]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + 1 + LeftSideColumnId][0]);
				FaceDesc[FaceDescId++] = (VertexIds[Step + LeftSideColumnId][LastStepRow]);
			}
			break;
		}
		}
	}
	LastFaceId = FaceDescId;

	auto IsUnsharedFace = [this](ESide Side, int SideFaceId) -> bool
	{
		switch (Side)
		{
		case ESide::Right:
		case ESide::Left:
			return false;
		case ESide::Front:
		case ESide::Top:
			return true;
		case ESide::Back:
			return SideFaceId < NumSteps - 2;
		case ESide::Bottom:
			return SideFaceId > 1;
		}
		return true;
	};

	// Compute Normals & UVs
	NormalDesc.SetNum(4 * NumQuads);
	int NormalId = 0;

	UVDesc.SetNum(4 * NumQuads);
	int UVId = 0;
	float MaxDimension = GetMaxDimension();
	float UVScale = (bScaleUVByAspectRatio) ? (1.0f / (float)MaxDimension) : 1.0f;

	const int NumFrontFaceVertex = TopStartFaceId - FrontStartFaceId;
	const int NumTopFaceVertex = BackStartFaceId - TopStartFaceId;
	const int NumBackFaceVertex = BottomStartFaceId - BackStartFaceId;
	const int NumBottomFaceVertex = LastFaceId - BottomStartFaceId;
	for (const ESide& Side : AllSides)
	{
		switch (Side)
		{
		case ESide::Right:
		case ESide::Left:
		{
			const int StartVertexId = (Side == ESide::Right ? 0 : NumVertsPerSide);
			for (int SideVertex = 0; SideVertex < NumVertsPerSide; SideVertex++)
			{
				int VId = StartVertexId + SideVertex;
				Normals[NormalId++] = GenerateNormal(Side, VId);
				UVs[UVId++] = GenerateUV(Side, SideVertex, VId, UVScale);
			}
			// For the Side Quads, the Normal/UV descriptor is identical to the Face descriptor.
			const int StartSideFaceVertex = (Side == ESide::Right ? 0 : 4 * NumQuadsPerSide);
			const int NumSideFaceVertex = 4 * NumQuadsPerSide;
			for (int SideFaceVertex = 0; SideFaceVertex < NumSideFaceVertex; SideFaceVertex++)
			{
				NormalDesc[StartSideFaceVertex + SideFaceVertex] = FaceDesc[StartSideFaceVertex + SideFaceVertex];
				UVDesc[StartSideFaceVertex + SideFaceVertex] = FaceDesc[StartSideFaceVertex + SideFaceVertex];
			}
			break;
		}
		case ESide::Front:
		case ESide::Top:
		{
			const int NumFaceVertex = (Side == ESide::Front ? NumFrontFaceVertex : NumTopFaceVertex);
			const int StartFaceId = (Side == ESide::Front ? FrontStartFaceId : TopStartFaceId);
			for (int FaceVertexId = 0; FaceVertexId < NumFaceVertex; FaceVertexId++)
			{
				int VId = FaceDesc[StartFaceId + FaceVertexId];
				Normals[NormalId] = GenerateNormal(Side, VId);
				NormalDesc[StartFaceId + FaceVertexId] = NormalId;
				NormalId++;

				UVs[UVId] = GenerateUV(Side, FaceVertexId / 4, VId, UVScale);
				UVDesc[StartFaceId + FaceVertexId] = UVId;
				UVId++;
			}
			break;
		}
		case ESide::Back:
		case ESide::Bottom:
		{
			const int NormalStartId = NormalId;
			const int UVStartId = UVId;
			const int NumFaceVertex = (Side == ESide::Back ? NumBackFaceVertex : NumBottomFaceVertex);
			const int StartFaceId = (Side == ESide::Back ? BackStartFaceId : BottomStartFaceId);
			for (int FaceVertexId = 0; FaceVertexId < NumFaceVertex; FaceVertexId += 4)
			{
				const int SideFaceId = FaceVertexId / 4;
				if (IsUnsharedFace(Side, SideFaceId))
				{
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId]);
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 3]);
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 1]);
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 2]);

					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId], UVScale);
					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 3], UVScale);
					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 1], UVScale);
					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 2], UVScale);
				}
				else
				{
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId]);
					Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 3]);

					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId], UVScale);
					UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 3], UVScale);

					// For the last shared face, add the last/cap vertices.
					if ((Side == ESide::Back && SideFaceId == NumSteps - 1) ||
						(Side == ESide::Bottom && SideFaceId == 1))
					{
						Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 1]);
						Normals[NormalId++] = GenerateNormal(Side, FaceDesc[StartFaceId + FaceVertexId + 2]);

						UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 1], UVScale);
						UVs[UVId++] = GenerateUV(Side, SideFaceId, FaceDesc[StartFaceId + FaceVertexId + 2], UVScale);
					}
				}
			}
			// Assign the normals/UVs.
			int FaceVertexId = StartFaceId;
			int FaceNormalId = NormalStartId;
			int FaceUVId = UVStartId;
			for (int FaceId = 0; FaceId < NumSteps; FaceId++)
			{
				// Face descriptor index order for back/bottom
				//
				//         1.___.2
				//          |   |
				//          |___|
				//         0     3
				//
				// The normals/UVs are generated along each vertex row
				// in order: (0, 3, 1, 2). Thus the respective normal/UV
				// IDs follow: (0, 2, 3, 1).
				//
				NormalDesc[FaceVertexId] = FaceNormalId;
				NormalDesc[FaceVertexId + 1] = FaceNormalId + 2;
				NormalDesc[FaceVertexId + 2] = FaceNormalId + 3;
				NormalDesc[FaceVertexId + 3] = FaceNormalId + 1;
				UVDesc[FaceVertexId] = FaceUVId;
				UVDesc[FaceVertexId + 1] = FaceUVId + 2;
				UVDesc[FaceVertexId + 2] = FaceUVId + 3;
				UVDesc[FaceVertexId + 3] = FaceUVId + 1;
				FaceVertexId += 4;

				// The last shared face on the bottom should increment like
				// an unshared face so that subsequent bottom faces reference
				// the correct attributes.
				if (IsUnsharedFace(Side, FaceId) || (Side == ESide::Bottom && FaceId == 1))
				{
					FaceNormalId += 4;
					FaceUVId += 4;
				}
				else
				{
					FaceNormalId += 2;
					FaceUVId += 2;
				}
			}
			break;
		}
		}
	}

	// Triangulate quad mesh into the output mesh.
	constexpr int Tris[2][3] = { {0, 3, 2}, {2, 1, 0} };
	int CurrentTriId = 0;
	int GroupId = 0;
	for (int FaceId = 0; FaceId < NumQuads; FaceId++)
	{
		ESide Side = FaceToSide(FaceId);

		const bool Flipped = (Side == ESide::Left);
		for (int TriId = 0; TriId < 2; TriId++)
		{
			const int AId = 4 * FaceId + Tris[TriId][0];
			const int BId = 4 * FaceId + Tris[TriId][1];
			const int CId = 4 * FaceId + Tris[TriId][2];
			const int FA = FaceDesc[AId];
			const int FB = FaceDesc[BId];
			const int FC = FaceDesc[CId];
			SetTriangle(CurrentTriId, FA, FB, FC, Flipped);
			const int NA = NormalDesc[AId];
			const int NB = NormalDesc[BId];
			const int NC = NormalDesc[CId];
			SetTriangleNormals(CurrentTriId, NA, NB, NC, Flipped);
			const int UVA = UVDesc[AId];
			const int UVB = UVDesc[BId];
			const int UVC = UVDesc[CId];
			SetTriangleUVs(CurrentTriId, UVA, UVB, UVC, Flipped);
			SetTrianglePolygon(CurrentTriId, GroupId);
			CurrentTriId++;
		}

		if (bPolygroupPerQuad)
		{
			GroupId += 1;
		}
		else
		{
			switch (Side)
			{
			// Right and Left sides are all in the same group. Since all faces within each side are
			// consecutive we increment GroupId only when the side is going to change in the next iteration
			case ESide::Right:
			case ESide::Left:
			{
				ESide NextSide = FaceToSide(FaceId + 1 < NumQuads ? FaceId + 1 : FaceId);
				GroupId += (NextSide != Side);
				break;
			}
				
			// Front/Top and Back/Bottom faces have incrementing GroupIds. Note: Previously faces on the same step were
			// grouped together but we changed that because having non-planar groups made edge loop insertion with
			// retriangulation screw up the step geometry
			case ESide::Front:
			case ESide::Top:
			case ESide::Back:
			case ESide::Bottom:
			{
				GroupId += 1;
				break;
			}
			}
		}
		
	}

	return *this;
}

/**
 * Helper to identify the side based on the face index.
 *
 * This method is only valid after the face description array has been
 * initialized.
 */
FStairGenerator::ESide FStairGenerator::FaceToSide(int FaceId)
{
	ESide Side = ESide::Right;
	const int FaceVertexId = FaceId * 4;
	if (FaceVertexId >= RightStartFaceId && FaceVertexId < LeftStartFaceId)
	{
		Side = ESide::Right;
	}
	else if (FaceVertexId >= LeftStartFaceId && FaceVertexId < FrontStartFaceId)
	{
		Side = ESide::Left;
	}
	else if (FaceVertexId >= FrontStartFaceId && FaceVertexId < TopStartFaceId)
	{
		Side = ESide::Front;
	}
	else if (FaceVertexId >= TopStartFaceId && FaceVertexId < BackStartFaceId)
	{
		Side = ESide::Top;
	}
	else if (FaceVertexId >= BackStartFaceId && FaceVertexId < BottomStartFaceId)
	{
		Side = ESide::Back;
	}
	else if (FaceVertexId >= BottomStartFaceId && FaceVertexId < LastFaceId)
	{
		Side = ESide::Bottom;
	}
	else
	{
		check(false);
	}
	return Side;
}


/**
 * Reset state data on the generator.
 *
 * This is invoked at the head of the Generate() method.
 */
void FStairGenerator::ResetData()
{
	if (NumSteps < 2)
	{
		NumSteps = 2;
	}

	VertexIds.Reset();
	VertexIdsToColumnRow.Reset();
	FaceDesc.Reset();
	NormalDesc.Reset();
	UVDesc.Reset();

	NumQuadsPerSide = 0;
	NumQuads = 0;
	NumVertsPerSide = 0;
	NumVerts = 0;
	NumAttrs = 0;

	RightSideColumnId = 0;
	LeftSideColumnId = 0;

	RightStartFaceId = 0;
	LeftStartFaceId = 0;
	FrontStartFaceId = 0;
	TopStartFaceId = 0;
	BackStartFaceId = 0;
	BottomStartFaceId = 0;
	LastFaceId = 0;
}


/**
 * FLinearStairGenerator
 */


FVector3d FLinearStairGenerator::GenerateVertex(ESide Side, int VertexColumn, int VertexRow)
{
	if (Side != ESide::Right && Side != ESide::Left)
	{
		// Vertices are only generated for Right & Left sides.
		check(false);
	}
	float X = float(VertexColumn) * StepDepth;
	float Y = (Side == ESide::Right ? float(0.5 * StepWidth) : -float(0.5 * StepWidth));
	float Z = float(VertexRow) * StepHeight;
	return FVector3d(X, Y, Z);
}

FVector3f FLinearStairGenerator::GenerateNormal(ESide Side, int VertexId)
{
	FVector3f N;
	switch (Side)
	{
	case ESide::Right:
		N = FVector3f::UnitY();
		break;
	case ESide::Left:
		N = -FVector3f::UnitY();
		break;
	case ESide::Front:
		N = -FVector3f::UnitX();
		break;
	case ESide::Top:
		N = FVector3f::UnitZ();
		break;
	case ESide::Back:
		N = FVector3f::UnitX();
		break;
	case ESide::Bottom:
		N = -FVector3f::UnitZ();
		break;
	default:
		check(false);
		break;
	}
	return N;
}

FVector2f FLinearStairGenerator::GenerateUV(ESide Side, int Step, int VertexId, float UVScale)
{
	const int Col = VertexIdsToColumnRow[VertexId].A;
	const int Row = VertexIdsToColumnRow[VertexId].B;
	FVector2f UV(0.0f, 0.0f);
	switch (Side)
	{
	case ESide::Right:
	case ESide::Left:
	{
		const float UScale = float(NumSteps) * StepDepth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		UV.X = FMathf::Lerp(-0.5f, 0.5f, float(Col % LeftSideColumnId) / float(NumSteps + 1)) * UScale + 0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Row) / float(NumSteps + 1)) * VScale + 0.5f;
		break;
	}
	case ESide::Front:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = StepHeight * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = Row == VertexIds[Col].Num() - 1 ? 0.5f : -0.5f;
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Top:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = StepDepth * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = (Col % LeftSideColumnId) > Step ? -0.5f : 0.5f;
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Back:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Row) / float(NumSteps + 1));
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Bottom:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = float(NumSteps) * StepDepth * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Col % LeftSideColumnId) / float(NumSteps + 1));
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	}
	return UV;
}

float FLinearStairGenerator::GetMaxDimension()
{
	return FMathf::Max(FMathf::Max(float(NumSteps) * StepDepth, float(NumSteps) * StepHeight), StepWidth);
}


/**
 * FFloatingStairGenerator
 */

FVector3d FFloatingStairGenerator::GenerateVertex(ESide Side, int VertexColumn, int VertexRow)
{
	if (Side != ESide::Right && Side != ESide::Left)
	{
		// Vertices are only generated for Right & Left sides.
		check(false);
	}
	float X = float(VertexColumn) * StepDepth;
	float Y = (Side == ESide::Right ? float(0.5 * StepWidth) : -float(0.5 * StepWidth));
	float Z = VertexColumn > 1 ? float((VertexColumn - 2) + VertexRow) * StepHeight : float(VertexRow) * StepHeight;
	return FVector3d(X, Y, Z);
}

FVector2f FFloatingStairGenerator::GenerateUV(ESide Side, int Step, int VertexId, float UVScale)
{
	const int Col = VertexIdsToColumnRow[VertexId].A;
	const int Row = VertexIdsToColumnRow[VertexId].B;
	FVector2f UV(0.0f, 0.0f);
	switch (Side)
	{
	case ESide::Right:
	case ESide::Left:
	{
		const float UScale = float(NumSteps) * StepDepth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		float X = float(Col % LeftSideColumnId);
		float Y = X < 3.0f ? float(Row) : float(Row) + X - 2.0f;
		X /= float(NumSteps + 1);
		Y /= float(NumSteps + 1);
		UV.X = FMathf::Lerp(-0.5f, 0.5f, X) * UScale + 0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, Y) * VScale + 0.5f;
		break;
	}
	case ESide::Back:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		float Y = Step < NumSteps - 1 ? float(Row + Step) : float(Row + Step - 1);
		Y /= float(NumSteps + 1);
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, Y);
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	default:
	{
		UV = Super::GenerateUV(Side, Step, VertexId, UVScale);
		break;
	}
	}
	return UV;
}


/**
 * FCurvedStairGenerator
 */

void FCurvedStairGenerator::ResetData()
{
	Super::ResetData();

	bIsClockwise = CurveAngle > 0.0f;
	CurveRadians = CurveAngle * TMathUtilConstants<float>::DegToRad;
	CurveRadiansPerStep = CurveRadians / float(NumSteps);
	OuterRadius = InnerRadius + StepWidth;
	RadiusRatio = OuterRadius / InnerRadius;
	BackNormal = FVector3f::Zero();
}

FVector3d FCurvedStairGenerator::GenerateVertex(ESide Side, int VertexColumn, int VertexRow)
{
	if (Side != ESide::Right && Side != ESide::Left)
	{
		// Vertices are only generated for Right & Left sides.
		check(false);
	}
	float X = 0.0f;
	float Y = 0.0f;
	float Z = float(VertexRow) * StepHeight;

	float XCoeff = FMathf::Cos(float(VertexColumn) * CurveRadiansPerStep);
	float YCoeff = FMathf::Sin(float(VertexColumn) * CurveRadiansPerStep);
	if (bIsClockwise)
	{
		X = (Side == ESide::Right ? XCoeff * InnerRadius : XCoeff * OuterRadius);
		Y = (Side == ESide::Right ? YCoeff * InnerRadius : YCoeff * OuterRadius);
	}
	else
	{
		X = (Side == ESide::Right ? XCoeff * -OuterRadius : XCoeff * -InnerRadius);
		Y = (Side == ESide::Right ? YCoeff * -OuterRadius : YCoeff * -InnerRadius);
	}
	return FVector3d(X, Y, Z);
}

FVector3f FCurvedStairGenerator::GenerateNormal(ESide Side, int VertexId)
{
	const int Col = VertexIdsToColumnRow[VertexId].A;
	const int Row = VertexIdsToColumnRow[VertexId].B;

	FVector3f N = FVector3f::ZeroVector;
	switch (Side)
	{
	case ESide::Right:
	case ESide::Left:
	{
		float X = FMathf::Cos(float(Col % (NumSteps + 1)) * CurveRadiansPerStep);
		float Y = FMathf::Sin(float(Col % (NumSteps + 1)) * CurveRadiansPerStep);
		N = (Side == ESide::Right ? FVector3f(-X, -Y, 0.0f) : FVector3f(X, Y, 0.0f));
		Normalize(N);
		break;
	}
	case ESide::Front:
	{
		float X = FMathf::Cos(float(Col % (NumSteps + 1)) * CurveRadiansPerStep);
		float Y = FMathf::Sin(float(Col % (NumSteps + 1)) * CurveRadiansPerStep);
		N = FVector3f(Y, -X, 0.0f);
		Normalize(N);
		break;
	}
	case ESide::Top:
	{
		N = FVector3f::UnitZ();
		break;
	}
	case ESide::Back:
	{
		if (BackNormal == FVector3f::Zero())
		{
			float X = FMathf::Cos(float(NumSteps) * CurveRadiansPerStep);
			float Y = FMathf::Sin(float(NumSteps) * CurveRadiansPerStep);
			BackNormal = FVector3f(-Y, X, 0.0f);
		}
		N = BackNormal;
		break;
	}
	case ESide::Bottom:
	{
		N = -FVector3f::UnitZ();
		break;
	}
	}
	return N;
}

FVector2f FCurvedStairGenerator::GenerateUV(ESide Side, int Step, int VertexId, float UVScale)
{
	const int Col = VertexIdsToColumnRow[VertexId].A;
	const int Row = VertexIdsToColumnRow[VertexId].B;
	FVector2f UV;
	switch (Side)
	{
	case ESide::Right:
	case ESide::Left:
	{
		const float UScale = OuterRadius * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		UV.X = FMathf::Lerp(-0.5f, 0.5f, float(Col % LeftSideColumnId) / float(NumSteps + 1));
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Row) / float(NumSteps + 1));

		// Proportionally scale the UVs of each side according
		// to the RadiusRatio and ensure [0,1].
		if (RadiusRatio * UScale > 1.0f)
		{
			UV.X /= (Side == ESide::Left ? UScale : UScale * RadiusRatio);
		}
		else if (Side == ESide::Left)
		{
			UV.X *= RadiusRatio;
		}
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Front:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = StepHeight * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = Row == VertexIds[Col].Num() - 1 ? -0.5f : 0.5f;
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Top:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = OuterRadius / float(NumSteps) * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = (Col % LeftSideColumnId) > Step ? -0.5f : 0.5f;

		// Proportionally scale the outer edge of top faces
		// to avoid stretching while ensuring [0,1].
		if (RadiusRatio * VScale > 1.0f)
		{
			UV.Y /= (Col >= LeftSideColumnId ? VScale : VScale * RadiusRatio);
		}
		else if (Col >= LeftSideColumnId)
		{
			UV.Y *= RadiusRatio;
		}

		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Back:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Row) / float(NumSteps + 1));
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Bottom:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = OuterRadius * UVScale;
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, float(Col % LeftSideColumnId) / float(NumSteps + 1));

		// Proportionally scale outer edge of bottom faces
		// to avoid stretching while ensuring [0,1].
		if (RadiusRatio * VScale > 1.0f)
		{
			UV.Y /= (Col >= LeftSideColumnId ? VScale : VScale * RadiusRatio);
		}
		else if (Col >= LeftSideColumnId)
		{
			UV.Y *= RadiusRatio;
		}
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	default:
	{
		check(false);
		break;
	}
	}
	return UV;
}

float FCurvedStairGenerator::GetMaxDimension()
{
	float MaxDepth = FMathf::Abs(CurveRadians) * OuterRadius;
	return FMathf::Max(FMathf::Max(MaxDepth, float(NumSteps) * StepHeight), StepWidth);
}


/**
 * FSpiralStairGenerator
 */

FVector3d FSpiralStairGenerator::GenerateVertex(ESide Side, int VertexColumn, int VertexRow)
{
	if (Side != ESide::Right && Side != ESide::Left)
	{
		// Vertices are only generated for Right & Left sides.
		check(false);
	}
	float X = 0.0f;
	float Y = 0.0f;
	float Z = VertexColumn > 1 ? float((VertexColumn - 2) + VertexRow) * StepHeight : float(VertexRow) * StepHeight;

	float XCoeff = FMathf::Cos(float(VertexColumn) * CurveRadiansPerStep);
	float YCoeff = FMathf::Sin(float(VertexColumn) * CurveRadiansPerStep);
	if (bIsClockwise)
	{
		X = (Side == ESide::Right ? XCoeff * InnerRadius : XCoeff * OuterRadius);
		Y = (Side == ESide::Right ? YCoeff * InnerRadius : YCoeff * OuterRadius);
	}
	else
	{
		X = (Side == ESide::Right ? XCoeff * -OuterRadius : XCoeff * -InnerRadius);
		Y = (Side == ESide::Right ? YCoeff * -OuterRadius : YCoeff * -InnerRadius);
	}
	return FVector3d(X, Y, Z);
}

FVector2f FSpiralStairGenerator::GenerateUV(ESide Side, int Step, int VertexId, float UVScale)
{
	const int Col = VertexIdsToColumnRow[VertexId].A;
	const int Row = VertexIdsToColumnRow[VertexId].B;
	FVector2f UV;
	switch (Side)
	{
	case ESide::Right:
	case ESide::Left:
	{
		const float UScale = OuterRadius * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		float X = float(Col % LeftSideColumnId);
		float Y = X < 3.0f ? float(Row) : float(Row) + X - 2.0f;
		UV.X = FMathf::Lerp(-0.5f, 0.5f, X / float(NumSteps + 1));
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, Y / float(NumSteps + 1));

		// Proportionally scale the UVs of each side according
		// to the RadiusRatio and ensure [0,1].
		if (RadiusRatio * UScale > 1.0f)
		{
			UV.X /= (Side == ESide::Left ? UScale : UScale * RadiusRatio);
		}
		else if (Side == ESide::Left)
		{
			UV.X *= RadiusRatio;
		}
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	case ESide::Back:
	{
		const float UScale = StepWidth * UVScale;
		const float VScale = float(NumSteps) * StepHeight * UVScale;
		float Y = Step < NumSteps - 1 ? float(Row + Step) : float(Row + Step - 1);
		Y /= float(NumSteps + 1);
		UV.X = Col < LeftSideColumnId ? 0.5f : -0.5f;
		UV.Y = FMathf::Lerp(-0.5f, 0.5f, Y);
		UV.X = UV.X * UScale + 0.5f;
		UV.Y = UV.Y * VScale + 0.5f;
		break;
	}
	default:
	{
		UV = Super::GenerateUV(Side, Step, VertexId, UVScale);
		break;
	}
	}
	return UV;
}
