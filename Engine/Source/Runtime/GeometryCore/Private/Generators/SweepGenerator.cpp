// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/SweepGenerator.h"

#include "Async/ParallelFor.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Curve/CurveUtil.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"

using namespace UE::Geometry;

void FSweepGeneratorBase::ConstructMeshTopology(const FPolygon2d& CrossSection,
	const TArrayView<const int32>& UVSections,
	const TArrayView<const int32>& NormalSections,
	const TArrayView<const int32>& SharpNormalsAlongLength,
	bool bEvenlySpaceUVs,
	const TArrayView<const FVector3d>& Path,
	int32 NumCrossSections,
	bool bLoop,
	const ECapType Caps[2],
	FVector2f SectionsUVScale,
	FVector2f CapUVScale,
	FVector2f CapUVOffset,
	const TArrayView<const float>& CustomCrossSectionTexCoord,
	const TArrayView<const float>& CustomPathTexCoord)
{
	// per cross section
	const int32 XVerts = CrossSection.VertexCount();
	const int32 XSegments = bProfileCurveIsClosed ? XVerts : XVerts - 1;
	const int32 XNormals = XVerts + NormalSections.Num();
	const int32 XUVs = XSegments + UVSections.Num() + 1;

	float TotalPerimeter = 0.0f, TotalPathLength = 0.0f;
	TArray<float> CrossSectionTexCoord, PathTexCoord;

	// Compute texture coordinates along the cross section (U coordinates)
	if (bProfileCurveIsClosed && CustomCrossSectionTexCoord.Num() >= XVerts + 1)
	{
		CrossSectionTexCoord = CustomCrossSectionTexCoord;
	}
	else if (bProfileCurveIsClosed && CustomCrossSectionTexCoord.Num() == XVerts)
	{
		CrossSectionTexCoord = CustomCrossSectionTexCoord;

		// If the cross section curve is closed and we are missing texture coordinate for the 
		// last element we wrap araound and use the coordinate of the first element
		CrossSectionTexCoord.Add(CrossSectionTexCoord[0]);
	}
	else if (bProfileCurveIsClosed == false && CustomCrossSectionTexCoord.Num() >= XVerts)
	{
		CrossSectionTexCoord = CustomCrossSectionTexCoord;
	}
	else if (bEvenlySpaceUVs)
	{
		CrossSectionTexCoord.Add(0.0f);
		for (int Idx = 0; Idx < XSegments; Idx++)
		{
			float SegLen = float(Distance(CrossSection[Idx], CrossSection[(Idx + 1) % XSegments]));
			TotalPerimeter += SegLen;
			CrossSectionTexCoord.Add(TotalPerimeter);
		}
		TotalPerimeter = FMath::Max(TotalPerimeter, FMathf::ZeroTolerance);
		for (int Idx = 0; Idx < CrossSectionTexCoord.Num(); Idx++)
		{
			CrossSectionTexCoord[Idx] /= TotalPerimeter;
			CrossSectionTexCoord[Idx] = 1.0f - CrossSectionTexCoord[Idx];
		}
	}
	else
	{
		for (int Idx = 0; Idx < XSegments; Idx++)
		{
			float U = float(Idx) / float(XSegments);
			CrossSectionTexCoord.Add(1.0f - U);
		}
		CrossSectionTexCoord.Add(0.0f);
	}

	// Compute texture coordinates along the path (V coordinates)
	if (CustomPathTexCoord.Num() >= NumCrossSections)
	{
		PathTexCoord = CustomPathTexCoord;
	}
	else if (bLoop && CustomPathTexCoord.Num() == NumCrossSections - 1)
	{
		PathTexCoord = CustomPathTexCoord;

		// If the path curve is closed and we are missing texture coordinate for the 
		// last element we wrap araound and use the coordinate of the first element
		PathTexCoord.Add(PathTexCoord[0]);
	}
	else if (bEvenlySpaceUVs)
	{
		PathTexCoord.Add(0.0f);
		int NumPathSegs = bLoop ? Path.Num() : Path.Num() - 1;
		for (int Idx = 0; Idx < NumPathSegs; Idx++)
		{
			float SegLen = float(Distance(Path[Idx], Path[(Idx + 1) % Path.Num()]));
			TotalPathLength += SegLen;
			PathTexCoord.Add(TotalPathLength);
		}
		TotalPathLength = FMath::Max(TotalPathLength, FMathf::ZeroTolerance);
		for (int Idx = 0; Idx < PathTexCoord.Num(); Idx++)
		{
			PathTexCoord[Idx] /= TotalPathLength;
			PathTexCoord[Idx] = 1.0f - PathTexCoord[Idx];
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < NumCrossSections; Idx++)
		{
			float V = float(Idx) / float(NumCrossSections - 1);
			PathTexCoord.Add(1.0f - V);
		}
	}

	int32 NumVerts = XVerts * NumCrossSections - (bLoop ? XVerts : 0);
	int32 NumNormals = NumCrossSections > 1 ? (XNormals * NumCrossSections - (bLoop ? XNormals : 0)) : 0;
	NumNormals += XNormals * SharpNormalsAlongLength.Num();
	int32 NumUVs = NumCrossSections > 1 ? XUVs * NumCrossSections : 0;
	int32 NumPolygons = (NumCrossSections - 1) * XSegments;
	int32 NumTriangles = NumPolygons * 2;

	TArray<FIndex3i> OutTriangles;

	// doesn't make sense to have cap types if the sweep is a loop
	ensure(!bLoop || (Caps[0] == ECapType::None && Caps[1] == ECapType::None));

	if (!bLoop)
	{
		for (int32 CapIdx = 0; !bLoop && CapIdx < 2; CapIdx++)
		{
			CapVertStart[CapIdx] = NumVerts;
			CapNormalStart[CapIdx] = NumNormals;
			CapUVStart[CapIdx] = NumUVs;
			CapTriangleStart[CapIdx] = NumTriangles;
			CapPolygonStart[CapIdx] = NumPolygons;

			if (Caps[CapIdx] == ECapType::FlatTriangulation)
			{
				NumTriangles += XVerts - 2;
				NumPolygons++;
				NumUVs += XVerts;
				NumNormals += XVerts;
			}
			else if (Caps[CapIdx] == ECapType::FlatMidpointFan)
			{
				NumTriangles += XSegments;
				NumPolygons++;
				NumUVs += XVerts + 1;
				NumNormals += XVerts + 1;
				NumVerts += 1;
			}
			// TODO: support more cap type; e.g.:
			//else if (Caps[CapIdx] == ECapType::Cone)
			//{
			//	NumTriangles += XVerts;
			//	NumPolygons += XVerts;
			//	NumUVs += XVerts + 1;
			//	NumNormals += XVerts * 2;
			//	NumVerts += 1;
			//}
		}
	}

	SetBufferSizes(NumVerts, NumTriangles, NumUVs, NumNormals);

	if (!bLoop)
	{
		for (int32 CapIdx = 0; CapIdx < 2; CapIdx++)
		{
			if (Caps[CapIdx] == ECapType::FlatTriangulation)
			{
				int32 VertOffset = CapIdx * (XVerts * (NumCrossSections - 1));

				PolygonTriangulation::TriangulateSimplePolygon(CrossSection.GetVertices(), OutTriangles);
				int32 TriIdx = CapTriangleStart[CapIdx];
				int32 PolyIdx = CapPolygonStart[CapIdx];
				for (const FIndex3i& Triangle : OutTriangles)
				{
					bool Flipped = CapIdx == 0;
					SetTriangle(TriIdx,
						Triangle.A + VertOffset, Triangle.B + VertOffset, Triangle.C + VertOffset,
						Flipped);
					SetTriangleUVs(TriIdx,
						Triangle.A + CapUVStart[CapIdx],
						Triangle.B + CapUVStart[CapIdx],
						Triangle.C + CapUVStart[CapIdx],
						Flipped);
					SetTriangleNormals(TriIdx,
						Triangle.A + CapNormalStart[CapIdx],
						Triangle.B + CapNormalStart[CapIdx],
						Triangle.C + CapNormalStart[CapIdx],
						Flipped);
					SetTrianglePolygon(TriIdx, PolyIdx);
					TriIdx++;
				}
				float SideScale = float(2 * CapIdx - 1);
				for (int32 Idx = 0; Idx < XVerts; Idx++)
				{
					FVector2f CenteredVert = (FVector2f)CrossSection.GetVertices()[Idx] * CapUVScale + CapUVOffset;
					SetUV(CapUVStart[CapIdx] + Idx, FVector2f(CenteredVert.X * SideScale, CenteredVert.Y), VertOffset + Idx);

					// correct normal to be filled by subclass
					SetNormal(CapNormalStart[CapIdx] + Idx, FVector3f::Zero(), VertOffset + Idx);
				}
			}
			else if (Caps[CapIdx] == ECapType::FlatMidpointFan)
			{
				int32 VertOffset = CapIdx * (XVerts * (NumCrossSections - 1));
				int32 CapVertStartIdx = CapVertStart[CapIdx];
				int32 TriIdx = CapTriangleStart[CapIdx];
				int32 PolyIdx = CapPolygonStart[CapIdx];
				for (int32 VertIdx = 0; VertIdx < XSegments; VertIdx++)
				{
					bool Flipped = CapIdx == 0;
					SetTriangle(TriIdx,
						VertOffset + VertIdx,
						CapVertStartIdx,
						VertOffset + (VertIdx + 1) % XVerts,
						Flipped);
					SetTriangleUVs(TriIdx,
						CapUVStart[CapIdx] + VertIdx,
						CapUVStart[CapIdx] + XVerts,
						CapUVStart[CapIdx] + (VertIdx + 1) % XVerts,
						Flipped);
					SetTriangleNormals(TriIdx,
						CapNormalStart[CapIdx] + VertIdx,
						CapNormalStart[CapIdx] + XVerts,
						CapNormalStart[CapIdx] + (VertIdx + 1) % XVerts,
						Flipped);
					SetTrianglePolygon(TriIdx, PolyIdx);
					TriIdx++;
				}

				// Set cap midpoint UV & Normal
				// (correct normal to be filled by subclass)
				SetUV(CapUVStart[CapIdx] + XVerts, FVector2f::Zero() + CapUVOffset, CapVertStartIdx);
				SetNormal(CapNormalStart[CapIdx] + XVerts, FVector3f::Zero(), CapVertStartIdx);

				// Set cap profile UVs & Normal
				for (int32 Idx = 0; Idx < XVerts; Idx++)
				{
					FVector2f CenteredVert = (FVector2f)CrossSection.GetVertices()[Idx] * CapUVScale + CapUVOffset;
					SetUV(CapUVStart[CapIdx] + Idx, FVector2f(CenteredVert.X, CenteredVert.Y), VertOffset + Idx);
					SetNormal(CapNormalStart[CapIdx] + Idx, FVector3f::Zero(), VertOffset + Idx);
				}
			}
		}
	}

	// fill in UVs and triangles along length
	int MinValidCrossSections = bLoop ? 3 : 2;
	int CurFaceGroupIndex = NumPolygons;
	if (NumCrossSections >= MinValidCrossSections)
	{
		int32 UVSection = 0, UVSubIdx = 0;

		int CrossSectionsMod = NumCrossSections;
		if (bLoop)
		{
			CrossSectionsMod--; // last cross section becomes the first
		}
		int NormalCrossSectionsMod = CrossSectionsMod + SharpNormalsAlongLength.Num();

		int32 NumSections = UVSections.Num();
		int32 NextDupVertIdx = UVSection < NumSections ? UVSections[UVSection] : -1;
		for (int32 VertSubIdx = 0; VertSubIdx < XSegments; UVSubIdx++)
		{
			for (int32 XIdx = 0; XIdx < NumCrossSections; XIdx++)
			{
				FVector2f UV = FVector2f(CrossSectionTexCoord[VertSubIdx], PathTexCoord[XIdx]);
				SetUV(XIdx * XUVs + UVSubIdx, UV * SectionsUVScale, (XIdx % CrossSectionsMod) * XVerts + VertSubIdx);
			}

			if (VertSubIdx == NextDupVertIdx)
			{
				NextDupVertIdx = UVSection < NumSections ? UVSections[UVSection] : -1;
			}
			else
			{
				for (int32 XIdx = 0; XIdx + 1 < NumCrossSections; XIdx++)
				{
					SetTriangleUVs(
						XSegments * 2 * XIdx + 2 * VertSubIdx,
						XIdx * XUVs + UVSubIdx,
						XIdx * XUVs + UVSubIdx + 1,
						(XIdx + 1) * XUVs + UVSubIdx, true);
					SetTriangleUVs(
						XSegments * 2 * XIdx + 2 * VertSubIdx + 1,
						(XIdx + 1) * XUVs + UVSubIdx + 1,
						(XIdx + 1) * XUVs + UVSubIdx,
						XIdx * XUVs + UVSubIdx + 1, true);
				}
				VertSubIdx++;
			}
		}
		{
			// final UV
			int32 VertSubIdx = bProfileCurveIsClosed ? 0 : XSegments;
			for (int32 XIdx = 0; XIdx < NumCrossSections; XIdx++)
			{
				FVector2f UV = FVector2f(CrossSectionTexCoord.Last(), PathTexCoord[XIdx]);
				SetUV(XIdx * XUVs + UVSubIdx, UV * SectionsUVScale, (XIdx % CrossSectionsMod) * XVerts + VertSubIdx);
			}
		}
		NumSections = NormalSections.Num();
		int32 NormalSection = 0;
		NextDupVertIdx = NormalSection < NumSections ? NormalSections[NormalSection] : -1;
		check(NextDupVertIdx < XVerts);
		for (int32 VertSubIdx = 0, NormalSubIdx = 0; VertSubIdx < XVerts; NormalSubIdx++)
		{
			int SharpNormalIdx = 0;
			for (int32 XIdx = 0, NormalXIdx = 0; XIdx < NumCrossSections; XIdx++, NormalXIdx++)
			{
				// just set the normal parent; don't compute normal yet
				SetNormal((NormalXIdx % NormalCrossSectionsMod) * XNormals + NormalSubIdx, FVector3f(0, 0, 0), (XIdx % CrossSectionsMod) * XVerts + VertSubIdx);
				// duplicate normals for cross sections that are 'sharp'
				if (SharpNormalIdx < SharpNormalsAlongLength.Num() && XIdx == SharpNormalsAlongLength[SharpNormalIdx])
				{
					NormalXIdx++;
					SetNormal((NormalXIdx % NormalCrossSectionsMod) * XNormals + NormalSubIdx, FVector3f(0, 0, 0), (XIdx % CrossSectionsMod) * XVerts + VertSubIdx);
					SharpNormalIdx++;
				}
			}

			if (VertSubIdx == NextDupVertIdx)
			{
				NextDupVertIdx = NormalSection < NumSections ? NormalSections[NormalSection] : -1;
				check(NextDupVertIdx < XVerts);
			}
			else
			{
				if (VertSubIdx < XSegments) // if bProfileCurveIsClosed == false skip the last triangle strip generation
				{
					int32 WrappedNextNormalSubIdx = (NormalSubIdx + 1) % XNormals;
					int32 WrappedNextVertexSubIdx = (VertSubIdx + 1) % XVerts;
					SharpNormalIdx = 0;
					for (int32 XIdx = 0, NXIdx = 0; XIdx + 1 < NumCrossSections; XIdx++, NXIdx++)
					{
						int32 T0Idx = XSegments * 2 * XIdx + 2 * VertSubIdx;
						int32 T1Idx = T0Idx + 1;
						int32 PIdx = XSegments * XIdx + VertSubIdx;
						int32 NextXIdx = (XIdx + 1) % CrossSectionsMod;
						int32 NextNXIdx = (NXIdx + 1) % NormalCrossSectionsMod;
						SetTrianglePolygon(T0Idx, (bPolygroupPerQuad) ? PIdx : (CurFaceGroupIndex + XIdx));
						SetTrianglePolygon(T1Idx, (bPolygroupPerQuad) ? PIdx : (CurFaceGroupIndex + XIdx));
						SetTriangle(T0Idx,
							XIdx * XVerts + VertSubIdx,
							XIdx * XVerts + WrappedNextVertexSubIdx,
							NextXIdx * XVerts + VertSubIdx, true);
						SetTriangle(T1Idx,
							NextXIdx * XVerts + WrappedNextVertexSubIdx,
							NextXIdx * XVerts + VertSubIdx,
							XIdx * XVerts + WrappedNextVertexSubIdx, true);
						SetTriangleNormals(
							T0Idx,
							NXIdx * XNormals + NormalSubIdx,
							NXIdx * XNormals + WrappedNextNormalSubIdx,
							NextNXIdx * XNormals + NormalSubIdx, true);
						SetTriangleNormals(
							T1Idx,
							NextNXIdx * XNormals + WrappedNextNormalSubIdx,
							NextNXIdx * XNormals + NormalSubIdx,
							NXIdx * XNormals + WrappedNextNormalSubIdx, true);
						if (SharpNormalIdx < SharpNormalsAlongLength.Num() && XIdx + 1 == SharpNormalsAlongLength[SharpNormalIdx])
						{
							NXIdx++;
							SharpNormalIdx++;
						}
					}
				}
				VertSubIdx++;
			}
		}
	}
}

float FVerticalCylinderGeneratorBase::ComputeSegLengths(const TArrayView<float>& Radii, const TArrayView<float>& Heights, TArray<float>& AlongPercents)
{
	float LenAlong = 0;
	int32 NumX = Radii.Num();
	AlongPercents.SetNum(NumX);
	AlongPercents[0] = 0;
	for (int XIdx = 0; XIdx + 1 < NumX; XIdx++)
	{
		double Dist = Distance(FVector2d(Radii[XIdx], Heights[XIdx]), FVector2d(Radii[XIdx + 1], Heights[XIdx + 1]));
		LenAlong += float(Dist);
		AlongPercents[XIdx + 1] = LenAlong;
	}
	for (int XIdx = 0; XIdx + 1 < NumX; XIdx++)
	{
		AlongPercents[XIdx + 1] /= LenAlong;
	}
	return LenAlong;
}

bool FVerticalCylinderGeneratorBase::GenerateVerticalCircleSweep(const TArrayView<float>& Radii, const TArrayView<float>& Heights, const TArrayView<int>& SharpNormalsAlongLength)
{
	FPolygon2d X = FPolygon2d::MakeCircle(1.0, AngleSamples);
	const TArray<FVector2d>& XVerts = X.GetVertices();
	ECapType Caps[2] = { ECapType::None, ECapType::None };

	if (bCapped)
	{
		Caps[0] = CapType;
		Caps[1] = CapType;
	}

	int NumX = Radii.Num();
	if (!ensure(NumX == Heights.Num()))
	{
		return false;
	}
	// first and last cross sections can't be sharp, so can't have more than NumX-2 sharp normal indices
	if (!ensure(SharpNormalsAlongLength.Num() + 2 <= NumX))
	{
		return false;
	}

	TArray<float> AlongPercents;
	float LenAlong = ComputeSegLengths(Radii, Heights, AlongPercents);

	ConstructMeshTopology(X, {}, {}, SharpNormalsAlongLength, false, {}, NumX, false, Caps, FVector2f(1, 1), FVector2f(.5f, .5f), FVector2f(.5f, .5f));

	TArray<FVector2d> NormalSides; NormalSides.SetNum(NumX - 1);
	for (int XIdx = 0; XIdx + 1 < NumX; XIdx++)
	{
		FVector2d Vec = FVector2d(Radii[XIdx + 1], Heights[XIdx + 1]) - FVector2d(Radii[XIdx], Heights[XIdx]);
		NormalSides[XIdx] = Normalized(PerpCW(Vec));
	}
	TArray<FVector2d> SmoothedNormalSides; SmoothedNormalSides.SetNum(NumX);
	// smooth internal normals
	SmoothedNormalSides[0] = NormalSides[0];
	SmoothedNormalSides.Last() = NormalSides.Last();
	for (int XIdx = 1; XIdx + 1 < NumX; XIdx++)
	{
		SmoothedNormalSides[XIdx] = Normalized(NormalSides[XIdx] + NormalSides[XIdx - 1]);
	}


	// set vertex positions and normals for all cross sections along length
	for (int SubIdx = 0; SubIdx < X.VertexCount(); SubIdx++)
	{
		int SharpNormalIdx = 0;
		for (int XIdx = 0, NormalXIdx = 0; XIdx < NumX; ++XIdx, ++NormalXIdx)
		{
			double AlongRadius = Radii[XIdx];
			Vertices[SubIdx + XIdx * AngleSamples] =
				FVector3d(XVerts[SubIdx].X * AlongRadius, XVerts[SubIdx].Y * AlongRadius, Heights[XIdx]);
			if (SharpNormalIdx < SharpNormalsAlongLength.Num() && XIdx == SharpNormalsAlongLength[SharpNormalIdx])
			{
				// write sharp normals
				if (ensure(XIdx > 0)) // very first index cannot be sharp
				{
					Normals[SubIdx + NormalXIdx * AngleSamples] = FVector3f(float(XVerts[SubIdx].X * NormalSides[XIdx - 1].X), float(XVerts[SubIdx].Y * NormalSides[XIdx - 1].X), float(NormalSides[XIdx - 1].Y));
				}
				NormalXIdx++;
				if (ensure(XIdx + 1 < NumX)) // very last index cannot be sharp
				{
					Normals[SubIdx + NormalXIdx * AngleSamples] = FVector3f(float(XVerts[SubIdx].X * NormalSides[XIdx].X), float(XVerts[SubIdx].Y * NormalSides[XIdx].X), float(NormalSides[XIdx].Y));
				}
				SharpNormalIdx++;
			}
			else
			{
				// write smoothed normal
				Normals[SubIdx + NormalXIdx * AngleSamples] = FVector3f(float(XVerts[SubIdx].X * SmoothedNormalSides[XIdx].X), float(XVerts[SubIdx].Y * SmoothedNormalSides[XIdx].X), float(SmoothedNormalSides[XIdx].Y));
			}
		}
	}

	if (bCapped)
	{
		// if capped, set vertices.
		for (int CapIdx = 0; CapIdx < 2; CapIdx++)
		{
			if (Caps[CapIdx] == ECapType::FlatMidpointFan)
			{
				Vertices[CapVertStart[CapIdx]] = FVector3d::UnitZ() * (double)Heights[CapIdx * (Heights.Num() - 1)];
			}
		}

		// if capped, set top/bottom normals
		for (int SubIdx = 0; SubIdx < X.VertexCount(); SubIdx++)
		{
			for (int XBotTop = 0; XBotTop < 2; ++XBotTop)
			{
				Normals[CapNormalStart[XBotTop] + SubIdx] = FVector3f(0.f, 0.f, float(2 * XBotTop - 1));
			}
		}
		for (int CapIdx = 0; CapIdx < 2; CapIdx++)
		{
			if (Caps[CapIdx] == ECapType::FlatMidpointFan)
			{
				Normals[CapNormalStart[CapIdx] + X.VertexCount()] = FVector3f(0.f, 0.f, float(2 * CapIdx - 1));
			}
		}
	}

	if (bUVScaleMatchSidesAndCaps)
	{
		float MaxAbsRad = FMathf::Abs(Radii[0]);
		for (int XIdx = 0; XIdx < NumX; XIdx++)
		{
			MaxAbsRad = FMathf::Max(FMathf::Abs(Radii[XIdx]), MaxAbsRad);
		}
		float AbsHeight = LenAlong;
		float MaxAbsCircumference = MaxAbsRad * FMathf::TwoPi;

		// scales to put each differently-scaled UV coordinate into the same space
		float ThetaScale = MaxAbsCircumference;
		float HeightScale = AbsHeight;
		float CapScale = MaxAbsRad * 2;

		float MaxScale = FMathf::Max3(ThetaScale, HeightScale, CapScale);
		ThetaScale /= MaxScale;
		HeightScale /= MaxScale;
		CapScale /= MaxScale;
		for (int UVIdx = 0; UVIdx < CapUVStart[0]; UVIdx++)
		{
			UVs[UVIdx].X *= ThetaScale;
			UVs[UVIdx].Y *= HeightScale;
		}
		for (int UVIdx = CapUVStart[0]; UVIdx < UVs.Num(); UVIdx++)
		{
			UVs[UVIdx] *= CapScale;
		}
	}

	return true;
}

FMeshShapeGenerator& FCylinderGenerator::Generate()
{
	if (bCapped)
	{
		AngleSamples = FMath::Max(AngleSamples, 3);
	}

	TArray<float> Radii, Heights;

	Radii.Add(Radius[0]);
	Heights.Add(0);
	for (int ExtraIdx = 0; ExtraIdx < LengthSamples; ExtraIdx++)
	{
		float Along = float(ExtraIdx + 1) / float(LengthSamples + 1);
		Radii.Add(FMath::Lerp(Radius[0], Radius[1], Along));
		Heights.Add(Height * Along);
	}
	Radii.Add(Radius[1]);
	Heights.Add(Height);

	GenerateVerticalCircleSweep(Radii, Heights, {});

	return *this;
}

FMeshShapeGenerator& FArrowGenerator::Generate()
{
	const float SrcRadii[]{ StickRadius, StickRadius, HeadBaseRadius, HeadTipRadius };
	const float SrcHeights[]{ 0, StickLength, StickLength, StickLength + HeadLength };

	TArray<float> Radii, Heights;
	const int NumVerts = 4 + AdditionalLengthSamples[0] + AdditionalLengthSamples[1] + AdditionalLengthSamples[2];
	Radii.SetNumUninitialized(NumVerts);
	Heights.SetNumUninitialized(NumVerts);

	int VertIdx = 0;
	auto SetVert = [&Radii, &Heights, &VertIdx](float Radius, float Height)
		{
			Radii[VertIdx] = Radius;
			Heights[VertIdx] = Height;
			++VertIdx;
		};

	for (int SegIdx = 0; true; ++SegIdx)
	{
		SetVert(SrcRadii[SegIdx], SrcHeights[SegIdx]);

		if (SegIdx == 3)
		{
			break;
		}

		for (int ExtraSeg = 1, NumExtraSegs = AdditionalLengthSamples[SegIdx] + 1; ExtraSeg < NumExtraSegs; ++ExtraSeg)
		{
			const float Along = float(ExtraSeg) / float(NumExtraSegs);
			SetVert(FMath::Lerp(SrcRadii[SegIdx], SrcRadii[SegIdx + 1], Along),
				FMath::Lerp(SrcHeights[SegIdx], SrcHeights[SegIdx + 1], Along));
		}
	}

	TArray<int> SharpNormalsAlongLength{ 1 + AdditionalLengthSamples[0], 2 + AdditionalLengthSamples[0] + AdditionalLengthSamples[1] };

	GenerateVerticalCircleSweep(Radii, Heights, SharpNormalsAlongLength);

	return *this;
}

FMeshShapeGenerator& FGeneralizedCylinderGenerator::Generate()
{
	const TArray<FVector2d>& XVerts = CrossSection.GetVertices();
	ECapType Caps[2] = { ECapType::None, ECapType::None };

	if (bCapped && !bLoop)
	{
		Caps[0] = CapType;
		Caps[1] = CapType;
	}
	int PathNum = Path.Num();

	bool bHavePathScaling = (PathScales.Num() == PathNum);
	bool bApplyScaling = (bHavePathScaling || (StartScale != 1.0) || (EndScale != 1.0)) && (bLoop == false);
	bool bNeedArcLength = (bApplyScaling || bUVScaleRelativeWorld);
	double TotalPathArcLength = (bNeedArcLength) ? UE::Geometry::CurveUtil::ArcLength<double, FVector3d>(Path, bLoop) : 1.0;

	FAxisAlignedBox2f Bounds = (FAxisAlignedBox2f)CrossSection.Bounds();
	double BoundsMaxDimInv = 1.0 / FMathd::Max(Bounds.MaxDim(), .001);
	FVector2f SectionScale(1.f, 1.f), CapScale((float)BoundsMaxDimInv, (float)BoundsMaxDimInv);
	if (bUVScaleRelativeWorld)
	{
		double Perimeter = CrossSection.Perimeter();
		SectionScale.X = float(Perimeter / UnitUVInWorldCoordinates);
		SectionScale.Y = float(TotalPathArcLength / UnitUVInWorldCoordinates);
		CapScale.X = CapScale.Y = 1.0f / UnitUVInWorldCoordinates;
	}
	ConstructMeshTopology(CrossSection, {}, {}, {}, true, Path, PathNum + (bLoop ? 1 : 0), bLoop, Caps, SectionScale, CapScale, Bounds.Center(), CrossSectionTexCoord, PathTexCoord);

	int XNum = CrossSection.VertexCount();
	TArray<FVector2d> XNormals; XNormals.SetNum(XNum);
	for (int Idx = 0; Idx < XNum; Idx++)
	{
		XNormals[Idx] = CrossSection.GetNormal_FaceAvg(Idx);
	}

	double AccumArcLength = 0;
	FFrame3d CrossSectionFrame = InitialFrame;
	bool bHaveExplicitFrames = (PathFrames.Num() == Path.Num());
	for (int PathIdx = 0; PathIdx < PathNum; ++PathIdx)
	{
		FVector3d C = Path[PathIdx];
		FVector3d X, Y;
		FMatrix2d MiterPosScale = FMatrix2d::Identity();
		if (bHaveExplicitFrames)
		{
			CrossSectionFrame = PathFrames[PathIdx];
			C = PathFrames[PathIdx].Origin;
		}
		if (!bHaveExplicitFrames || bAlignFramesToSampledTangents)
		{
			FVector3d ToPrev, ToNext;
			UE::Geometry::CurveUtil::GetVectorsToPrevNext<double, FVector3d>(Path, PathIdx, ToPrev, ToNext, true, bLoop);

			FVector3d Tangent = Normalized(-ToPrev + ToNext);
			CrossSectionFrame.AlignAxis(2, Tangent);
			if (MiterLimit > 1)
			{
				// only miter if neither neighbor vector is exactly zero
				if (!ToPrev.IsZero() && !ToNext.IsZero())
				{
					FVector3d TurnDir = ToPrev + ToNext;
					FVector2d ScaleAlong = (FVector2d)CrossSectionFrame.ToFrameVector(TurnDir);
					double ScaleAmount = 1.0;
					if (ScaleAlong.Normalize())
					{
						double CosTwoTheta = ToPrev.Dot(ToNext);
						double SqrScale = 2.0 / (1.0 - CosTwoTheta);
						if (!FMath::IsFinite(SqrScale) || SqrScale > MiterLimit * MiterLimit)
						{
							ScaleAmount = MiterLimit;
						}
						else
						{
							ScaleAmount = FMath::Sqrt(SqrScale);
						}
						MiterPosScale = FMatrix2d::AxisScale(ScaleAlong, ScaleAmount, false /* axis was already normalized */);
					}
				}
			}
		}
		X = CrossSectionFrame.X();
		Y = CrossSectionFrame.Y();

		double T = FMathd::Clamp((AccumArcLength / TotalPathArcLength), 0.0, 1.0);
		double UniformScale = (bApplyScaling) ? FMathd::Lerp(StartScale, EndScale, T) : 1.0;
		FVector2d PathScaling = (bHavePathScaling) ? PathScales[PathIdx] : FVector2d::One();

		for (int SubIdx = 0; SubIdx < XNum; SubIdx++)
		{
			FVector2d XP = MiterPosScale * (UniformScale * PathScaling * CrossSection[SubIdx]);
			FVector2d XN = XNormals[SubIdx];
			// Note: Arguably, one could apply an inverse mitering scale to the normals, and then re-normalize, here ...
			// However, it's better to think about these normals an average from the two unscaled cross sections that are meeting at the cross section plane,
			// and in that model they won't be affected by the miter scale
			Vertices[SubIdx + PathIdx * XNum] = C + X * XP.X + Y * XP.Y;
			Normals[SubIdx + PathIdx * XNum] = (FVector3f)(X * XN.X + Y * XN.Y);
		}

		if (PathIdx < PathNum - 1)
		{
			AccumArcLength += Distance(C, Path[PathIdx + 1]);
		}
	}
	if (bCapped && !bLoop)
	{
		// if capped, set vertices.
		for (int CapIdx = 0; CapIdx < 2; CapIdx++)
		{
			if (Caps[CapIdx] == ECapType::FlatMidpointFan)
			{
				Vertices[CapVertStart[CapIdx]] = Path[CapIdx * (Path.Num() - 1)];
			}
		}

		for (int CapIdx = 0; CapIdx < 2; CapIdx++)
		{
			FVector3d Normal = CurveUtil::Tangent<double, FVector3d>(Path, CapIdx * (PathNum - 1), bLoop)* (double)(CapIdx * 2 - 1);
			for (int SubIdx = 0; SubIdx < XNum; SubIdx++)
			{
				Normals[CapNormalStart[CapIdx] + SubIdx] = (FVector3f)Normal;
			}
		}
	}

	for (int k = 0; k < Normals.Num(); ++k)
	{
		Normalize(Normals[k]);
	}

	return *this;
}

// FProfileSweepGenerator

// Various indexing utility functions. See the comment in Generate() for a description of vertex/triangle/uv layout.
namespace UE::SweepGeneratorLocals
{
	static int32 GetVertIndex(bool VertIsWelded, int32 SweepIndex, int32 ProfileIndex, int32 NumWelded, int32 NumNonWelded, const TArray<int32>& VertPositionOffsets)
	{
		return VertIsWelded ?
			VertPositionOffsets[ProfileIndex]
			: NumWelded + SweepIndex * NumNonWelded + VertPositionOffsets[ProfileIndex];
	}
	static int32 GetTriangleIndex(int32 SweepIndex, int32 ProfileIndex, int32 NumTrisPerSweepSegment, const TArray<int32>& VertTriangleOffsets)
	{
		return (SweepIndex * NumTrisPerSweepSegment) + VertTriangleOffsets[ProfileIndex];
	}
	static int32 GetUvIndex(int32 SweepIndex, int32 ProfileIndex, int32 NumUvColumns)
	{
		return (SweepIndex * NumUvColumns) + ProfileIndex;
	}
	static int32 GetPolygonGroup(EProfileSweepPolygonGrouping PolygonGroupingMode, int32 SweepIndex, int32 ProfileIndex, int32 NumProfileSegments)
	{
		int32 PolygonId = -1;
		switch (PolygonGroupingMode)
		{
		case EProfileSweepPolygonGrouping::Single:
			PolygonId = 0;
			break;
		case EProfileSweepPolygonGrouping::PerFace:
			PolygonId = SweepIndex * NumProfileSegments + ProfileIndex;
			break;
		case EProfileSweepPolygonGrouping::PerProfileSegment:
			PolygonId = ProfileIndex;
			break;
		case EProfileSweepPolygonGrouping::PerSweepSegment:
			PolygonId = SweepIndex;
			break;
		}
		return PolygonId;
	}
}

/**
 * Utility function for calculating the triangle normal contributions to average normals.
 */
void FProfileSweepGenerator::AdjustNormalsForTriangle(int32 TriIndex, int32 FirstIndex, int32 SecondIndex, int32 ThirdIndex,
	TArray<FVector3d> &WeightedNormals)
{
	FVector3d AbNormalized = Normalized(Vertices[SecondIndex] - Vertices[FirstIndex]);
	AdjustNormalsForTriangle(TriIndex, FirstIndex, SecondIndex, ThirdIndex, WeightedNormals, AbNormalized);
}

/**
 * Utility function for calculating the contribution of triangle normals to average normals. AbNormalized (ie,
 * the normalized vector from the first vertex to the second) is taken as a parameter so that it can be reused in
 * dealing with a planar quad.
 */
void FProfileSweepGenerator::AdjustNormalsForTriangle(int32 TriIndex, int32 FirstIndex, int32 SecondIndex, int32 ThirdIndex,
	TArray<FVector3d>& WeightedNormals, const FVector3d& AbNormalized)
{
	// For the code below, this is the naming of the vertices:
	//  a-c
	//   \|
	//    b

	// We store contribution of this normal to each vertex's average. These will need to get summed
	// per vertex later- we avoid adding the result into the per-vertex sums as we go along to avoid
	// concurrent writes since this is done in parallel.

	FVector3d Bc = Normalized(Vertices[ThirdIndex] - Vertices[SecondIndex]);
	FVector3d Ac = Normalized(Vertices[ThirdIndex] - Vertices[FirstIndex]);
	FVector3d TriangleNormal = Normalized(Ac.Cross(AbNormalized));

	// Note that AngleR requires normalized inputs
	WeightedNormals[TriIndex * 3] = TriangleNormal * AngleR(AbNormalized,Ac);
	WeightedNormals[TriIndex * 3 + 1] = TriangleNormal * AngleR(Bc, -AbNormalized);
	WeightedNormals[TriIndex * 3 + 2] = TriangleNormal * AngleR(Ac, Bc);

	// We can safely hook the triangle normals up, even though their calculation is incomplete.
	SetTriangleNormals(TriIndex, FirstIndex, SecondIndex, ThirdIndex);
}

FMeshShapeGenerator& FProfileSweepGenerator::Generate()
{
	using namespace UE::SweepGeneratorLocals;

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Check that we have our inputs
	if (ProfileCurve.Num() < 2 || SweepCurve.Num() < 2 || (!SweepScaleCurve.IsEmpty() && SweepScaleCurve.Num() != SweepCurve.Num()))
	{
		Reset();
		return *this;;
	}

	// If all points are welded, nothing to do
	int32 NumWelded = WeldedVertices.Num();
	int32 NumNonWelded = ProfileCurve.Num() - WeldedVertices.Num();
	check(NumNonWelded >= 0); // We should never have more welded points than there are points total.
	if (NumNonWelded == 0)
	{
		Reset();
		return *this;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(ProfileSweepGenerator_Generate);

	/*
	The generated vertices are organized and connected as follows:

		o-o-o-o-o-o-o-o
		| | | |/ \| | |   |
		o-o-o-o-o-o-o-o   | Sweep index direction
		| | | |/ \| | |   v
		o-o-o-o-o-o-o-o
		| | | |/ \| | |
		o-o-o-o   o-o-o
		        ^
		        | welded vertex

		---> Profile index direction

	The welded vertex actually has a single position but multiple UV coordinates for different triangles,
	and those UV coordinates are actually centered vertically like this in the UV grid:
	o     o
	| >o< |
	o     o
	
	Vertex positions are stored with the welded vertices first, followed by rows of non welded vertices, one
	row for each sweep point. Triangles, similarly, are stored as rows of "sweep segments" (horizontal rows
	of triangles corresponding to a sweep step). We can index into the rows of these data structures with
	the sweep index and into the columns using a mapping from profile index to offset, since these offsets
	vary depending on the presence of welded vertices (which also generate fewer triangles per sweep segment).

	UVs are similary stored as rows per sweep point, but because welded points have different UV coordinates
	per triangle and because coincident UV's (when there are welded-to-welded edges) still need separate UV
	elements per vertex, there is no need to have special offseting per profile index. However, there is 
	an extra UV row/column in cases where the sweep/profile curves are closed, to avoid wrapping back around
	to 0, so indexing to the next element should not use the modulo operator.
	
	Note that despite their arrangement in memory, U's actually increase in the sweep direction and V's increase
	in the profile direction, since we typically imagine a vertical profile being swept or rotated in a horizontal
	direction. The spacing in the UV grid is weighted by the distance between profile and sweep points.
	
	For calculating normals, we sum together the triangle normals at each vertex weighted by the angle of
	that triangle vertex, and we normalize at the end.
	*/

	// A few additional convenience variables
	int32 NumProfilePoints = ProfileCurve.Num();
	int32 NumSweepPoints = SweepCurve.Num();
	int32 NumSweepSegments = bSweepCurveIsClosed ? NumSweepPoints : NumSweepPoints - 1 ;
	int32 NumProfileSegments = bProfileCurveIsClosed ? NumProfilePoints : NumProfilePoints - 1;

	int32 NumVerts = NumWelded + NumNonWelded * NumSweepPoints;

	// Set up structures needed to index to the correct vertex or triangle within the rows that we 
	// store per sweep instance.
	TArray<int32> VertPositionOffsets;
	TArray<int32> TriangleOffsets;
	VertPositionOffsets.SetNum(NumProfilePoints);
	TriangleOffsets.SetNum(NumProfilePoints);

	int32 NextWeldedOffset = 0;
	int32 NextNonWeldedOffset = 0;
	int32 NextTriangleOffset = 0;
	for (int32 ProfileIndex = 0; ProfileIndex < NumProfilePoints; ++ProfileIndex)
	{
		bool CurrentIsWelded = WeldedVertices.Contains(ProfileIndex);

		VertPositionOffsets[ProfileIndex] = CurrentIsWelded ? NextWeldedOffset++ : NextNonWeldedOffset++;

		if (bProfileCurveIsClosed || ProfileIndex < NumProfilePoints - 1) // if there's a next profile point
		{
			// Set up indexing for the triangle (or triangles in the quad) immediately to the right of this vertex.
			int32 NextProfileIndex = (ProfileIndex + 1) % NumProfilePoints;
			bool NextIsWelded = WeldedVertices.Contains(NextProfileIndex);
			if (CurrentIsWelded)
			{
				// No triangles for a welded-to-welded connection, otherwise one triangle.
				TriangleOffsets[ProfileIndex] = NextIsWelded ? -1 : NextTriangleOffset++;
			}
			else
			{
				// Depending on whether next is welded or not, we're adding one or two triangles.
				TriangleOffsets[ProfileIndex] = NextTriangleOffset;
				NextTriangleOffset += (NextIsWelded ? 1 : 2);
			}
		}
	}
	int32 NumTrisPerSweepSegment = NextTriangleOffset;
	int32 NumTriangles = NumTrisPerSweepSegment * NumSweepSegments;

	// Determine number of normals needed. When converting to a dynamic mesh later, we are not allowed to have vertices of
	// the same triangle share a normal, so for sharp normals, we need  NumTriangles*3 rather than NumTriangles normals.
	int32 NumNormals = NumVerts;

	// If we're going to be averaging normals for vertices, we'll want some temporary storage so that we can avoid
	// coincident writes when we're parallelized.
	TArray<FVector3d> WeightedNormals;
	WeightedNormals.SetNum(NumTriangles * 3);

	// Perform all allocations except UV's, which will get done later and for which we may want some vertex positions.
	SetBufferSizes(NumVerts, NumTriangles, 0, NumNormals);

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Create positions of all our vertices. We don't connect triangles yet because that is best done at
	// the same time as dealing with normals, and normals will require the positions to have been set.
	ParallelFor(NumProfilePoints, 
		[this, NumWelded, NumNonWelded, NumSweepPoints, &VertPositionOffsets]
		(int ProfileIndex)
	{
		if (WeldedVertices.Contains(ProfileIndex))
		{
			FVector3d FramePoint = ProfileCurve[ProfileIndex];
			// Position stays locked into the first frame
			if (!SweepScaleCurve.IsEmpty())
			{
				FramePoint *= SweepScaleCurve[0];
			}
			SetVertex(GetVertIndex(true, 0, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets), 
				SweepCurve[0].FromFramePoint(FramePoint));
		}
		else
		{
			// Generate copies of the vertex in all the sweep frames.
			for (int32 SweepIndex = 0; SweepIndex < NumSweepPoints; ++SweepIndex)
			{
				FVector3d FramePoint = ProfileCurve[ProfileIndex];
				if (!SweepScaleCurve.IsEmpty())
				{
					FramePoint *= SweepScaleCurve[SweepIndex];
				}
				SetVertex(GetVertIndex(false, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets),
					SweepCurve[SweepIndex].FromFramePoint(FramePoint));
			}
		}
	});

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Now set up UV's and UV indexing. This performs the UV allocation for us.
	int32 NumUvColumns = 0;
	int32 NumUvRows = 0;
	InitializeUvBuffer(VertPositionOffsets, NumUvRows, NumUvColumns);

	// Connect up the triangles, calculate normals, and associate with UV's
	ParallelFor(NumSweepSegments, 
		[this, NumSweepPoints, NumProfilePoints, NumWelded, NumNonWelded, &VertPositionOffsets,  
			NumProfileSegments, NumTrisPerSweepSegment, &TriangleOffsets, &WeightedNormals, NumUvColumns]
		(int SweepIndex)
	{
		int32 NextSweepIndex = (SweepIndex + 1) % NumSweepPoints;

		for (int32 ProfileIndex = 0; ProfileIndex < NumProfileSegments; ++ProfileIndex)
		{
			int32 NextProfileIndex = (ProfileIndex + 1) % NumProfilePoints;
			bool CurrentIsWelded = WeldedVertices.Contains(ProfileIndex);
			bool NextIsWelded = WeldedVertices.Contains(NextProfileIndex);

			if (CurrentIsWelded)
			{
				if (NextIsWelded)
				{
					// No triangles between adjacent welded triangles
					continue;
				}
				else
				{
					// Welded to non-welded: one triangle
					int32 CurrentVert = GetVertIndex(true, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 BottomRightVert = GetVertIndex(false, NextSweepIndex, NextProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 RightVert = GetVertIndex(false, SweepIndex, NextProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);

					int32 TriIndex = GetTriangleIndex(SweepIndex, ProfileIndex, NumTrisPerSweepSegment, TriangleOffsets);

					SetTriangle(TriIndex, CurrentVert, BottomRightVert, RightVert);
					AdjustNormalsForTriangle(TriIndex, CurrentVert, BottomRightVert, RightVert, WeightedNormals);
					SetTriangleUVs(TriIndex,
						// Do not wrap around when looking for UV elements, since there will be an extra on the end if needed
						GetUvIndex(SweepIndex, ProfileIndex, NumUvColumns),
						GetUvIndex(SweepIndex + 1, ProfileIndex + 1, NumUvColumns),
						GetUvIndex(SweepIndex, ProfileIndex + 1, NumUvColumns));
					SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
				}
			}
			else
			{
				if (NextIsWelded)
				{
					// Non-welded to welded: one triangle
					int32 CurrentVert = GetVertIndex(false, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 BottomVert = GetVertIndex(false, NextSweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 RightVert = GetVertIndex(true, SweepIndex, NextProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);

					int32 TriIndex = GetTriangleIndex(SweepIndex, ProfileIndex, NumTrisPerSweepSegment, TriangleOffsets);

					SetTriangle(TriIndex, CurrentVert, BottomVert, RightVert);
					AdjustNormalsForTriangle(TriIndex, CurrentVert, BottomVert, RightVert, WeightedNormals);
					SetTriangleUVs(TriIndex,
						// Do not wrap around when looking for UV elements, since there will be an extra on the end if needed
						GetUvIndex(SweepIndex, ProfileIndex, NumUvColumns),
						GetUvIndex(SweepIndex + 1, ProfileIndex, NumUvColumns),
						GetUvIndex(SweepIndex, ProfileIndex + 1, NumUvColumns));
					SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
				}
				else
				{
					// Non-welded to non-welded creates a quad.
					int32 CurrentVert = GetVertIndex(false, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 BottomVert = GetVertIndex(false, NextSweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 BottomRightVert = GetVertIndex(false, NextSweepIndex, NextProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
					int32 RightVert = GetVertIndex(false, SweepIndex, NextProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);

					int32 TriIndex = GetTriangleIndex(SweepIndex, ProfileIndex, NumTrisPerSweepSegment, TriangleOffsets);

					// The currently supported modes are to either always connect diagonally down, or connect the shorter diagonal.
					// For comparing diagonals, we allow some percent difference to triangulate symmetric quads uniformly.
					if (QuadSplitMethod == EProfileSweepQuadSplit::Uniform
						|| DistanceSquared(Vertices[CurrentVert], Vertices[BottomRightVert])
							/ DistanceSquared(Vertices[BottomVert], Vertices[RightVert]) <= (1 + DiagonalTolerance)*(1 + DiagonalTolerance))
					{
						FVector3d DiagonalDown = Normalized(Vertices[BottomRightVert] - Vertices[CurrentVert]);

						SetTriangle(TriIndex, CurrentVert, BottomRightVert, RightVert);
						AdjustNormalsForTriangle(TriIndex, CurrentVert, BottomRightVert, RightVert, WeightedNormals, DiagonalDown);
						SetTriangleUVs(TriIndex,
							GetUvIndex(SweepIndex, ProfileIndex, NumUvColumns),
							GetUvIndex(SweepIndex + 1, ProfileIndex + 1, NumUvColumns),
							GetUvIndex(SweepIndex, ProfileIndex + 1, NumUvColumns));
						SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
						++TriIndex;

						// Do the second triangle in such a way that the diagonal is goes from first vertex to second.
						SetTriangle(TriIndex, BottomRightVert, CurrentVert, BottomVert);
						AdjustNormalsForTriangle(TriIndex, BottomRightVert, CurrentVert, BottomVert, WeightedNormals, -DiagonalDown);
						SetTriangleUVs(TriIndex,
							GetUvIndex(SweepIndex + 1, ProfileIndex + 1, NumUvColumns),
							GetUvIndex(SweepIndex, ProfileIndex, NumUvColumns),
							GetUvIndex(SweepIndex + 1, ProfileIndex, NumUvColumns));
						SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
					}
					else
					{
						FVector3d DiagonalUp = Normalized(Vertices[RightVert] - Vertices[BottomVert]);
						SetTriangle(TriIndex, BottomVert, RightVert, CurrentVert);
						AdjustNormalsForTriangle(TriIndex, BottomVert, RightVert, CurrentVert, WeightedNormals, DiagonalUp);
						SetTriangleUVs(TriIndex,
							GetUvIndex(SweepIndex + 1, ProfileIndex, NumUvColumns),
							GetUvIndex(SweepIndex, ProfileIndex + 1, NumUvColumns),
							GetUvIndex(SweepIndex, ProfileIndex, NumUvColumns));
						SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
						++TriIndex;

						SetTriangle(TriIndex, RightVert, BottomVert, BottomRightVert);
						AdjustNormalsForTriangle(TriIndex, RightVert, BottomVert, BottomRightVert, WeightedNormals, -DiagonalUp);
						SetTriangleUVs(TriIndex,
							GetUvIndex(SweepIndex, ProfileIndex + 1, NumUvColumns),
							GetUvIndex(SweepIndex + 1, ProfileIndex, NumUvColumns),
							GetUvIndex(SweepIndex + 1, ProfileIndex + 1, NumUvColumns));
						SetTrianglePolygon(TriIndex, GetPolygonGroup(PolygonGroupingMode, SweepIndex, ProfileIndex, NumProfileSegments));
					}//end splitting quad
				}//end if nonwelded to nonwelded
			}//end if nonwelded
		}//end for profile points
	});//end parallel across sweep segments

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Combine the weighted normals that we accumulated.
	TArray<FVector3d> NormalSums;
	NormalSums.SetNumZeroed(NumVerts);
	for (int32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
	{
		FIndex3i TriangleVerts = Triangles[TriIndex];
		NormalSums[TriangleVerts.A] += WeightedNormals[TriIndex * 3];
		NormalSums[TriangleVerts.B] += WeightedNormals[TriIndex * 3 + 1];
		NormalSums[TriangleVerts.C] += WeightedNormals[TriIndex * 3 + 2];
	}

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Normalize and set them
	ParallelFor(NumVerts, [this, &NormalSums](int VertIndex)
	{
		SetNormal(VertIndex, (FVector3f)(Normalized(NormalSums[VertIndex])), VertIndex);
	});

	if (Progress && Progress->Cancelled())
	{
		return *this;
	}

	// Save the beginning and end profile curve instances, if relevant
	if (!bSweepCurveIsClosed)
	{
		for (int32 ProfileIndex = 0; ProfileIndex < NumProfilePoints; ++ProfileIndex)
		{
			EndProfiles[0].Add(GetVertIndex(WeldedVertices.Contains(ProfileIndex), 0, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets));
			EndProfiles[1].Add(GetVertIndex(WeldedVertices.Contains(ProfileIndex), NumSweepPoints-1, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets));
		}
	}

	return *this;
}//end Generate()

/**
 * Initializes the UV buffer with UV's that are set according to the diagram at the start of Generate(), with an extra
 * element on each end in the case of closed curves, and distances weighted by distances in the corresponding curves.
 * This function should get called after setting vertex positions so that the function can use them in case 
 * bUVScaleRelativeWorld is true.
 *
 * @param VertPositionOffsets Offsets needed to get the correct vertex indices, for setting the parents of UV elements
 * @param NumUvRowsOut Number of resulting UV rows allocated (a row corresponds to an instance of the profile curve)
 * @param NumUvColumnsOut Number of resulting UV columns allocated (number of columns relates to number of profile points)
 */
void FProfileSweepGenerator::InitializeUvBuffer(const TArray<int32>& VertPositionOffsets,
	int32& NumUvRowsOut, int32& NumUvColumnsOut)
{
	using namespace UE::SweepGeneratorLocals;

	// Convenience variables
	int32 NumProfilePoints = ProfileCurve.Num();
	int32 NumProfileSegments = bProfileCurveIsClosed ? NumProfilePoints : NumProfilePoints - 1;
	int32 NumSweepPoints = SweepCurve.Num();
	int32 NumSweepSegments = bSweepCurveIsClosed ? NumSweepPoints : NumSweepPoints - 1;
	int32 NumWelded = WeldedVertices.Num();
	int32 NumNonWelded = NumProfilePoints - NumWelded;

	// Since we're working with a grid, the coordinates we generate are combinations of a limited number of U and V options, which are
	// determined by spacing between the corresponding curve points. The V elements are determined by ProfileIndex.
	TArray<double> Vs;
	double CumulativeDistance = 0;
	Vs.Add(CumulativeDistance);
	for (int32 ProfileIndex = 0; ProfileIndex < NumProfileSegments; ++ProfileIndex)
	{
		int32 NextProfileIndex = (ProfileIndex + 1) % NumProfilePoints;
		if (!bUVsSkipFullyWeldedEdges
			|| !(WeldedVertices.Contains(ProfileIndex) && WeldedVertices.Contains(NextProfileIndex))) // check for welded-to-welded
		{
			CumulativeDistance += Distance(ProfileCurve[ProfileIndex], ProfileCurve[NextProfileIndex]);
		}
		Vs.Add(CumulativeDistance);
	}

	// Figure out how we'll be normalizing/scaling the V's
	double VScale = UVScale[1];
	if (!bUVScaleRelativeWorld)
	{
		// The normal case: normalizing and scaling.
		VScale = ensure(CumulativeDistance != 0) ? // Profile points shouldn't be coincident
			VScale /= CumulativeDistance 
			: 0;
	}
	else
	{
		// Convert using custom scale
		VScale  = (UnitUVInWorldCoordinates != 0) ? VScale / UnitUVInWorldCoordinates : 0;
	}

	// Scale and adjust
	for (int i = 0; i < Vs.Num(); ++i)
	{
		Vs[i] =  Vs[i] * VScale + UVOffset[1];
	}

	// U elements depend on distances in the sweep direction. For that, we need to accumulate the displacement of
	// the profile vertices across sweep frames. We can divide by NumProfileVertices later, if we need to.
	TArray<double> Distances;
	Distances.SetNumZeroed(NumSweepSegments);
	ParallelFor(NumSweepSegments,
		[this, NumSweepPoints, NumProfilePoints, NumWelded, NumNonWelded, &VertPositionOffsets, &Distances]
	(int SweepIndex)
	{
		int32 NextSweepIndex = (SweepIndex + 1) % NumSweepPoints;
		for (int32 ProfileIndex = 0; ProfileIndex < NumProfilePoints; ++ProfileIndex)
		{
			if (!WeldedVertices.Contains(ProfileIndex))
			{
				FVector3d A = Vertices[GetVertIndex(false, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets)];
				FVector3d B = Vertices[GetVertIndex(false, NextSweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets)];
				Distances[SweepIndex] += Distance(A, B);
			}
		}
	});

	// U elements differ between regular and welded vertices, since welded are centered between adjacent non-welded
	TArray<double> WeldedUs;
	TArray<double> RegularUs;
	CumulativeDistance = 0;
	RegularUs.Add(0);
	for (int i = 0; i < NumSweepSegments; ++i)
	{
		CumulativeDistance += Distances[i];
		RegularUs.Add(CumulativeDistance);
		WeldedUs.Add(CumulativeDistance - Distances[i] / 2);
	}
	// There's one fewer welded U, but it's more convenient to make the two arrays the same length
	// so we can index into them the same way and initialize all UV elements including the few
	// that we don't use.
	WeldedUs.Add(0);

	double UScale = UVScale[0];
	if (!bUVScaleRelativeWorld)
	{
		UScale = (CumulativeDistance != 0) ? UScale / CumulativeDistance : 0;
	}
	else
	{
		UScale = (UnitUVInWorldCoordinates != 0) ? 
			// Get an average and convert to UV from world using the scale.
			UScale / (float(NumProfilePoints) * UnitUVInWorldCoordinates) 
			: 0;
	}
	// Adjust
	for (int i = 1; i < RegularUs.Num(); ++i)
	{
		RegularUs[i] = RegularUs[i] * UScale + UVOffset[0];
		WeldedUs[i - 1] = WeldedUs[i-1] * UScale + UVOffset[0];
	}

	// Set up storage
	NumUvRowsOut = RegularUs.Num();
	NumUvColumnsOut = Vs.Num();
	SetBufferSizes(0, 0, NumUvRowsOut * NumUvColumnsOut, 0);

	// Initialize the UV buffer
	ParallelFor(NumUvRowsOut, 
		[this, NumSweepPoints, NumProfilePoints, NumUvColumnsOut, NumWelded,
		NumNonWelded, &VertPositionOffsets, &WeldedUs, &RegularUs, &Vs]
		(int i)
	{
		int32 SweepIndex = i % NumSweepPoints;
		for (int j = 0; j < NumUvColumnsOut; ++j)
		{
			int32 ProfileIndex = j % NumProfilePoints;
			if (WeldedVertices.Contains(ProfileIndex))
			{
				int32 VertIndex = GetVertIndex(true, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
				SetUV(i * NumUvColumnsOut + j, FVector2f((float)WeldedUs[i], (float)Vs[j]), VertIndex);
			}
			else
			{
				int32 VertIndex = GetVertIndex(false, SweepIndex, ProfileIndex, NumWelded, NumNonWelded, VertPositionOffsets);
				SetUV(i * NumUvColumnsOut + j, FVector2f((float)RegularUs[i], (float)Vs[j]), VertIndex);
			}
		}
	});
}