// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCoreModule.h"

#include "CompGeom/ExactPredicates.h"
#include "Splines/MultiSpline.h"
#include "Splines/SplineTypeRegistry.h"
#include "Splines/TangentBezierSpline.h"

#define LOCTEXT_NAMESPACE "FGeometryCoreModule"

DEFINE_LOG_CATEGORY(LogGeometry);

void FGeometryCoreModule::StartupModule()
{
	RegisterSplineTypes();
	
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	UE::Geometry::ExactPredicates::GlobalInit();
}

void FGeometryCoreModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

void FGeometryCoreModule::RegisterSplineTypes() const
{
	using namespace UE::Geometry::Spline;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	/**
	 * TMultiSpline<FTangentBezierSpline3d> once had an incorrect type ID which failed to take into account the wrapped spline type (FTangentBezierSpline3d).
	 * Now, the implementation name of a multi-spline is in the form of "MultiSpline.[WrappedSplineName]".
	 * This registered factory exists to allow for proper loading of multi-splines which were saved before this bug was fixed.
	 */
	const FString LegacyMultiSplineImplName = "MultiSpline";
	const FString LegacyMultiSplineValueTypeName = "Vector";
	const FSplineTypeId::IdType LegacyMultiSplineID = FSplineTypeId::GenerateTypeId(*LegacyMultiSplineImplName, *LegacyMultiSplineValueTypeName);
	ensure (LegacyMultiSplineID == 0x54E3AD22);
	FSplineTypeRegistry::RegisterType(LegacyMultiSplineID, LegacyMultiSplineImplName, LegacyMultiSplineValueTypeName, []() -> TUniquePtr<ISplineInterface>
	{
		return MakeUnique<TMultiSpline<FTangentBezierSpline3d>>();
	});
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryCoreModule, GeometryCore)