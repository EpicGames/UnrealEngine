// Copyright Epic Games, Inc. All Rights Reserved.

#include "Image/DisplacementMap.h"

#include "MathUtil.h"

namespace UE
{
namespace Geometry
{

FDisplacementMap::FDisplacementMap()
	: SourceFormat(TSF_G8)
	, BytesPerPixel(1)
	, SizeX(1)
	, SizeY(1)
	, NumLevels(1)
	, Magnitude(0.0f)
	, Center(0.0f)
	, AddressX(TA_Wrap)
	, AddressY(TA_Wrap)
{
	Storage.Add(0);
	SourceData = TArrayView64<const uint8>(Storage);
}
	
FDisplacementMap::FDisplacementMap( 
		TArray64<uint8>&& InSourceStorage, 
		ETextureSourceFormat InSourceFormat,
		const int32 InSizeX,
		const int32 InSizeY,
		float InMagnitude, 
		float InCenter, 
		TextureAddress InAddressX, 
		TextureAddress InAddressY )
	: SourceFormat(InSourceFormat)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, NumLevels(1)
	, Magnitude( InMagnitude )
	, Center( InCenter )
	, AddressX( InAddressX )
	, AddressY( InAddressY )
	, Storage(MoveTemp(InSourceStorage))
	, SourceData(Storage)
{
	Init();
}

FDisplacementMap::FDisplacementMap( 
		TArrayView64<const uint8> InSourceData, 
		ETextureSourceFormat InSourceFormat,
		const int32 InSizeX,
		const int32 InSizeY,
		float InMagnitude, 
		float InCenter, 
		TextureAddress InAddressX, 
		TextureAddress InAddressY )
	: SourceFormat(InSourceFormat)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, NumLevels(1)
	, Magnitude( InMagnitude )
	, Center( InCenter )
	, AddressX( InAddressX )
	, AddressY( InAddressY )
	, SourceData( InSourceData )
{
	Init();
}

int FDisplacementMap::GetBytesPerPixel(const ETextureSourceFormat SourceFormat) 
{
	switch (SourceFormat)
	{
	case TSF_BGRA8:
		return 4;
	case TSF_RGBA16:
		return 8;
	case TSF_R16F:
		return 2;
	case TSF_G8:
		return 1;
	case TSF_G16:
		return 2;
	case TSF_RGBA16F:
		return 8;
	case TSF_RGBA32F:
		return 16;
	case TSF_R32F:
		return 4;
	}
	return 0;
}

void FDisplacementMap::Init()
{
	if ((BytesPerPixel = GetBytesPerPixel(SourceFormat)) == 0)
	{
		ensureMsgf(false, TEXT("Unsupported texture format."));
		InitDefault();
		return;
	}
		
	if (SizeX > MAX_RESOLUTION || SizeY > MAX_RESOLUTION)
	{
		ensureMsgf(false, TEXT("Maximum displacement map resolution exceeded."));
		InitDefault();
		return;
	}

	// replace void textures by 1x1 black pixel
	if (SizeX == 0 || SizeY == 0)
	{
		InitDefault();
		return;
	}

	check(SourceData.Num() == SizeX * SizeY * BytesPerPixel);

	if (SizeX == 1 && SizeY == 1)
	{
		// don't add mipmaps for 1x1 image
		check(NumLevels == 1);
		return;
	}
	
	uint32 PrevSizeX = SizeX;
	uint32 PrevSizeY = SizeY;
	for( uint32 Level = 1; ; Level++ )
	{
		uint32 MipSizeX = ( ( SizeX - 1 ) >> Level ) + 1;
		uint32 MipSizeY = ( ( SizeY - 1 ) >> Level ) + 1;
		
		MipData[ Level - 1 ].AddUninitialized( MipSizeX * MipSizeY );
		MipDataFiltered[ Level - 1 ].AddUninitialized( MipSizeX * MipSizeY );

		for( uint32 y = 0; y < MipSizeY; y++ )
		{
			for( uint32 x = 0; x < MipSizeX; x++ )
			{
				uint32 x0 = x*2;
				uint32 y0 = y*2;
				uint32 x1 = FMath::Min( x0 + 1, PrevSizeX - 1 );
				uint32 y1 = FMath::Min( y0 + 1, PrevSizeY - 1 );

				if ( Level == 1 )
				{
					float d0 = Load( x0, y0 );
					float d1 = Load( x1, y0 );
					float d2 = Load( x0, y1 );
					float d3 = Load( x1, y1 );

					MipData[ Level - 1 ][ x + y * MipSizeX ] = FVector2f(
						FMath::Min( d0, FMath::Min3( d1, d2, d3 ) ),
						FMath::Max( d0, FMath::Max3( d1, d2, d3 ) ) );

					MipDataFiltered[ Level - 1 ][ x + y * MipSizeX ] = 0.25f * (d0 + d1 + d2 + d3);
				}
				else
				{
					FVector2f d0 = Load( x0, y0, Level - 1 );
					FVector2f d1 = Load( x1, y0, Level - 1 );
					FVector2f d2 = Load( x0, y1, Level - 1 );
					FVector2f d3 = Load( x1, y1, Level - 1 );

					MipData[ Level - 1 ][ x + y * MipSizeX ] = FVector2f(
						FMath::Min( d0.X, FMath::Min3( d1.X, d2.X, d3.X ) ),
						FMath::Max( d0.Y, FMath::Max3( d1.Y, d2.Y, d3.Y ) ) );

					float v0 = LoadFiltered( x0, y0, Level - 1 );
					float v1 = LoadFiltered( x1, y0, Level - 1 );
					float v2 = LoadFiltered( x0, y1, Level - 1 );
					float v3 = LoadFiltered( x1, y1, Level - 1 );

					MipDataFiltered[Level - 1][x + y * MipSizeX] = 0.25f * (v0 + v1 + v2 + v3);
				}
			}
		}

		PrevSizeX = MipSizeX;
		PrevSizeY = MipSizeY;
		NumLevels++;

		// Level NumLevel-1 corresponds to coarsest 1x1 average 
		if( MipSizeX == 1 && MipSizeY == 1 ) 
		{
			break;
		}
	}
}

void FDisplacementMap::InitDefault()
{
	SourceFormat = TSF_G8;
	BytesPerPixel = 1;
	SizeX = 1;
	SizeY = 1;
	NumLevels = 1;
	Magnitude = 0.f;
	Center = 0.f;
	AddressX = TA_Wrap;
	AddressY = TA_Wrap;

	Storage.Init(0, 1);
	SourceData = TArrayView64<const uint8>(Storage);
}

// Bilinear filtered
float FDisplacementMap::Sample( FVector2f UV ) const
{
	// Half texel
	UV.X = UV.X * static_cast<float>(SizeX) - 0.5f;
	UV.Y = UV.Y * static_cast<float>(SizeY) - 0.5f;

	int32 x0 = FMath::FloorToInt32( UV.X );
	int32 y0 = FMath::FloorToInt32( UV.Y );
	int32 x1 = x0 + 1;
	int32 y1 = y0 + 1;

	float wx1 = UV.X - static_cast<float>(x0);
	float wy1 = UV.Y - static_cast<float>(y0);
	float wx0 = 1.0f - wx1;
	float wy0 = 1.0f - wy1;

	return
		Sample( x0, y0 ) * wx0 * wy0 +
		Sample( x1, y0 ) * wx1 * wy0 +
		Sample( x0, y1 ) * wx0 * wy1 +
		Sample( x1, y1 ) * wx1 * wy1;
}

// Returns min/max over bilinear footprint
FVector2f FDisplacementMap::Sample( FVector2f MinUV, FVector2f MaxUV ) const
{
	// Half texel
	MinUV = MinUV * FVector2f( (float)SizeX, (float)SizeY ) - FVector2f( 0.5f );
	MaxUV = MaxUV * FVector2f( (float)SizeX, (float)SizeY ) - FVector2f( 0.5f );

	int32 x0 = FMath::FloorToInt32( MinUV.X );
	int32 y0 = FMath::FloorToInt32( MinUV.Y );
	int32 x1 = FMath::FloorToInt32( MaxUV.X ) + 1;
	int32 y1 = FMath::FloorToInt32( MaxUV.Y ) + 1;

	uint32 Level = FMath::FloorLog2( FMath::Max( x1 - x0, y1 - y0 ) );

	if( (x1 >> Level) - (x0 >> Level) > 1 ||
		(y1 >> Level) - (y0 >> Level) > 1 )
		Level++;

	Level = FMath::Min( Level, NumLevels - 1 );

	if ( Level == 0 )
	{
		float d0 = Sample( x0, y0 );
		float d1 = Sample( x1, y0 );
		float d2 = Sample( x0, y1 );
		float d3 = Sample( x1, y1 );

		return FVector2f(
			FMath::Min( d0, FMath::Min3( d1, d2, d3 ) ),
			FMath::Max( d0, FMath::Max3( d1, d2, d3 ) ) );
	}
	else
	{
		FVector2f d0 = Sample( x0, y0, Level );
		FVector2f d1 = Sample( x1, y0, Level );
		FVector2f d2 = Sample( x0, y1, Level );
		FVector2f d3 = Sample( x1, y1, Level );

		return FVector2f(
			FMath::Min( d0.X, FMath::Min3( d1.X, d2.X, d3.X ) ),
			FMath::Max( d0.Y, FMath::Max3( d1.Y, d2.Y, d3.Y ) ) );
	}
}


// Returns min/max over rectangular region footprint
FVector2f FDisplacementMap::SampleHierarchical(FVector2f MinUV, FVector2f MaxUV, const uint32 MaxRefinements) const
{	
	if (MaxRefinements == 0) 
	{
		// equivalent
		return Sample(MinUV, MaxUV);
	}

	// Half texel
	MinUV = MinUV * FVector2f((float)SizeX, (float)SizeY) - FVector2f(0.5f);
	MaxUV = MaxUV * FVector2f((float)SizeX, (float)SizeY) - FVector2f(0.5f);

	int32 X0 = FMath::FloorToInt32(MinUV.X);
	int32 Y0 = FMath::FloorToInt32(MinUV.Y);

	// inclusive, +1 to account for bilinear filtering
	int32 X1 = FMath::FloorToInt32(MaxUV.X) + 1; 
	int32 Y1 = FMath::FloorToInt32(MaxUV.Y) + 1;

	const FIntRect QueryWindow(X0, Y0, X1 + 1, Y1 + 1); // FIntRect is half-open

	struct FNode
	{
		uint32 Level;
		uint32 X, Y;
	};
	
	TArray<FNode, TInlineAllocator<64>> Stack;

	// initialize root notes
	uint32 Level = FMath::FloorLog2(FMath::Max(X1 - X0, Y1 - Y0));

	if ((X1 >> Level) - (X0 >> Level) > 1 ||
		(Y1 >> Level) - (Y0 >> Level) > 1)
	{
		Level++;
	}

	Level = FMath::Min(Level, NumLevels - 1);

	const uint32 Mask = ~((1<<Level)-1);
	const uint32 X0M = X0 & Mask;
	const uint32 X1M = X1 & Mask;
	const uint32 Y0M = Y0 & Mask;
	const uint32 Y1M = Y1 & Mask;

	Stack.Push( { Level, X0M, Y0M } );
	if ( X1M != X0M )	
	{
		Stack.Push( { Level, X1M, Y0M } );
	}
	if ( Y1M != Y0M )
	{
		Stack.Push( { Level, X0M, Y1M } );
		if ( X1M != X0M )	
		{
			Stack.Push( { Level, X1M, Y1M } );
		}
	}

	const uint32 MinLevel = Level - FMath::Min(Level, MaxRefinements);
	
	// result
	FVector2f MinMax( std::numeric_limits<float>::max(),
	                 -std::numeric_limits<float>::max() );

	while (!Stack.IsEmpty())
    {
        FNode Node = Stack.Pop();
		const uint32 NodeSize = 1u << Node.Level;
		const FIntRect NodeRect(Node.X, Node.Y, Node.X + NodeSize, Node.Y + NodeSize);

		if (!QueryWindow.Intersect(NodeRect))
		{
			continue;
		}

		// if node fully contained in query region or we can't refine do accumulate 
		if (QueryWindow.Contains(NodeRect) || (MinLevel > 0 && Node.Level == MinLevel))
		{
			FVector2f Bounds;
			if (Node.Level == 0) 
			{
				Bounds[0] = Bounds[1] = Sample(Node.X, Node.Y);
			}
			else
			{
				Bounds = Sample(Node.X, Node.Y, Node.Level);
			}

			MinMax[0] = FMath::Min(MinMax[0], Bounds[0]);
			MinMax[1] = FMath::Max(MinMax[1], Bounds[1]);

			continue;
		}

		if (Node.Level > MinLevel)
		{
			// partially overlaps -> refine
			const uint32 NextNodeSize = NodeSize >> 1;
			
			Stack.Push( { Node.Level-1, Node.X               , Node.Y                } );
			Stack.Push( { Node.Level-1, Node.X + NextNodeSize, Node.Y                } );
			Stack.Push( { Node.Level-1, Node.X               , Node.Y + NextNodeSize } );
			Stack.Push( { Node.Level-1, Node.X + NextNodeSize, Node.Y + NextNodeSize } );
		}
	}

	return MinMax;
}

float FDisplacementMap::SampleEWA(FVector2f UV, FVector2f Axis0, FVector2f Axis1) const
{
	// major/minor
	float Len1 = Axis1.Length();

	float LODLevelf  = FMath::Max(0.f, static_cast<float>(NumLevels) - 1.f + FMath::Log2(Len1));
	uint32 LODLeveli = FMath::FloorToInt(LODLevelf);
	float d = LODLevelf - static_cast<float>(LODLeveli);

	return ((1.f - d) * EWA(LODLeveli    , UV, Axis0, Axis1)
				 + d  * EWA(LODLeveli + 1, UV, Axis0, Axis1) - Center) * Magnitude;

}

float FDisplacementMap::EWA(uint32 Level, FVector2f UV, FVector2f Axis0, FVector2f Axis1) const
{
	// adapted from [ Pharr, Jakob, Humphreys 2023, "Physically Based Rendering" (4th Edition) ]

	if (Level >= NumLevels)
	{
		return LoadFiltered(0, 0, NumLevels-1);
	}

	const uint32 MipSizeX = ((SizeX - 1) >> Level) + 1;
	const uint32 MipSizeY = ((SizeY - 1) >> Level) + 1;

	FVector2f MipScale((float)MipSizeX, (float)MipSizeY);

	UV.X = UV.X * (float)MipScale.X - 0.5f;
	UV.Y = UV.Y * (float)MipScale.Y - 0.5f;
	Axis0 *= MipScale;
	Axis1 *= MipScale;

	float A = Axis0.Y * Axis0.Y + Axis1.Y * Axis1.Y + 1.f;
	float B = -2.f * (Axis0.X * Axis0.Y + Axis1.X * Axis1.Y);
	float C = Axis0.X * Axis0.X + Axis1.X * Axis1.X + 1.f;

	float InvF = 1.f / (A * C - 0.25f * B * B);
	A *= InvF;
	B *= InvF;
	C *= InvF;

	float Det = -B * B + 4.f * A * C;
	float InvDet = 1.f / Det;

	if (!FMath::IsFinite(InvDet))
	{
		return Sample(UV) / Magnitude + Center;
	}

	float USqrt = FMath::Sqrt(Det * C);
	float VSqrt = FMath::Sqrt(Det * A);

	const int32 U0 = FMath::CeilToInt32(UV.X - 2.f * InvDet * USqrt);
	const int32 U1 = FMath::FloorToInt32(UV.X + 2.f * InvDet * USqrt);

	const int32 V0 = FMath::CeilToInt32(UV.Y - 2.f * InvDet * VSqrt);
	const int32 V1 = FMath::FloorToInt32(UV.Y + 2.f * InvDet * VSqrt);

	float Sum = 0.f;
	float SumWts = 0.f;

	auto ClippedGaussian = [](const float XSqr) -> float
	{
		constexpr float Offset = 1.f / (UE_EULERS_NUMBER * UE_EULERS_NUMBER);
		return FMath::Exp(-XSqr * 2.f) - Offset;
	};

	// goes to 0 smoothly, avoids clipping, sharper than Gaussian
	// @todo needs testing
	auto BlackmanHarris = [](const float XSqr) -> float
	{
		const float X = FMath::Sqrt(XSqr);
		return 0.35875f + 0.48829f * FMath::Cos(UE_PI * X) + 0.14128f * FMath::Cos(2.f * UE_PI * X) + 0.01168f * FMath::Cos(3.f * UE_PI * X);
	};

	for (int32 Vi = V0; Vi <= V1; ++Vi)
	{
		const float Vf = static_cast<float>(Vi) - UV.Y;
		for (int32 Ui = U0; Ui <= U1; ++Ui)
		{
			const float Uf = static_cast<float>(Ui) - UV.X;
			const float RSqr = A * Uf * Uf + B * Uf * Vf + C * Vf * Vf;

			if (RSqr < 1.f)
			{
				const float Weight = ClippedGaussian(RSqr);

				const int32 X = TMathUtil<int32>::Clamp(Ui, 0, MipSizeX - 1);
				const int32 Y = TMathUtil<int32>::Clamp(Vi, 0, MipSizeY - 1);
				if (Level == 0)
				{
					Sum += Weight * Load(X, Y);
				}
				else
				{
					Sum += Weight * MipDataFiltered[Level - 1][X + Y * MipSizeX];
				}
				SumWts += Weight;
			}
		}
	}

	return Sum / SumWts;
}

FVector2f FDisplacementMap::WarpSample(const FVector2f& UV) const
{
	float U = UV[0];
	float V = UV[1];

	{
		uint32 MipSizeX = ( ( SizeX - 1 ) >> (NumLevels-1) ) + 1;
		uint32 MipSizeY = ( ( SizeY - 1 ) >> (NumLevels-1) ) + 1;
		check(MipSizeX == 1 && MipSizeY == 1);
	}

	uint32 X = 0; 
	uint32 Y = 0;

	for (uint32 Level = NumLevels-1; Level > 0; --Level )
	{
		const int ChildLevel = Level - 1;
		const uint32 MipSizeX = ((SizeX - 1) >> ChildLevel) + 1;
		const uint32 MipSizeY = ((SizeY - 1) >> ChildLevel) + 1;

		const uint32 ChildX = X << 1;
		const uint32 ChildY = Y << 1;

		float Child00 = 0.f, Child01 = 0.f, Child10 = 0.f, Child11 = 0.f;
		
		if (ChildLevel == 0)
		{
			Child00 = Load(ChildX  , ChildY  );

			if (ChildX+1 < MipSizeX)
			{
				Child10 = Load(ChildX+1, ChildY  );
			}
			if (ChildY+1 < MipSizeY)
			{
				Child01 = Load(ChildX  , ChildY+1);
			
				if (ChildX+1 < MipSizeX)
				{
					Child11 = Load(ChildX+1, ChildY+1);
				}
			}
		}
		else
		{
			const TArray<float>& ChildMipData = MipDataFiltered[ChildLevel-1];
			Child00 = ChildMipData[ ChildX   + (ChildY  ) * MipSizeX ];
			if (ChildX+1 < MipSizeX)
			{
				Child10 = ChildMipData[ ChildX+1 + (ChildY  ) * MipSizeX ];
			}
			if (ChildY+1 < MipSizeY)
			{
				Child01 = ChildMipData[ ChildX   + (ChildY+1) * MipSizeX ];
				if (ChildX+1 < MipSizeX)
				{
					Child11 = ChildMipData[ ChildX+1 + (ChildY+1) * MipSizeX ];
				}
			}
		}

		const float Sum0 = Child00 + Child10;
		const float Sum1 = Child01 + Child11;

		float ChildLeft, ChildRight;
		// top-bottom
		if (V * (Sum0 + Sum1) < Sum0 || ChildY+1 >= MipSizeY)
		{
			V *= (Sum0 + Sum1) / Sum0;
			Y = ChildY;
			ChildLeft  = Child00;
			ChildRight = Child10;
		}
		else
		{
			V = (V * (Sum0 + Sum1) - Sum0) / Sum1;
			Y = ChildY + 1;
			ChildLeft  = Child01;
			ChildRight = Child11;
		}

		// left-right
		if (U*(ChildLeft + ChildRight) < ChildLeft || ChildX+1 >= MipSizeX)
		{
			U *= (ChildLeft + ChildRight) / ChildLeft;
			X = ChildX; 
		}
		else
		{
			U = (U*(ChildLeft + ChildRight) - ChildLeft) / ChildRight;
			X = ChildX + 1;
		}
	}

	return FVector2f( (static_cast<float>(X) + U) / static_cast<float>(SizeX),
                	  (static_cast<float>(Y) + V) / static_cast<float>(SizeY) );
}

} // namespace Geometry
} // namespace UE
