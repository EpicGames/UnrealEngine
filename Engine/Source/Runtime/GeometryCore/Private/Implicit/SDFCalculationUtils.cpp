// Copyright Epic Games, Inc. All Rights Reserved.


#include "Implicit/SDFCalculationUtils.h"

using namespace UE::Geometry;

// find distance x0 is from triangle x1-x2-x3
double UE::Geometry::PointTriangleDistance(const FVector3d& x0, const FVector3d& x1, const FVector3d& x2, const FVector3d& x3)
{
	// first find barycentric coordinates of closest point on infinite plane
	FVector3d x13 = (x1 - x3);
	FVector3d x23 = (x2 - x3);
	FVector3d x03 = (x0 - x3);
	double m13 = x13.SquaredLength(), m23 = x23.SquaredLength(), d = x13.Dot(x23);
	double invdet = 1.0 / FMath::Max(m13 * m23 - d * d, 1e-30);
	double a = x13.Dot(x03), b = x23.Dot(x03);
	// the barycentric coordinates themselves
	double w23 = invdet * (m23 * a - d * b);
	double w31 = invdet * (m13 * b - d * a);
	double w12 = 1 - w23 - w31;
	if (w23 >= 0 && w31 >= 0 && w12 >= 0) // if we're inside the triangle
	{
		return Distance(x0, w23*x1 + w31*x2 + w12*x3);
	}
	else // we have to clamp to one of the edges
	{
		if (w23 > 0) // this rules out edge 2-3 for us
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x1, x3));
		}
		else if (w31 > 0) // this rules out edge 1-3
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x2, x3));
		}
		else // w12 must be >0, ruling out edge 1-2
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x3), PointSegmentDistance(x0, x2, x3));
		}
	}
}



// find distance x0 is from triangle x1-x2-x3
float UE::Geometry::PointTriangleDistance(const FVector3f& x0, const FVector3f& x1, const FVector3f& x2, const FVector3f& x3)
{
	// first find barycentric coordinates of closest point on infinite plane
	FVector3f x13 = (x1 - x3);
	FVector3f x23 = (x2 - x3);
	FVector3f x03 = (x0 - x3);
	float m13 = x13.SquaredLength(), m23 = x23.SquaredLength(), d = x13.Dot(x23);
	float invdet = 1.0f / FMath::Max(m13 * m23 - d * d, 1e-30f);
	float a = x13.Dot(x03), b = x23.Dot(x03);
	// the barycentric coordinates themselves
	float w23 = invdet * (m23 * a - d * b);
	float w31 = invdet * (m13 * b - d * a);
	float w12 = 1 - w23 - w31;
	if (w23 >= 0 && w31 >= 0 && w12 >= 0) // if we're inside the triangle
	{
		return Distance(x0, w23*x1 + w31*x2 + w12*x3);
	}
	else // we have to clamp to one of the edges
	{
		if (w23 > 0) // this rules out edge 2-3 for us
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x1, x3));
		}
		else if (w31 > 0) // this rules out edge 1-3
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x2, x3));
		}
		else // w12 must be >0, ruling out edge 1-2
		{
			return FMath::Min(PointSegmentDistance(x0, x1, x3), PointSegmentDistance(x0, x2, x3));
		}
	}
}



// robust test of (X0,Y0) in the triangle (X1,Y1)-(X2,Y2)-(X3,Y3)
// if true is returned, the barycentric coordinates are set in A,B,C.
bool UE::Geometry::PointInTriangle2d(double X0, double Y0, double X1, double Y1, double X2, double Y2, double X3, double Y3, double& A, double& B, double& C)
{
	A = B = C = 0;
	X1 -= X0; X2 -= X0; X3 -= X0;
	Y1 -= Y0; Y2 -= Y0; Y3 -= Y0;
	int signa = Orientation(X2, Y2, X3, Y3, A);
	if (signa == 0) return false;
	int signb = Orientation(X3, Y3, X1, Y1, B);
	if (signb != signa) return false;
	int signc = Orientation(X1, Y1, X2, Y2, C);
	if (signc != signa) return false;
	double sum = A + B + C;
	// if the SOS signs match and are nonzero, there's no way all of A, B, and C are zero.
	// TODO: is this mathematically impossible? can we just remove the check?
	checkSlow(sum != 0); // TEXT("TCachingMeshSDF::PointInTriangle2d: impossible config?"));
	A /= sum;
	B /= sum;
	C /= sum;
	return true;
}

