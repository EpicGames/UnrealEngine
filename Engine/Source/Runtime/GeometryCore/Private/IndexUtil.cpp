// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/IndexUtil.h"

using namespace UE::Geometry;

// integer indices offsets in x/y directions
const FVector2i IndexUtil::GridOffsets4[] =
{
	FVector2i(-1, 0), FVector2i(1, 0),
	FVector2i(0, -1), FVector2i(0, 1)
};

// integer indices offsets in x/y directions and diagonals
const FVector2i IndexUtil::GridOffsets8[] =
{
	FVector2i(-1, 0),  FVector2i(1, 0),
	FVector2i(0, -1),  FVector2i(0, 1),
	FVector2i(-1, 1),  FVector2i(1, 1),
	FVector2i(-1, -1), FVector2i(1, -1)
};

// integer indices offsets in x/y/z directions, corresponds w/ BoxFaces directions
const FVector3i IndexUtil::GridOffsets6[] =
{
	FVector3i( 0, 0,-1), FVector3i( 0, 0, 1),
	FVector3i(-1, 0, 0), FVector3i( 1, 0, 0),
	FVector3i( 0,-1, 0), FVector3i( 0, 1, 0)
};

// integer indices offsets in x/y/z directions and diagonals
const FVector3i IndexUtil::GridOffsets26[] = 
{
	// face-nbrs
	FVector3i(0, 0,-1), FVector3i(0, 0, 1),
	FVector3i(-1, 0, 0), FVector3i(1, 0, 0),
	FVector3i(0,-1, 0), FVector3i(0, 1, 0),
	// edge-nbrs (+y, 0, -y)
	FVector3i(1, 1, 0), FVector3i(-1, 1, 0),
	FVector3i(0, 1, 1), FVector3i(0, 1,-1),
	FVector3i(1, 0, 1), FVector3i(-1, 0, 1),
	FVector3i(1, 0,-1), FVector3i(-1, 0,-1),
	FVector3i(1, -1, 0), FVector3i(-1,-1, 0),
	FVector3i(0, -1, 1), FVector3i(0,-1,-1),
	// corner-nbrs (+y,-y)
	FVector3i(1, 1, 1), FVector3i(-1, 1, 1),
	FVector3i(1, 1,-1), FVector3i(-1, 1,-1),
	FVector3i(1,-1, 1), FVector3i(-1,-1, 1),
	FVector3i(1,-1,-1), FVector3i(-1,-1,-1)
};




// corners [ (-x,-y), (x,-y), (x,y), (-x,y) ], -z, then +z
//
//   7---6     +z       or        3---2     -z
//   |\  |\                       |\  |\
//   4-\-5 \                      0-\-1 \
//    \ 3---2                      \ 7---6   
//     \|   |                       \|   |
//      0---1  -z                    4---5  +z
//

const int IndexUtil::BoxFaces[6][4] = 
{
	{ 0, 1, 2, 3 },     // back, -z
	{ 5, 4, 7, 6 },     // front, +z
	{ 4, 0, 3, 7 },     // left, -x
	{ 1, 5, 6, 2 },     // right, +x,
	{ 4, 5, 1, 0 },     // bottom, -y
	{ 3, 2, 6, 7 }      // top, +y
};


static const FVector2i UV00(0, 0);
static const FVector2i UV10(1, 0);
static const FVector2i UV11(1, 1);
static const FVector2i UV01(0, 1);
const FVector2i IndexUtil::BoxFacesUV[4] = { UV00, UV10, UV11, UV01 };

const int IndexUtil::BoxFaceNormals[6] = { -3, 3, -1, 1, -2, 2 };
