// Copyright Epic Games, Inc. All Rights Reserved.

#include "Intersection/ContainmentQueries3.h"

using namespace UE::Geometry;
using namespace UE::Math;

template<typename RealType>
bool UE::Geometry::IsInside(const TSphere3<RealType>& OuterSphere, const TSphere3<RealType>& InnerSphere)
{
	return OuterSphere.Contains(InnerSphere);
}

template<typename RealType>
bool UE::Geometry::IsInside(const TSphere3<RealType>& OuterSphere, const TCapsule3<RealType>& InnerCapsule)
{
	return OuterSphere.Contains(TSphere3<RealType>(InnerCapsule.Segment.StartPoint(), InnerCapsule.Radius))
		&& OuterSphere.Contains(TSphere3<RealType>(InnerCapsule.Segment.EndPoint(), InnerCapsule.Radius));
}

template<typename RealType>
bool UE::Geometry::IsInside(const TSphere3<RealType>& OuterSphere, const TOrientedBox3<RealType>& InnerBox)
{
	return InnerBox.TestCorners([&](const TVector<RealType>& Point) {
		return OuterSphere.Contains(Point);
	});
}






template<typename RealType>
bool UE::Geometry::IsInside(const TCapsule3<RealType>& OuterCapsule, const TSphere3<RealType>& InnerSphere)
{
	RealType DistSegPt = TMathUtil<RealType>::Sqrt(OuterCapsule.Segment.DistanceSquared(InnerSphere.Center));
	return (DistSegPt + InnerSphere.Radius) <= OuterCapsule.Radius;
}


template<typename RealType>
bool UE::Geometry::IsInside(const TCapsule3<RealType>& OuterCapsule, const TCapsule3<RealType>& InnerCapsule)
{
	// is this correct?
	RealType Dist0 = TMathUtil<RealType>::Sqrt(OuterCapsule.Segment.DistanceSquared(InnerCapsule.Segment.StartPoint()));
	RealType Dist1 = TMathUtil<RealType>::Sqrt(OuterCapsule.Segment.DistanceSquared(InnerCapsule.Segment.EndPoint()));
	RealType MaxSegDist = TMathUtil<RealType>::Max(Dist0, Dist1);

	return (MaxSegDist + InnerCapsule.Radius) <= OuterCapsule.Radius;
}


template<typename RealType>
bool UE::Geometry::IsInside(const TCapsule3<RealType>& OuterCapsule, const TOrientedBox3<RealType>& InnerBox)
{
	// todo: possibly more efficient to do this by calculating distance to box center and then adding on box radius?
	// not sure if this would an exact test though, or just a bound

	return InnerBox.TestCorners([&](const TVector<RealType>& Point) {
		return OuterCapsule.Contains(Point);
	});
}





template<typename RealType>
bool UE::Geometry::IsInside(const TOrientedBox3<RealType>& OuterBox, const TOrientedBox3<RealType>& InnerBox)
{
	const TFrame3<RealType>& Frame = OuterBox.Frame;
	const TVector<RealType>& Extents = OuterBox.Extents;
	TVector<RealType> Axes[3];
	OuterBox.Frame.GetAxes(Axes[0], Axes[1], Axes[2]);
	for (int32 k = 0; k < 3; ++k)
	{
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(Axes[k], Frame.Origin + Extents[k]*Axes[k]), InnerBox))
		{
			return false;
		}
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(-Axes[k], Frame.Origin - Extents[k]*Axes[k]), InnerBox))
		{
			return false;
		}
	}
	return true;
}


template<typename RealType>
bool UE::Geometry::IsInside(const TOrientedBox3<RealType>& OuterBox, const TSphere3<RealType>& InnerSphere)
{
	const TFrame3<RealType>& Frame = OuterBox.Frame;
	const TVector<RealType>& Extents = OuterBox.Extents;
	TVector<RealType> Axes[3];
	OuterBox.Frame.GetAxes(Axes[0], Axes[1], Axes[2]);
	for (int32 k = 0; k < 3; ++k)
	{
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(Axes[k], Frame.Origin + Extents[k]*Axes[k]), InnerSphere))
		{
			return false;
		}
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(-Axes[k], Frame.Origin - Extents[k]*Axes[k]), InnerSphere))
		{
			return false;
		}
	}
	return true;
}



template<typename RealType>
bool UE::Geometry::IsInside(const TOrientedBox3<RealType>& OuterBox, const TCapsule3<RealType>& InnerCapsule)
{
	const TFrame3<RealType>& Frame = OuterBox.Frame;
	const TVector<RealType>& Extents = OuterBox.Extents;
	TVector<RealType> Axes[3];
	OuterBox.Frame.GetAxes(Axes[0], Axes[1], Axes[2]);
	for (int32 k = 0; k < 3; ++k)
	{
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(Axes[k], Frame.Origin + Extents[k]*Axes[k]), InnerCapsule))
		{
			return false;
		}
		if (UE::Geometry::TestIntersection<RealType>(THalfspace3<RealType>(-Axes[k], Frame.Origin - Extents[k]*Axes[k]), InnerCapsule))
		{
			return false;
		}
	}
	return true;
}



template<typename RealType>
bool UE::Geometry::DoesCylinderContainPoint(const TVector<RealType>& CylinderCenter, const TVector<RealType>& NormalizedCylinderAxis, 
	RealType CylinderRadius, RealType CylinderHeight, const TVector<RealType>& QueryPoint)
{
	// In debug build, verify that the cylinder axis is indeed normalized
	checkSlow(FMath::Abs(1 - NormalizedCylinderAxis.SquaredLength()) <= KINDA_SMALL_NUMBER);

	RealType TValue = (QueryPoint - CylinderCenter).Dot(NormalizedCylinderAxis);
	if (FMath::Abs(TValue) > CylinderHeight / 2)
	{
		return false;
	}
	TVector<RealType> ProjectedPoint = CylinderCenter + TValue * NormalizedCylinderAxis;
	return (QueryPoint - ProjectedPoint).SquaredLength() <= CylinderRadius * CylinderRadius;
}



namespace UE
{
	namespace Geometry
	{
		template bool GEOMETRYCORE_API IsInside(const TSphere3<float>& OuterSphere, const TSphere3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TSphere3<double>& OuterSphere, const TSphere3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TSphere3<float>& OuterSphere, const TCapsule3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TSphere3<double>& OuterSphere, const TCapsule3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TSphere3<float>& OuterSphere, const TOrientedBox3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TSphere3<double>& OuterSphere, const TOrientedBox3<double>& InnerSphere);

		template bool GEOMETRYCORE_API IsInside(const TCapsule3<float>& OuterSphere, const TCapsule3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TCapsule3<double>& OuterSphere, const TCapsule3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TCapsule3<float>& OuterSphere, const TSphere3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TCapsule3<double>& OuterSphere, const TSphere3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TCapsule3<float>& OuterSphere, const TOrientedBox3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TCapsule3<double>& OuterSphere, const TOrientedBox3<double>& InnerSphere);

		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<float>& OuterSphere, const TOrientedBox3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<double>& OuterSphere, const TOrientedBox3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<float>& OuterSphere, const TSphere3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<double>& OuterSphere, const TSphere3<double>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<float>& OuterSphere, const TCapsule3<float>& InnerSphere);
		template bool GEOMETRYCORE_API IsInside(const TOrientedBox3<double>& OuterSphere, const TCapsule3<double>& InnerSphere);

		template bool GEOMETRYCORE_API DoesCylinderContainPoint(const TVector<float>& CylinderCenter, const TVector<float>& CylinderAxis, 
			float CylinderRadius, float CylinderHeight, const TVector<float>& QueryPoint);
		template bool GEOMETRYCORE_API DoesCylinderContainPoint(const TVector<double>& CylinderCenter, const TVector<double>& CylinderAxis, 
			double CylinderRadius, double CylinderHeight, const TVector<double>& QueryPoint);
	}
}
