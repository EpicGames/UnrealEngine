// Copyright Epic Games, Inc. All Rights Reserved.

#include "Intersection/IntersectionQueries2.h"

using namespace UE::Geometry;

template<typename Real>
bool UE::Geometry::TestIntersection(const TSegment2<Real>& Segment, const TAxisAlignedBox2<Real>& Box)
{
	// TODO Port Wild Magic IntrSegment2Box2 (which requires porting IntrLine2Box2) so we can call segment-box intersection here
	
	// If either endpoint is inside, then definitely (at least partially) contained
	if (Box.Contains(Segment.StartPoint()) || Box.Contains(Segment.EndPoint()))
	{
		return true;
	}

	// If both outside, have to do some intersections with the box sides

	if (Segment.Intersects(TSegment2<Real>(Box.GetCorner(0), Box.GetCorner(1))))
	{
		return true;
	}

	if (Segment.Intersects(TSegment2<Real>(Box.GetCorner(1), Box.GetCorner(2))))
	{
		return true;
	}

	return Segment.Intersects(TSegment2<Real>(Box.GetCorner(3), Box.GetCorner(2)));

	// Don't need to intersect with the fourth side because segment would have to intersect two sides
	// of box if both endpoints are outside the box.
}

template<typename Real>
bool UE::Geometry::DoesTriangleIntersectCircle2D(
	const TVector2<Real>& A, const TVector2<Real>& B, const TVector2<Real>& C,
	const TVector2<Real>& Center, double RadiusSquared)
{
	// Eliminate the fully contained case by checking for containment of circle center. Could use
	//  TTriangle2::IsInside, but we need to reuse the edge vectors anyway for the other tests, so
	//  we do it ourselves.
	TVector2<Real> SideVectors[3]{ B - A, C - B, A - C };
	TVector2<Real> CenterRelativeCorner[3]{ Center - A, Center - B, Center - C };
	double Signs[3]
	{
		DotPerp(SideVectors[0], CenterRelativeCorner[0]),
		DotPerp(SideVectors[1], CenterRelativeCorner[1]),
		DotPerp(SideVectors[2], CenterRelativeCorner[2]),
	};
	// Note that we don't use >= here because that is susceptible to issues with degenerate triangles.
	//  If one or more of the signs are legitimately zero, then we are on a side of the triangle,
	//  and we should succeed in the checks further below.
	if (Signs[0] * Signs[1] > 0 && Signs[1] * Signs[2] > 0 && Signs[2] * Signs[0] > 0)
	{
		return true;
	}

	// If not inside, try seeing if the circle covers one of the corner points
	for (int i = 0; i < 3; ++i)
	{
		if (CenterRelativeCorner[i].SizeSquared() <= RadiusSquared)
		{
			return true;
		}
	}

	// If still no intersection, try projecting onto each of the sides and see if the distance 
	//  to the projected point is close enough.
	for (int i = 0; i < 3; ++i)
	{
		// First check to see if we're inside the bounds of the segment (i.e. inside the two
		//  perpendicular lines on each end). Note that we don't allow equality here because 
		//  that would mean that we're in-line with the end corner, at which point we should have
		//  succeeded in the corner containment tests above, if we were going to succeed. At the 
		//  same time, not allowing equality to zero allows us to not worry about degenerate edges
		//  causing the dot product to go to zero.
		if (SideVectors[i].Dot(CenterRelativeCorner[i]) > 0
			&& SideVectors[i].Dot(CenterRelativeCorner[(i + 1) % 3]) < 0

			// Now check the projected distance. Signs is dot of unnormalized side perpendicular
			//  vector with the relative center location, so dividing by side length would give us
			//  the distance from that side. Rearranging a bit to avoid divide and sqrt, we get:
			&& FMath::Square(Signs[i]) <= RadiusSquared * SideVectors[i].SizeSquared())
		{
			// Note about Signs[i] being 0 "incorrectly": the dot product not being zero tells us
			//  that the side was not zero. Thus, Signs[i] should be a valid value, except possibly
			//  up to two TNumericLimits<Real>::Min() lower than it should be if it underflowed in
			//  DotPerp.
			return true;
		}
	}

	return false;
}

namespace UE
{
namespace Geometry
{

template bool GEOMETRYCORE_API TestIntersection(const TSegment2<float>& Segment, const TAxisAlignedBox2<float>& Box);
template bool GEOMETRYCORE_API TestIntersection(const TSegment2<double>& Segment, const TAxisAlignedBox2<double>& Box);
		
template bool GEOMETRYCORE_API DoesTriangleIntersectCircle2D(
	const TVector2<float>& A, const TVector2<float>& B, const TVector2<float>& C,
	const TVector2<float>& Center, double RadiusSquared);
template bool GEOMETRYCORE_API DoesTriangleIntersectCircle2D(
	const TVector2<double>& A, const TVector2<double>& B, const TVector2<double>& C,
	const TVector2<double>& Center, double RadiusSquared);

} // namespace UE::Geometry
} // namespace UE
