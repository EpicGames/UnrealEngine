// Copyright Epic Games, Inc. All Rights Reserved.

#include "Intersection/IntersectionQueries3.h"

using namespace UE::Geometry;



template<typename RealType>
bool UE::Geometry::TestIntersection(const THalfspace3<RealType>& Halfspace, const TSphere3<RealType>& Sphere)
{
	RealType ProjectedCenter = Halfspace.Normal.Dot(Sphere.Center) - Halfspace.Constant;
	return (ProjectedCenter + Sphere.Radius >= 0);
}


template<typename RealType>
bool UE::Geometry::TestIntersection(const THalfspace3<RealType>& Halfspace, const TCapsule3<RealType>& Capsule)
{
	RealType ProjectedP0 = Halfspace.Normal.Dot(Capsule.Segment.StartPoint()) - Halfspace.Constant;
	RealType ProjectedP1 = Halfspace.Normal.Dot(Capsule.Segment.EndPoint()) - Halfspace.Constant;
	return (TMathUtil<RealType>::Max(ProjectedP0, ProjectedP1) + Capsule.Radius >= 0);
}



template<typename RealType>
bool UE::Geometry::TestIntersection(const THalfspace3<RealType>& Halfspace, const TOrientedBox3<RealType>& Box)
{
	RealType Center = Halfspace.Normal.Dot(Box.Frame.Origin) - Halfspace.Constant;
	TVector<RealType> X, Y, Z;
	Box.Frame.GetAxes(X, Y, Z);
	RealType Radius =
		TMathUtil<RealType>::Abs(Box.Extents.X * Halfspace.Normal.Dot(X)) +
		TMathUtil<RealType>::Abs(Box.Extents.Y * Halfspace.Normal.Dot(Y)) +
		TMathUtil<RealType>::Abs(Box.Extents.Z * Halfspace.Normal.Dot(Z));
	return (Center + Radius >= 0);
}


namespace UE
{
	namespace Geometry
	{
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<float>& Halfspace, const TSphere3<float>& InnerSphere);
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<double>& Halfspace, const TSphere3<double>& InnerSphere);
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<float>& Halfspace, const TCapsule3<float>& InnerSphere);
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<double>& Halfspace, const TCapsule3<double>& InnerSphere);
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<float>& Halfspace, const TOrientedBox3<float>& InnerSphere);
		template bool GEOMETRYCORE_API TestIntersection(const THalfspace3<double>& Halfspace, const TOrientedBox3<double>& InnerSphere);
	}
}
