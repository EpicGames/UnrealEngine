// Copyright Epic Games, Inc. All Rights Reserved.

#include "MathUtil.h"

#include "Math/UnrealMath.h"

#include <cfloat>

using namespace UE::Geometry;

constexpr float TMathUtilConstants<float>::Epsilon;
constexpr float TMathUtilConstants<float>::ZeroTolerance;
constexpr float TMathUtilConstants<float>::MaxReal;
constexpr float TMathUtilConstants<float>::Pi;
constexpr float TMathUtilConstants<float>::FourPi;
constexpr float TMathUtilConstants<float>::TwoPi;
constexpr float TMathUtilConstants<float>::HalfPi;
constexpr float TMathUtilConstants<float>::InvPi;
constexpr float TMathUtilConstants<float>::InvTwoPi;
constexpr float TMathUtilConstants<float>::DegToRad;
constexpr float TMathUtilConstants<float>::RadToDeg;
constexpr float TMathUtilConstants<float>::Sqrt2;
constexpr float TMathUtilConstants<float>::InvSqrt2;
constexpr float TMathUtilConstants<float>::Sqrt3;
constexpr float TMathUtilConstants<float>::InvSqrt3;

constexpr double TMathUtilConstants<double>::Epsilon;
constexpr double TMathUtilConstants<double>::ZeroTolerance;
constexpr double TMathUtilConstants<double>::MaxReal;
constexpr double TMathUtilConstants<double>::Pi;
constexpr double TMathUtilConstants<double>::FourPi;
constexpr double TMathUtilConstants<double>::TwoPi;
constexpr double TMathUtilConstants<double>::HalfPi;
constexpr double TMathUtilConstants<double>::InvPi;
constexpr double TMathUtilConstants<double>::InvTwoPi;
constexpr double TMathUtilConstants<double>::DegToRad;
constexpr double TMathUtilConstants<double>::RadToDeg;
constexpr double TMathUtilConstants<double>::Sqrt2;
constexpr double TMathUtilConstants<double>::InvSqrt2;
constexpr double TMathUtilConstants<double>::Sqrt3;
constexpr double TMathUtilConstants<double>::InvSqrt3;
