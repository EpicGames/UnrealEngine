// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshBoundaryLoops.h"

using namespace UE::Geometry;

int FMeshBoundaryLoops::GetMaxVerticesLoopIndex() const
{
	int j = 0;
	for (int i = 1; i < Loops.Num(); ++i)
	{
		if (Loops[i].Vertices.Num() > Loops[j].Vertices.Num())
		{
			j = i;
		}
	}
	return j;
}



int FMeshBoundaryLoops::GetLongestLoopIndex() const
{
	int32 LongestLoopIdx = -1;
	double LongestLoopLen = 0;
	for (int i = 0; i < Loops.Num(); ++i)
	{
		int32 LoopNum = Loops[i].Vertices.Num();
		double LoopLen = 0;
		for (int32 k = 0; k < LoopNum; ++k)
		{
			FVector3d NextPos = Mesh->GetVertex(Loops[i].Vertices[(k+1)%LoopNum]);
			LoopLen += Distance(Mesh->GetVertex(Loops[i].Vertices[k]), NextPos);
		}
		if (LoopLen > LongestLoopLen)
		{
			LongestLoopLen = LoopLen;
			LongestLoopIdx = i;
		}
	}
	return LongestLoopIdx;
}


FIndex2i FMeshBoundaryLoops::FindVertexInLoop(int VertexID) const
{
	int N = Loops.Num();
	for (int li = 0; li < N; ++li)
	{
		int idx = Loops[li].FindVertexIndex(VertexID);
		if (idx >= 0)
		{
			return FIndex2i(li, idx);
		}
	}
	return FIndex2i::Invalid();
}



int FMeshBoundaryLoops::FindLoopContainingVertex(int VertexID) const
{
	int N = Loops.Num();
	for (int li = 0; li < N; ++li)
	{
		if (Loops[li].Vertices.Contains(VertexID))
		{
			return li;
		}
	}
	return -1;
}


int FMeshBoundaryLoops::FindLoopContainingEdge(int EdgeID) const
{
	int N = Loops.Num();
	for (int li = 0; li < N; ++li)
	{
		if (Loops[li].Edges.Contains(EdgeID))
		{
			return li;
		}
	}
	return -1;
}




bool FMeshBoundaryLoops::Compute()
{
	// This algorithm assumes that triangles are oriented consistently, 
	// so closed boundary-loop can be followed by walking edges in-order

	Loops.Reset(); Spans.Reset();
	bSawOpenSpans = bFellBackToSpansOnFailure = false;

	// early-out if we don't actually have boundaries
	if (Mesh->IsClosed())
	{
		return true;
	}

	int NE = Mesh->MaxEdgeID();

	// Temporary memory used to indicate when we have "used" an edge.
	TArray<bool> used_edge;
	used_edge.Init(false, Mesh->MaxEdgeID());

	// current loop is stored here, cleared after each loop extracted
	TArray<int> loop_edges;
	TArray<int> loop_verts;
	TArray<int> bowties;

	// Temp buffer for reading back all boundary edges of a vertex.
	// probably always small but : pathological cases it could be large...
	TArray<int> all_e;
	all_e.Reserve(32);

	// [TODO] might make sense to precompute some things here, like num_be for each bdry vtx?

	// process all edges of mesh
	for (int eid = 0; eid < NE; ++eid)
	{
		if (Mesh->IsEdge(eid) == false)
		{
			continue;
		}
		if (used_edge[eid] == true)
		{
			continue;
		}
		if (Mesh->IsBoundaryEdge(eid) == false)
		{
			continue;
		}

		if (EdgeFilterFunc != nullptr && EdgeFilterFunc(eid) == false)
		{
			used_edge[eid] = true;
			continue;
		}

		// ok this is start of a boundary chain
		int eStart = eid;
		used_edge[eStart] = true;
		loop_edges.Add(eStart);

		int eCur = eid;

		// follow the chain in order of oriented edges
		bool bClosed = false;
		bool bIsOpenSpan = false;
		while (!bClosed)
		{
			FIndex2i ev = Mesh->GetOrientedBoundaryEdgeV(eCur);
			int cure_a = ev.A, cure_b = ev.B;
			if (bIsOpenSpan)
			{
				cure_a = ev.B; cure_b = ev.A;
			}
			else
			{
				loop_verts.Add(cure_a);
			}

			int e0 = -1, e1 = 1;
			int bdry_nbrs = Mesh->GetVtxBoundaryEdges(cure_b, e0, e1);

			// have to filter this list, if we are filtering. this is ugly.
			if (EdgeFilterFunc != nullptr)
			{
				if (bdry_nbrs > 2)
				{
					// we may repreat this below...irritating...
					all_e.Reset();
					int num_be = Mesh->GetAllVtxBoundaryEdges(cure_b, all_e);
					num_be = BufferUtil::CountValid(all_e, EdgeFilterFunc, num_be);
				}
				else
				{
					if (EdgeFilterFunc(e0) == false) bdry_nbrs--;
					if (EdgeFilterFunc(e1) == false) bdry_nbrs--;
				}
			}


			if (bdry_nbrs < 2)
			{   // hit an 'endpoint' vertex (should only happen when Filter is on...)
				if (SpanBehavior == ESpanBehaviors::Abort)
				{
					bSawOpenSpans = true;
					goto CATASTROPHIC_ABORT;
				}
				if (bIsOpenSpan)
				{
					bClosed = true;
					continue;
				}
				else
				{
					bIsOpenSpan = true;    // begin open span
					eCur = loop_edges[0];  // restart at other end of loop
					Algo::Reverse(loop_edges);  // do this so we can push to front
					continue;
				}
			}

			int eNext = -1;

			if (bdry_nbrs > 2)
			{
				// found "bowtie" vertex...things just got complicated!

				if (cure_b == loop_verts[0])
				{
					// The "end" of the current edge is the same as the start vertex.
					// This means we can close the loop here. Might as well!
					eNext = -2;   // sentinel value used below

				}
				else
				{
					// try to find an unused outgoing edge that is oriented properly.
					// This could create sub-loops, we will handle those later
					all_e.Reset();
					int num_be = Mesh->GetAllVtxBoundaryEdges(cure_b, all_e);
					check(num_be == bdry_nbrs);

					if (EdgeFilterFunc != nullptr)
					{
						num_be = BufferUtil::FilterInPlace(all_e, EdgeFilterFunc, num_be);
					}

					// Try to pick the best "turn left" vertex.
					eNext = FindLeftTurnEdge(eCur, cure_b, all_e, num_be, used_edge);
					if (eNext == -1)
					{
						if (FailureBehavior == EFailureBehaviors::Abort || SpanBehavior == ESpanBehaviors::Abort)
						{
							goto CATASTROPHIC_ABORT;
						}

						// ok, we are stuck. all we can do now is terminate this loop and keep it as a span
						if (bIsOpenSpan)
						{
							bClosed = true;
						}
						else
						{
							bIsOpenSpan = true;
							bClosed = true;
						}
						continue;
					}
				}

				if (bowties.Contains(cure_b) == false)
				{
					bowties.Add(cure_b);
				}

			}
			else
			{
				// walk forward to next available edge
				check(e0 == eCur || e1 == eCur);
				eNext = (e0 == eCur) ? e1 : e0;
			}

			if (eNext == -2)
			{
				// found a bowtie vert that is the same as start-of-loop, so we
				// are just closing it off explicitly
				bClosed = true;
			}
			else if (eNext == eStart)
			{
				// found edge at start of loop, so loop is done.
				bClosed = true;
			}
			else if (used_edge[eNext] != false)
			{
				// disaster case - the next edge is already used, but it is not the start of our loop
				// All we can do is convert to open span and terminate
				if (FailureBehavior == EFailureBehaviors::Abort || SpanBehavior == ESpanBehaviors::Abort)
				{
					goto CATASTROPHIC_ABORT;
				}
				bIsOpenSpan = true;
				bClosed = true;

			}
			else
			{
				// push onto accumulated list
				check(used_edge[eNext] == false);
				loop_edges.Add(eNext);
				used_edge[eNext] = true;
				eCur = eNext;
			}
		}

		if (bIsOpenSpan || bOnlyComputeSpans)
		{
			bSawOpenSpans = true;
			if (SpanBehavior == ESpanBehaviors::Compute)
			{
				Algo::Reverse(loop_edges);  // orient properly
				FEdgeSpan& NewSpan = Spans[Spans.Emplace()];
				NewSpan.InitializeFromEdges(Mesh, loop_edges);
			}
		}
		else if (bowties.Num() > 0)
		{
			// if we saw a bowtie vertex, we might need to break up this loop,
			// so call ExtractSubloops
			Subloops subloops;
			bool bSubloopsOK = ExtractSubloops(loop_verts, loop_edges, bowties, subloops);
			if (bSubloopsOK == false)
			{
				if (FailureBehavior == EFailureBehaviors::Abort)
				{
					goto CATASTROPHIC_ABORT;
				}

				if (subloops.Spans.Num() > 0)
				{
					bFellBackToSpansOnFailure = true;
					for (FEdgeSpan& span : subloops.Spans)
					{
						Spans.Add(span);
					}
				}
			}
			else
			{
				for (FEdgeLoop& loop : subloops.Loops)
				{
					Loops.Add(loop);
				}
			}
		}
		else
		{
			// clean simple loop, convert to EdgeLoop instance
			FEdgeLoop& NewLoop = Loops[Loops.Emplace()];
			NewLoop.Initialize(Mesh, loop_verts, loop_edges);
		}

		// reset these lists
		loop_edges.Reset();
		loop_verts.Reset();
		bowties.Reset();
	}

	return true;

CATASTROPHIC_ABORT:
	bAborted = true;
	return false;
}



FVector3d FMeshBoundaryLoops::GetVertexNormal(int vid)
{
	FVector3d n = FVector3d::Zero();
	for (int ti : Mesh->VtxTrianglesItr(vid))
	{
		n += Mesh->GetTriNormal(ti);
	}
	Normalize(n);
	return n;
}




// ok, bdry_edges[0...bdry_edges_count] contains the boundary edges coming out of bowtie_v.
// We want to pick the best one to continue the loop that came in to bowtie_v on incoming_e.
// If the loops are all sane, then we will get the smallest loops by "turning left" at bowtie_v.
// So, we compute the tangent plane at bowtie_v, and then the signed angle for each
// viable edge in this plane. 
//
// [TODO] handle degenerate edges. what do we do then? Currently will only chose
//  degenerate edge if there are no other options (I think...)
int FMeshBoundaryLoops::FindLeftTurnEdge(int incoming_e, int bowtie_v, TArray<int>& bdry_edges, int bdry_edges_count, TArray<bool>& used_edges)
{
	// compute normal and edge [a,bowtie]
	FVector3d n = GetVertexNormal(bowtie_v);
	//int other_v = Mesh->edge_other_v(incoming_e, bowtie_v);
	FIndex2i ev = Mesh->GetEdgeV(incoming_e);
	int other_v = (ev.A == bowtie_v) ? ev.B : ev.A;
	FVector3d ab = Mesh->GetVertex(bowtie_v) - Mesh->GetVertex(other_v);

	// our winner
	int best_e = -1;
	double best_angle = TNumericLimits<double>::Max();

	for (int i = 0; i < bdry_edges_count; ++i)
	{
		int bdry_eid = bdry_edges[i];
		if (used_edges[bdry_eid] == true)
		{
			continue;       // this edge is already used
		}

		FIndex2i bdry_ev = Mesh->GetOrientedBoundaryEdgeV(bdry_eid);
		if (bdry_ev.A != bowtie_v)
		{
			continue;       // have to be able to chain to end of current edge, orientation-wise
		}

		// compute projected angle
		FVector3d bc = Mesh->GetVertex(bdry_ev.B) - Mesh->GetVertex(bowtie_v);
		double fAngleS = -VectorUtil::PlaneAngleSignedD(ab, bc, n);

		// turn left!
		if (best_angle == TNumericLimits<double>::Max() || fAngleS < best_angle)
		{
			best_angle = fAngleS;
			best_e = bdry_eid;
		}
	}

	// Note w/ bowtie vertices and open spans, best_e CAN be invalid (== -1)
	return best_e;
}





// This is called when loopV contains one or more "bowtie" vertices.
// These vertices *might* be duplicated in loopV (but not necessarily)
// If they are, we have to break loopV into subloops that don't contain duplicates.
//
// The list bowties contains all the possible duplicates 
// (all v in bowties occur in loopV at least once)
//
// Currently loopE is not used, and the returned EdgeLoop objects do not have their Edges
// arrays initialized. Perhaps to improve in future.
//
// An unhandled case to think about is where we have a sequence [..A..B..A..B..] where
// A and B are bowties. In this case there are no A->A or B->B subloops. What should
// we do here??
bool FMeshBoundaryLoops::ExtractSubloops(TArray<int>& loopV, TArray<int>& loopE, TArray<int>& bowties, Subloops& SubloopsOut)
{
	Subloops& subs = SubloopsOut;

	// figure out which bowties we saw are actually duplicated in loopV
	TArray<int> dupes;
	for (int bv : bowties)
	{
		if (CountInList(loopV, bv) > 1)
		{
			dupes.Add(bv);
		}
	}

	// we might not actually have any duplicates, if we got luck. Early out in that case
	if (dupes.Num() == 0)
	{
		FEdgeLoop& NewLoop = subs.Loops[subs.Loops.Emplace()];
		NewLoop.Initialize(Mesh, loopV, loopE, &bowties);
		return true;
	}

	// This loop extracts subloops until we have dealt with all the
	// duplicate vertices in loopV
	while (dupes.Num() > 0)
	{

		// Find shortest "simple" loop, ie a loop from a bowtie to itself that
		// does not contain any other bowties. This is an independent loop.
		// We're doing a lot of extra work here if we only have one element in dupes...
		int bi = 0, bv = 0;
		int start_i = -1, end_i = -1;
		int bv_shortest = -1; int shortest = TNumericLimits<int>::Max();
		for (; bi < dupes.Num(); ++bi)
		{
			bv = dupes[bi];
			if (IsSimpleBowtieLoop(loopV, dupes, bv, start_i, end_i))
			{
				int len = CountSpan(loopV, start_i, end_i);
				if (len < shortest)
				{
					bv_shortest = bv;
					shortest = len;
				}
			}
		}

		// failed to find a simple loop. Not sure what to do in this situation. 
		// If we don't want to throw, all we can do is convert the remaining 
		// loop to a span and return. 
		// (Or should we keep it as a loop and set flag??)
		if (bv_shortest == -1)
		{
			if (FailureBehavior == EFailureBehaviors::Abort)
			{
				FailureBowties = dupes;
				bAborted = true;
				return false;
			}

			VerticesTemp.Reset();
			for (int i = 0; i < loopV.Num(); ++i)
			{
				if (loopV[i] != -1)
				{
					VerticesTemp.Add(loopV[i]);
				}
			}

			FEdgeSpan& NewSpan = subs.Spans[subs.Spans.Emplace()];
			NewSpan.InitializeFromVertices(Mesh, VerticesTemp, false);
			NewSpan.SetBowtieVertices(bowties);
			return false;
		}

		if (bv != bv_shortest)
		{
			bv = bv_shortest;
			// running again just to get start_i and end_i...
			IsSimpleBowtieLoop(loopV, dupes, bv, start_i, end_i);
		}

		check(loopV[start_i] == bv && loopV[end_i] == bv);

		FEdgeLoop& NewLoop = subs.Loops[subs.Loops.Emplace()];

		VerticesTemp.Reset();
		ExtractSpan(loopV, start_i, end_i, true, VerticesTemp);
		NewLoop.InitializeFromVertices(Mesh, VerticesTemp, false);
		NewLoop.SetBowtieVertices(bowties);

		// If there are no more duplicates of this bowtie, we can treat
		// it like a regular vertex now
		if (CountInList(loopV, bv) < 2)
		{
			dupes.Remove(bv);
		}
	}

	// Should have one loop left that contains duplicates. 
	// Extract this as a separate loop
	int nLeft = 0;
	for (int i = 0; i < loopV.Num(); ++i)
	{
		if (loopV[i] != -1)
			nLeft++;
	}
	if (nLeft > 0)
	{
		FEdgeLoop& NewLoop = subs.Loops[subs.Loops.Emplace()];

		VerticesTemp.Reset();
		for (int i = 0; i < loopV.Num(); ++i)
		{
			if (loopV[i] != -1)
			{
				VerticesTemp.Add(loopV[i]);
			}
		}
		NewLoop.InitializeFromVertices(Mesh, VerticesTemp, false);
		NewLoop.SetBowtieVertices(bowties);
	}

	return true;
}



// Check if the loop from bowtieV to bowtieV inside loopV contains any other bowtie verts.
// Also returns start and end indices in loopV of "clean" loop
// Note that start may be < end, if the "clean" loop wraps around the end
bool FMeshBoundaryLoops::IsSimpleBowtieLoop(const TArray<int>& LoopVerts, const TArray<int>& BowtieVerts, int BowtieVertex, int& start_i, int& end_i)
{
	// find two indices of bowtie vert
	start_i = FindIndex(LoopVerts, 0, BowtieVertex);
	end_i = FindIndex(LoopVerts, start_i + 1, BowtieVertex);

	if (IsSimplePath(LoopVerts, BowtieVerts, BowtieVertex, start_i, end_i))
	{
		return true;
	}
	else if (IsSimplePath(LoopVerts, BowtieVerts, BowtieVertex, end_i, start_i))
	{
		int tmp = start_i; start_i = end_i; end_i = tmp;
		return true;
	}
	else
	{
		return false;       // not a simple bowtie loop!
	}
}


// check if forward path from loopV[i1] to loopV[i2] contains any bowtie verts other than bowtieV
bool FMeshBoundaryLoops::IsSimplePath(const TArray<int>& LoopVerts, const TArray<int>& BowtieVerts, int BowtieVertex, int i1, int i2)
{
	int N = LoopVerts.Num();
	for (int i = i1; i != i2; i = (i + 1) % N)
	{
		int vi = LoopVerts[i];
		if (vi == -1)
		{
			continue;       // skip removed vertices
		}
		if (vi != BowtieVertex && BowtieVerts.Contains(vi))
		{
			return false;
		}
	}
	return true;
}


// Read out the span from loop[i0] to loop [i1-1] into an array.
// If bMarkInvalid, then these values are set to -1 in loop
void FMeshBoundaryLoops::ExtractSpan(TArray<int>& Loop, int i0, int i1, bool bMarkInvalid, TArray<int>& OutSpan)
{
	int num = CountSpan(Loop, i0, i1);
	OutSpan.SetNum(num);
	int ai = 0;
	int N = Loop.Num();
	for (int i = i0; i != i1; i = (i + 1) % N)
	{
		if (Loop[i] != -1)
		{
			OutSpan[ai++] = Loop[i];
			if (bMarkInvalid)
			{
				Loop[i] = -1;
			}
		}
	}
}


// count number of valid vertices in l between loop[i0] and loop[i1-1]
int FMeshBoundaryLoops::CountSpan(const TArray<int>& Loop, int i0, int i1)
{
	int c = 0;
	int N = Loop.Num();
	for (int i = i0; i != i1; i = (i + 1) % N)
	{
		if (Loop[i] != -1)
		{
			c++;
		}
	}
	return c;
}

// find the index of item in loop, starting at start index
int FMeshBoundaryLoops::FindIndex(const TArray<int>& Loop, int Start, int Item)
{
	for (int i = Start; i < Loop.Num(); ++i)
	{
		if (Loop[i] == Item)
		{
			return i;
		}
	}
	return -1;
}

// count number of times item appears in loop
int FMeshBoundaryLoops::CountInList(const TArray<int>& Loop, int Item)
{
	int c = 0;
	for (int i = 0; i < Loop.Num(); ++i)
	{
		if (Loop[i] == Item)
		{
			c++;
		}
	}
	return c;
}

int FMeshBoundaryLoops::FindLoopTrianglesHint(const TArray<int>& HintTris) const
{
	TSet<int> HintEdges;
	for (int TriangleID : HintTris)
	{
		if (Mesh->IsTriangle(TriangleID) == false)
		{
			continue;
		}

		FIndex3i TriangleEdges = Mesh->GetTriEdges(TriangleID);
		for (int j = 0; j < 3; ++j)
		{
			if (Mesh->IsBoundaryEdge(TriangleEdges[j]))
			{
				HintEdges.Add(TriangleEdges[j]);
			}
		}
	}

	return FindLoopEdgesHint(HintEdges);
}


int FMeshBoundaryLoops::FindLoopEdgesHint(const TSet<int>& HintEdges) const
{
	int NumLoops = GetLoopCount();
	int BestLoop = -1;
	int MaxVotes = 0;
	for (int LoopIndex = 0; LoopIndex < NumLoops; ++LoopIndex)
	{
		int Votes = 0;
		const FEdgeLoop& CurrentLoop = Loops[LoopIndex];
		for (int EdgeID : CurrentLoop.Edges)
		{
			if (HintEdges.Contains(EdgeID))
			{
				++Votes;
			}
		}

		if (Votes > MaxVotes)
		{
			BestLoop = LoopIndex;
			MaxVotes = Votes;
		}
	}

	return BestLoop;
}


int FMeshBoundaryLoops::FindSpanContainingEdge(int EdgeID) const
{
	const int32 NumSpans = Spans.Num();
	for (int32 SpanIndex = 0; SpanIndex < NumSpans; ++SpanIndex)
	{
		if (Spans[SpanIndex].Edges.Contains(EdgeID))
		{
			return SpanIndex;
		}
	}
	return -1;
}
