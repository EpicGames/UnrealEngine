// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshRegionBoundaryLoops.h"

#include "Algo/ForEach.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshBoundaryLoops.h"   // has a set of internal static functions we re-use
#include "VectorUtil.h"
#include "Util/SparseIndexCollectionTypes.h"

using namespace UE::Geometry;

FMeshRegionBoundaryLoops::FMeshRegionBoundaryLoops(const FDynamicMesh3* MeshIn, const TArray<int>& RegionTris, bool bAutoCompute)
{
	SetMesh(MeshIn, RegionTris);

	if (bAutoCompute)
	{
		Compute();
	}
}



void FMeshRegionBoundaryLoops::SetMesh(const FDynamicMesh3* MeshIn, const TArray<int>& RegionTris)
{
	this->Mesh = MeshIn;

	// make flag set for included triangles
	Triangles.InitAuto(Mesh->MaxTriangleID(), RegionTris.Num());
	for (int i = 0; i < RegionTris.Num(); ++i)
	{
		Triangles.Add(RegionTris[i]);
	}

	// make flag set for included edges
	// NOTE: this currently processes non-boundary-edges twice. Could
	// avoid w/ another IndexFlagSet, but the check is inexpensive...
	Edges.InitAuto(Mesh->MaxEdgeID(), RegionTris.Num());
	for (int i = 0; i < RegionTris.Num(); ++i)
	{
		int tid = RegionTris[i];
		FIndex3i te = Mesh->GetTriEdges(tid);
		for (int j = 0; j < 3; ++j)
		{
			int eid = te[j];
			if (!Edges.Contains(eid))
			{
				FIndex2i et = Mesh->GetEdgeT(eid);
				if (et.B == IndexConstants::InvalidID || Triangles[et.A] != Triangles[et.B])
				{
					edges_roi.Add(eid);
					Edges.Add(eid);
				}
			}
		}
	}
}



int FMeshRegionBoundaryLoops::GetMaxVerticesLoopIndex() const
{
	int j = 0;
	for (int i = 1; i < Loops.Num(); ++i)
	{
		if (Loops[i].Vertices.Num() > Loops[j].Vertices.Num())
		{
			j = i;
		}
	}
	return j;
}



bool FMeshRegionBoundaryLoops::Compute()
{
	bFailed = false; // reset

	// This algorithm assumes that triangles are oriented consistently, 
	// so closed boundary-loop can be followed by walking edges in-order
	Loops.SetNum(0);

	// Temporary memory used to indicate when we have "used" an edge.
	FIndexFlagSet used_edge;
	used_edge.InitAuto(Mesh->MaxEdgeID(), edges_roi.Num());

	// current loop is stored here, cleared after each loop extracted
	TArray<int> loop_edges;
	TArray<int> loop_verts;
	TArray<int> bowties;

	// Temp buffer for reading back all boundary edges of a vertex.
	// probably always small but : pathological cases it could be large...
	TArray<int> all_e;
	all_e.SetNum(16);

	// process all edges of mesh
	for (int eid : edges_roi)
	{
		if (used_edge[eid] == true)
		{
			continue;
		}
		if (IsEdgeOnBoundary(eid) == false)
		{
			continue;
		}

		// ok this is start of a boundary chain
		int eStart = eid;
		used_edge.Add(eStart);
		loop_edges.Add(eStart);

		int eCur = eid;
		int eFirstVert = -1; // the first vertex on eCur, in terms of our walking order

		// follow the chain : order of oriented edges
		bool bClosed = false;
		while (!bClosed)
		{

			// [TODO] can do this more efficiently?
			int tid_in = IndexConstants::InvalidID, tid_out = IndexConstants::InvalidID;
			IsEdgeOnBoundary(eCur, tid_in, tid_out);

			int cure_a, cure_b;
			if (eFirstVert == -1)
			{
				FIndex2i ev = GetOrientedEdgeVerts(eCur, tid_in);
				cure_a = ev.A;
				cure_b = ev.B;
			}
			else
			{
				// once we've walked on at least one edge, no longer need to rely on triangle orientation to know which way we were walking
				FIndex2i edgev = Mesh->GetEdgeV(eCur);
				checkSlow(edgev.Contains(eFirstVert));
				cure_a = eFirstVert;
				cure_b = edgev.A == cure_a ? edgev.B : edgev.A;
			}
			loop_verts.Add(cure_a);

			int e0 = -1, e1 = 1;
			int bdry_nbrs = GetVertexBoundaryEdges(cure_b, e0, e1);

			if (bdry_nbrs < 2)
			{
				// found broken neighbourhood at vertex cure_b -- unrecoverable failure (unclosed loop)
				bFailed = true;
				return false;
			}

			int eNext = -1;
			if (bdry_nbrs > 2)
			{
				// found "bowtie" vertex...things just got complicated!

				if (cure_b == loop_verts[0])
				{
					// The "end" of the current edge is the same as the start vertex.
					// This means we can close the loop here. Might as well!
					eNext = -2;   // sentinel value used below

				}
				else
				{
					// try to find an unused outgoing edge that is oriented properly.
					// This could create sub-loops, we will handle those later
					if (bdry_nbrs >= all_e.Num())
						all_e.SetNum(bdry_nbrs);
					int num_be = GetAllVertexBoundaryEdges(cure_b, all_e);

					check(num_be == bdry_nbrs);

					// Try to pick the best "turn left" vertex.
					eNext = FindLeftTurnEdge(eCur, cure_b, all_e, num_be, used_edge);

					if (eNext == -1)
					{
						// Cannot find valid outgoing edge at bowtie vertex cure_b -- unrecoverable failure
						bFailed = true;
						return false;
					}
				}

				if (bowties.Contains(cure_b) == false)
				{
					bowties.Add(cure_b);
				}

			}
			else
			{
				check(e0 == eCur || e1 == eCur);
				eNext = (e0 == eCur) ? e1 : e0;
			}

			if (eNext == -2)
			{
				// found a bowtie vert that is the same as start-of-loop, so we
				// are just closing it off explicitly
				bClosed = true;
			}
			else if (eNext == eStart)
			{
				// found edge at start of loop, so loop is done.
				bClosed = true;
			}
			else
			{
				// push onto accumulated list
				check(used_edge[eNext] == false);
				loop_edges.Add(eNext);
				eCur = eNext;
				used_edge.Add(eCur);
			}

			eFirstVert = cure_b;
		}

		// if we saw a bowtie vertex, we might need to break up this loop,
		// so call ExtractSubloops
		if (bowties.Num() > 0)
		{
			TArray<FEdgeLoop> subloops;
			bool bExtractedLoops = TryExtractSubloops(loop_verts, loop_edges, bowties, subloops);
			if (!bExtractedLoops)
			{
				// skip adding subloops and mark as failure (but go on computing the rest of the boundary loops)
				bFailed = true;
			}
			else
			{
				for (int i = 0; i < subloops.Num(); ++i)
				{
					Loops.Add(subloops[i]);
				}
			}
		}
		else
		{
			// clean simple loop, convert to FEdgeLoop instance
			FEdgeLoop loop(Mesh);
			loop.Vertices = loop_verts;
			loop.Edges = loop_edges;
			Loops.Add(loop);
		}

		// reset these lists
		loop_edges.SetNum(0);
		loop_verts.SetNum(0);
		bowties.SetNum(0);
	}

	return !bFailed;
}






// returns true for both internal and mesh boundary edges
// tid_in and tid_out are triangles 'in' and 'out' of set, respectively
bool FMeshRegionBoundaryLoops::IsEdgeOnBoundary(int eid, int& tid_in, int& tid_out) const
{
	if (Edges.Contains(eid) == false)
	{
		return false;
	}

	tid_in = tid_out = IndexConstants::InvalidID;
	FIndex2i et = Mesh->GetEdgeT(eid);
	if (et.B == IndexConstants::InvalidID)	// boundary edge!
	{
		tid_in = et.A;
		tid_out = et.B;
		return true;
	}

	bool in0 = Triangles[et.A];
	bool in1 = Triangles[et.B];
	if (in0 != in1)
	{
		tid_in = (in0) ? et.A : et.B;
		tid_out = (in0) ? et.B : et.A;
		return true;
	}
	return false;
}



// return same indices as GetEdgeV, but oriented based on attached triangle
FIndex2i FMeshRegionBoundaryLoops::GetOrientedEdgeVerts(int eID, int tid_in)
{
	FIndex2i edgev = Mesh->GetEdgeV(eID);
	int a = edgev.A, b = edgev.B;
	FIndex3i tri = Mesh->GetTriangle(tid_in);
	int ai = IndexUtil::FindEdgeIndexInTri(a, b, tri);
	return FIndex2i(tri[ai], tri[(ai + 1) % 3]);
}


int FMeshRegionBoundaryLoops::GetVertexBoundaryEdges(int vID, int& e0, int& e1)
{
	int count = 0;
	for (int eid : Mesh->VtxEdgesItr(vID))
	{
		if (IsEdgeOnBoundary(eid))
		{
			if (count == 0)
			{
				e0 = eid;
			}
			else if (count == 1)
			{
				e1 = eid;
			}
			count++;
		}
	}
	return count;
}


int FMeshRegionBoundaryLoops::GetAllVertexBoundaryEdges(int vID, TArray<int>& e)
{
	int count = 0;
	for (int eid : Mesh->VtxEdgesItr(vID))
	{
		if (IsEdgeOnBoundary(eid))
		{
			e[count++] = eid;
		}
	}
	return count;
}


FVector3d FMeshRegionBoundaryLoops::GetVertexNormal(int vid)
{
	FVector3d n = FVector3d::Zero();
	for (int ti : Mesh->VtxTrianglesItr(vid))
	{
		n += Mesh->GetTriNormal(ti);
	}
	Normalize(n);
	return n;
}



//
// [TODO] for internal vertices, there is no ambiguity : which is the left-turn edge,
//   we should be using 'closest' left-neighbour edge.
//
// ok, bdry_edges[0...bdry_edges_count] contains the boundary edges coming out of bowtie_v.
// We want to pick the best one to continue the loop that came : to bowtie_v on incoming_e.
// If the loops are all sane, then we will get the smallest loops by "turning left" at bowtie_v.
// So, we compute the tangent plane at bowtie_v, and then the signed angle for each
// viable edge : this plane. 
int FMeshRegionBoundaryLoops::FindLeftTurnEdge(int incoming_e, int bowtie_v, TArray<int>& bdry_edges, int bdry_edges_count, const FIndexFlagSet& used_edges)
{
	// compute normal and edge [a,bowtie]
	FVector3d n = GetVertexNormal(bowtie_v);
	//int other_v = Mesh->edge_other_v(incoming_e, bowtie_v);
	FIndex2i ev = Mesh->GetEdgeV(incoming_e);
	int other_v = (ev.A == bowtie_v) ? ev.B : ev.A;
	FVector3d ab = Mesh->GetVertex(bowtie_v) - Mesh->GetVertex(other_v);

	// our winner
	int best_e = -1;
	double best_angle = TNumericLimits<double>::Max();

	for (int i = 0; i < bdry_edges_count; ++i)
	{
		int bdry_eid = bdry_edges[i];
		if (used_edges[bdry_eid] == true)
			continue;       // this edge is already used

		// [TODO] can do this more efficiently?
		int tid_in = IndexConstants::InvalidID, tid_out = IndexConstants::InvalidID;
		IsEdgeOnBoundary(bdry_eid, tid_in, tid_out);
		FIndex2i bdry_ev = GetOrientedEdgeVerts(bdry_eid, tid_in);
		//FIndex2i bdry_ev = Mesh.GetOrientedBoundaryEdgeV(bdry_eid);

		if (bdry_ev.A != bowtie_v) {
			continue;       // have to be able to chain to end of current edge, orientation-wise
		}

		// compute projected angle
		FVector3d bc = Mesh->GetVertex(bdry_ev.B) - Mesh->GetVertex(bowtie_v);
		double fAngleS = -VectorUtil::PlaneAngleSignedD(ab, bc, n);

		// turn left!
		if (best_angle == TNumericLimits<double>::Max() || fAngleS < best_angle)
		{
			best_angle = fAngleS;
			best_e = bdry_eid;
		}
	}

	return best_e;
}




// This is called when LoopV contains one or more "bowtie" vertices.
// These vertices *might* be duplicated in LoopV (but not necessarily)
// If they are, we have to break LoopV into subloops that don't contain duplicates.
//
// The list Bowties contains all the possible duplicates 
// (all v in Bowties occur in LoopV at least once)
//
// Currently LoopE is not used.
//
// Note: the approach used here, which doesn't fail, should also be used in FMeshBoundaryLoops. However we're
// going to do additional tests there with hole filling to make sure that not failing has an overall positive 
// impact on hole filling, since we would rather fail there if the resulting subloops are more likely to be
// bad for hole filling than good.
bool FMeshRegionBoundaryLoops::TryExtractSubloops(TArray<int>& LoopV, const TArray<int>& LoopE, const TArray<int>& Bowties, TArray<FEdgeLoop>& SubloopsOut)
{
	// We keep track of the last place we've seen a bowtie in our list. As soon as we see a bowtie a second
	// time, we extract that subloop. As long as we do this immediately, the intervening vids will not be 
	// duplicates even if they are bowties.
	TMap<int32, int32> BowtieVidToLoopIndex;
	TSet<int32> BowtieVids(Bowties);
	for (int LoopIndex = 0; LoopIndex < LoopV.Num(); ++LoopIndex)
	{
		int32 Vid = LoopV[LoopIndex];
		int32* SeenBowtieIndex = BowtieVidToLoopIndex.Find(Vid);
		if (SeenBowtieIndex)
		{
			// Check that the place we saw this wasn't in an extracted subloop (in which case the vert is set to -1 in LoopV)
			if (LoopV[*SeenBowtieIndex] >= 0)
			{
				// Now that we've seen the bowtie twice, extract the subloop
				TArray<int32> SubloopVertices;
				FMeshBoundaryLoops::ExtractSpan(LoopV, *SeenBowtieIndex, LoopIndex, true, SubloopVertices);

				FEdgeLoop& NewLoop = SubloopsOut[SubloopsOut.Emplace()];
				NewLoop.InitializeFromVertices(Mesh, SubloopVertices, false);
				NewLoop.SetBowtieVertices(Bowties);
			}

			// Update the last place we saw the bowtie, in case a subsequent loop goes through here.
			BowtieVidToLoopIndex[Vid] = LoopIndex;
		}
		else if (BowtieVids.Contains(Vid))
		{
			BowtieVidToLoopIndex.Add(Vid, LoopIndex);
		}
	}

	// Should have one loop left.
	TArray<int32> RemainingLoopVids;
	for (int32 Vid : LoopV)
	{
		if (Vid >= 0)
		{
			RemainingLoopVids.Add(Vid);
		}
	}
	if (ensure(RemainingLoopVids.Num() > 2))
	{
		FEdgeLoop& NewLoop = SubloopsOut[SubloopsOut.Emplace()];
		NewLoop.InitializeFromVertices(Mesh, RemainingLoopVids, false);
		NewLoop.SetBowtieVertices(Bowties);
	}
	else
	{
		return false;
	}

	return true;
}

template<typename StorageType, int ElementSize, typename ElementType>
bool FMeshRegionBoundaryLoops::GetLoopOverlayMap(const FEdgeLoop& LoopIn,
	const TDynamicMeshOverlay<StorageType, ElementSize>& Overlay,
	VidOverlayMap<ElementType>& LoopVidsToOverlayElementsOut)
{
	for (int32 i = 0; i < LoopIn.Vertices.Num(); ++i)
	{
		int32 Vid = LoopIn.Vertices[i];

		// Get the inner triangle associated with the edges going forward from this vertex
		int32 TidInside, TidOutside;
		IsEdgeOnBoundary(LoopIn.Edges[i], TidInside, TidOutside);
		check(TidInside != IndexConstants::InvalidID);

		// Find the overlay element associated with the vertex
		FIndex3i TriangleVerts = Mesh->GetTriangle(TidInside);
		int32 VidTriIndex = TriangleVerts.IndexOf(Vid);
		check(VidTriIndex >= 0);

		FIndex3i TriangleElements = Overlay.GetTriangle(TidInside);
		int32 UVElementID = TriangleElements[VidTriIndex];
		if (!Overlay.IsElement(UVElementID))
		{
			return false;
		}

		ElementType Element; 
		Overlay.GetElement(UVElementID, Element);
		LoopVidsToOverlayElementsOut.Add(Vid,
			ElementIDAndValue<ElementType>(UVElementID, Element));
	}

	return true;
}

template<typename StorageType, int ElementSize, typename ElementType>
void FMeshRegionBoundaryLoops::UpdateLoopOverlayMapValidity(
	VidOverlayMap<ElementType>& LoopVidsToOverlayElements, 
	const TDynamicMeshOverlay<StorageType, ElementSize>& Overlay)
{
	// Go through all the overlay element ids's and see if they are still an element
	// in the overlay. If not, make that id an invalid ID.
	Algo::ForEachIf(LoopVidsToOverlayElements,
		[&Overlay](const auto& Entry) { return !Overlay.IsElement(Entry.Value.Key); },
		[](auto& Entry) { Entry.Value.Key = IndexConstants::InvalidID; });
}

// Right now we use our templated functions just for UV layers. If we need other overlay layers,
// we'll need to add instantiations here.
template GEOMETRYCORE_API bool FMeshRegionBoundaryLoops::GetLoopOverlayMap<float, 2, FVector2f>(
	const FEdgeLoop& LoopIn, const TDynamicMeshOverlay<float, 2>& Overlay,
	VidOverlayMap<FVector2f>& LoopVidsToOverlayElementsOut);
template GEOMETRYCORE_API void FMeshRegionBoundaryLoops::UpdateLoopOverlayMapValidity<float, 2, FVector2f>(
	VidOverlayMap<FVector2f>& LoopVidsToOverlayElements, const TDynamicMeshOverlay<float, 2>& Overlay);


bool FMeshRegionBoundaryLoops::GetTriangleSetBoundaryLoop(const FDynamicMesh3& Mesh, const TArray<int32>& Tris, FEdgeLoop& Loop)
{
	// todo: special-case single triangle
	// collect list of border edges
	TArray<int32> Edges;
	for (int32 tid : Tris)
	{
		FIndex3i TriEdges = Mesh.GetTriEdges(tid);
		for (int32 j = 0; j < 3; ++j)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(TriEdges[j]);
			int32 OtherT = (EdgeT.A == tid) ? EdgeT.B : EdgeT.A;
			if (OtherT == FDynamicMesh3::InvalidID || Tris.Contains(OtherT) == false)
			{
				Edges.AddUnique(TriEdges[j]);
			}
		}
	}

	if (Edges.Num() == 0)
	{
		return false;
	}

	Loop.Mesh = &Mesh;

	// Start at first edge and walk around loop, adding one vertex and edge each time.
	// Abort if we encounter any nonmanifold configuration 
	int32 NumEdges = Edges.Num();
	int32 StartEdge = Edges[0];
	FIndex2i StartEdgeT = Mesh.GetEdgeT(StartEdge);
	int32 InTri = Tris.Contains(StartEdgeT.A) ? StartEdgeT.A : StartEdgeT.B;
	FIndex2i StartEdgeV = Mesh.GetEdgeV(StartEdge);
	IndexUtil::OrientTriEdge(StartEdgeV.A, StartEdgeV.B, Mesh.GetTriangle(InTri));
	Loop.Vertices.Reset();
	Loop.Vertices.Add(StartEdgeV.A);
	Loop.Vertices.Add(StartEdgeV.B);
	int32 CurEndVert = Loop.Vertices.Last();
	int32 PrevEdge = StartEdge;
	Loop.Edges.Reset();
	Loop.Edges.Add(StartEdge);
	int32 NumEdgesUsed = 1;
	bool bContinue = true;
	do 
	{
		bContinue = false;
		for (int32 eid : Mesh.VtxEdgesItr(CurEndVert))
		{
			if (eid != PrevEdge && Edges.Contains(eid) && Loop.Edges.Contains(eid) == false)
			{
				FIndex2i EdgeV = Mesh.GetEdgeV(eid);
				int32 NextV = (EdgeV.A == CurEndVert) ? EdgeV.B : EdgeV.A;
				if (NextV == Loop.Vertices[0])		// closed loop
				{
					Loop.Edges.Add(eid);
					NumEdgesUsed++;
					bContinue = false;
					break;
				}
				else
				{
					if (Loop.Vertices.Contains(NextV))
					{
						return false;		// hit a middle vertex, we have nonmanifold set of edges, abort
					}
					Loop.Edges.Add(eid);
					PrevEdge = eid;
					Loop.Vertices.Add(NextV);
					NumEdgesUsed++;
					CurEndVert = NextV;
					bContinue = true;
					break;
				}
			}
		}
	} while (bContinue);

	if (NumEdgesUsed != Edges.Num())	// closed loop but we still have edges? must have nonmanifold configuration, abort.
	{
		return false;
	}

	return true;
}

