// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/EmbedSurfacePath.h"
#include "MathUtil.h"
#include "VectorUtil.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "Distance/DistPoint3Triangle3.h"
#include "DynamicMesh/DynamicMesh3.h"

using namespace UE::Geometry;

FVector3d FMeshSurfacePoint::Pos(const FDynamicMesh3 *Mesh) const
{
	if (PointType == ESurfacePointType::Vertex)
	{
		return Mesh->GetVertex(ElementID);
	}
	else if (PointType == ESurfacePointType::Edge)
	{
		FVector3d EA, EB;
		Mesh->GetEdgeV(ElementID, EA, EB);
		// Note this is equivalent to Lerp(EA, EB, BaryCoord[1])
		return BaryCoord[0] * EA + BaryCoord[1] * EB;
	}
	else // PointType == ESurfacePointType::Triangle
	{
		FVector3d TA, TB, TC;
		Mesh->GetTriVertices(ElementID, TA, TB, TC);
		return BaryCoord[0] * TA + BaryCoord[1] * TB + BaryCoord[2] * TC;
	}
}

namespace UE::EmbedSurfacePathLocals
{

/**
 * Helper function to snap a triangle surface point to the triangle vertices or edges if it's close enough.  Input SurfacePt must be a triangle.
 */
static void RefineSurfacePtFromTriangleToSubElement(const FDynamicMesh3* Mesh, FVector3d Pos, FMeshSurfacePoint& SurfacePt, double SnapElementThresholdSq)
{
	// expect this to only be called on SurfacePoints with PointType == Triangle; otherwise indicative of incorrect usage
	if (!ensure(SurfacePt.PointType == ESurfacePointType::Triangle))
	{
		return;
	}
	int TriID = SurfacePt.ElementID;

	FIndex3i TriVertIDs = Mesh->GetTriangle(TriID);
	int BestSubIdx = -1;
	double BestElementDistSq = 0;
	for (int VertSubIdx = 0; VertSubIdx < 3; VertSubIdx++)
	{
		double DistSq = DistanceSquared(Pos, Mesh->GetVertex(TriVertIDs[VertSubIdx]));
		if (DistSq <= SnapElementThresholdSq && (BestSubIdx == -1 || DistSq < BestElementDistSq))
		{
			BestSubIdx = VertSubIdx;
			BestElementDistSq = DistSq;
		}
	}

	if (BestSubIdx > -1)
	{
		SurfacePt.ElementID = TriVertIDs[BestSubIdx];
		SurfacePt.PointType = ESurfacePointType::Vertex;
		return;
	}

	// failed to snap to vertex, try snapping to edge
	FIndex3i TriEdgeIDs = Mesh->GetTriEdges(TriID);
	
	check(BestSubIdx == -1); // otherwise would have returned the within-threshold vertex!
	double BestEdgeParam = 0;
	for (int EdgeSubIdx = 0; EdgeSubIdx < 3; EdgeSubIdx++)
	{
		int EdgeID = TriEdgeIDs[EdgeSubIdx];
		FVector3d EPosA, EPosB;
		Mesh->GetEdgeV(EdgeID, EPosA, EPosB);
		FSegment3d EdgeSeg(EPosA, EPosB);
		double DistSq = EdgeSeg.DistanceSquared(Pos);
		if (DistSq <= SnapElementThresholdSq && (BestSubIdx == -1 || DistSq < BestElementDistSq))
		{
			BestSubIdx = EdgeSubIdx;
			BestElementDistSq = DistSq;
			BestEdgeParam = EdgeSeg.ProjectUnitRange(Pos);
		}
	}

	if (BestSubIdx > -1)
	{
		SurfacePt = FMeshSurfacePoint::MakeEdgePoint(TriEdgeIDs[BestSubIdx], BestEdgeParam);
		return;
	}

	// no snapping to be done, leave surfacept on the triangle
}

// For when a triangle is replaced by multiple triangles, create a new surface point for the point's new location among the smaller triangles.
//  Note input position is in the coordinate space of the mesh vertices, not e.g. a transformed space (unlike the positions passed to WalkMeshPlanar below!)
template<typename TIterableTrisType>
static FMeshSurfacePoint RelocateTrianglePointAfterRefinement(const FDynamicMesh3* Mesh, const FVector3d& PosInVertexCoordSpace, const TIterableTrisType& TriIDs, double SnapElementThresholdSq)
{
	double BestTriDistSq = 0;
	FVector3d BestBaryCoords;
	int BestTriID = -1;
	for (int TriID : TriIDs)
	{
		check(Mesh->IsTriangle(TriID));
		FIndex3i TriVertIDs = Mesh->GetTriangle(TriID);
		FTriangle3d Tri(Mesh->GetVertex(TriVertIDs.A), Mesh->GetVertex(TriVertIDs.B), Mesh->GetVertex(TriVertIDs.C));
		FDistPoint3Triangle3d TriDist(PosInVertexCoordSpace, Tri); // heavy duty way to get barycentric coordinates and check if on triangle; should be robust to degenerate triangles unlike VectorUtil's barycentric coordinate function
		double DistSq = TriDist.GetSquared();
		if (BestTriID == -1 || DistSq < BestTriDistSq)
		{
			BestTriID = TriID;
			BestTriDistSq = DistSq;
			BestBaryCoords = TriDist.TriangleBaryCoords;
		}
	}

	ensure(Mesh->IsTriangle(BestTriID));
	FMeshSurfacePoint SurfacePt(BestTriID, BestBaryCoords);
	RefineSurfacePtFromTriangleToSubElement(Mesh, PosInVertexCoordSpace, SurfacePt, SnapElementThresholdSq);
	return SurfacePt;
}

// Helper to snap a point to the nearest point in a C-style array
template<int NumPoints>
static bool SelectSnapPoint(FVector3d(&PointArray)[NumPoints], const FVector3d& Point, int32& OutIdx, double SnapThresholdSq)
{
	double BestDistSq = SnapThresholdSq;
	OutIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < NumPoints; ++Idx)
	{
		double DSq = FVector3d::DistSquared(Point, PointArray[Idx]);
		if (DSq < BestDistSq)
		{
			OutIdx = Idx;
			BestDistSq = DSq;
		}
	}
	return OutIdx != INDEX_NONE;
}

struct FIndexDistance
{
	int Index;
	double PathLength;
	double DistanceToEnd;
	bool operator<(const FIndexDistance& Other) const
	{
		return PathLength + DistanceToEnd < Other.PathLength + Other.DistanceToEnd;
	}
};

static bool WalkMeshPlanar(const FDynamicMesh3* Mesh, int StartTri, int StartVID, FVector3d StartPt, int EndTri, int EndVertID, FVector3d EndPt, FVector3d WalkPlaneNormal, TFunction<FVector3d(const FDynamicMesh3*, int)> VertexToPosnFn,
	bool bAllowBackwardsSearch, double AcceptEndPtOutsideDist, double PtOnPlaneThresholdSq, TArray<TPair<FMeshSurfacePoint, int>>& WalkedPath, double BackwardsTolerance)
{
	// Even when bAllowBackwardsSearch is false, there may be multiple paths from start point to end point. 
	// The approach we take is a breadth-first-like one where we keep track of multiple paths and always extend the one
	// that has the potential to be shortest, i.e. whose current path length + distance to destination is smallest.
	// (we can't go by distance to destination alone because a sub optimal path can curve around the destination in the
	// plane in such a way that it always seems closer than the next step in a more direct path that happens to pass
	// through a more tesselated region).

	auto SetTriVertPositions = [&VertexToPosnFn, &Mesh](FIndex3i TriVertIDs, FTriangle3d& Tri)
	{
		Tri.V[0] = VertexToPosnFn(Mesh, TriVertIDs.A);
		Tri.V[1] = VertexToPosnFn(Mesh, TriVertIDs.B);
		Tri.V[2] = VertexToPosnFn(Mesh, TriVertIDs.C);
	};

	auto PtInsideTri = [](const FVector3d& BaryCoord, double BaryThreshold = FMathd::ZeroTolerance)
	{
		return BaryCoord[0] >= -BaryThreshold && BaryCoord[1] >= -BaryThreshold && BaryCoord[2] >= -BaryThreshold;
	};

	// track where you came from and where you are going
	struct FWalkIndices
	{
		FVector3d Position;		// Position in the coordinate space used for the walk (note: may not be the same space as the mesh vertices, e.g. if we walk on UV positions)
		int WalkedFromPt,		// index into ComputedPointsAndSources (or -1)
			WalkingOnTri;		// ID of triangle in mesh (or -1)

		FWalkIndices() : WalkedFromPt(-1), WalkingOnTri(-1)
		{}

		FWalkIndices(FVector3d Position, int FromPt, int OnTri) : Position(Position), WalkedFromPt(FromPt), WalkingOnTri(OnTri)
		{}
	};

	// TODO: vertex/edge snapping?
	TArray<TPair<FMeshSurfacePoint, FWalkIndices>> ComputedPointsAndSources;

	// This is treated as a heap, sorted to have the lowest potential path length (i.e. sum of the path length so far and
	// the distance to end point) at the top, so that we can pop off of it to grow the best candidate path.
	// The actual payloads are indices into ComputedPointsAndSources.
	TArray<FIndexDistance> UnexploredEnds;

	TSet<int> ExploredTriangles, CrossedVertices, CrossedEdges;

	// When we've found a path, the index into ComputedPointsAndSources of the best path end.
	int BestKnownEnd = -1;

	FTriangle3d CurrentTri;
	FIndex3i StartTriVertIDs = Mesh->GetTriangle(StartTri);
	SetTriVertPositions(StartTriVertIDs, CurrentTri);
	FDistPoint3Triangle3d CurrentTriDist(StartPt, CurrentTri); // heavy duty way to get barycentric coordinates and check if on triangle; should be robust to degenerate triangles unlike VectorUtil's barycentric coordinate function
	int StartVIDIndex = -1;
	if (StartVID != -1)
	{
		StartVIDIndex = StartTriVertIDs.IndexOf(StartVID);
	}
	if (StartVIDIndex == -1)
	{
		// TODO: use TrianglePosToSurfacePoint to snap to edge or vertex as needed (OR do this as a post-process and delete the point if doing so leads to a duplicate point!)
		CurrentTriDist.ComputeResult();
		// TODO: replace barycoords result with edge or vertex surface point data if within distance threshold of vertex or edge!
		ComputedPointsAndSources.Emplace(FMeshSurfacePoint(StartTri, CurrentTriDist.TriangleBaryCoords), FWalkIndices(StartPt, -1, StartTri));
	}
	else
	{
		// if a valid StartVID was given, assume that's our closest point
		CurrentTriDist.TriangleBaryCoords = FVector3d::Zero();
		CurrentTriDist.TriangleBaryCoords[StartVIDIndex] = 1.0;
		CurrentTriDist.ClosestTrianglePoint = StartPt;
		ComputedPointsAndSources.Emplace(FMeshSurfacePoint(StartTri, CurrentTriDist.TriangleBaryCoords), FWalkIndices(StartPt, -1, StartTri));
	}

	FVector3d ForwardsDirection = EndPt - StartPt;

	int CurrentEnd = 0;
	double CurrentPathLength = 0;
	double CurrentDistanceToEnd = ForwardsDirection.Length();

	// Our start point is our first unexplored end
	UnexploredEnds.Add({0, CurrentPathLength, CurrentDistanceToEnd });

	int IterCountSafety = 0;
	int NumTriangles = Mesh->TriangleCount();
	while (true)
	{
		if (!ensure(IterCountSafety++ < NumTriangles * 2)) // safety check to protect against infinite loop
		{
			return false;
		}

		// Grab the best potential path, if there is still a viable one.
		if (UnexploredEnds.Num())
		{
			FIndexDistance TopEndWithDistance;
			UnexploredEnds.HeapPop(TopEndWithDistance);

			CurrentEnd = TopEndWithDistance.Index;
			CurrentPathLength = TopEndWithDistance.PathLength;
			CurrentDistanceToEnd = TopEndWithDistance.DistanceToEnd;
		}
		else
		{
			return false; // failed to find path
		}

		FMeshSurfacePoint FromPt = ComputedPointsAndSources[CurrentEnd].Key;
		FWalkIndices CurrentWalk = ComputedPointsAndSources[CurrentEnd].Value;
		int TriID = CurrentWalk.WalkingOnTri;
		check(Mesh->IsTriangle(TriID));
		FIndex3i TriVertIDs = Mesh->GetTriangle(TriID);
		SetTriVertPositions(TriVertIDs, CurrentTri);

		// Note about ending the search: we happen to know that the final step of our path will be a direct line
		// to the destination, which means that the final path length will be CurrentPathLength + DistanceToDestination
		// at that point. Since we've been grabbing the minimal path (for extending) by the same criteria, we know that
		// we don't need to try extending any other paths at that point (we have the shortest).
		// This breaks down if AcceptEndPtOutsideDist is large enough to have multiple candidate end points and we want
		// the absolute shortest one, or if we find ourselves in a world where the distance contributed by the last step
		// is somehow potentially greater than the distance to destination at that time ("curved" triangles of some kind);
		// In that case we would want to keep track of a PathLengthUpperBound, which we would update once we find a path,
		// and only stop looking once we have no more unfinished paths, or if the next path we grab can't possibly beat 
		// the upper bound.

		// if we're on a triangle that is connected to the known final vertex, end the search!
		if (EndVertID >= 0 && TriVertIDs.Contains(EndVertID))
		{
			CurrentEnd = ComputedPointsAndSources.Emplace(FMeshSurfacePoint(EndVertID), FWalkIndices(EndPt, CurrentEnd, TriID));
			BestKnownEnd = CurrentEnd;
			break;
		}

		bool OnEndTri = EndTri == TriID;
		bool ComputedEndPtOnTri = false;
		if (EndVertID < 0 && EndTri == -1) // if we need to check if this is the end tri, and it could be the end tri
		{
			CurrentTriDist.Triangle = CurrentTri;
			CurrentTriDist.Point = EndPt;
			ComputedEndPtOnTri = true;
			double DistSq = CurrentTriDist.GetSquared();
			if (DistSq < AcceptEndPtOutsideDist/* && PtInsideTri(CurrentTriDist.TriangleBaryCoords, FMathd::Epsilon)*/)  // TODO: we don't really need to check the barycentric coordinates for being 'inside' the triangle if the distance is within epsilon?  especially since the barycoords test also has n epsilon threshold?
			{
				OnEndTri = true;
			}
		}

		// if we're on the final triangle, end the search!
		if (OnEndTri)
		{
			if (!ComputedEndPtOnTri)
			{
				CurrentTriDist.Triangle = CurrentTri;
				CurrentTriDist.Point = EndPt;
				ComputedEndPtOnTri = true;
				CurrentTriDist.GetSquared();
			}
			CurrentEnd = ComputedPointsAndSources.Emplace(FMeshSurfacePoint(TriID, CurrentTriDist.TriangleBaryCoords), FWalkIndices(EndPt, CurrentEnd, TriID));

			BestKnownEnd = CurrentEnd;
			break;
		}

		if (ExploredTriangles.Contains(TriID))
		{
			// note we only add explored triangles to the search at all to handle the specific case where we go 'the long way' around and back to the start triangle ... otherwise we could have just not added them to the search at all
			// currently that case is not even possible, but if it were, it would have been handled in the above `if (OnEndTri)` so we should be able to safely kill this branch of search here
			continue;
		}
		ExploredTriangles.Add(TriID);

		// not on a terminal triangle, cross the triangle and continue the search
		double SignDist[3];
		int Side[3];
		int32 InitialComputedPointsNum = ComputedPointsAndSources.Num();
		for (int TriSubIdx = 0; TriSubIdx < 3; TriSubIdx++)
		{
			double SD = (CurrentTri.V[TriSubIdx] - StartPt).Dot(WalkPlaneNormal);
			SignDist[TriSubIdx] = SD;
			if (FMathd::Abs(SD) <= PtOnPlaneThresholdSq)
			{
				// Vertex crossing
				Side[TriSubIdx] = 0;
				int CandidateVertID = TriVertIDs[TriSubIdx];
				if (FromPt.PointType != ESurfacePointType::Vertex || CandidateVertID != FromPt.ElementID)
				{
					FMeshSurfacePoint SurfPt(CandidateVertID);
					FWalkIndices WalkInds(CurrentTri.V[TriSubIdx], CurrentEnd, -1);
					//double DSq = EndPt.DistanceSquared(CurrentTri.V[TriSubIdx]);
					bool bIsForward = ForwardsDirection.Dot(CurrentTri.V[TriSubIdx] - StartPt) >= -BackwardsTolerance;
					// not allowed to go in a direction that gets us further from the destination than our initial point if backwards search not allowed
					if ((bAllowBackwardsSearch || bIsForward) && !CrossedVertices.Contains(CandidateVertID))
					{
						// consider going over this vertex
						CrossedVertices.Add(CandidateVertID);

						// TODO: extract this "next triangle candidate" logic to be used in more places??
						// walking over a vertex is gross because we have to search the whole one ring for candidate next triangles and there might be multiple of them
						// note that currently this means I compute signs for all vertices of neighboring triangles here, and do not re-use those signs when I actually process the triangle later; TODO reconsider if/when optimizing this fn
						for (int32 NbrTriID : Mesh->VtxTrianglesItr(CandidateVertID))
						{
							if (NbrTriID != TriID)
							{
								FIndex3i NbrTriVertIDs = Mesh->GetTriangle(NbrTriID);
								FTriangle3d NbrTri;
								SetTriVertPositions(NbrTriVertIDs, NbrTri);
								int SignsMultiplied = 1;
								for (int NbrTriSubIdx = 0; NbrTriSubIdx < 3; NbrTriSubIdx++)
								{
									if (NbrTriVertIDs[NbrTriSubIdx] == CandidateVertID)
									{
										continue;
									}
									double NbrSD = (NbrTri.V[NbrTriSubIdx] - StartPt).Dot(WalkPlaneNormal);
									int NbrSign = FMathd::Abs(NbrSD) <= PtOnPlaneThresholdSq ? 0 : NbrSD > 0 ? 1 : -1;
									SignsMultiplied *= NbrSign;
								}
								if (SignsMultiplied < 1) // plane will cross this triangle, so try walking it
								{
									WalkInds.WalkingOnTri = NbrTriID;
									ComputedPointsAndSources.Emplace(SurfPt, WalkInds);
								}
							}
						}
					}
				}
			}
			else
			{
				Side[TriSubIdx] = SD > 0 ? 1 : -1;
			}
		}
		FIndex3i TriEdgeIDs = Mesh->GetTriEdges(TriID);
		for (int TriSubIdx = 0; TriSubIdx < 3; TriSubIdx++)
		{
			int NextSubIdx = (TriSubIdx + 1) % 3;
			if (Side[TriSubIdx] * Side[NextSubIdx] < 0)
			{
				// edge crossing
				int CandidateEdgeID = TriEdgeIDs[TriSubIdx];
				if (FromPt.PointType != ESurfacePointType::Edge || CandidateEdgeID != FromPt.ElementID)
				{
					double CrossingT = SignDist[TriSubIdx] / (SignDist[TriSubIdx] - SignDist[NextSubIdx]);
					FVector3d CrossingP = (1 - CrossingT) * CurrentTri.V[TriSubIdx] + CrossingT * CurrentTri.V[NextSubIdx];
					const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(CandidateEdgeID);
					if (Edge.Vert[0] != TriVertIDs[TriSubIdx]) // edge verts are stored backwards from the order in the local triangle, reverse the crossing accordingly
					{
						CrossingT = 1 - CrossingT;
					}
					int CrossToTriID = Edge.Tri[0];
					if (CrossToTriID == TriID)
					{
						CrossToTriID = Edge.Tri[1];
					}
					if (CrossToTriID == -1)
					{
						// We've walked off the border of the mesh
						// TODO: check if this is close enough to the EndPt, and if so just stop the walk here
						continue;
					}
					// not allowed to go in a direction that gets us further from the destination than our initial point if backwards search not allowed
					bool bIsForward = ForwardsDirection.Dot(CrossingP - StartPt) >= -BackwardsTolerance;
					if ((bAllowBackwardsSearch || bIsForward) && !CrossedEdges.Contains(CandidateEdgeID))
					{
						CrossedEdges.Add(CandidateEdgeID);
						ComputedPointsAndSources.Emplace(FMeshSurfacePoint::MakeEdgePoint(CandidateEdgeID, CrossingT), FWalkIndices(CrossingP, CurrentEnd, CrossToTriID));
					}
				}
			}
		}

		const FVector3d& PreviousPathPoint = ComputedPointsAndSources[CurrentEnd].Value.Position;
		for (int32 NewComputedPtIdx = InitialComputedPointsNum; NewComputedPtIdx < ComputedPointsAndSources.Num(); NewComputedPtIdx++)
		{
			const FVector3d& CurrentPathPoint = ComputedPointsAndSources[NewComputedPtIdx].Value.Position;
			double PathLength = CurrentPathLength + Distance(PreviousPathPoint, CurrentPathPoint);
			double DistanceToEnd = Distance(EndPt, CurrentPathPoint);

			// Theoretically we've already verified the "forward" thing above while grabbing points, but we do it again here just in case.
			bool bIsForward = ForwardsDirection.Dot(CurrentPathPoint - StartPt) >= -BackwardsTolerance;
			if (ensure(bAllowBackwardsSearch || bIsForward))
			{
				UnexploredEnds.HeapPush({ NewComputedPtIdx, PathLength, DistanceToEnd });
			}
		}
	}


	int TrackedPtIdx = BestKnownEnd;
	int SafetyIdxBacktrack = 0;
	TArray<int> AcceptedIndices;
	while (TrackedPtIdx > -1)
	{
		if (!ensure(SafetyIdxBacktrack++ < 2*ComputedPointsAndSources.Num())) // infinite loop guard
		{
			return false;
		}
		AcceptedIndices.Add(TrackedPtIdx);
		TrackedPtIdx = ComputedPointsAndSources[TrackedPtIdx].Value.WalkedFromPt;
	}
	WalkedPath.Reset();
	for (int32 IdxIdx = AcceptedIndices.Num() - 1; IdxIdx >= 0; IdxIdx--)
	{
		WalkedPath.Emplace(ComputedPointsAndSources[AcceptedIndices[IdxIdx]].Key, ComputedPointsAndSources[AcceptedIndices[IdxIdx]].Value.WalkingOnTri);
	}

	// try refining start and end points if they were on triangles, and remove them if they turn out to be duplicates after refinement
	//  (note we could instead do the refinement up front and avoid the possibility of duplicate points, but that would slightly complicate the traversal logic ...)
	// note we don't even check that the barycoords match in the edge case -- conceptually the path can only cross the edge at one point, so it 'should' be fine to treat them as close enough, and having two points on the same edge would mess up the simple embedding code
	if (WalkedPath.Num() && WalkedPath[0].Key.PointType == ESurfacePointType::Triangle)
	{
		FMeshSurfacePoint& SurfacePt = WalkedPath[0].Key;
		
		// special case code to handle if we started on an exact vertex ID, to ensure refinement gets the same vertex ID
		// (note we do this after-the-fact fix rather than refining up-front to keep the traversal logic simpler)
		if (StartVIDIndex > -1 && SurfacePt.BaryCoord[StartVIDIndex] == 1.0) // if we had an exact start vertex
		{
			FIndex3i TriVertIDs = Mesh->GetTriangle(SurfacePt.ElementID);
			SurfacePt.ElementID = TriVertIDs[StartVIDIndex];
			SurfacePt.PointType = ESurfacePointType::Vertex;
		}
		else
		{
			RefineSurfacePtFromTriangleToSubElement(Mesh, SurfacePt.Pos(Mesh), SurfacePt, PtOnPlaneThresholdSq);
		}
		if (WalkedPath.Num() > 1 &&
			SurfacePt.PointType != ESurfacePointType::Triangle &&
			SurfacePt.PointType == WalkedPath[1].Key.PointType &&
			SurfacePt.ElementID == WalkedPath[1].Key.ElementID)
		{
			if (SurfacePt.PointType == ESurfacePointType::Edge) // copy closer barycoord
			{
				WalkedPath[1].Key.BaryCoord = SurfacePt.BaryCoord;
			}
			WalkedPath.RemoveAt(0);
		}
	}
	if (WalkedPath.Num() && WalkedPath.Last().Key.PointType == ESurfacePointType::Triangle)
	{
		FMeshSurfacePoint& SurfacePt = WalkedPath.Last().Key;
		RefineSurfacePtFromTriangleToSubElement(Mesh, SurfacePt.Pos(Mesh), SurfacePt, PtOnPlaneThresholdSq);
		if (WalkedPath.Num() > 1 &&
			SurfacePt.PointType != ESurfacePointType::Triangle &&
			SurfacePt.PointType == WalkedPath.Last(1).Key.PointType &&
			SurfacePt.ElementID == WalkedPath.Last(1).Key.ElementID)
		{
			if (SurfacePt.PointType == ESurfacePointType::Edge) // copy closer barycoord
			{
				WalkedPath.Last(1).Key.BaryCoord = SurfacePt.BaryCoord;
			}
			WalkedPath.Pop();
		}
	}

	return true;
}


} // UE::EmbedSurfacePathLocals


//static int FMeshSurfacePath::FindSharedTriangle(const FDynamicMesh3* Mesh, const FMeshSurfacePoint& A, const FMeshSurfacePoint& B)
//{
//	if (A.PointType == B.PointType && A.PointType == ESurfacePointType::Vertex)
//	{
//		int Edge = Mesh->FindEdge(A.ElementID, B.ElementID);
//		if (Edge == FDynamicMesh3::InvalidID)
//		{
//			return FDynamicMesh3::InvalidID;
//		}
//		return Mesh->GetEdgeT(Edge).A;
//	}
//	else if (A.PointType == ESurfacePointType::Vertex || B.PointType == ESurfacePointType::Vertex)
//	{
//		int VertexID = A.ElementID, OtherID = B.ElementID;
//		ESurfacePointType OtherType = B.PointType;
//		if (A.PointType != ESurfacePointType::Vertex)
//		{
//			VertexID = B.ElementID;
//			OtherID = A.ElementID;
//			OtherType = A.PointType;
//		}
//		if (OtherType == ESurfacePointType::Triangle)
//		{
//			if (!Mesh->GetTriangle(OtherID).Contains(VertexID))
//			{
//				return FDynamicMesh3::InvalidID;
//			}
//			return OtherID;
//		}
//		else // OtherType == ESurfacePointType::Edge
//		{
//			FIndex2i TriIDs = Mesh->GetEdgeT(OtherID);
//			if (Mesh->GetTriangle(TriIDs.A).Contains(VertexID))
//			{
//				return TriIDs.A;
//			}
//			else if (TriIDs.B != FDynamicMesh3::InvalidID && Mesh->GetTriangle(TriIDs.B).Contains(VertexID))
//			{
//				return TriIDs.B;
//			}
//			else
//			{
//				return FDynamicMesh3::InvalidID;
//			}
//		}
//	}
//	else if (A.PointType == ESurfacePointType::Edge || B.FMeshSurfacePoint == ESurfacePointType::Edge)
//	{
//		int EdgeID = A.ElementID, OtherID = B.ElementID;
//		ESurfacePointType OtherType = B.PointType;
//		if (A.PointType != ESurfacePointType::Edge)
//		{
//			EdgeID = B.ElementID;
//			OtherID = A.ElementID;
//			OtherType = A.PointType;
//		}
//		if (OtherType == ESurfacePointType::Triangle)
//		{
//			FIndex2i 
//		}
//	}
//	else // both ESurfacePointType::Triangle
//	{
//		if (A.ElementID == B.ElementID)
//		{
//			return A.ElementID;
//		}
//		else
//		{
//			return FDynamicMesh3::InvalidID;
//		}
//	}
//}

bool FMeshSurfacePath::IsConnected() const
{
	int Idx = 1, LastIdx = 0;
	if (bIsClosed)
	{
		LastIdx = Path.Num() - 1;
		Idx = 0;
	}
	for (; Idx < Path.Num(); LastIdx = Idx++)
	{
		int WalkingOnTri = Path[LastIdx].Value;
		if (!Mesh->IsTriangle(WalkingOnTri))
		{
			return false;
		}
		int Inds[2] = { LastIdx, Idx };
		for (int IndIdx = 0; IndIdx < 2; IndIdx++)
		{
			const FMeshSurfacePoint& P = Path[Inds[IndIdx]].Key;
			switch (P.PointType)
			{
			case ESurfacePointType::Triangle:
				if (P.ElementID != WalkingOnTri)
				{
					return false;
				}
				break;
			case ESurfacePointType::Edge:
				if (!Mesh->GetEdgeT(P.ElementID).Contains(WalkingOnTri))
				{
					return false;
				}
				break;
			case ESurfacePointType::Vertex:
				if (!Mesh->GetTriangle(WalkingOnTri).Contains(P.ElementID))
				{
					return false;
				}
				break;
			}
		}
	}
	return true;
}


bool FMeshSurfacePath::AddViaPlanarWalk(
	int StartTri, int StartVID, FVector3d StartPt, int EndTri, int EndVertID, 
	FVector3d EndPt, FVector3d WalkPlaneNormal, TFunction<FVector3d(const FDynamicMesh3*, int)> VertexToPosnFn,
	bool bAllowBackwardsSearch, double AcceptEndPtOutsideDist, double PtOnPlaneThresholdSq, double BackwardsTolerance)
{
	using namespace UE::EmbedSurfacePathLocals;
	
	if (!VertexToPosnFn)
	{
		VertexToPosnFn = [](const FDynamicMesh3* MeshArg, int VertexID)
		{
			return MeshArg->GetVertex(VertexID);
		};
	}
	return WalkMeshPlanar(Mesh, StartTri, StartVID, StartPt, EndTri, EndVertID, EndPt, WalkPlaneNormal, VertexToPosnFn,
		bAllowBackwardsSearch, AcceptEndPtOutsideDist, PtOnPlaneThresholdSq, Path, BackwardsTolerance);
}

// TODO: general path embedding becomes an arbitrary 2D remeshing problem per triangle; requires support from e.g. GeometryAlgorithms CDT. Not implemented yet; this is a vague sketch of what might go there.
//bool FMeshSurfacePath::EmbedPath(bool bUpdatePath, TArray<int>& PathVertices, TFunction<bool(const TArray<FVector2d>& Vertices, const TArray<FIndex3i>& LabelledEdges, TArray<FVector2d>& OutVertices, TArray<int32>& OutVertexMap, TArray<FIndex3i>& OutTriangles)> MeshGraphFn)
//{
//	// Array mapping surface pts in Path to IDs of new vertices in the mesh
//	TArray<int> SurfacePtToNewVertexID;
//
//	struct FRemeshTriInfo
//	{
//		TArray<int> AddSurfacePtIdx;
//		TArray<FVector2d> LocalVertexPosns;
//		TArray<FIndex2i> LocalEdges;
//	};
//
//	TMap<int, FRemeshTriInfo> TrianglesToRemesh;
//
//	// STEP 1: Collect all triangles that need re-triangulation because of pts added to edges or faces
//	// STEP 2: Perform re-triangulations in local coordinate space per triangle.  Store in FRemeshTriInfo map.  If any failure detected, stop process here (before altering source mesh)
//	// STEP 3: Add floating verts for all points that require a new vert, track their correspondence to surface pts via SurfacePtToNewVertexID
//	// STEP 4: Remove all triangles that needed re-triangulation (ignoring creation of bowties, etc)
//	// STEP 5: Add re-triangulations into mesh
//	// TODO: consider how to handle dynamic mesh overlay / attribute stuff
//	
//	return false;
//}



bool FMeshSurfacePath::EmbedSimplePath(bool bUpdatePath, TArray<int>& PathVertices, bool bDoNotDuplicateFirstVertexID, double SnapElementThresholdSq, const FEmbedSimplePathSettings& Settings)
{
	using namespace UE::EmbedSurfacePathLocals;

	// used to track where the new vertices for *this* path start; used for bDoNotDuplicateFirstVertexID
	int32 InitialPathIdx = PathVertices.Num();
	if (!Path.Num())
	{
		return true;
	}


	if (Settings.bSimplifyPathBySnapping)
	{
		// Helper to snap tri and edge surface points to vertices, if they're within the threshold distance
		auto SnapToVertex = [this, SnapElementThresholdSq](FMeshSurfacePoint& Pt) -> bool
		{
			double CloseDSq = SnapElementThresholdSq;
			if (Pt.PointType == ESurfacePointType::Triangle)
			{
				FTriangle3d Tri;
				Mesh->GetTriVertices(Pt.ElementID, Tri.V[0], Tri.V[1], Tri.V[2]);
				FVector3d NewPt = Tri.BarycentricPoint(Pt.BaryCoord);
				int32 ClosestIdx = INDEX_NONE;
				if (SelectSnapPoint(Tri.V, NewPt, ClosestIdx, SnapElementThresholdSq))
				{
					Pt = FMeshSurfacePoint(Mesh->GetTriangle(Pt.ElementID)[ClosestIdx]);
					return true;
				}
			}
			else if (Pt.PointType == ESurfacePointType::Edge)
			{
				FIndex2i EdgeV = Mesh->GetEdgeV(Pt.ElementID);
				FVector3d EdgePos[2];
				Mesh->GetEdgeV(Pt.ElementID, EdgePos[0], EdgePos[1]);
				FVector3d NewPt = Pt.BaryCoord[0] * EdgePos[0] + Pt.BaryCoord[1] * EdgePos[1];
				int32 ClosestIdx = INDEX_NONE;
				if (SelectSnapPoint(EdgePos, NewPt, ClosestIdx, SnapElementThresholdSq))
				{
					Pt = FMeshSurfacePoint(EdgeV[ClosestIdx]);
					return true;
				}
			}
			return false;
		};

		int32 NumRemoved = 0;
		TMap<int32, int32> FirstIndices;
		FirstIndices.Reserve(Path.Num());
		for (int32 PathIdx = 0; PathIdx < Path.Num(); ++PathIdx)
		{
			int32 OffsetPathIdx = PathIdx - NumRemoved;
			if (NumRemoved > 0)
			{
				Path[OffsetPathIdx] = Path[PathIdx];
			}

			if (SnapToVertex(Path[OffsetPathIdx].Key))
			{
				int32 LastValidIdx = PathIdx - 1 - NumRemoved;
				// if snapped point is a duplicate, can skip it
				if (LastValidIdx >= 0 &&
					Path[LastValidIdx].Key.PointType == ESurfacePointType::Vertex &&
					Path[LastValidIdx].Key.ElementID == Path[OffsetPathIdx].Key.ElementID)
				{
					NumRemoved++;
				}
			}
		}
		if (NumRemoved > 0)
		{
			Path.RemoveAt(Path.Num() - NumRemoved, NumRemoved);
		}
	}

	// Snapping on near-degenerate paths can cause the path to revisit a vertex.
	// For paths that were intended to be straight line connections, we can skip all points between the duplicated vertex(/vertices)
	if (Settings.bRemovePathLoops)
	{
		int32 NumRemoved = 0;
		TMap<int32, int32> SeenVertices;
		SeenVertices.Reserve(Path.Num());
		
		for (int32 PathIdx = 0; PathIdx < Path.Num(); ++PathIdx)
		{
			int32 OffsetPathIdx = PathIdx - NumRemoved;
			if (NumRemoved > 0)
			{
				Path[OffsetPathIdx] = Path[PathIdx];
			}

			if (Path[OffsetPathIdx].Key.PointType == ESurfacePointType::Vertex)
			{
				int32* SeenBefore = SeenVertices.Find(Path[OffsetPathIdx].Key.ElementID);
				if (SeenBefore)
				{
					int32 SeenBeforeVal = *SeenBefore;
					NumRemoved += OffsetPathIdx - SeenBeforeVal;
					ensure(OffsetPathIdx > SeenBeforeVal);
					for (int32 WalkBack = OffsetPathIdx - 1; WalkBack > SeenBeforeVal; --WalkBack)
					{
						if (Path[WalkBack].Key.PointType == ESurfacePointType::Vertex)
						{
							SeenVertices.Remove(Path[WalkBack].Key.ElementID);
						}
					}
				}
				else
				{
					SeenVertices.Add(Path[OffsetPathIdx].Key.ElementID, OffsetPathIdx);
				}
			}
		}
		if (NumRemoved > 0)
		{
			Path.RemoveAt(Path.Num() - NumRemoved, NumRemoved);
		}
	}


	int32 PathNum = Path.Num();
	const FMeshSurfacePoint& OrigEndPt = Path[PathNum - 1].Key;
	
	// If FinalTri is split or poked, we will need to re-locate the last point in the path
	int StartProcessIdx = 0, EndSimpleProcessIdx = PathNum - 1;
	bool bEndPointSpecialProcess = false;
	if (PathNum > 1 && OrigEndPt.PointType == ESurfacePointType::Triangle)
	{
		EndSimpleProcessIdx = PathNum - 2;
		bEndPointSpecialProcess = true;
	}
	bool bNeedFinalRelocate = false;
	FMeshSurfacePoint EndPtUpdated = Path.Last().Key;
	FVector3d EndPtPos = OrigEndPt.Pos(Mesh);
	
	if (Path[0].Key.PointType == ESurfacePointType::Triangle)
	{
		// poke triangle, and place initial vertex
		FDynamicMesh3::FPokeTriangleInfo PokeInfo;
		Mesh->PokeTriangle(Path[0].Key.ElementID, Path[0].Key.BaryCoord, PokeInfo);
		if (EndPtUpdated.PointType == ESurfacePointType::Triangle && Path[0].Key.ElementID == EndPtUpdated.ElementID)
		{
			TStaticArray<int, 3> EndCandidateTris{ PokeInfo.NewTriangles.A, PokeInfo.NewTriangles.B, PokeInfo.OriginalTriangle };
			EndPtUpdated = RelocateTrianglePointAfterRefinement(Mesh, EndPtPos, EndCandidateTris, SnapElementThresholdSq);
		}
		PathVertices.Add(PokeInfo.NewVertex);
		StartProcessIdx = 1;
	}
	for (int32 PathIdx = StartProcessIdx; PathIdx <= EndSimpleProcessIdx; PathIdx++)
	{
		if (!ensure(Path[PathIdx].Key.PointType != ESurfacePointType::Triangle))
		{
			// Input assumptions violated -- Simple path can only have Triangle points at the very first and/or last points!  Would need a more powerful embed function to handle this case.
			return false;
		}
		const FMeshSurfacePoint& Pt = Path[PathIdx].Key;
		if (Pt.PointType == ESurfacePointType::Edge)
		{
			ensure(Mesh->IsEdge(Pt.ElementID));

			if (Settings.bAllowEdgeFlipToCollapseTinyEdges && !PathVertices.IsEmpty() && !Mesh->IsBoundaryEdge(Pt.ElementID))
			{
				// If the edge we insert would be degenerate, try to flip the edge rather than splitting it
				// Note this is equivalent to splitting the edge then immediately collapsing the split
				FVector3d EdgeVerts[2];
				Mesh->GetEdgeV(Pt.ElementID, EdgeVerts[0], EdgeVerts[1]);
				FVector3d EdgePt = EdgeVerts[0] * Pt.BaryCoord[0] + EdgeVerts[1] * Pt.BaryCoord[1];
				FVector3d LastPos = Mesh->GetVertex(PathVertices.Last());
				if (FVector3d::DistSquared(LastPos, EdgePt) < SnapElementThresholdSq)
				{
					FIndex2i OppVIDs = Mesh->GetEdgeOpposingV(Pt.ElementID);
					FVector3d OpposingVerts[2]{ Mesh->GetVertex(OppVIDs.A), Mesh->GetVertex(OppVIDs.B) };
					// Note these Normals might be facing the opposite direction from the normals as the edge vertex order doesn't necessarily match the first tri winding
					// but we are just testing for relative consistency in orientation, so this is ok
					FVector3d Normals[2]{ VectorUtil::Normal(EdgeVerts[0], EdgeVerts[1], OpposingVerts[0]), VectorUtil::Normal(EdgeVerts[1], EdgeVerts[0], OpposingVerts[1]) };
					FVector3d FlipNormals[2]{ VectorUtil::Normal(OpposingVerts[0],OpposingVerts[1],EdgeVerts[1]), VectorUtil::Normal(OpposingVerts[1], OpposingVerts[0], EdgeVerts[0]) };
					// Only flip if (1) the original triangle are facing the same way and (2) flipping won't create a 'fold over' 
					// If the triangle(s) are degenerate, we don't have a clear normal so this test is invalid; in this case we allow the flip,
					// to avoid introducing even more degenerate geometry
					if (Normals[1].IsZero() || (Normals[0].Dot(Normals[1]) >= 0 && Normals[0].Dot(FlipNormals[0]) >= 0. && Normals[0].Dot(FlipNormals[1]) >= 0))
					{
						// Note because EmbedSimplePath is only allowed to cross each triangle at most once (except the start/end tri)
						// we can assume no future path points reference the edge we flip.

						FDynamicMesh3::FEdgeFlipInfo FlipInfo;
						if (Mesh->FlipEdge(Pt.ElementID, FlipInfo) == EMeshResult::Ok)
						{
							if (EndPtUpdated.PointType == ESurfacePointType::Triangle && FlipInfo.Triangles.Contains(EndPtUpdated.ElementID))
							{
								TStaticArray<int, 2> TriInds = { FlipInfo.Triangles.A, FlipInfo.Triangles.B };
								EndPtUpdated = RelocateTrianglePointAfterRefinement(Mesh, EndPtPos, TriInds, SnapElementThresholdSq);
							}
							// don't add a vertex to the path in this case; elements accessible from the split edge will be accessible from the previous path vertex after the flip
							continue;
						}
					}
				}
			}

			FDynamicMesh3::FEdgeSplitInfo SplitInfo;
			Mesh->SplitEdge(Pt.ElementID, SplitInfo, Pt.GetEdgeSplitParam());
			PathVertices.Add(SplitInfo.NewVertex);
			if (EndPtUpdated.PointType == ESurfacePointType::Triangle && SplitInfo.OriginalTriangles.Contains(EndPtUpdated.ElementID))
			{
				TStaticArray<int, 2> TriInds { EndPtUpdated.ElementID, 
				 SplitInfo.OriginalTriangles.A == EndPtUpdated.ElementID ? SplitInfo.NewTriangles.A : SplitInfo.NewTriangles.B };
				EndPtUpdated = RelocateTrianglePointAfterRefinement(Mesh, EndPtPos, TriInds, SnapElementThresholdSq);
			}
			else if (PathIdx != PathNum - 1 && EndPtUpdated.PointType == ESurfacePointType::Edge && Pt.ElementID == EndPtUpdated.ElementID)
			{
				// TODO: in this case we would need to relocate the endpoint, as its edge is gone ... 
				ensure(false);
			}
		}
		else
		{
			ensure(Pt.PointType == ESurfacePointType::Vertex);
			ensure(Mesh->IsVertex(Pt.ElementID));
			// make sure we don't add a duplicate vertex for the very first vertex (occurs when appending paths sequentially)
			if (!bDoNotDuplicateFirstVertexID || PathVertices.Num() != InitialPathIdx || 0 == PathVertices.Num() || PathVertices.Last() != Pt.ElementID)
			{
				PathVertices.Add(Pt.ElementID);
			}
		}
	}

	if (bEndPointSpecialProcess)
	{
		if (EndPtUpdated.PointType == ESurfacePointType::Triangle)
		{
			FDynamicMesh3::FPokeTriangleInfo PokeInfo;
			Mesh->PokeTriangle(EndPtUpdated.ElementID, EndPtUpdated.BaryCoord, PokeInfo);
			PathVertices.Add(PokeInfo.NewVertex);
		}
		else if (EndPtUpdated.PointType == ESurfacePointType::Edge)
		{
			FDynamicMesh3::FEdgeSplitInfo SplitInfo;
			Mesh->SplitEdge(EndPtUpdated.ElementID, SplitInfo, EndPtUpdated.GetEdgeSplitParam());
			PathVertices.Add(SplitInfo.NewVertex);
		}
		else
		{
			if (PathVertices.Num() == 0 || PathVertices.Last() != EndPtUpdated.ElementID)
			{
				PathVertices.Add(EndPtUpdated.ElementID);
			}
		}
	}


	// TODO: rm this debugging check
	//for (int PathIdx = InitialPathIdx; PathIdx + 1 < PathVertices.Num(); PathIdx++)
	//{
	//	if (!ensure(Mesh->IsEdge(Mesh->FindEdge(PathVertices[PathIdx], PathVertices[PathIdx + 1]))))
	//	{
	//		return false;
	//	}
	//}

	if (bUpdatePath)
	{
		ensure(false); // todo implement this
	}

	return true;
}


bool AddEdgesOnPath(const FDynamicMesh3& Mesh, FFrame3d Frame, const TArray<int>& PathVertices, bool bClosePath, TSet<int>& OutEdges, int& OutSeedTriID)
{
	OutSeedTriID = -1;
	int32 NumEdges = bClosePath ? PathVertices.Num() : PathVertices.Num() - 1;
	for (int32 IdxA = 0; IdxA < NumEdges; IdxA++)
	{
		int32 IdxB = (IdxA + 1) % PathVertices.Num();
		int32 IDA = PathVertices[IdxA];
		int32 IDB = PathVertices[IdxB];

		ensure(IDA != IDB);
		int EID = Mesh.FindEdge(IDA, IDB);
		if (!ensure(EID != FDynamicMesh3::InvalidID))
		{
			// TODO: some recovery?  This could occur e.g. if you have a self-intersecting path over the mesh surface
			return false;
		}
		OutEdges.Add(EID);

		if (OutSeedTriID == -1)
		{
			FVector2d PA = Frame.ToPlaneUV(Mesh.GetVertex(IDA));
			FVector2d PB = Frame.ToPlaneUV(Mesh.GetVertex(IDB));

			FIndex2i OppVIDs = Mesh.GetEdgeOpposingV(EID);
			double SignedAreaA = FTriangle2d::SignedArea(PA, PB, Frame.ToPlaneUV(Mesh.GetVertex(OppVIDs.A)));
			if (SignedAreaA > FMathd::Epsilon)
			{
				OutSeedTriID = Mesh.GetEdgeT(EID).A;
			}
			else if (OppVIDs.B != FDynamicMesh3::InvalidID)
			{
				double SignedAreaB = FTriangle2d::SignedArea(PA, PB, Frame.ToPlaneUV(Mesh.GetVertex(OppVIDs.B)));
				if (SignedAreaB > FMathd::ZeroTolerance)
				{
					OutSeedTriID = Mesh.GetEdgeT(EID).B;
				}
			}
		}
	}

	return true;
}


bool UE::Geometry::EmbedProjectedPaths(FDynamicMesh3* Mesh, const TArrayView<const int> StartTriIDs, FFrame3d Frame, const TArrayView<const TArray<FVector2d>> AllPaths, TArray<TArray<int>>& OutAllPathVertices, TArray<TArray<int>>& OutAllVertexCorrespondence, bool bClosePaths, FMeshFaceSelection* EnclosedFaces, double PtSnapVertexOrEdgeThresholdSq)
{
	check(Mesh);
	check(AllPaths.Num() == StartTriIDs.Num());
	
	int32 NumPaths = AllPaths.Num();
	OutAllPathVertices.Reset();
	OutAllPathVertices.SetNum(NumPaths);
	OutAllVertexCorrespondence.Reset();
	OutAllVertexCorrespondence.SetNum(NumPaths);

	TFunction<FVector3d(const FDynamicMesh3*, int)> ProjectToFrame = [&Frame](const FDynamicMesh3* MeshArg, int VertexID)
	{
		FVector2d ProjPt = Frame.ToPlaneUV(MeshArg->GetVertex(VertexID));
		return FVector3d(ProjPt.X, ProjPt.Y, 0);
	};
	
	// embed each path
	for (int32 PathIdx = 0; PathIdx < NumPaths; PathIdx++)
	{
		int StartTriID = StartTriIDs[PathIdx];
		const TArray<FVector2d>& Path2D = AllPaths[PathIdx];
		TArray<int>& OutPathVertices = OutAllPathVertices[PathIdx];
		TArray<int>& OutVertexCorrespondence = OutAllVertexCorrespondence[PathIdx];
		bool bClosePath = bClosePaths;
		if (StartTriID == FDynamicMesh3::InvalidID)
		{
			return false;
		}

		int32 EndIdxA = Path2D.Num() - (bClosePath ? 1 : 2);
		int CurrentSeedTriID = StartTriID;
		OutPathVertices.Reset();

		OutVertexCorrespondence.Add(0);
		for (int32 IdxA = 0; IdxA <= EndIdxA; IdxA++)
		{
			int32 IdxB = (IdxA + 1) % Path2D.Num();
			FMeshSurfacePath SurfacePath(Mesh);
			int LastVert = -1;
			// for closed paths, tell the final segment to connect back to the first vertex
			if (bClosePath && IdxB == 0 && OutPathVertices.Num() > 0)
			{
				LastVert = OutPathVertices[0];
			}
			FVector3d StartPos;
			if (OutPathVertices.Num())  // shift walk start pos to the actual place the last segment ended
			{
				StartPos = ProjectToFrame(Mesh, OutPathVertices.Last());
			}
			else
			{
				StartPos = FVector3d(Path2D[IdxA].X, Path2D[IdxA].Y, 0);
			}
			FVector2d WalkDir = Path2D[IdxB] - FVector2d(StartPos.X, StartPos.Y);
			double WalkLen = WalkDir.Length();
			bool bEmbedSuccess = true;
			if (WalkLen >= PtSnapVertexOrEdgeThresholdSq || (LastVert != -1 && LastVert != OutPathVertices.Last()))
			{
				WalkDir /= WalkLen;
				FVector3d WalkNormal(-WalkDir.Y, WalkDir.X, 0);
				if (!ensureMsgf(Mesh->IsTriangle(CurrentSeedTriID), TEXT("Invalid triangle somehow passed as seed for mesh path embedding: %d"), CurrentSeedTriID))
				{
					return false;
				}
				bool bWalkSuccess = SurfacePath.AddViaPlanarWalk(CurrentSeedTriID, -1, StartPos, -1, LastVert, FVector3d(Path2D[IdxB].X, Path2D[IdxB].Y, 0), WalkNormal, ProjectToFrame, false, FMathf::ZeroTolerance, PtSnapVertexOrEdgeThresholdSq);
				if (!bWalkSuccess)
				{
					return false;
				}
				bEmbedSuccess = SurfacePath.EmbedSimplePath(false, OutPathVertices, true, PtSnapVertexOrEdgeThresholdSq);
			}

			if (OutPathVertices.Num() == 0)
			{
				return false;
			}


			OutVertexCorrespondence.Add(OutPathVertices.Num() - 1);
			if (!bEmbedSuccess)
			{
				return false;
			}
			FDynamicMesh3::FLocalIntArray TrianglesOut;
			Mesh->GetVtxTriangles(OutPathVertices.Last(), TrianglesOut);

			check(TrianglesOut.Num());
			CurrentSeedTriID = TrianglesOut[0];
		}

		if (OutPathVertices.Num() == 0) // no path?
		{
			return false;
		}

		// special handling to remove redundant vertex + correspondence at the start and end of a looping path
		if (bClosePath && OutPathVertices.Num() > 1)
		{
			if (OutPathVertices[0] == OutPathVertices.Last())
			{
				OutPathVertices.Pop();
			}
			else
			{
				// TODO: we may consider worrying about the case where the start and end are 'almost' connected / separated by some degenerate triangles that are easily crossed, which would currently fail
				// for now we only handle the case where the start and end vertices are on the same triangle, which could happen for a single degenerate triangle case, which is the most likely case ...
				if (Mesh->FindEdge(OutPathVertices[0], OutPathVertices.Last()) == FDynamicMesh3::InvalidID)
				{
					return false; // failed to properly close path
				}
			}
			OutVertexCorrespondence.Pop();

			// wrap any trailing correspondence verts that happened to point to the last vertex
			for (int i = OutVertexCorrespondence.Num() - 1; i >= 0; i--)
			{
				if (OutVertexCorrespondence[i] == OutPathVertices.Num())
				{
					OutVertexCorrespondence[i] = 0;
				}
				else
				{
					break;
				}
			}
		}
	}

	if (EnclosedFaces)
	{
		TSet<int> Edges;
		TArray<int> SeedTriIDs;
		for (int32 PathIdx = 0; PathIdx < NumPaths; PathIdx++)
		{
			if (OutAllPathVertices[PathIdx].Num() < 2)
			{
				// path was a single point so could not have enclosed anything
				continue;
			}
			int SeedTriID = -1;
			if (!AddEdgesOnPath(*Mesh, Frame, OutAllPathVertices[PathIdx], bClosePaths, Edges, SeedTriID))
			{
				return false;
			}
			if (SeedTriID != -1)
			{
				SeedTriIDs.Add(SeedTriID);
			}
		}

		if (SeedTriIDs.Num() > 0)
		{
			EnclosedFaces->FloodFill(SeedTriIDs, nullptr, [&Edges](int ID)
			{
				return !Edges.Contains(ID);
			});
		}
	}

	return true;
}


bool UE::Geometry::EmbedProjectedPath(FDynamicMesh3* Mesh, int StartTriID, FFrame3d Frame, const TArray<FVector2d>& Path2D, TArray<int>& OutPathVertices, TArray<int>& OutVertexCorrespondence, bool bClosePath, FMeshFaceSelection* EnclosedFaces, double PtSnapVertexOrEdgeThresholdSq)
{
	TArrayView<int> StartTriIDs(&StartTriID, 1);
	const TArrayView<const TArray<FVector2d>> AllPaths(&Path2D, 1);
	TArray<TArray<int>> OutAllPathVertices, OutAllVertexCorrespondence;
	bool bResult = EmbedProjectedPaths(Mesh, StartTriIDs, Frame, AllPaths, OutAllPathVertices, OutAllVertexCorrespondence, bClosePath, EnclosedFaces, PtSnapVertexOrEdgeThresholdSq);
	if (bResult && ensure(OutAllPathVertices.Num() == 1 && OutAllVertexCorrespondence.Num() == 1))
	{
		OutPathVertices = OutAllPathVertices[0];
		OutVertexCorrespondence = OutAllVertexCorrespondence[0];
	}
	return bResult;
}
