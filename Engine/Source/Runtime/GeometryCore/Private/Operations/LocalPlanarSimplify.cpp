// Copyright Epic Games, Inc. All Rights Reserved.


#include "Operations/LocalPlanarSimplify.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMesh3.h"

using namespace UE::Geometry;


bool FLocalPlanarSimplify::IsFlat(const FDynamicMesh3& Mesh, int VID, double DotTolerance, FVector3d& OutFirstNormal)
{
	bool bHasFirst = false;
	bool bIsFlat = true;
	Mesh.EnumerateVertexTriangles(VID, [&Mesh, DotTolerance, &OutFirstNormal, &bHasFirst, &bIsFlat](int32 TID)
		{
			if (!bIsFlat)
			{
				return;
			}
			FVector3d Normal = Mesh.GetTriNormal(TID);
			if (!bHasFirst)
			{
				OutFirstNormal = Normal;
				bHasFirst = true;
			}
			else
			{
				bIsFlat = bIsFlat && Normal.Dot(OutFirstNormal) >= DotTolerance;
			}
		});
	return bIsFlat;
}

bool FLocalPlanarSimplify::IsDevelopableAlongEdge(const FDynamicMesh3& Mesh, int EID, int VID, double DotTolerance, FVector3d& NormalA, bool& bIsFlat)
{
	FIndex2i EdgeT = Mesh.GetEdgeT(EID);
	NormalA = Mesh.GetTriNormal(EdgeT.A);
	bIsFlat = true;
	FVector3d NormalB;
	if (EdgeT.B != FDynamicMesh3::InvalidID)
	{
		NormalB = Mesh.GetTriNormal(EdgeT.B);
		// Only consider a second normal if the across-edge triangle normal doesn't match the first normal
		if (NormalA.Dot(NormalB) < DotTolerance)
		{
			bIsFlat = false;
		}
	}
	bool bIsDevelopable = true;
	Mesh.EnumerateVertexTriangles(VID, [&bIsDevelopable, EdgeT, &Mesh, NormalA, NormalB, DotTolerance, bIsFlat](int32 TID)
		{
			if (!bIsDevelopable || EdgeT.Contains(TID))
			{
				return;
			}
			FVector3d Normal = Mesh.GetTriNormal(TID);
			bIsDevelopable = Normal.Dot(NormalA) >= DotTolerance;
			if (!bIsDevelopable && !bIsFlat) // if we didn't match first normal, test the second normal (if we have one)
			{
				bIsDevelopable = Normal.Dot(NormalB) >= DotTolerance;
			}
		});
	bIsFlat = bIsFlat && bIsDevelopable;

	return bIsDevelopable;
}

/**
 * Test if a given edge collapse would cause a triangle flip or other unacceptable decrease in mesh quality
 */
bool FLocalPlanarSimplify::CollapseWouldHurtTriangleQuality(
	const FDynamicMesh3& Mesh, const FVector3d& RemoveVNormal,
	int32 RemoveV, const FVector3d& RemoveVPos, int32 KeepV, const FVector3d& KeepVPos,
	double TryToImproveTriQualityThreshold, bool bHasMultipleNormals
	)
{
	double WorstQualityNewTriangle = FMathd::MaxReal;

	bool bIsHurt = false;
	Mesh.EnumerateVertexTriangles(RemoveV,
		[&Mesh, &bIsHurt, &KeepVPos, RemoveV, KeepV, &RemoveVNormal,
		TryToImproveTriQualityThreshold, &WorstQualityNewTriangle, bHasMultipleNormals](int32 TID)
		{
			if (bIsHurt)
			{
				return;
			}
			FIndex3i Tri = Mesh.GetTriangle(TID);
			// Skip re-computing the triangle normal for flat surfaces
			FVector3d TriNormal = bHasMultipleNormals ? Mesh.GetTriNormal(TID) : RemoveVNormal;
			FVector3d Verts[3];
			for (int Idx = 0; Idx < 3; Idx++)
			{
				int VID = Tri[Idx];
				if (VID == KeepV)
				{
					// this tri has both RemoveV and KeepV, so it'll be removed and we don't need to consider it
					return;
				}
				else if (VID == RemoveV)
				{
					// anything at RemoveV is reconnected to KeepV's position
					Verts[Idx] = KeepVPos;
				}
				else
				{
					// it's not on the collapsed edge so it stays still
					Verts[Idx] = Mesh.GetVertex(Tri[Idx]);
				}
			}
			FVector3d Edge1(Verts[1] - Verts[0]);
			FVector3d Edge2(Verts[2] - Verts[0]);
			FVector3d VCross(Edge2.Cross(Edge1));

			double EdgeFlipTolerance = 1.e-5;
			double Area2 = Normalize(VCross);
			if (TryToImproveTriQualityThreshold > 0)
			{
				FVector3d Edge3(Verts[2] - Verts[1]);
				double MaxLenSq = FMathd::Max3(Edge1.SquaredLength(), Edge2.SquaredLength(), Edge3.SquaredLength());
				double Quality = Area2 / (MaxLenSq + FMathd::ZeroTolerance);
				WorstQualityNewTriangle = FMathd::Min(Quality, WorstQualityNewTriangle);
			}

			bIsHurt = VCross.Dot(TriNormal) <= EdgeFlipTolerance;
		}
	);

	// note tri quality was computed as 2*Area / MaxEdgeLenSquared
	//  so need to multiply it by 2/sqrt(3) to get actual aspect ratio
	if (!bIsHurt && WorstQualityNewTriangle * 2 * FMathd::InvSqrt3 < TryToImproveTriQualityThreshold)
	{
		// we found a bad tri; tentatively switch to rejecting this edge collapse
		bIsHurt = true;
		// but if there was an even worse tri in the original neighborhood, accept the collapse after all
		Mesh.EnumerateVertexTriangles(RemoveV, [&Mesh, &bIsHurt, WorstQualityNewTriangle](int TID)
			{
				if (!bIsHurt) // early out if we already found an originally-worse tri
				{
					return;
				}
				FVector3d A, B, C;
				Mesh.GetTriVertices(TID, A, B, C);
				FVector3d E1 = B - A, E2 = C - A, E3 = C - B;
				double Area2 = E1.Cross(E2).Length();
				double MaxLenSq = FMathd::Max3(E1.SquaredLength(), E2.SquaredLength(), E3.SquaredLength());
				double Quality = Area2 / (MaxLenSq + FMathd::ZeroTolerance);
				if (Quality < WorstQualityNewTriangle)
				{
					bIsHurt = false;
				}
			}
		);
	}
	return bIsHurt;
}


/**
 * Test if a given edge collapse would change the mesh shape or UVs unacceptably
 */
bool FLocalPlanarSimplify::CollapseWouldChangeShapeOrUVs(
	const FDynamicMesh3& Mesh, const TSet<int>& PathEdgeSet, double DotTolerance, int SourceEID,
	int32 RemoveV, const FVector3d& RemoveVPos, int32 KeepV, const FVector3d& KeepVPos, const FVector3d& EdgeDir,
	bool bPreserveTriangleGroups, bool bPreserveUVsForMesh, bool bPreserveVertexUVs, bool bPreserveOverlayUVs,
	float UVEqualThresholdSq, bool bPreserveVertexNormals, float NormalEqualCosThreshold)
{
	// Search the edges connected to the vertex to find one in the boundary set that points in the opposite direction
	// If we don't find that edge, or if there are other boundary/seam edges attached, we can't remove this vertex
	// We also can't remove the vertex if doing so would distort the UVs
	bool bHasBadEdge = false;

	int OpposedEdge = -1;
	FIndex2i EdgeT = Mesh.GetEdgeT(SourceEID);
	int SourceGroupID = Mesh.GetTriangleGroup(EdgeT.A);
	int OtherGroupID = -1;
	// Note: In many cases (e.g. plane cut, new edge insertions) we know there will be only one group,
	// so we could use a slightly faster path with bAllowTwoGroups==false. But this is probably not
	// a significant performance difference, so for now we just expose the allows-two-group path.
	constexpr bool bAllowTwoGroups = true;
	if (bAllowTwoGroups && EdgeT.B != FDynamicMesh3::InvalidID)
	{
		OtherGroupID = Mesh.GetTriangleGroup(EdgeT.B);
	}
	
	Mesh.EnumerateVertexEdges(RemoveV,
		[&](int32 VertEID)
		{
			if (bHasBadEdge || VertEID == SourceEID)
			{
				return;
			}

			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(VertEID);
			if (bPreserveTriangleGroups && Mesh.HasTriangleGroups())
			{
				int GroupA = Mesh.GetTriangleGroup(Edge.Tri.A);
				int GroupB = Edge.Tri.B == FDynamicMesh3::InvalidID ? SourceGroupID : Mesh.GetTriangleGroup(Edge.Tri.B);

				if (
					(GroupA != SourceGroupID && (!bAllowTwoGroups || GroupA != OtherGroupID)) ||
					(GroupB != SourceGroupID && (!bAllowTwoGroups || GroupB != OtherGroupID))
					)
				{
					// RemoveV is bordering too many groups, so the edge collapse would change the shape of the groups
					bHasBadEdge = true;
					return;
				}
			}

			// it's a path edge; check if it's the opposite-facing one we need
			if (PathEdgeSet.Contains(VertEID))
			{
				if (OpposedEdge != -1)
				{
					bHasBadEdge = true;
					return;
				}
				FIndex2i OtherEdgeV = Edge.Vert;
				int OtherV = IndexUtil::FindEdgeOtherVertex(OtherEdgeV, RemoveV);
				FVector3d OtherVPos = Mesh.GetVertex(OtherV);
				FVector3d OtherEdgeDir = OtherVPos - RemoveVPos;
				if (Normalize(OtherEdgeDir) == 0)
				{
					// collapsing degenerate edges above should prevent this
					bHasBadEdge = true;
					return; // break instead of continue to skip the whole edge
				}
				if (OtherEdgeDir.Dot(EdgeDir) <= -DotTolerance)
				{
					OpposedEdge = VertEID;
				}
				else
				{
					bHasBadEdge = true;
					return;
				}

				// LerpT used for vertex normal and UV interpolation
				float LerpT = float((OtherVPos - RemoveVPos).Dot(OtherEdgeDir) / (OtherVPos - KeepVPos).Dot(OtherEdgeDir));

				if (bPreserveVertexNormals && Mesh.HasVertexNormals())
				{
					FVector3f OtherN = Mesh.GetVertexNormal(OtherV);
					FVector3f RemoveN = Mesh.GetVertexNormal(RemoveV);
					FVector3f KeepN = Mesh.GetVertexNormal(KeepV);
					if (Normalized(Lerp(OtherN, KeepN, LerpT)).Dot(Normalized(RemoveN)) < NormalEqualCosThreshold)
					{
						bHasBadEdge = true;
						return;
					}
				}

				// Controls whether any UV distortion is tested
				if (!bPreserveUVsForMesh)
				{
					return;
				}
				
				if (bPreserveVertexUVs && Mesh.HasVertexUVs())
				{
					FVector2f OtherUV = Mesh.GetVertexUV(OtherV);
					FVector2f RemoveUV = Mesh.GetVertexUV(RemoveV);
					FVector2f KeepUV = Mesh.GetVertexUV(KeepV);
					if ( DistanceSquared( Lerp(OtherUV, KeepUV, LerpT), RemoveUV) > UVEqualThresholdSq)
					{
						bHasBadEdge = true;
						return;
					}
				}
				if (bPreserveOverlayUVs && Mesh.HasAttributes())
				{
					int NumLayers = Mesh.Attributes()->NumUVLayers();
					FIndex2i SourceEdgeTris = Mesh.GetEdgeT(SourceEID);
					FIndex2i OppEdgeTris = Edge.Tri;

					// returns true if collapse will result in acceptable overlay UVs
					auto CanCollapseOverlayUVs = [&Mesh, KeepV, RemoveV, OtherV, NumLayers, LerpT, UVEqualThresholdSq](int SourceEdgeTID, int OppEdgeTID) -> bool
					{
						FIndex3i SourceBaseTri = Mesh.GetTriangle(SourceEdgeTID);
						FIndex3i OppBaseTri = Mesh.GetTriangle(OppEdgeTID);
						int KeepSourceIdx = IndexUtil::FindTriIndex(KeepV, SourceBaseTri);
						int RemoveSourceIdx = IndexUtil::FindTriIndex(RemoveV, SourceBaseTri);
						int OtherOppIdx = IndexUtil::FindTriIndex(OtherV, OppBaseTri);
						if (!ensure(KeepSourceIdx != -1 && RemoveSourceIdx != -1 && OtherOppIdx != -1))
						{
							return false;
						}

						// get the UVs per overlay off the triangle(s) attached the two edges
						for (int UVLayerIdx = 0; UVLayerIdx < NumLayers; UVLayerIdx++)
						{
							const FDynamicMeshUVOverlay* UVs = Mesh.Attributes()->GetUVLayer(UVLayerIdx);
							if (UVs->ElementCount() < 3)
							{
								// overlay is not actually in use; skip it
								continue;
							}
							FIndex3i SourceT = UVs->GetTriangle(SourceEdgeTID);
							FIndex3i OppT = UVs->GetTriangle(OppEdgeTID);
							int KeepE = SourceT[KeepSourceIdx];
							int RemoveE = SourceT[RemoveSourceIdx];
							int OtherE = OppT[OtherOppIdx];
							if (KeepE == -1 || RemoveE == -1 || OtherE == -1)
							{
								// overlay is not set on relevant triangles; skip it
								continue;
							}
							FVector2f OtherUV = UVs->GetElement(OtherE);
							FVector2f RemoveUV = UVs->GetElement(RemoveE);
							FVector2f KeepUV = UVs->GetElement(KeepE);
							if (DistanceSquared(Lerp(OtherUV, KeepUV, LerpT), RemoveUV) > UVEqualThresholdSq)
							{
								return false;
							}
						}

						return true;
					};

					// If we're not on a boundary, check if we're on a seam -- and if so, test that
					// the collapse won't change the UVs on the other side of the seam
					if (SourceEdgeTris.B != -1 || OppEdgeTris.B != -1)
					{
						// the edges must be both boundary or neither
						if (SourceEdgeTris.B == -1 || OppEdgeTris.B == -1)
						{
							bHasBadEdge = true;
							return;
						}
						// likewise must be both seam or neither
						bool bSourceIsSeam = Mesh.Attributes()->IsSeamEdge(SourceEID);
						bool bOtherIsSeam = Mesh.Attributes()->IsSeamEdge(VertEID);
						if (bSourceIsSeam != bOtherIsSeam)
						{
							bHasBadEdge = true;
							return;
						}
						// if both are seam, make sure that the A and B sides are consistent,
						// then test the UVs on the other side of the seam
						if (bSourceIsSeam)
						{
							// test whether the A/B sides are consistent for SourceEdgeTris and OppEdgeTris,
							// based on the ordering of the remove and keep/other vertices in each triangle
							FIndex3i SourceBaseTri = Mesh.GetTriangle(SourceEdgeTris.A);
							FIndex3i OppBaseTri = Mesh.GetTriangle(OppEdgeTris.A);
							int32 SrcTriRmVSubIdx = IndexUtil::FindTriIndex(RemoveV, SourceBaseTri);
							int32 OppTriRmVSubIdx = IndexUtil::FindTriIndex(RemoveV, OppBaseTri);
							// Test whether the Triangle "A" for the Source Edge is wound such that the 'Remove' Vertex is before the 'Keep' Vertex
							bool bSrcAOrderIsRmThenKeep = SourceBaseTri[(SrcTriRmVSubIdx + 1) % 3] == KeepV;
							// Test whether the Triangle "A" for the Opposite Edge is wound such that the 'Remove' Vertex is before the 'Other' Vertex
							bool bOppAOrderIsRmThenOther = OppBaseTri[(OppTriRmVSubIdx + 1) % 3] == OtherV;
							// If the two "A" sides are on the same side of the seam, we expect these two vertex ordering bools *not* to match --
							// i.e., the oriented edges should be "Keep Remove, then Remove Other" or "Other Remove, then Remove Keep"
							// If the bools match, we swap the A and B of the Opposite Edge Triangles, to move the A side back to the same side of the seam
							// as its Source Edge Triangle counterpart.
							if (bSrcAOrderIsRmThenKeep == bOppAOrderIsRmThenOther)
							{
								Swap(OppEdgeTris.A, OppEdgeTris.B);
							}
							if (!CanCollapseOverlayUVs(SourceEdgeTris.B, OppEdgeTris.B))
							{
								bHasBadEdge = true;
								return;
							}
						}
					}

					if (!CanCollapseOverlayUVs(SourceEdgeTris.A, OppEdgeTris.A))
					{
						bHasBadEdge = true;
						return;
					}
				}
			}
			else // it wasn't in the boundary edge set; check if it's one that would prevent us from safely removing the vertex
			{
				if (Mesh.IsBoundaryEdge(VertEID) || (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamEdge(VertEID)))
				{
					bHasBadEdge = true;
				}
			}
		});

	return bHasBadEdge;
}


void FLocalPlanarSimplify::SimplifyAlongEdges(FDynamicMesh3& Mesh, TSet<int32>& InOutEdges, TUniqueFunction<void(const DynamicMeshInfo::FEdgeCollapseInfo&)> ProcessCollapse) const
{
	double DotTolerance = FMathd::Cos(SimplificationAngleTolerance * FMathd::DegToRad);

	// For some simplification passes we can safely only consider locally-flat vertices for simplification, ignoring "straight crease" cases because
	// the new edges that we consider simplifying were generated by cutting across flat triangles. This makes the simplification logic a bit simpler.
	// You can use this variable to toggle that path.
	// TODO: We could consider exposing this variable as a parameter (but it does not seem likely to affect performance enough to really be worth doing so)
	constexpr bool bOnlySimplifyWhereFlat = false;

	// Save the input list of edges to iterate over, so we can edit the set w/ new edges while safely iterating over the old ones
	TArray<int32> CutBoundaryEdgesArray = InOutEdges.Array();

	int NumCollapses = 0, CollapseIters = 0;
	int MaxCollapseIters = 1; // TODO: is there a case where we need more iterations?  Perhaps if we add some triangle quality criteria?
	while (CollapseIters < MaxCollapseIters)
	{
		int LastNumCollapses = NumCollapses;
		for (int EID : CutBoundaryEdgesArray)
		{
			// this can happen if a collapse removes another cut boundary edge
			// (which can happen e.g. if you have a degenerate (colinear) tri flat on the cut boundary)
			if (!Mesh.IsEdge(EID))
			{
				continue;
			}

			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID);

			// track whether the neighborhood of the vertex is flat
			bool Developable[2]{ false, false };
			// normals for each flat vertex
			FVector3d FlatNormals[2]{ FVector3d::Zero(), FVector3d::Zero() };
			bool Flat[2]{ false, false };
			int NumDevelopable = 0;
			for (int VIdx = 0; VIdx < 2; VIdx++)
			{
				if (bOnlySimplifyWhereFlat)
				{
					Flat[VIdx] = Developable[VIdx] = IsFlat(Mesh, Edge.Vert[VIdx], DotTolerance, FlatNormals[VIdx]);
				}
				else
				{
					Developable[VIdx] = IsDevelopableAlongEdge(Mesh, EID, Edge.Vert[VIdx], DotTolerance, FlatNormals[VIdx], Flat[VIdx]);
				}

				if (Developable[VIdx])
				{
					NumDevelopable++;
				}
			}

			if (NumDevelopable == 0)
			{
				continue;
			}

			// see if we can collapse to remove either vertex
			for (int RemoveVIdx = 0; RemoveVIdx < 2; RemoveVIdx++)
			{
				if (!Developable[RemoveVIdx])
				{
					continue;
				}
				int KeepVIdx = 1 - RemoveVIdx;
				FVector3d RemoveVPos = Mesh.GetVertex(Edge.Vert[RemoveVIdx]);
				FVector3d KeepVPos = Mesh.GetVertex(Edge.Vert[KeepVIdx]);
				FVector3d EdgeDir = KeepVPos - RemoveVPos;
				if (Normalize(EdgeDir) == 0) // 0 is returned as a special case when the edge was too short to normalize
				{
					// This case is often avoided by collapsing degenerate edges in a separate pre-pass, so we just skip over it here for now
					// TODO: Consider adding degenerate edge-collapse logic here
					break; // break instead of continue to skip the whole edge
				}

				bool bHasBadEdge = false; // will be set if either mesh can't collapse the edge
				int RemoveV = Edge.Vert[RemoveVIdx];
				int KeepV = Edge.Vert[KeepVIdx];
				int SourceEID = EID;

				bHasBadEdge = bHasBadEdge || CollapseWouldHurtTriangleQuality(
					Mesh, FlatNormals[RemoveVIdx], RemoveV, RemoveVPos, KeepV, KeepVPos, TryToImproveTriQualityThreshold, !Flat[RemoveVIdx]);

				bHasBadEdge = bHasBadEdge || CollapseWouldChangeShapeOrUVs(
					Mesh, InOutEdges, DotTolerance,
					SourceEID, RemoveV, RemoveVPos, KeepV, KeepVPos, EdgeDir, bPreserveTriangleGroups,
					true, bPreserveVertexUVs, bPreserveOverlayUVs, UVDistortTolerance * UVDistortTolerance,
					bPreserveVertexNormals, FMathf::Cos(NormalDistortTolerance * FMathf::DegToRad));

				if (bHasBadEdge)
				{
					continue;
				}

				FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
				EMeshResult CollapseResult = Mesh.CollapseEdge(KeepV, RemoveV, 0, CollapseInfo);
				if (CollapseResult == EMeshResult::Ok)
				{
					if (ProcessCollapse)
					{
						ProcessCollapse(CollapseInfo);
					}
					NumCollapses++;
					InOutEdges.Remove(CollapseInfo.CollapsedEdge);
					InOutEdges.Remove(CollapseInfo.RemovedEdges[0]);
					if (CollapseInfo.RemovedEdges[1] != -1)
					{
						InOutEdges.Remove(CollapseInfo.RemovedEdges[1]);
					}
				}
				break; // if we got through to trying to collapse the edge, don't try to collapse from the other vertex.
			}
		}

		CutBoundaryEdgesArray = InOutEdges.Array();

		if (NumCollapses == LastNumCollapses)
		{
			break;
		}

		CollapseIters++;
	}
}


void FLocalPlanarSimplify::CollapseDegenerateEdges(FDynamicMesh3& Mesh, TSet<int32>& InOutEdges, bool bRemoveAllDegenerateFromInputSet, double DegenerateEdgeLenTol, TSet<int>* TriangleSelection)
{
	double Tol2 = DegenerateEdgeLenTol * DegenerateEdgeLenTol;
	FVector3d A, B;
	TArray<int32> ToRemove;
	int Collapsed = 0;
	do
	{
		Collapsed = 0;
		for (auto EdgesIter = InOutEdges.CreateIterator(); EdgesIter; ++EdgesIter)
		{
			int32 EID = *EdgesIter;
			if (!Mesh.IsEdge(EID))
			{
				continue;
			}
			Mesh.GetEdgeV(EID, A, B);
			if (DistanceSquared(A, B) > Tol2)
			{
				continue;
			}

			FIndex2i EV = Mesh.GetEdgeV(EID);
			// if the vertex we'd remove is on a seam, try removing the other one instead
			if (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamVertex(EV.B, false))
			{
				Swap(EV.A, EV.B);
				// if they were both on seams, then collapse should not happen?  (& would break OnCollapseEdge assumptions in overlay)
				if (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamVertex(EV.B, false))
				{
					continue;
				}
			}
			FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
			EMeshResult Result = Mesh.CollapseEdge(EV.A, EV.B, CollapseInfo);
			if (Result == EMeshResult::Ok)
			{
				Collapsed++;
				EdgesIter.RemoveCurrent();
				if (TriangleSelection)
				{
					TriangleSelection->Remove(CollapseInfo.RemovedEdges.A);
					if (CollapseInfo.RemovedEdges.B != FDynamicMesh3::InvalidID)
					{
						TriangleSelection->Remove(CollapseInfo.RemovedEdges.B);
					}
				}
				// Add the other edge for later removal, so we don't potentially invalidate our iterator
				// (but only if we really need to remove invalid edges here; some algorithms may be ok with leaving invalid EIDs in the set)
				if (bRemoveAllDegenerateFromInputSet && CollapseInfo.RemovedEdges.B != FDynamicMesh3::InvalidID)
				{
					ToRemove.Add(CollapseInfo.RemovedEdges.B);
				}
			}
		}
	} while (Collapsed != 0);

	if (bRemoveAllDegenerateFromInputSet)
	{
		for (int32 EID : ToRemove)
		{
			InOutEdges.Remove(EID);
		}
	}
}

