// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshBoolean

#include "Operations/MeshBoolean.h"
#include "Operations/MeshMeshCut.h"
#include "Operations/LocalPlanarSimplify.h"
#include "Selections/MeshConnectedComponents.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "Spatial/SparseDynamicOctree3.h"

#include "Algo/RemoveIf.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"

using namespace UE::Geometry;

// TODO: Commented out below is a custom thick triangle intersection.
// It's much better at finding all the near-tolerance collision edges.
// But it ends up creating harder problems downstream in terms of
//  tiny edges, almost-exactly-at-tolerance coplanar faces, etc.
// Consider re-enabling it in combination with more robust downstream processing!

//// helper for ThickTriTriIntersection, using the plane of Tri0 as reference
//// (factored out so we can try using both planes as reference, and make the result less dependent on triangle ordering)
//bool ThickTriTriHelper(const FTriangle3d& Tri0, const FTriangle3d& Tri1, const FPlane3d& Plane0,
//					   const FVector3d& IntersectionDir, const FVector3d& dist1, const FIndex3i& sign1,
//					   int pos1, int  neg1, int zero1,
//					   FIntrTriangle3Triangle3d& Intr, double Tolerance)
//{
//	int SegmentsFound = 0;
//	int PtsFound[2]{ 0,0 }; // points found on the positive and negative sides
//	FVector3d CrossingPts[4]; // space for triangle-plane intersection segments; with negative-side endpoints first
//	int PtsSide[4]; // -1, 1 or 0
//
//	// offset tolerance -- used to accept intersections off the plane, when we'd otherwise miss "near intersections"
//	double ToleranceOffset = Tolerance * .99; // scale down to be extra sure not to create un-snappable geometry
//	// only accept 'offset plane' points if not doing so would miss a much-larger-than-tolerance edge
//	double AcceptOffsetPointsThresholdSq = 1e-2 * 1e-2;
//
//	double InPlaneTolerance = FMathd::ZeroTolerance;
//
//	// consider all crossings
//	for (int i = 0, lasti = 2; i < 3; lasti = i++)
//	{
//		if (sign1[lasti] == sign1[i])
//		{
//			continue;
//		}
//		// 
//		if (sign1[lasti] == 0 || sign1[i] == 0)
//		{
//			int nzi = lasti, zi = i;
//			if (sign1[lasti] == 0)
//			{
//				nzi = i;
//				zi = lasti;
//			}
//			int SideIdx = (sign1[nzi] + 1) / 2;
//			int PtIdx = SideIdx * 2 + PtsFound[SideIdx];
//			int Side = sign1[nzi];
//
//			double ParamOnTolPlane = (dist1[nzi] - sign1[nzi] * ToleranceOffset) / (dist1[nzi] - dist1[zi]);
//			FVector3d IntrPt;
//			if (ParamOnTolPlane < 1)
//			{
//				IntrPt = Tri1.V[nzi] + (Tri1.V[zi] - Tri1.V[nzi]) * ParamOnTolPlane;
//				if (IntrPt.DistanceSquared(Tri1.V[zi]) < AcceptOffsetPointsThresholdSq)
//				{
//					Side = 0;
//					IntrPt = Tri1.V[zi];
//				}
//			}
//			else
//			{
//				IntrPt = Tri1.V[zi];
//			}
//
//			// record crossing pt
//			PtsSide[PtIdx] = Side;
//			CrossingPts[PtIdx] = IntrPt;
//			PtsFound[SideIdx]++;
//		}
//		else
//		{
//			double OffsetParamDiff = sign1[i] * ToleranceOffset / (dist1[i] - dist1[lasti]);
//			FVector3d Edge = Tri1.V[lasti] - Tri1.V[i];
//			double OffsetDSq = Edge.SquaredLength() * OffsetParamDiff * OffsetParamDiff;
//			if (OffsetDSq < AcceptOffsetPointsThresholdSq)
//			{
//				FVector3d IntrPt = Tri1.V[i] + Edge * dist1[i] / (dist1[i] - dist1[lasti]);
//				for (int SideIdx = 0; SideIdx < 2; SideIdx++)
//				{
//					int PtIdx = SideIdx * 2 + PtsFound[SideIdx];
//					PtsSide[PtIdx] = 0;
//					CrossingPts[PtIdx] = IntrPt;
//					PtsFound[SideIdx]++;
//				}
//			}
//			else
//			{
//				for (int Sign = -1; Sign <= 1; Sign += 2)
//				{
//					double ParamOnPlane = (dist1[i] - Sign * ToleranceOffset) / (dist1[i] - dist1[lasti]);
//					FVector3d IntrPt = Tri1.V[i] + (Tri1.V[lasti] - Tri1.V[i]) * ParamOnPlane;
//					int SideIdx = (Sign + 1) / 2;
//					int PtIdx = SideIdx * 2 + PtsFound[SideIdx];
//					PtsSide[PtIdx] = Sign;
//					CrossingPts[PtIdx] = IntrPt;
//					PtsFound[SideIdx]++;
//				}
//			}
//		}
//	}
//
//	bool bMadeZeroEdge = false;
//	int AddedPts = 0;
//	for (int SideIdx = 0; SideIdx < 2; SideIdx++)
//	{
//		if (PtsFound[SideIdx] == 2)
//		{
//			int PtIdx0 = SideIdx * 2;
//			if (PtsSide[PtIdx0] == 0 && PtsSide[PtIdx0 + 1] == 0)
//			{
//				if (bMadeZeroEdge)
//				{
//					continue;
//				}
//				bMadeZeroEdge = true;
//			}
//			FVector3d OutA, OutB;
//			int IntrQ = FIntrTriangle3Triangle3d::IntersectTriangleWithCoplanarSegment(Plane0, Tri0, CrossingPts[PtIdx0], CrossingPts[PtIdx0 + 1], OutA, OutB, InPlaneTolerance);
//
//			if (IntrQ == 2)
//			{
//				Intr.Points[AddedPts++] = OutA;
//				Intr.Points[AddedPts++] = OutB;
//			}
//		}
//	}
//	Intr.Quantity = AddedPts;
//	if (AddedPts == 4)
//	{
//		Intr.Result = EIntersectionResult::Intersects;
//		Intr.Type = EIntersectionType::MultiSegment;
//	}
//	else if (AddedPts == 2)
//	{
//		Intr.Result = EIntersectionResult::Intersects;
//		Intr.Type = EIntersectionType::Segment;
//	}
//	else
//	{
//		Intr.Result = EIntersectionResult::NoIntersection;
//		Intr.Type = EIntersectionType::Empty;
//		return false;
//	}
//
//	return true;
//}
//
//bool ThickTriTriIntersection(FIntrTriangle3Triangle3d& Intr, double Tolerance)
//{
//	// intersection tolerance is applied one dimension at a time, so we scale down by 1/sqrt(2)
//	//  to ensure approximations remain within snapping distance
//	Intr.SetTolerance(Tolerance);
//	const FTriangle3d& Triangle0 = Intr.GetTriangle0();
//	const FTriangle3d& Triangle1 = Intr.GetTriangle1();
//	FPlane3d Plane0(Triangle0.V[0], Triangle0.V[1], Triangle0.V[2]);
//
//	// Compute the signed distances of Triangle1 vertices to Plane0.  Use an epsilon-thick plane test.
//	int pos1, neg1, zero1;
//	FVector3d dist1;
//	FIndex3i sign1;
//	FIntrTriangle3Triangle3d::TrianglePlaneRelations(Triangle1, Plane0, dist1, sign1, pos1, neg1, zero1, Tolerance);
//	if (pos1 == 3 || neg1 == 3)
//	{
//		// ignore triangles that are more than tolerance-separated
//		Intr.SetResultNone();
//		return false;
//	}
//
//	FPlane3d Plane1(Triangle1.V[0], Triangle1.V[1], Triangle1.V[2]);
//	FVector3d IntersectionDir = Plane0.Normal.Cross(Plane1.Normal);
//
//	FVector3d SegA, SegB;
//	bool bFound = false;
//	bFound = zero1 < 3 && ThickTriTriHelper(Triangle0, Triangle1, Plane0, IntersectionDir, dist1, sign1, pos1, neg1, zero1, Intr, Tolerance);
//	if (!bFound || Intr.Quantity == 1)
//	{
//		int pos0, neg0, zero0;
//		FVector3d dist0;
//		FIndex3i sign0;
//		FIntrTriangle3Triangle3d::TrianglePlaneRelations(Triangle0, Plane1, dist0, sign0, pos0, neg0, zero0, Tolerance);
//		bFound = zero1 < 3 && ThickTriTriHelper(Triangle1, Triangle0, Plane1, IntersectionDir, dist0, sign0, pos0, neg0, zero0, Intr, Tolerance);
//	}
//	if (!bFound) // make sure the Intr values are set in the coplanar case
//	{
//		Intr.SetResultNone();
//	}
//
//	return bFound;
//}

bool FMeshBoolean::Compute()
{
	FCustomInsideMeshTest WindingTest(
		[Threshold = WindingThreshold](FVector3d Pt, const FCustomInsideTestContext& Context) -> bool
		{
			return Context.OptionalWindingTree->IsInside(Pt, Threshold);
		}, true);
	return ComputeWithCustomInside(WindingTest, WindingTest);
}

bool FMeshBoolean::ComputeWithCustomInside(FCustomInsideMeshTest InsideMeshA, FCustomInsideMeshTest InsideMeshB)
{
	// copy meshes
	FDynamicMesh3 CutMeshB(*Meshes[1]);
	if (Result != Meshes[0])
	{
		*Result = *Meshes[0];
	}
	FDynamicMesh3* CutMesh[2]{ Result, &CutMeshB }; // just an alias to keep things organized

	// transform the copies to a shared space (centered at the origin and scaled to a unit cube)
	FAxisAlignedBox3d CombinedAABB(CutMesh[0]->GetBounds(true), Transforms[0]);
	FAxisAlignedBox3d MeshB_AABB(CutMesh[1]->GetBounds(true), Transforms[1]);
	CombinedAABB.Contain(MeshB_AABB);
	double ScaleFactor = 1.0 / FMath::Clamp(CombinedAABB.MaxDim(), 0.01, 1000000.0);
	for (int MeshIdx = 0; MeshIdx < 2; MeshIdx++)
	{
		FTransformSRT3d CenteredTransform = Transforms[MeshIdx];
		CenteredTransform.SetTranslation(ScaleFactor*(CenteredTransform.GetTranslation() - CombinedAABB.Center()));
		CenteredTransform.SetScale(ScaleFactor*CenteredTransform.GetScale());
		MeshTransforms::ApplyTransform(*CutMesh[MeshIdx], CenteredTransform, true);
	}
	ResultTransform = FTransformSRT3d(CombinedAABB.Center());
	ResultTransform.SetScale(FVector3d::One() * (1.0 / ScaleFactor));

	if (Cancelled())
	{
		return false;
	}

	// build spatial data and use it to find intersections
	FDynamicMeshAABBTree3 Spatial[2]{ CutMesh[0], CutMesh[1] };
	Spatial[0].SetTolerance(SnapTolerance);
	Spatial[1].SetTolerance(SnapTolerance);
	MeshIntersection::FIntersectionsQueryResult Intersections
		= Spatial[0].FindAllIntersections(Spatial[1], nullptr, IMeshSpatial::FQueryOptions(), IMeshSpatial::FQueryOptions(),
			[this](FIntrTriangle3Triangle3d& Intr)
			{
				Intr.SetTolerance(SnapTolerance);
				return Intr.Find();

				// TODO: if we revisit "thick" tri tri collisions, this is where we'd call:
				// 	ThickTriTriIntersection(Intr, SnapTolerance);
			});

	if (Cancelled())
	{
		return false;
	}

	bool bOpOnSingleMesh = OperatesOnSingleMesh(Operation);

	// cut the meshes
	FMeshMeshCut Cut(CutMesh[0], CutMesh[1]);
	Cut.bTrackInsertedVertices = bCollapseDegenerateEdgesOnCut; // to collect candidates to collapse
	Cut.bMutuallyCut = !bOpOnSingleMesh;
	Cut.SnapTolerance = SnapTolerance;
	Cut.Cut(Intersections);

	if (Cancelled())
	{
		return false;
	}

	int NumMeshesToProcess = bOpOnSingleMesh ? 1 : 2;

	// collapse tiny edges along cut boundary
	if (bCollapseDegenerateEdgesOnCut)
	{
		double DegenerateEdgeTolSq = DegenerateEdgeTolFactor * DegenerateEdgeTolFactor * SnapTolerance * SnapTolerance;
		for (int MeshIdx = 0; MeshIdx < NumMeshesToProcess; MeshIdx++)
		{
			// convert vertex chains to edge IDs to simplify logic of finding remaining candidate edges after collapses
			TArray<int> EIDs;
			for (int ChainIdx = 0; ChainIdx < Cut.VertexChains[MeshIdx].Num();)
			{
				int ChainLen = Cut.VertexChains[MeshIdx][ChainIdx];
				int ChainEnd = ChainIdx + 1 + ChainLen;
				for (int ChainSubIdx = ChainIdx + 1; ChainSubIdx + 1 < ChainEnd; ChainSubIdx++)
				{
					int VID[2]{ Cut.VertexChains[MeshIdx][ChainSubIdx], Cut.VertexChains[MeshIdx][ChainSubIdx + 1] };
					if (DistanceSquared(CutMesh[MeshIdx]->GetVertex(VID[0]), CutMesh[MeshIdx]->GetVertex(VID[1])) < DegenerateEdgeTolSq)
					{
						EIDs.Add(CutMesh[MeshIdx]->FindEdge(VID[0], VID[1]));
					}
				}
				ChainIdx = ChainEnd;
			}
			TSet<int> AllEIDs(EIDs);
			for (int Idx = 0; Idx < EIDs.Num(); Idx++)
			{
				int EID = EIDs[Idx];
				if (!CutMesh[MeshIdx]->IsEdge(EID))
				{
					continue;
				}
				FVector3d A, B;
				CutMesh[MeshIdx]->GetEdgeV(EID, A, B);
				if (DistanceSquared(A,B) > DegenerateEdgeTolSq)
				{
					continue;
				}
				FIndex2i EV = CutMesh[MeshIdx]->GetEdgeV(EID);

				// if the vertex we'd remove is on a seam, try removing the other one instead
				if (CutMesh[MeshIdx]->HasAttributes() && CutMesh[MeshIdx]->Attributes()->IsSeamVertex(EV.B, false))
				{
					Swap(EV.A, EV.B);
					// if they were both on seams, then collapse should not happen?  (& would break OnCollapseEdge assumptions in overlay)
					if (CutMesh[MeshIdx]->HasAttributes() && CutMesh[MeshIdx]->Attributes()->IsSeamVertex(EV.B, false))
					{
						continue;
					}
				}
				FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
				EMeshResult CollapseResult = CutMesh[MeshIdx]->CollapseEdge(EV.A, EV.B, .5, CollapseInfo);
				if (CollapseResult == EMeshResult::Ok)
				{
					for (int i = 0; i < 2; i++)
					{
						if (AllEIDs.Contains(CollapseInfo.RemovedEdges[i]))
						{
							int ToAdd = CollapseInfo.KeptEdges[i];
							bool bWasPresent;
							AllEIDs.Add(ToAdd, &bWasPresent);
							if (!bWasPresent)
							{
								EIDs.Add(ToAdd);
							}
						}
					}
				}
			}
		}
	}

	if (Cancelled())
	{
		return false;
	}

	// edges that will become new boundary edges after the boolean op removes triangles on each mesh
	TArray<int> CutBoundaryEdges[2];
	// Vertices on the cut boundary that *may* not have a corresonding vertex on the other mesh
	TSet<int> PossUnmatchedBdryVerts[2];

	// delete geometry according to boolean rules, tracking the boundary edges
	{ // (just for scope)
		// first decide what triangles to delete for both meshes (*before* deleting anything so winding doesn't get messed up!)
		TArray<bool> KeepTri[2];
		// This array is used to double-check the assumption that we will delete the other surface when we keep a coplanar tri
		// Note we only need it for mesh 0 (i.e., the mesh we try to keep triangles from when we preserve coplanar surfaces)
		TArray<int32> DeleteIfOtherKept;
		if (NumMeshesToProcess > 1)
		{
			DeleteIfOtherKept.Init(-1, CutMesh[0]->MaxTriangleID());
		}
		for (int MeshIdx = 0; MeshIdx < NumMeshesToProcess; MeshIdx++)
		{
			FCustomInsideMeshTest& UseInsideTest = MeshIdx == 0 ? InsideMeshB : InsideMeshA;
			FDynamicMeshAABBTree3& OtherSpatial = Spatial[1 - MeshIdx];
			if (!OtherSpatial.IsValid(false))
			{
				OtherSpatial.Build();
			}
			TFastWindingTree<FDynamicMesh3> LocalWinding(&OtherSpatial, UseInsideTest.bRequiresWinding);
			TFastWindingTree<FDynamicMesh3>* UseWinding = UseInsideTest.bRequiresWinding ? &LocalWinding : nullptr;
			FCustomInsideTestContext InsideTestContext
			{
				.Spatial = &OtherSpatial,
				.OptionalWindingTree = UseWinding
			};
			
			FDynamicMesh3& ProcessMesh = *CutMesh[MeshIdx];
			int MaxTriID = ProcessMesh.MaxTriangleID();
			KeepTri[MeshIdx].SetNumUninitialized(MaxTriID);
			bool bCoplanarKeepSameDir = (Operation != EBooleanOp::Difference && Operation != EBooleanOp::TrimInside && Operation != EBooleanOp::NewGroupInside);
			bool bRemoveInside = 1; // whether to remove the inside triangles (e.g. for union) or the outside ones (e.g. for intersection)
			if (Operation == EBooleanOp::NewGroupOutside || Operation == EBooleanOp::TrimOutside || Operation == EBooleanOp::Intersect || (Operation == EBooleanOp::Difference && MeshIdx == 1))
			{
				bRemoveInside = 0;
			}
			FMeshNormals OtherNormals(OtherSpatial.GetMesh());
			OtherNormals.ComputeTriangleNormals();
			const double OnPlaneTolerance = SnapTolerance;
			IMeshSpatial::FQueryOptions NonDegenCoplanarCandidateFilter(OnPlaneTolerance,
				[&OtherNormals](int TID) -> bool // filter degenerate triangles from matching
				{
					// By convention, the normal for degenerate triangles is the zero vector
					return !OtherNormals[TID].IsZero();
				});
			ParallelFor(MaxTriID, [&](int TID)
				{
					if (!ProcessMesh.IsTriangle(TID))
					{
						return;
					}
					
					FTriangle3d Tri;
					ProcessMesh.GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
					FVector3d Centroid = Tri.Centroid();

					// first check for the coplanar case
					{
						double DSq;
						int OtherTID = OtherSpatial.FindNearestTriangle(Centroid, DSq, NonDegenCoplanarCandidateFilter);
						if (OtherTID > -1) // only consider it coplanar if there is a matching tri
						{
							
							FVector3d OtherNormal = OtherNormals[OtherTID];
							FVector3d Normal = ProcessMesh.GetTriNormal(TID);
							double DotNormals = OtherNormal.Dot(Normal);

							//if (FMath::Abs(DotNormals) > .9) // TODO: do we actually want to check for a normal match? coplanar vertex check below is more robust?
							{
								// To be extra sure it's a coplanar match, check the vertices are *also* on the other mesh (w/in SnapTolerance)

								bool bAllTrisOnOtherMesh = true;
								for (int Idx = 0; Idx < 3; Idx++)
								{
									// use a slightly more forgiving tolerance to account for the likelihood that these vertices were mesh-cut right to the boundary of the coplanar region and have some additional error
									if (OtherSpatial.FindNearestTriangle(Tri.V[Idx], DSq, OnPlaneTolerance * 2) == FDynamicMesh3::InvalidID)
									{
										bAllTrisOnOtherMesh = false;
										break;
									}
								}
								if (bAllTrisOnOtherMesh)
								{
									// for coplanar tris favor the first mesh; just delete from the other mesh
									// for fully degenerate tris, favor deletion also
									//  (Note: For degenerate tris we have no orientation info, so we are choosing between
									//         potentially leaving 'cracks' in solid regions or 'spikes' in empty regions)
									if (MeshIdx != 0 || Normal.IsZero())
									{
										KeepTri[MeshIdx][TID] = false;
										return;
									}
									else // for the first mesh, & with a valid normal, logic depends on orientation of matching tri
									{
										bool bKeep = DotNormals > 0 == bCoplanarKeepSameDir;
										KeepTri[MeshIdx][TID] = bKeep;
										if (NumMeshesToProcess > 1 && bKeep)
										{
											// If we kept this tri, remember the coplanar pair we expect to be deleted, in case
											// it isn't deleted (e.g. because it wasn't coplanar); to then delete this one instead.
											// This can help clean up sliver triangles near a cut boundary that look locally coplanar
											DeleteIfOtherKept[TID] = OtherTID;
										}
										return;
									}
								}
							}
						}
					}

					// didn't already return a coplanar result; use the winding number
					KeepTri[MeshIdx][TID] = (UseInsideTest.IsPointInsideFn(Centroid, InsideTestContext)) != bRemoveInside;
				});
		}

		// Don't keep coplanar tris if the matched, second-mesh tri that we expected to delete was actually kept
		if (NumMeshesToProcess > 1)
		{
			for (int TID : CutMesh[0]->TriangleIndicesItr())
			{
				int32 DeleteIfOtherKeptTID = DeleteIfOtherKept[TID];
				if (DeleteIfOtherKeptTID > -1 && KeepTri[1][DeleteIfOtherKeptTID])
				{
					KeepTri[0][TID] = false;
				}
			}
		}

		for (int MeshIdx = 0; MeshIdx < NumMeshesToProcess; MeshIdx++)
		{
			FDynamicMesh3& ProcessMesh = *CutMesh[MeshIdx];
			for (int EID : ProcessMesh.EdgeIndicesItr())
			{
				FDynamicMesh3::FEdge Edge = ProcessMesh.GetEdge(EID);
				if (Edge.Tri.B == IndexConstants::InvalidID || KeepTri[MeshIdx][Edge.Tri.A] == KeepTri[MeshIdx][Edge.Tri.B])
				{
					continue;
				}

				CutBoundaryEdges[MeshIdx].Add(EID);
				PossUnmatchedBdryVerts[MeshIdx].Add(Edge.Vert.A);
				PossUnmatchedBdryVerts[MeshIdx].Add(Edge.Vert.B);
			}
		}
		// now go ahead and delete from both meshes
		bool bRegroupInsteadOfDelete = Operation == EBooleanOp::NewGroupInside || Operation == EBooleanOp::NewGroupOutside;
		int NewGroupID = -1;
		TArray<int> NewGroupTris;
		if (bRegroupInsteadOfDelete)
		{
			ensure(NumMeshesToProcess == 1);
			NewGroupID = CutMesh[0]->AllocateTriangleGroup();
		}
		for (int MeshIdx = 0; MeshIdx < NumMeshesToProcess; MeshIdx++)
		{
			FDynamicMesh3& ProcessMesh = *CutMesh[MeshIdx];

			for (int TID = 0; TID < KeepTri[MeshIdx].Num(); TID++)
			{
				if (ProcessMesh.IsTriangle(TID) && !KeepTri[MeshIdx][TID])
				{
					if (bRegroupInsteadOfDelete)
					{
						ProcessMesh.SetTriangleGroup(TID, NewGroupID);
						NewGroupTris.Add(TID);
					}
					else
					{
						ProcessMesh.RemoveTriangle(TID, true, false);
					}
				}
			}
		}
		if (bRegroupInsteadOfDelete)
		{
			// the new triangle group could include disconnected components; best to give them separate triangle groups
			FMeshConnectedComponents Components(CutMesh[0]);
			Components.FindConnectedTriangles(NewGroupTris);
			for (int ComponentIdx = 1; ComponentIdx < Components.Num(); ComponentIdx++)
			{
				int SplitGroupID = CutMesh[0]->AllocateTriangleGroup();
				for (int TID : Components.GetComponent(ComponentIdx).Indices)
				{
					CutMesh[0]->SetTriangleGroup(TID, SplitGroupID);
				}
			}
		}
	}

	if (Cancelled())
	{
		return false;
	}

	// correspond vertices across both meshes (in cases where both meshes were processed)
	TMap<int, int> AllVIDMatches; // mapping of matched vertex IDs from cutmesh 0 to cutmesh 1
	if (NumMeshesToProcess == 2)
	{
		TMap<int, int> FoundMatchesMaps[2]; // mappings of matched vertex IDs from mesh 1->0 and mesh 0->1
		double SnapToleranceSq = SnapTolerance * SnapTolerance;

		// ensure segments that are now on boundaries have 1:1 vertex correspondence across meshes
		for (int MeshIdx = 0; MeshIdx < 2; MeshIdx++)
		{
			int OtherMeshIdx = 1 - MeshIdx;
			FDynamicMesh3& OtherMesh = *CutMesh[OtherMeshIdx];

			TPointHashGrid3d<int> OtherMeshPointHash(OtherMesh.GetBounds(true).MaxDim() / 64, -1);
			for (int BoundaryVID : PossUnmatchedBdryVerts[OtherMeshIdx])
			{
				OtherMeshPointHash.InsertPointUnsafe(BoundaryVID, OtherMesh.GetVertex(BoundaryVID));
			}

			FSparseDynamicOctree3 EdgeOctree;
			EdgeOctree.RootDimension = .25;
			EdgeOctree.SetMaxTreeDepth(7);
			auto EdgeBounds = [&OtherMesh](int EID)
			{
				FDynamicMesh3::FEdge Edge = OtherMesh.GetEdge(EID);
				FVector3d A = OtherMesh.GetVertex(Edge.Vert.A);
				FVector3d B = OtherMesh.GetVertex(Edge.Vert.B);
				if (A.X > B.X)
				{
					Swap(A.X, B.X);
				}
				if (A.Y > B.Y)
				{
					Swap(A.Y, B.Y);
				}
				if (A.Z > B.Z)
				{
					Swap(A.Z, B.Z);
				}
				return FAxisAlignedBox3d(A, B);
			};
			auto AddEdge = [&EdgeOctree, &OtherMesh, EdgeBounds](int EID)
			{
				EdgeOctree.InsertObject(EID, EdgeBounds(EID));
			};
			auto UpdateEdge = [&EdgeOctree, &OtherMesh, EdgeBounds](int EID)
			{
				EdgeOctree.ReinsertObject(EID, EdgeBounds(EID));
			};
			for (int EID : CutBoundaryEdges[OtherMeshIdx])
			{
				AddEdge(EID);
			}
			TArray<int> EdgesInRange;


			// mapping from OtherMesh VIDs to ProcessMesh VIDs
			// used to ensure we only keep the best match, in cases where multiple boundary vertices map to a given vertex on the other mesh boundary
			TMap<int, int>& FoundMatches = FoundMatchesMaps[MeshIdx];

			for (int BoundaryVID : PossUnmatchedBdryVerts[MeshIdx])
			{
				if (MeshIdx == 1 && FoundMatchesMaps[0].Contains(BoundaryVID))
				{
					continue; // was already snapped to a vertex
				}

				FVector3d Pos = CutMesh[MeshIdx]->GetVertex(BoundaryVID);
				TPair<int, double> VIDDist = OtherMeshPointHash.FindNearestInRadius(Pos, SnapTolerance, [&Pos, &OtherMesh](int VID)
					{
						return DistanceSquared(Pos, OtherMesh.GetVertex(VID));
					});
				int NearestVID = VIDDist.Key; // ID of nearest vertex on other mesh
				double DSq = VIDDist.Value;   // square distance to that vertex

				if (NearestVID != FDynamicMesh3::InvalidID)
				{

					int* Match = FoundMatches.Find(NearestVID);
					if (Match)
					{
						double OldDSq = DistanceSquared(CutMesh[MeshIdx]->GetVertex(*Match), OtherMesh.GetVertex(NearestVID));
						if (DSq < OldDSq) // new vertex is a better match than the old one
						{
							int OldVID = *Match; // copy old VID out of match before updating the TMap
							FoundMatches.Add(NearestVID, BoundaryVID); // new VID is recorded as best match

							// old VID is swapped in as the one to consider as unmatched
							// it will now be matched below
							BoundaryVID = OldVID;
							Pos = CutMesh[MeshIdx]->GetVertex(BoundaryVID);
							DSq = OldDSq;
						}
						NearestVID = FDynamicMesh3::InvalidID; // one of these vertices will be unmatched
					}
					else
					{
						FoundMatches.Add(NearestVID, BoundaryVID);
					}
				}

				// if we didn't find a valid match, try to split the nearest edge to create a match
				if (NearestVID == FDynamicMesh3::InvalidID)
				{
					// vertex had no match -- try to split edge to match it
					FAxisAlignedBox3d QueryBox(Pos, SnapTolerance);
					EdgesInRange.Reset();
					EdgeOctree.RangeQuery(QueryBox, EdgesInRange);

					int OtherEID = FindNearestEdge(OtherMesh, EdgesInRange, Pos);
					if (OtherEID != FDynamicMesh3::InvalidID)
					{
						FVector3d EdgePts[2];
						OtherMesh.GetEdgeV(OtherEID, EdgePts[0], EdgePts[1]);
						// only accept the match if it's not going to create a degenerate edge -- TODO: filter already-matched edges from the FindNearestEdge query!
						if (DistanceSquared(EdgePts[0], Pos) > SnapToleranceSq&& DistanceSquared(EdgePts[1], Pos) > SnapToleranceSq)
						{
							FSegment3d Seg(EdgePts[0], EdgePts[1]);
							double Along = Seg.ProjectUnitRange(Pos);
							FDynamicMesh3::FEdgeSplitInfo SplitInfo;
							if (ensure(EMeshResult::Ok == OtherMesh.SplitEdge(OtherEID, SplitInfo, Along)))
							{
								FoundMatches.Add(SplitInfo.NewVertex, BoundaryVID);
								OtherMesh.SetVertex(SplitInfo.NewVertex, Pos);
								CutBoundaryEdges[OtherMeshIdx].Add(SplitInfo.NewEdges.A);
								UpdateEdge(OtherEID);
								AddEdge(SplitInfo.NewEdges.A);
								// Note: Do not update PossUnmatchedBdryVerts with the new vertex, because it is already matched by construction
								// Likewise do not update the pointhash -- we don't want it to find vertices that were already perfectly matched
							}
						}
					}
				}
			}

			// actually snap the positions together for final matches
			for (TPair<int, int>& Match : FoundMatches)
			{
				CutMesh[MeshIdx]->SetVertex(Match.Value, OtherMesh.GetVertex(Match.Key));

				// Copy match to AllVIDMatches; note this is always mapping from CutMesh 0 to 1
				int VIDs[2]{ Match.Key, Match.Value }; // just so we can access by index
				AllVIDMatches.Add(VIDs[1 - MeshIdx], VIDs[MeshIdx]);
			}
		}
	}

	if (bSimplifyAlongNewEdges)
	{
		SimplifyAlongNewEdges(NumMeshesToProcess, CutMesh, CutBoundaryEdges, AllVIDMatches);
	}

	if (Operation == EBooleanOp::Difference)
	{
		// TODO: implement a way to flip all the triangles in the mesh without building this AllTID array
		TArray<int> AllTID;
		for (int TID : CutMesh[1]->TriangleIndicesItr())
		{
			AllTID.Add(TID);
		}
		FDynamicMeshEditor FlipEditor(CutMesh[1]);
		FlipEditor.ReverseTriangleOrientations(AllTID, true);
	}

	if (Cancelled())
	{
		return false;
	}

	bool bSuccess = true;

	if (NumMeshesToProcess > 1)
	{
		FDynamicMeshEditor Editor(Result);
		FMeshIndexMappings IndexMaps;
		Editor.AppendMesh(CutMesh[1], IndexMaps);

		if (TrackPerTriangleSourceMesh.IsSet())
		{
			TrackPerTriangleSourceMesh->SetNumZeroed(Result->MaxTriangleID());
			for (const TPair<int32, int32> TriSrcDst : IndexMaps.GetTriangleMap().GetForwardMap())
			{
				(*TrackPerTriangleSourceMesh)[TriSrcDst.Value] = 1;
			}
		}

		if (bPopulateSecondMeshGroupMap)
		{
			SecondMeshGroupMap = IndexMaps.GetGroupMap();
		}

		if (bWeldSharedEdges)
		{
			bool bWeldSuccess = MergeEdges(IndexMaps, CutMesh, CutBoundaryEdges, AllVIDMatches);
			bSuccess = bSuccess && bWeldSuccess;
		}
		else
		{
			CreatedBoundaryEdges = CutBoundaryEdges[0];
			for (int OldMeshEID : CutBoundaryEdges[1])
			{
				if (!CutMesh[1]->IsEdge(OldMeshEID))
				{
					ensure(false);
					continue;
				}
				FIndex2i OtherEV = CutMesh[1]->GetEdgeV(OldMeshEID);
				int MappedEID = Result->FindEdge(IndexMaps.GetNewVertex(OtherEV.A), IndexMaps.GetNewVertex(OtherEV.B));
				checkSlow(Result->IsBoundaryEdge(MappedEID));
				CreatedBoundaryEdges.Add(MappedEID);
			}
		}
	}
	// For NewGroupInside and NewGroupOutside, the cut doesn't create boundary edges.
	else if (Operation != EBooleanOp::NewGroupInside && Operation != EBooleanOp::NewGroupOutside)
	{
		CreatedBoundaryEdges = CutBoundaryEdges[0];
	}

	if (bTrackAllNewEdges)
	{
		for (int32 eid : CreatedBoundaryEdges)
		{
			AllNewEdges.Add(eid);
		}
	}

	if (bPutResultInInputSpace)
	{
		MeshTransforms::ApplyTransform(*Result, ResultTransform);
		ResultTransform = FTransformSRT3d::Identity();
	}

	return bSuccess;
}



void FMeshBoolean::SimplifyAlongNewEdges(int NumMeshesToProcess, FDynamicMesh3* CutMesh[2], TArray<int> CutBoundaryEdges[2], TMap<int, int>& AllVIDMatches)
{
	double DotTolerance = FMathd::Cos(SimplificationAngleTolerance * FMathd::DegToRad);

	TSet<int> CutBoundaryEdgeSets[2]; // set versions of CutBoundaryEdges, for faster membership tests
	for (int MeshIdx = 0; MeshIdx < NumMeshesToProcess; MeshIdx++)
	{
		CutBoundaryEdgeSets[MeshIdx].Append(CutBoundaryEdges[MeshIdx]);
	}

	int NumCollapses = 0, CollapseIters = 0;
	int MaxCollapseIters = 1; // TODO: is there a case where we need more iterations?  Perhaps if we add some triangle quality criteria?
	while (CollapseIters < MaxCollapseIters)
	{
		int LastNumCollapses = NumCollapses;
		for (int EID : CutBoundaryEdges[0])
		{
			// this can happen if a collapse removes another cut boundary edge
			// (which can happen e.g. if you have a degenerate (colinear) tri flat on the cut boundary)
			if (!CutMesh[0]->IsEdge(EID))
			{
				continue;
			}
			// don't allow collapses if we somehow get down to our last triangle on either mesh
			if (CutMesh[0]->TriangleCount() <= 1 || (NumMeshesToProcess == 2 && CutMesh[1]->TriangleCount() <= 1))
			{
				break;
			}

			FDynamicMesh3::FEdge Edge = CutMesh[0]->GetEdge(EID);
			int Matches[2]{ -1, -1 };
			bool bHasMatches = NumMeshesToProcess == 2;
			if (bHasMatches)
			{
				for (int MatchIdx = 0; MatchIdx < 2; MatchIdx++)
				{
					int* Match = AllVIDMatches.Find(Edge.Vert[MatchIdx]);
					if (Match)
					{
						Matches[MatchIdx] = *Match;
					}
					else
					{
						bHasMatches = false;
						// TODO: if we switch to allow collapse on unmatched edges, we shouldn't break here
						//        b/c we may be partially matched, and need to track which is matched.
						break;
					}
				}
				if (!bHasMatches)
				{
					continue; // edge wasn't matched up on the other mesh; can't collapse it?
					// TODO: consider supporting collapses in this case?
				}
			}
			// if we have matched vertices, we also need a matched edge to collapse
			int OtherEID = -1;
			if (bHasMatches)
			{
				OtherEID = CutMesh[1]->FindEdge(Matches[0], Matches[1]);
				if (OtherEID == -1)
				{
					continue;
				}
			}
			// track whether the neighborhood of the vertex is flat (and likewise its matched pair's neighborhood, if present)
			bool Flat[2]{ false, false };
			// normals for each flat vertex, and each "side" (mesh 0 and mesh 1, if mesh 1 is present)
			FVector3d FlatNormals[2][2]{ {FVector3d::Zero(), FVector3d::Zero()}, {FVector3d::Zero(), FVector3d::Zero()} };
			int NumFlat = 0;
			for (int VIdx = 0; VIdx < 2; VIdx++)
			{
				if (FLocalPlanarSimplify::IsFlat(*CutMesh[0], Edge.Vert[VIdx], DotTolerance, FlatNormals[VIdx][0]))
				{
					Flat[VIdx] = (Matches[VIdx] == -1) || FLocalPlanarSimplify::IsFlat(*CutMesh[1], Matches[VIdx], DotTolerance, FlatNormals[VIdx][1]);
				}

				if (Flat[VIdx])
				{
					NumFlat++;
				}
			}

			if (NumFlat == 0)
			{
				continue;
			}

			// see if we can collapse to remove either vertex
			for (int RemoveVIdx = 0; RemoveVIdx < 2; RemoveVIdx++)
			{
				if (!Flat[RemoveVIdx])
				{
					continue;
				}
				int KeepVIdx = 1 - RemoveVIdx;
				FVector3d RemoveVPos = CutMesh[0]->GetVertex(Edge.Vert[RemoveVIdx]);
				FVector3d KeepVPos = CutMesh[0]->GetVertex(Edge.Vert[KeepVIdx]);
				FVector3d EdgeDir = KeepVPos - RemoveVPos;
				if (Normalize(EdgeDir) == 0) // 0 is returned as a special case when the edge was too short to normalize
				{
					// collapsing degenerate edges above should prevent this
					ensure(!bCollapseDegenerateEdgesOnCut);
					// Just skip these edges, because in practice we generally have bCollapseDegenerateEdgesOnCut enabled
					break; // break instead of continue to skip the whole edge
				}

				bool bHasBadEdge = false; // will be set if either mesh can't collapse the edge
				for (int MeshIdx = 0; !bHasBadEdge && MeshIdx < NumMeshesToProcess; MeshIdx++)
				{
					int RemoveV = MeshIdx == 0 ? Edge.Vert[RemoveVIdx] : Matches[RemoveVIdx];
					int KeepV = MeshIdx == 0 ? Edge.Vert[KeepVIdx] : Matches[KeepVIdx];
					int SourceEID = MeshIdx == 0 ? EID : OtherEID;

					bHasBadEdge = bHasBadEdge || FLocalPlanarSimplify::CollapseWouldHurtTriangleQuality(*CutMesh[MeshIdx],
						FlatNormals[RemoveVIdx][MeshIdx], RemoveV, RemoveVPos, KeepV, KeepVPos, TryToImproveTriQualityThreshold);
					
					bHasBadEdge = bHasBadEdge || FLocalPlanarSimplify::CollapseWouldChangeShapeOrUVs(
						*CutMesh[MeshIdx], CutBoundaryEdgeSets[MeshIdx], DotTolerance,
						SourceEID, RemoveV, RemoveVPos, KeepV, KeepVPos, EdgeDir, bPreserveTriangleGroups,
						PreserveUVsOnlyForMesh == -1 || MeshIdx == PreserveUVsOnlyForMesh,
						bPreserveVertexUVs, bPreserveOverlayUVs, UVDistortTolerance * UVDistortTolerance,
						bPreserveVertexNormals, FMathf::Cos(NormalDistortTolerance * FMathf::DegToRad));
				};

				if (bHasBadEdge)
				{
					continue;
				}

				// do some pre-collapse sanity checks on the matched edge (if present) to see if it will fail to collapse
				bool bAttemptCollapse = true;
				if (bHasMatches)
				{
					int OtherRemoveV = Matches[RemoveVIdx];
					int OtherKeepV = Matches[KeepVIdx];

					int a = OtherRemoveV, b = OtherKeepV;
					int eab = CutMesh[1]->FindEdge(OtherRemoveV, OtherKeepV);

					const FDynamicMesh3::FEdge EdgeAB = CutMesh[1]->GetEdge(eab);
					int t0 = EdgeAB.Tri[0];
					if (t0 == FDynamicMesh3::InvalidID)
					{
						bAttemptCollapse = false;
					}
					else
					{
						FIndex3i T0tv = CutMesh[1]->GetTriangle(t0);
						int c = IndexUtil::FindTriOtherVtx(a, b, T0tv);
						checkSlow(EdgeAB.Tri[1] == FDynamicMesh3::InvalidID);
						// We cannot collapse if edge lists of a and b share vertices other
						//  than c and d  (because then we will make a triangle [x b b].
						//  Brute-force search logic adapted from FDynamicMesh3::CollapseEdge implementation.
						//  (simplified because we know this is a boundary edge)
						CutMesh[1]->EnumerateVertexVertices(a, [&](int VID)
						{
							if (!bAttemptCollapse || VID == c || VID == b)
							{
								return;
							}
							CutMesh[1]->EnumerateVertexVertices(b, [&](int VID2)
							{
								bAttemptCollapse &= (VID != VID2);
							});
						});
					}
				}
				if (!bAttemptCollapse)
				{
					break; // don't try starting from other vertex if the match edge couldn't be collapsed
				}

				FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
				int RemoveV = Edge.Vert[RemoveVIdx];
				int KeepV = Edge.Vert[KeepVIdx];
				// Detect the case of a triangle with two boundary edges, where collapsing 
				// the target boundary edge would keep the non-boundary edge.
				// This collapse will remove the triangle, so we add the
				// (formerly) non-boundary edge as our new boundary edge.
				auto WouldRemoveTwoBoundaryEdges = [](const FDynamicMesh3& Mesh, int EID, int RemoveV)
				{
					checkSlow(Mesh.IsEdge(EID));
					int OppV = Mesh.GetEdgeOpposingV(EID).A;
					int NextOnTri = Mesh.FindEdge(RemoveV, OppV);
					return Mesh.IsBoundaryEdge(NextOnTri);
				};
				bool bWouldRemoveNext = WouldRemoveTwoBoundaryEdges(*CutMesh[0], EID, RemoveV);
				EMeshResult CollapseResult = CutMesh[0]->CollapseEdge(KeepV, RemoveV, 0, CollapseInfo);
				if (CollapseResult == EMeshResult::Ok)
				{
					if (bWouldRemoveNext && ensure(CutMesh[0]->IsBoundaryEdge(CollapseInfo.KeptEdges.A)))
					{
						CutBoundaryEdgeSets[0].Add(CollapseInfo.KeptEdges.A);
					}

					if (bHasMatches)
					{
						int OtherRemoveV = Matches[RemoveVIdx];
						int OtherKeepV = Matches[KeepVIdx];
						bool bOtherWouldRemoveNext = WouldRemoveTwoBoundaryEdges(*CutMesh[1], OtherEID, OtherRemoveV);
						FDynamicMesh3::FEdgeCollapseInfo OtherCollapseInfo;
						EMeshResult OtherCollapseResult = CutMesh[1]->CollapseEdge(OtherKeepV, OtherRemoveV, 0, OtherCollapseInfo);
						if (OtherCollapseResult != EMeshResult::Ok)
						{
							// if we get here, we've somehow managed to collapse the first edge but failed on the second (matched) edge
							// which will leave a crack in the result unless we can somehow undo the first collapse, which would require a bunch of extra work
							// but the only case where I could see this happen is if the second edge is on an isolated triangle, which means there is a hole anyway
							// or if the mesh topology is somehow invalid
							ensureMsgf(OtherCollapseResult == EMeshResult::Failed_CollapseTriangle, TEXT("Collapse failed with result: %d"), (int)OtherCollapseResult);
						}
						else
						{
							if (bOtherWouldRemoveNext && ensure(CutMesh[1]->IsBoundaryEdge(OtherCollapseInfo.KeptEdges.A)))
							{
								CutBoundaryEdgeSets[1].Add(OtherCollapseInfo.KeptEdges.A);
							}
							AllVIDMatches.Remove(RemoveV);
							CutBoundaryEdgeSets[1].Remove(OtherCollapseInfo.CollapsedEdge);
							CutBoundaryEdgeSets[1].Remove(OtherCollapseInfo.RemovedEdges[0]);
							if (OtherCollapseInfo.RemovedEdges[1] != -1)
							{
								CutBoundaryEdgeSets[1].Remove(OtherCollapseInfo.RemovedEdges[1]);
							}
						}
					}

					NumCollapses++;
					CutBoundaryEdgeSets[0].Remove(CollapseInfo.CollapsedEdge);
					CutBoundaryEdgeSets[0].Remove(CollapseInfo.RemovedEdges[0]);
					if (CollapseInfo.RemovedEdges[1] != -1)
					{
						CutBoundaryEdgeSets[0].Remove(CollapseInfo.RemovedEdges[1]);
					}
				}
				break; // if we got through to trying to collapse the edge, don't try to collapse from the other vertex.
			}
		}

		CutBoundaryEdges[0] = CutBoundaryEdgeSets[0].Array();
		CutBoundaryEdges[1] = CutBoundaryEdgeSets[1].Array();

		if (NumCollapses == LastNumCollapses)
		{
			break;
		}

		CollapseIters++;
	}
}


bool FMeshBoolean::MergeEdges(const FMeshIndexMappings& IndexMaps, FDynamicMesh3* CutMesh[2], const TArray<int> CutBoundaryEdges[2], const TMap<int, int>& AllVIDMatches)
{
	// translate the edge IDs from CutMesh[1] over to Result mesh edge IDs
	TArray<int> OtherMeshEdges;
	for (int OldMeshEID : CutBoundaryEdges[1])
	{
		if (!ensure(CutMesh[1]->IsEdge(OldMeshEID)))
		{
			continue;
		}
		FIndex2i OtherEV = CutMesh[1]->GetEdgeV(OldMeshEID);
		int MappedEID = Result->FindEdge(IndexMaps.GetNewVertex(OtherEV.A), IndexMaps.GetNewVertex(OtherEV.B));
		if (ensure(Result->IsBoundaryEdge(MappedEID)))
		{
			OtherMeshEdges.Add(MappedEID);
		}
	}

	// find "easy" match candidates using the already-made vertex correspondence
	TArray<FIndex2i> CandidateMatches;
	TArray<int> UnmatchedEdges;
	for (int EID : CutBoundaryEdges[0])
	{
		if (!ensure(Result->IsBoundaryEdge(EID)))
		{
			continue;
		}
		FIndex2i VIDs = Result->GetEdgeV(EID);
		const int* OtherA = AllVIDMatches.Find(VIDs.A);
		const int* OtherB = AllVIDMatches.Find(VIDs.B);
		bool bAddedCandidate = false;
		if (OtherA && OtherB)
		{
			int MapOtherA = IndexMaps.GetNewVertex(*OtherA);
			int MapOtherB = IndexMaps.GetNewVertex(*OtherB);
			int OtherEID = Result->FindEdge(MapOtherA, MapOtherB);
			if (OtherEID != FDynamicMesh3::InvalidID)
			{
				CandidateMatches.Add(FIndex2i(EID, OtherEID));
				bAddedCandidate = true;
			}
		}
		if (!bAddedCandidate)
		{
			UnmatchedEdges.Add(EID);
		}
	}

	// merge the easy matches
	for (FIndex2i Candidate : CandidateMatches)
	{
		if (!Result->IsEdge(Candidate.A) || !Result->IsBoundaryEdge(Candidate.A))
		{
			continue;
		}

		FDynamicMesh3::FMergeEdgesInfo MergeInfo;
		EMeshResult EdgeMergeResult = Result->MergeEdges(Candidate.A, Candidate.B, MergeInfo);
		if (EdgeMergeResult != EMeshResult::Ok)
		{
			UnmatchedEdges.Add(Candidate.A);
		}
		else
		{
			if (bTrackAllNewEdges)
			{
				AllNewEdges.Add(Candidate.A);
			}
		}
	}

	// filter matched edges from the edge array for the other mesh
	OtherMeshEdges.SetNum(Algo::RemoveIf(OtherMeshEdges, [this](int EID)
		{
			return !Result->IsEdge(EID) || !Result->IsBoundaryEdge(EID);
		}));

	// see if we can match anything else
	bool bAllMatched = true;
	if (UnmatchedEdges.Num() > 0)
	{
		// greedily match within snap tolerance
		double SnapToleranceSq = SnapTolerance * SnapTolerance;
		for (int OtherEID : OtherMeshEdges)
		{
			if (!Result->IsEdge(OtherEID) || !Result->IsBoundaryEdge(OtherEID))
			{
				continue;
			}
			FVector3d OA, OB;
			Result->GetEdgeV(OtherEID, OA, OB);
			for (int UnmatchedIdx = 0; UnmatchedIdx < UnmatchedEdges.Num(); UnmatchedIdx++)
			{
				int EID = UnmatchedEdges[UnmatchedIdx];
				if (!Result->IsEdge(EID) || !Result->IsBoundaryEdge(EID))
				{
					UnmatchedEdges.RemoveAtSwap(UnmatchedIdx, EAllowShrinking::No);
					UnmatchedIdx--;
					continue;
				}
				FVector3d A, B;
				Result->GetEdgeV(EID, A, B);
				if (DistanceSquared(OA, A) < SnapToleranceSq && DistanceSquared(OB, B) < SnapToleranceSq)
				{
					FDynamicMesh3::FMergeEdgesInfo MergeInfo;
					EMeshResult EdgeMergeResult = Result->MergeEdges(EID, OtherEID, MergeInfo);
					if (EdgeMergeResult == EMeshResult::Ok)
					{
						UnmatchedEdges.RemoveAtSwap(UnmatchedIdx, EAllowShrinking::No);
						if (bTrackAllNewEdges)
						{
							AllNewEdges.Add(EID);
						}
						break;
					}
				}
			}
		}

		// store the failure cases from the first mesh's array
		for (int EID : UnmatchedEdges)
		{
			if (Result->IsEdge(EID) && Result->IsBoundaryEdge(EID))
			{
				CreatedBoundaryEdges.Add(EID);
				bAllMatched = false;
			}
		}
	}
	// store the failure cases from the second mesh's array
	for (int OtherEID : OtherMeshEdges)
	{
		if (Result->IsEdge(OtherEID) && Result->IsBoundaryEdge(OtherEID))
		{
			CreatedBoundaryEdges.Add(OtherEID);
			bAllMatched = false;
		}
	}
	return bAllMatched;
}


int FMeshBoolean::FindNearestEdge(const FDynamicMesh3& OnMesh, const TArray<int>& EIDs, FVector3d Pos)
{
	int NearEID = FDynamicMesh3::InvalidID;
	double NearSqr = SnapTolerance * SnapTolerance;
	FVector3d EdgePts[2];
	for (int EID : EIDs) {
		OnMesh.GetEdgeV(EID, EdgePts[0], EdgePts[1]);

		FSegment3d Seg(EdgePts[0], EdgePts[1]);
		double DSqr = Seg.DistanceSquared(Pos);
		if (DSqr < NearSqr)
		{
			NearEID = EID;
			NearSqr = DSqr;
		}
	}
	return NearEID;
}
