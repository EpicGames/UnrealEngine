// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshMeshCut

#include "Operations/MeshMeshCut.h"
#include "Operations/EmbedSurfacePath.h"

using namespace UE::Geometry;

namespace MeshCut
{
	enum class EVertexType
	{
		Unknown = -1,
		Vertex = 0,
		Edge = 1,
		Face = 2
	};

	/**
	 * An intersection point + where it maps to on the surface
	 */
	struct FPtOnMesh
	{
		FVector3d Pos;
		EVertexType Type = EVertexType::Unknown;
		int ElemID = IndexConstants::InvalidID;
	};

	/**
	 * Mapping from intersection segments to source triangle and intersection points
	 */
	struct FSegmentToElements
	{
		int BaseTID; // triangle ID that the segment was on *before* mesh was cut
		int PtOnMeshIdx[2]; // indices into IntersectionVerts array for the two endpoints of the segment 
	};

	/**
	 * Per mesh info about an in-progress cut.  Only stored temporarily while cut is performed; not retained
	 */
	struct FCutWorkingInfo
	{
		FCutWorkingInfo(FDynamicMesh3* WorkingMesh, double SnapTolerance) : Mesh(WorkingMesh), SnapToleranceSq(SnapTolerance*SnapTolerance)
		{
			Init(WorkingMesh);
		}

		// get the direction of the first non-degenerate edge (or a default vector if all edges were degenerate)
		static FVector3d GetDegenTriangleEdgeDirection(const FDynamicMesh3* Mesh, int TID, FVector3d DefaultDir = FVector3d::ZAxisVector)
		{ 
			FVector3d V[3];
			Mesh->GetTriVertices(TID, V[0], V[1], V[2]);
			for (int Prev = 2, Idx = 0; Idx < 3; Prev = Idx++)
			{
				FVector3d E = V[Idx] - V[Prev];
				if (E.Normalize())
				{
					return E;
				}
			}
			return DefaultDir;
		}

		void Init(FDynamicMesh3* WorkingMesh)
		{
			BaseFaceNormals.SetNumUninitialized(WorkingMesh->MaxTriangleID());
			double area = 0;
			for (int TID : WorkingMesh->TriangleIndicesItr())
			{
				BaseFaceNormals[TID] = WorkingMesh->GetTriNormal(TID);
			}

			FaceVertices.Reset();
			EdgeVertices.Reset();
			IntersectionVerts.Reset();
			Segments.Reset();
		}

		// mesh to operate on
		FDynamicMesh3* Mesh;

		// snapping tolerance squared
		double SnapToleranceSq;


		// triangle ID -> IntersectionVerts index for intersection pts that we still have to insert
		TMultiMap<int, int> FaceVertices;

		// edge ID -> IntersectionVerts index for intersection pts that we still have to insert
		TMultiMap<int, int> EdgeVertices;

		// Normals of original triangles (before cut is performed)
		// only needed for walking mesh, in more-expensive fallback case that just inserting segment vertices doesn't make a connected path
		TArray<FVector3d> BaseFaceNormals;

		// points on the mesh -- after cut, these will all correspond to mesh vertices
		TArray<FPtOnMesh> IntersectionVerts;

		// Stores the mapping of intersection segments to original mesh triangles and IntersectionVerts
		TArray<FSegmentToElements> Segments;


		void AddSegments(const MeshIntersection::FIntersectionsQueryResult& Intersections, int WhichSide)
		{
			int SegStart = Segments.Num();
			Segments.SetNum(SegStart + Intersections.Segments.Num());
			
			// classify the points of each intersection segment as on-vertex, on-edge, or on-face
			for (int SegIdx = 0, SegCount = Intersections.Segments.Num(); SegIdx < SegCount; SegIdx++)
			{
				const MeshIntersection::FSegmentIntersection& Seg = Intersections.Segments[SegIdx];
				FSegmentToElements& SegToEls = Segments[SegStart+SegIdx];
				SegToEls.BaseTID = Seg.TriangleID[WhichSide];

				FTriangle3d Tri;
				Mesh->GetTriVertices(SegToEls.BaseTID, Tri.V[0], Tri.V[1], Tri.V[2]);
				FIndex3i TriVIDs = Mesh->GetTriangle(SegToEls.BaseTID);
				int PrevOnEdgeIdx = -1;
				FVector3d PrevOnEdgePos;
				for (int SegPtIdx = 0; SegPtIdx < 2; SegPtIdx++)
				{
					int NewPtIdx = IntersectionVerts.Num();
					FPtOnMesh& PtOnMesh = IntersectionVerts.Emplace_GetRef();
					PtOnMesh.Pos = Seg.Point[SegPtIdx];
					SegToEls.PtOnMeshIdx[SegPtIdx] = NewPtIdx;

					// decide whether the point is on a vertex, edge or triangle

					int OnVertexIdx = OnVertex(Tri, PtOnMesh.Pos);
					if (OnVertexIdx > -1)
					{
						PtOnMesh.Type = EVertexType::Vertex;
						PtOnMesh.ElemID = TriVIDs[OnVertexIdx];
						continue;
					}

					// check for an edge match
					int OnEdgeIdx = OnEdge(Tri, PtOnMesh.Pos);
					if (OnEdgeIdx > -1)
					{
						// if segment is degenerate and stuck to one edge, see if it could cross
						if (PrevOnEdgeIdx == OnEdgeIdx && FVector3d::DistSquared(PrevOnEdgePos, PtOnMesh.Pos) < SnapToleranceSq)
						{
							int OnEdgeReplaceIdx = OnEdgeWithSkip(Tri, PtOnMesh.Pos, OnEdgeIdx);
							if (OnEdgeReplaceIdx > -1)
							{
								OnEdgeIdx = OnEdgeReplaceIdx;
							}
						}
						PtOnMesh.Type = EVertexType::Edge;
						PtOnMesh.ElemID = Mesh->GetTriEdge(SegToEls.BaseTID, OnEdgeIdx);

						check(PtOnMesh.ElemID > -1);
						EdgeVertices.Add(PtOnMesh.ElemID, NewPtIdx);

						PrevOnEdgeIdx = OnEdgeIdx;
						PrevOnEdgePos = PtOnMesh.Pos;

						continue;
					}

					// wasn't vertex or edge, so it's a face vertex
					PtOnMesh.Type = EVertexType::Face;
					PtOnMesh.ElemID = SegToEls.BaseTID;
					FaceVertices.Add(PtOnMesh.ElemID, NewPtIdx);
				}
			}
		}

		void InsertFaceVertices()
		{
			FTriangle3d Tri;
			TArray<int> PtIndices;

			while (FaceVertices.Num() > 0)
			{
				int TID = -1, PtIdx = -1;
				// use an iterator to get a single element
				{
					const auto TIDToPtIdxItr = FaceVertices.CreateConstIterator();
					TID = TIDToPtIdxItr.Key();
					PtIdx = TIDToPtIdxItr.Value();
				}
				PtIndices.Reset();
				FaceVertices.MultiFind(TID, PtIndices);

				FPtOnMesh& Pt = IntersectionVerts[PtIdx];

				Mesh->GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
				// TODO: I think it should be ok to use this fn for BarycentricCoords even though it's not robust to degenerate triangles
				//       because in degenerate tri cases we would have snapped the point to a vertex or edge.
				//       This won't be the case however if vertex or edge snapping tolerances are too low!
				FVector3d BaryCoords = VectorUtil::BarycentricCoords(Pt.Pos, Tri.V[0], Tri.V[1], Tri.V[2]);
				DynamicMeshInfo::FPokeTriangleInfo PokeInfo;
				EMeshResult Result = Mesh->PokeTriangle(TID, BaryCoords, PokeInfo);
				checkf(Result == EMeshResult::Ok, TEXT("Failed to add vertex on Triangle ID %d"), TID); // should never happen unless TID is invalid?
				int PokeVID = PokeInfo.NewVertex;
				// set vertex position to intersection pos (even though it should already be close) so that new vertices on both meshes have matching positions
				Mesh->SetVertex(PokeVID, Pt.Pos);
				//WorkingInfo[MeshIdx].AddPokeSubFaces(PokeInfo); // TODO: add this back if we have a reason to track subface; otherwise delete it
				Pt.ElemID = PokeVID;
				Pt.Type = EVertexType::Vertex;

				FaceVertices.Remove(TID);
				FIndex3i PokeTriangles(TID, PokeInfo.NewTriangles.A, PokeInfo.NewTriangles.B);
				// if there were other points on the face, redistribute them among the newly created triangles
				if (PtIndices.Num() > 1)
				{
					for (int RelocatePtIdx : PtIndices)
					{
						if (PtIdx == RelocatePtIdx) // skip the pt we've already handled
						{
							continue;
						}

						FPtOnMesh& RelocatePt = IntersectionVerts[RelocatePtIdx];
						UpdateFromPoke(RelocatePt, PokeInfo.NewVertex, PokeInfo.NewEdges, PokeTriangles);
						if (RelocatePt.Type == EVertexType::Edge)
						{
							checkSlow(RelocatePt.ElemID > -1);
							EdgeVertices.Add(RelocatePt.ElemID, RelocatePtIdx);
						}
						else if (RelocatePt.Type == EVertexType::Face)
						{
							checkSlow(RelocatePt.ElemID > -1);
							FaceVertices.Add(RelocatePt.ElemID, RelocatePtIdx);
						}
					}
				}
			}
		}

		void InsertEdgeVertices()
		{
			TArray<int> PtIndices;
				
			while (EdgeVertices.Num() > 0)
			{
				int EID = -1, PtIdx = -1;
				// use an iterator to get a single element
				{
					const auto EIDToPtIdxItr = EdgeVertices.CreateConstIterator();
					EID = EIDToPtIdxItr.Key();
					PtIdx = EIDToPtIdxItr.Value();
				}
				PtIndices.Reset();
				EdgeVertices.MultiFind(EID, PtIndices);

				FPtOnMesh& Pt = IntersectionVerts[PtIdx];

				FVector3d EA, EB;
				Mesh->GetEdgeV(EID, EA, EB);
				FSegment3d Seg(EA, EB);
				double SplitParam = Seg.ProjectUnitRange(Pt.Pos);

				FIndex2i SplitTris = Mesh->GetEdgeT(EID);
				DynamicMeshInfo::FEdgeSplitInfo SplitInfo;
				EMeshResult Result = Mesh->SplitEdge(EID, SplitInfo, SplitParam);
				checkf(Result == EMeshResult::Ok, TEXT("Failed to add vertex on Edge ID %d"), EID); // should never happen unless EID is invalid or the mesh has broken topology?

				Mesh->SetVertex(SplitInfo.NewVertex, Pt.Pos);
				//WorkingInfo[MeshIdx].AddSplitSubfaces(SplitInfo);  // TODO: add this back if we have a reason to track subfaces; otherwise delete it
				Pt.ElemID = SplitInfo.NewVertex;
				Pt.Type = EVertexType::Vertex;

				EdgeVertices.Remove(EID);
				// if there were other points on the edge, redistribute them to the newly created edges
				if (PtIndices.Num() > 1)
				{
					FIndex2i SplitEdges{ SplitInfo.OriginalEdge, SplitInfo.NewEdges.A };
					for (int RelocatePtIdx : PtIndices)
					{
						if (PtIdx == RelocatePtIdx)
						{
							continue;
						}

						FPtOnMesh& RelocatePt = IntersectionVerts[RelocatePtIdx];
						UpdateFromSplit(RelocatePt, SplitInfo.NewVertex, SplitEdges);
						if (RelocatePt.Type == EVertexType::Edge)
						{
							checkSlow(RelocatePt.ElemID > -1);
							EdgeVertices.Add(RelocatePt.ElemID, RelocatePtIdx);
						}
					}
				}
			}
		}

		/**
		 * @param VertexChains Optional packed storage of chains of vertices inserted by mesh cutting, to allow downstream to track what was cut
		 * @param SegmentToChain Optional mapping of VertexChains to the segments that created them; useful for corresponding insertions across meshes
		 */
		bool ConnectEdges(TArray<int> *VertexChains = nullptr, TArray<int> *SegmentToChain = nullptr)
		{
			TArray<int> EmbeddedPath;

			bool bSuccess = true; // remains true if we successfully connect all edges

			checkf(VertexChains || !SegmentToChain, TEXT("If SegmentToChain isn't null, VertexChains must not be null"));

			if (SegmentToChain)
			{
				SegmentToChain->SetNumUninitialized(Segments.Num());
				for (int& ChainIdx : *SegmentToChain)
				{
					ChainIdx = IndexConstants::InvalidID;
				}
			}

			for (int SegIdx = 0, NumSegs = Segments.Num(); SegIdx < NumSegs; SegIdx++)
			{
				FSegmentToElements& Seg = Segments[SegIdx];
				if (Seg.PtOnMeshIdx[0] == Seg.PtOnMeshIdx[1])
				{
					continue; // degenerate case, but OK
				}
				FPtOnMesh& PtA = IntersectionVerts[Seg.PtOnMeshIdx[0]];
				FPtOnMesh& PtB = IntersectionVerts[Seg.PtOnMeshIdx[1]];
				if (!ensureMsgf(
					PtA.Type == EVertexType::Vertex
					&& PtB.Type == EVertexType::Vertex
					&& PtA.ElemID != IndexConstants::InvalidID
					&& PtB.ElemID != IndexConstants::InvalidID, TEXT("Point insertion failed during mesh mesh cut!")))
				{
					bSuccess = false;
					continue; // shouldn't happen!
				}
				if (PtA.ElemID == PtB.ElemID)
				{
					if (VertexChains)
					{
						if (SegmentToChain)
						{
							(*SegmentToChain)[SegIdx] = VertexChains->Num();
						}
						VertexChains->Add(1);
						VertexChains->Add(PtA.ElemID);
					}
					continue; // degenerate case, but OK
				}


				int EID = Mesh->FindEdge(PtA.ElemID, PtB.ElemID);
				if (EID != FDynamicMesh3::InvalidID)
				{
					if (VertexChains)
					{
						if (SegmentToChain)
						{
							(*SegmentToChain)[SegIdx] = VertexChains->Num();
						}
						VertexChains->Add(2);
						VertexChains->Add(PtA.ElemID);
						VertexChains->Add(PtB.ElemID);
					}
					continue; // already connected
				}

				FMeshSurfacePath SurfacePath(Mesh);
				int StartTID = Mesh->GetVtxSingleTriangle(PtA.ElemID); // TODO: would be faster to have a PlanarWalk call that takes a start vertex ID !
				FVector3d WalkPlaneNormal = BaseFaceNormals[Seg.BaseTID].Cross(PtB.Pos - PtA.Pos);
				if (Normalize(WalkPlaneNormal) == 0)
				{
					if (FVector3d::DistSquared(PtA.Pos, PtB.Pos) > SnapToleranceSq)
					{
						// path points are separated, expect degeneracy to come from colinear triangle vertices
						// this implies vertices are spread along the original edges, which would already be connected
						// so we shouldn't need to do any additional work to connect things
						continue;
					}
					// Path points are not separated; we may need to connect across a (likely degenerate) triangle
					// Use a walk normal that can separate the triangle vertices (even if the triangle's collapsed to a line segment)
					WalkPlaneNormal = GetDegenTriangleEdgeDirection(Mesh, StartTID);
					if (!ensure(Normalize(WalkPlaneNormal) > 0))
					{
						// there was no non-degenerate edge; triangle is a point, nothing to walk here
						continue;
					}
				}
				bool bWalkSuccess = SurfacePath.AddViaPlanarWalk(StartTID, PtA.ElemID,
					Mesh->GetVertex(PtA.ElemID), -1, PtB.ElemID,
					Mesh->GetVertex(PtB.ElemID), WalkPlaneNormal, nullptr /*TODO: transform fn goes here?*/, false, FMathd::ZeroTolerance, SnapToleranceSq, .001);
				if (!bWalkSuccess)
				{
					bSuccess = false;
				}
				else
				{
					EmbeddedPath.Reset();
					if (SurfacePath.EmbedSimplePath(false, EmbeddedPath, false, SnapToleranceSq, FMeshSurfacePath::FEmbedSimplePathSettings::WithSimplification()))
					{
						ensure(EmbeddedPath.Num() > 0 && EmbeddedPath[0] == PtA.ElemID);
						if (VertexChains)
						{
							if (SegmentToChain)
							{
								(*SegmentToChain)[SegIdx] = VertexChains->Num();
							}
							VertexChains->Add(EmbeddedPath.Num());
							VertexChains->Append(EmbeddedPath);
						}
					}
					else
					{
						bSuccess = false;
					}
				}
			}
			

			return bSuccess;
		}

		void UpdateFromSplit(FPtOnMesh& Pt, int SplitVertex, const FIndex2i& SplitEdges)
		{
			// check if within tolerance of the new vtx
			if (DistanceSquared(Pt.Pos, Mesh->GetVertex(SplitVertex)) < SnapToleranceSq)
			{
				Pt.Type = EVertexType::Vertex;
				Pt.ElemID = SplitVertex;
				return;
			}

			// it was already on the edge, so it must be on the sub-edges after split -- just pick the closest
			int EdgeIdx = ClosestEdge(SplitEdges, Pt.Pos);
			checkSlow(EdgeIdx > -1 && EdgeIdx < 2 && SplitEdges[EdgeIdx]>-1);
			Pt.Type = EVertexType::Edge;
			Pt.ElemID = SplitEdges[EdgeIdx];
		}



		void UpdateFromPoke(FPtOnMesh& Pt, int PokeVertex, const FIndex3i& PokeEdges, const FIndex3i& PokeTris)
		{
			// check if within tolerance of the new vtx
			if (DistanceSquared(Pt.Pos, Mesh->GetVertex(PokeVertex)) < SnapToleranceSq)
			{
				Pt.Type = EVertexType::Vertex;
				Pt.ElemID = PokeVertex;
				return;
			}

			int EdgeIdx = OnEdge(PokeEdges, Pt.Pos, SnapToleranceSq);
			if (EdgeIdx > -1)
			{
				Pt.Type = EVertexType::Edge;
				Pt.ElemID = PokeEdges[EdgeIdx];
				return;
			}

			for (int j = 0; j < 3; ++j) {

				if (IsInTriangle(PokeTris[j], Pt.Pos))
				{
					check(Pt.Type == EVertexType::Face); // how would it be anything else?
					Pt.ElemID = PokeTris[j];
					return;
				}
			}

			// failsafe case: pt was outside of triangle -- project to edge
			EdgeIdx = OnEdge(PokeEdges, Pt.Pos, FMathd::MaxReal);
			Pt.Type = EVertexType::Edge;
			Pt.ElemID = PokeEdges[EdgeIdx];
		}

		int OnVertex(const FTriangle3d& Tri, const FVector3d& V)
		{
			double BestDSq = SnapToleranceSq;
			int BestIdx = -1;
			for (int SubIdx = 0; SubIdx < 3; SubIdx++)
			{
				double DSq = DistanceSquared(Tri.V[SubIdx], V);
				if (DSq < BestDSq)
				{
					BestIdx = SubIdx;
					BestDSq = DSq;
				}
			}
			return BestIdx;
		}

		int OnEdge(const FTriangle3d& Tri, const FVector3d& V)
		{
			double BestDSq = SnapToleranceSq;
			int BestIdx = -1;
			for (int Idx = 0; Idx < 3; Idx++)
			{
				FSegment3d Seg{ Tri.V[Idx], Tri.V[(Idx + 1) % 3] };
				double DSq = Seg.DistanceSquared(V);
				if (DSq < BestDSq)
				{
					BestDSq = DSq;
					BestIdx = Idx;
				}
			}
			return BestIdx;
		}

		int OnEdgeWithSkip(const FTriangle3d& Tri, const FVector3d& V, int SkipIdx)
		{
			double BestDSq = SnapToleranceSq;
			int BestIdx = -1;
			for (int Idx = 0; Idx < 3; Idx++)
			{
				if (Idx == SkipIdx)
				{
					continue;
				}
				FSegment3d Seg{ Tri.V[Idx], Tri.V[(Idx + 1) % 3] };
				double DSq = Seg.DistanceSquared(V);
				if (DSq < BestDSq)
				{
					BestDSq = DSq;
					BestIdx = Idx;
				}
			}
			return BestIdx;
		}

		int ClosestEdge(FIndex2i EIDs, const FVector3d& Pos)
		{
			int BestIdx = -1;
			double BestDSq = FMathd::MaxReal;
			for (int Idx = 0; Idx < 2; Idx++)
			{
				int EID = EIDs[Idx];
				FIndex2i EVIDs = Mesh->GetEdgeV(EID);
				FSegment3d Seg(Mesh->GetVertex(EVIDs.A), Mesh->GetVertex(EVIDs.B));
				double DSq = Seg.DistanceSquared(Pos);
				if (DSq < BestDSq)
				{
					BestDSq = DSq;
					BestIdx = Idx;
				}
			}
			return BestIdx;
		}

		int OnEdge(FIndex3i EIDs, const FVector3d& Pos, double BestDSq)
		{
			int BestIdx = -1;
			for (int Idx = 0; Idx < 3; Idx++)
			{
				int EID = EIDs[Idx];
				FIndex2i EVIDs = Mesh->GetEdgeV(EID);
				FSegment3d Seg(Mesh->GetVertex(EVIDs.A), Mesh->GetVertex(EVIDs.B));
				double DSq = Seg.DistanceSquared(Pos);
				if (DSq < BestDSq)
				{
					BestDSq = DSq;
					BestIdx = Idx;
				}
			}
			return BestIdx;
		}

		bool IsInTriangle(int TID, const FVector3d& Pos)
		{
			FTriangle3d Tri;
			Mesh->GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
			FVector3d bary = VectorUtil::BarycentricCoords(Pos, Tri.V[0], Tri.V[1], Tri.V[2]);
			return (bary.X >= 0 && bary.Y >= 0 && bary.Z >= 0
				&& bary.X < 1 && bary.Y <= 1 && bary.Z <= 1);

		}


	};
}

bool FMeshSelfCut::Cut(const MeshIntersection::FIntersectionsQueryResult& Intersections)
{
	check(!bCutCoplanar); // not implemented yet

	ResetOutputs();

	MeshCut::FCutWorkingInfo WorkingInfo(Mesh, SnapTolerance);
	WorkingInfo.AddSegments(Intersections, 0);
	WorkingInfo.AddSegments(Intersections, 1);
	WorkingInfo.InsertFaceVertices();
	WorkingInfo.InsertEdgeVertices();
	return WorkingInfo.ConnectEdges(bTrackInsertedVertices ? &VertexChains : nullptr);
}


bool FMeshMeshCut::Cut(const MeshIntersection::FIntersectionsQueryResult& Intersections)
{
	check(!bCutCoplanar); // not implemented yet

	ResetOutputs();

	bool bSuccess = true;

	int MeshesToProcess = bMutuallyCut ? 2 : 1;
	for (int MeshIdx = 0; MeshIdx < MeshesToProcess; MeshIdx++)
	{
		MeshCut::FCutWorkingInfo WorkingInfo(Mesh[MeshIdx], SnapTolerance);
		WorkingInfo.AddSegments(Intersections, MeshIdx); // add intersection segments
		WorkingInfo.InsertFaceVertices(); // insert vertices for intersection segments w/ endpoints on faces
		WorkingInfo.InsertEdgeVertices(); // insert vertices for intersection segments w/ endpoints on edges

		// ensure that intersection segment endpoints are connected by direct edge paths
		bool bConnected = WorkingInfo.ConnectEdges(
			bTrackInsertedVertices ? &VertexChains[MeshIdx] : nullptr,
			bTrackInsertedVertices ? &SegmentToChain[MeshIdx] : nullptr
		);
		if (!bConnected)
		{
			bSuccess = false;
		}
	}

	return bSuccess;
}
