// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshBoolean

#include "Operations/MeshSelfUnion.h"
#include "Operations/MeshMeshCut.h"
#include "Operations/LocalPlanarSimplify.h"

#include "Selections/MeshConnectedComponents.h"

#include "DynamicMesh/MeshNormals.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "Spatial/PointHashGrid3.h"

#include "Algo/RemoveIf.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"

using namespace UE::Geometry;

bool FMeshSelfUnion::Compute()
{
	// transform the mesh to a shared space (centered at the origin, and scaled to a unit cube)
	FAxisAlignedBox3d AABB = Mesh->GetBounds(true);
	double ScaleFactor = 1.0 / FMath::Clamp(AABB.MaxDim(), 0.01, 1000000.0);
	FTransformSRT3d TransformToCenteredBox = FTransformSRT3d::Identity();
	TransformToCenteredBox.SetTranslation(ScaleFactor * (TransformToCenteredBox.GetTranslation() - AABB.Center()));
	TransformToCenteredBox.SetScale(ScaleFactor * FVector3d::One());
	MeshTransforms::ApplyTransform(*Mesh, TransformToCenteredBox);
	FTransformSRT3d ResultTransform(AABB.Center());
	ResultTransform.SetScale((1.0 / ScaleFactor) * FVector3d::One());

	// build spatial data and use it to find intersections
	FDynamicMeshAABBTree3 Spatial(Mesh);
	Spatial.SetTolerance(SnapTolerance);
	MeshIntersection::FIntersectionsQueryResult Intersections = Spatial.FindAllSelfIntersections(true, IMeshSpatial::FQueryOptions(),
		[this](FIntrTriangle3Triangle3d& Intr)
		{
			Intr.SetTolerance(SnapTolerance);
			return Intr.Find();
		}
	);

	if (Cancelled())
	{
		return false;
	}

	// cut the meshes
	FMeshSelfCut Cut(Mesh);
	Cut.SnapTolerance = SnapTolerance;
	Cut.bTrackInsertedVertices = bCollapseDegenerateEdgesOnCut; // to collect candidates to collapse
	Cut.Cut(Intersections);

	if (Cancelled())
	{
		return false;
	}

	// collapse tiny edges along cut boundary
	if (bCollapseDegenerateEdgesOnCut)
	{
		double DegenerateEdgeTolSq = DegenerateEdgeTolFactor * DegenerateEdgeTolFactor * SnapTolerance * SnapTolerance;

		// convert vertex chains to edge IDs to simplify logic of finding remaining candidate edges after collapses
		TArray<int> EIDs;
		for (int ChainIdx = 0; ChainIdx < Cut.VertexChains.Num();)
		{
			int ChainLen = Cut.VertexChains[ChainIdx];
			int ChainEnd = ChainIdx + 1 + ChainLen;
			for (int ChainSubIdx = ChainIdx + 1; ChainSubIdx + 1 < ChainEnd; ChainSubIdx++)
			{
				int VID[2]{ Cut.VertexChains[ChainSubIdx], Cut.VertexChains[ChainSubIdx + 1] };
				if ( DistanceSquared(Mesh->GetVertex(VID[0]), Mesh->GetVertex(VID[1])) < DegenerateEdgeTolSq)
				{
					EIDs.Add(Mesh->FindEdge(VID[0], VID[1]));
				}
			}
			ChainIdx = ChainEnd;
		}
		TSet<int> AllEIDs(EIDs);
		for (int Idx = 0; Idx < EIDs.Num(); Idx++)
		{
			int EID = EIDs[Idx];
			if (!Mesh->IsEdge(EID))
			{
				continue;
			}
			FVector3d A, B;
			Mesh->GetEdgeV(EID, A, B);
			if (DistanceSquared(A, B) > DegenerateEdgeTolSq)
			{
				continue;
			}
			FIndex2i EV = Mesh->GetEdgeV(EID);

			// if the vertex we'd remove is on a seam, try removing the other one instead
			if (Mesh->HasAttributes() && Mesh->Attributes()->IsSeamVertex(EV.B, false))
			{
				Swap(EV.A, EV.B);
				// if they were both on seams, then collapse should not happen?  (& would break OnCollapseEdge assumptions in overlay)
				if (Mesh->HasAttributes() && Mesh->Attributes()->IsSeamVertex(EV.B, false))
				{
					continue;
				}
			}
			FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
			EMeshResult CollapseResult = Mesh->CollapseEdge(EV.A, EV.B, .5, CollapseInfo);
			if (CollapseResult == EMeshResult::Ok)
			{
				for (int i = 0; i < 2; i++)
				{
					if (AllEIDs.Contains(CollapseInfo.RemovedEdges[i]))
					{
						int ToAdd = CollapseInfo.KeptEdges[i];
						bool bWasPresent;
						AllEIDs.Add(ToAdd, &bWasPresent);
						if (!bWasPresent)
						{
							EIDs.Add(ToAdd);
						}
					}
				}
			}
		}
	}

	if (Cancelled())
	{
		return false;
	}

	// edges that will become new boundary edges after the boolean op removes triangles on each mesh
	TArray<int> CutBoundaryEdges;
	// Vertices on the cut boundary that *may* not have a corresonding vertex on the other mesh
	TSet<int> PossUnmatchedBdryVerts;

	FMeshNormals Normals(Mesh);
	Normals.ComputeTriangleNormals();

	FMeshConnectedComponents ConnectedComponents(Mesh);
	ConnectedComponents.FindConnectedTriangles();
	TArray<int> TriToComponentID;  TriToComponentID.Init(-1, Mesh->MaxTriangleID());
	for (int ComponentIdx = 0; ComponentIdx < ConnectedComponents.Num(); ComponentIdx++)
	{
		const FMeshConnectedComponents::FComponent& Component = ConnectedComponents.GetComponent(ComponentIdx);
		for (int TID : Component.Indices)
		{
			TriToComponentID[TID] = ComponentIdx;
		}
	}
	// remap component IDs so they are ordered corresponding to the order of their first triangles in the mesh
	TArray<int> ComponentIDRemap; ComponentIDRemap.Init(-1, ConnectedComponents.Num());
	int RemapIdx = 0;
	for (int TID = 0; TID < Mesh->MaxTriangleID(); TID++)
	{
		int ComponentIdx = TriToComponentID[TID];
		if (ComponentIdx != -1 && ComponentIDRemap[ComponentIdx] == -1)
		{
			ComponentIDRemap[ComponentIdx] = RemapIdx++;
		}
	}
	for (int TID = 0; TID < Mesh->MaxTriangleID(); TID++)
	{
		int& ComponentIdx = TriToComponentID[TID];
		if (ComponentIdx > -1)
		{
			ComponentIdx = ComponentIDRemap[ComponentIdx];
		}
	}

	// delete geometry according to boolean rules, tracking the boundary edges
	{ // (just for scope)
		// decide what triangles to delete
		TArray<bool> KeepTri;
		TArray<int32> DeleteIfOtherKept;
		TFastWindingTree<FDynamicMesh3> Winding(&Spatial);
		int MaxTriID = Mesh->MaxTriangleID();
		KeepTri.SetNumUninitialized(MaxTriID);
		DeleteIfOtherKept.Init(-1, MaxTriID);
		
		ParallelFor(MaxTriID, [this, &Spatial, &Normals, &TriToComponentID, &KeepTri, &DeleteIfOtherKept, &Winding](int TID)
		{
			if (!Mesh->IsTriangle(TID))
			{
				return;
			}
			FVector3d Centroid = Mesh->GetTriCentroid(TID);

			double WindingNum = Winding.FastWindingNumber(Centroid + Normals[TID] * NormalOffset);
			bool bKeep = WindingNum < WindingThreshold; // keep if the outside of the tri is outside the shape
			if (bTrimFlaps && bKeep) // trimming flaps == also check that the inside of the tri is inside the shape
			{
				bKeep = Winding.FastWindingNumber(Centroid - Normals[TID] * NormalOffset) > WindingThreshold;
			}
			
			// if triangle is a candidate for keeping, check for the coplanar case
			if (bKeep)
			{
				double DSq;
				int MyComponentID = TriToComponentID[TID];
				IMeshSpatial::FQueryOptions QueryOptions(SnapTolerance,
					[&Normals, &TriToComponentID, MyComponentID](int OtherTID)
					{
						// By convention, the normal for degenerate triangles is the zero vector
						return !Normals[OtherTID].IsZero() && TriToComponentID[OtherTID] != MyComponentID;
					}
				);
				int OtherTID = Spatial.FindNearestTriangle(Centroid, DSq, QueryOptions);
				if (OtherTID > -1) // only consider it coplanar if there is a matching tri
				{
					double DotNormals = Normals[OtherTID].Dot(Normals[TID]);
					//if (FMath::Abs(DotNormals) > .9) // TODO: do we actually want to check for a normal match? coplanar vertex check below is more robust?
					{
						// To be extra sure it's a coplanar match, check the vertices are *also* on the other connected component (w/in SnapTolerance)
						FTriangle3d Tri;
						Mesh->GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
						bool bAllTrisOnOtherComponent = true;
						for (int Idx = 0; Idx < 3; Idx++)
						{
							if (Spatial.FindNearestTriangle(Tri.V[Idx], DSq, QueryOptions) == FDynamicMesh3::InvalidID)
							{
								bAllTrisOnOtherComponent = false;
								break;
							}
						}
						if (bAllTrisOnOtherComponent)
						{
							if (DotNormals <= 0) // include zero in range to also discard degenerate triangles w/ zero normals
							{
								KeepTri[TID] = false;
							}
							else
							{
								// for two coplanar components with matching normals,
								// just keep tris from the component with lower ID
								int OtherComponentID = TriToComponentID[OtherTID];
								bool bHasPriority = MyComponentID < OtherComponentID;
								KeepTri[TID] = bHasPriority;
								if (bHasPriority)
								{
									// If we kept this tri, remember the coplanar pair we expect to be deleted, in case
									// it isn't deleted (e.g. because it wasn't coplanar); to then delete this one instead.
									// This can help clean up sliver triangles near a cut boundary that look locally coplanar
									DeleteIfOtherKept[TID] = OtherTID;
								}
							}
							return;
						}
					}
				}
			}
			// didn't already return a coplanar result; use the winding-number-based decision
			KeepTri[TID] = bKeep;
		});

		// Don't keep coplanar tris if the matched, "lower priority" tri that we expected to delete was actually kept
		for (int TID : Mesh->TriangleIndicesItr())
		{
			int32 DeleteIfOtherKeptTID = DeleteIfOtherKept[TID];
			if (DeleteIfOtherKeptTID > -1 && KeepTri[DeleteIfOtherKeptTID])
			{
				KeepTri[TID] = false;
			}
		}

		// track where we will create new boundary edges
		for (int EID : Mesh->EdgeIndicesItr())
		{
			FIndex2i TriPair = Mesh->GetEdgeT(EID);
			if (TriPair.B == IndexConstants::InvalidID || KeepTri[TriPair.A] == KeepTri[TriPair.B])
			{
				continue;
			}

			CutBoundaryEdges.Add(EID);
			FIndex2i VertPair = Mesh->GetEdgeV(EID);
			PossUnmatchedBdryVerts.Add(VertPair.A);
			PossUnmatchedBdryVerts.Add(VertPair.B);
		}
		
		// actually delete triangles
		for (int TID = 0; TID < KeepTri.Num(); TID++)
		{
			if (Mesh->IsTriangle(TID) && !KeepTri[TID])
			{
				Mesh->RemoveTriangle(TID, true, false);
			}
		}
	}

	if (Cancelled())
	{
		return false;
	}

	// Hash boundary verts for faster search
	TPointHashGrid3d<int> PointHash(Mesh->GetBounds(true).MaxDim() / 64, -1);
	for (int BoundaryVID : PossUnmatchedBdryVerts)
	{
		PointHash.InsertPointUnsafe(BoundaryVID, Mesh->GetVertex(BoundaryVID));
	}

	FSparseDynamicOctree3 EdgeOctree;
	EdgeOctree.RootDimension = .25;
	EdgeOctree.SetMaxTreeDepth(7);
	auto EdgeBounds = [this](int EID)
	{
		FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EID);
		FVector3d A = Mesh->GetVertex(Edge.Vert.A);
		FVector3d B = Mesh->GetVertex(Edge.Vert.B);
		if (A.X > B.X)
		{
			Swap(A.X, B.X);
		}
		if (A.Y > B.Y)
		{
			Swap(A.Y, B.Y);
		}
		if (A.Z > B.Z)
		{
			Swap(A.Z, B.Z);
		}
		return FAxisAlignedBox3d(A, B);
	};
	auto AddEdge = [&EdgeOctree, EdgeBounds](int EID)
	{
		EdgeOctree.InsertObject(EID, EdgeBounds(EID));
	};
	auto UpdateEdge = [&EdgeOctree, EdgeBounds](int EID)
	{
		EdgeOctree.ReinsertObject(EID, EdgeBounds(EID));
	};
	for (int EID : CutBoundaryEdges)
	{
		AddEdge(EID);
	}
	TArray<int> EdgesInRange;
	
	// mapping of all accepted correspondences of boundary vertices (both ways -- so if A is connected to B we add both A->B and B->A) 
	TMap<int, int> FoundMatches;

	{ // for scope
		double SnapToleranceSq = SnapTolerance * SnapTolerance;
		TArray<int> BoundaryNbrEdges;
		TArray<int> ExcludeVertices;
		for (int BoundaryVID : PossUnmatchedBdryVerts)
		{
			// skip vertices that we've already matched up
			if (FoundMatches.Contains(BoundaryVID))
			{
				continue;
			}

			FVector3d Pos = Mesh->GetVertex(BoundaryVID);

			// Find a neighborhood of topologically-connected vertices, and exclude these from matching
			// TODO: in theory we should walk SnapTolerance away on the connected boundary edges to build the full exclusion set
			// (in practice just filtering the immediate neighbors should usually be ok?)
			BoundaryNbrEdges.Reset();
			ExcludeVertices.Reset();
			ExcludeVertices.Add(BoundaryVID);
			Mesh->GetAllVtxBoundaryEdges(BoundaryVID, BoundaryNbrEdges);
			for (int EID : BoundaryNbrEdges)
			{
				FIndex2i EdgeVID = Mesh->GetEdgeV(EID);
				ExcludeVertices.Add(EdgeVID.A == BoundaryVID ? EdgeVID.B : EdgeVID.A);
			}

			TPair<int, double> VIDDist = PointHash.FindNearestInRadius(
				Pos, SnapTolerance,
				[this, &Pos](int VID)
				{
					return DistanceSquared(Pos, Mesh->GetVertex(VID));
				},
				[&ExcludeVertices](int VID)
				{
					return ExcludeVertices.Contains(VID);
				}
			);
			int NearestVID = VIDDist.Key; // ID of nearest vertex on other mesh
			double DSq = VIDDist.Value;   // square distance to that vertex

			if (NearestVID != FDynamicMesh3::InvalidID)
			{

				int* Match = FoundMatches.Find(NearestVID);
				if (Match)
				{
					double OldDSq = DistanceSquared(Mesh->GetVertex(*Match), Mesh->GetVertex(NearestVID));
					if (DSq < OldDSq) // new vertex is a better match than the old one
					{
						int OldVID = *Match; // copy old VID out of match before updating the TMap
						FoundMatches.Add(NearestVID, BoundaryVID); // new VID is recorded as best match
						FoundMatches.Add(BoundaryVID, NearestVID);
						FoundMatches.Remove(OldVID);

						// old VID is swapped in as the one to consider as unmatched
						// it will now be matched below
						BoundaryVID = OldVID;
						// Note we don't reset BoundaryNbrEdges here, since we still want to filter
						// the edges near the (now-matched) initial BoundaryVID
						Mesh->GetAllVtxBoundaryEdges(BoundaryVID, BoundaryNbrEdges);
						Pos = Mesh->GetVertex(BoundaryVID);
						DSq = OldDSq;
					}
					NearestVID = FDynamicMesh3::InvalidID; // one of these vertices will be unmatched
				}
				else
				{
					FoundMatches.Add(NearestVID, BoundaryVID);
					FoundMatches.Add(BoundaryVID, NearestVID);
				}
			}

			// if we didn't find a valid match, try to split the nearest edge to create a match
			if (NearestVID == FDynamicMesh3::InvalidID)
			{
				// vertex had no match -- try to split edge to match it
				FAxisAlignedBox3d QueryBox(Pos, SnapTolerance);
				EdgesInRange.Reset();
				EdgeOctree.RangeQuery(QueryBox, EdgesInRange);

				int OtherEID = FindNearestEdge(EdgesInRange, BoundaryNbrEdges, Pos);
				if (OtherEID != FDynamicMesh3::InvalidID)
				{
					FVector3d EdgePts[2];
					Mesh->GetEdgeV(OtherEID, EdgePts[0], EdgePts[1]);
					// only accept the match if it's not going to create a degenerate edge -- TODO: filter already-matched edges from the FindNearestEdge query!
					if (DistanceSquared(EdgePts[0], Pos) > SnapToleranceSq && DistanceSquared(EdgePts[1], Pos) > SnapToleranceSq)
					{
						FSegment3d Seg(EdgePts[0], EdgePts[1]);
						double Along = Seg.ProjectUnitRange(Pos);
						FDynamicMesh3::FEdgeSplitInfo SplitInfo;
						if (ensure(EMeshResult::Ok == Mesh->SplitEdge(OtherEID, SplitInfo, Along)))
						{
							FoundMatches.Add(SplitInfo.NewVertex, BoundaryVID);
							FoundMatches.Add(BoundaryVID, SplitInfo.NewVertex);
							Mesh->SetVertex(SplitInfo.NewVertex, Pos);
							CutBoundaryEdges.Add(SplitInfo.NewEdges.A);
							UpdateEdge(OtherEID);
							AddEdge(SplitInfo.NewEdges.A);
							// Note: Do not update PossUnmatchedBdryVerts with the new vertex, because it is already matched by construction
							// Likewise do not update the pointhash -- we don't want it to find vertices that were already perfectly matched
						}
					}
				}
			}
		}
	}

	// actually snap the positions together for final matches
	for (TPair<int, int>& Match : FoundMatches)
	{
		if (Match.Value < Match.Key)
		{
			checkSlow(FoundMatches[Match.Value] == Match.Key);
			continue; // everything is in the map twice, so we only process the Key<Value entries
		}
		Mesh->SetVertex(Match.Value, Mesh->GetVertex(Match.Key));
	}


	if (bSimplifyAlongNewEdges)
	{
		SimplifyAlongNewEdges(CutBoundaryEdges, FoundMatches);
	}

	if (Cancelled())
	{
		return false;
	}

	bool bWeldSuccess = true;
	if (bWeldSharedEdges)
	{
		bWeldSuccess = MergeEdges(CutBoundaryEdges, FoundMatches);
	}

	if (bTrackAllNewEdges)
	{
		for (int32 eid : CreatedBoundaryEdges)
		{
			AllNewEdges.Add(eid);
		}
	}

	MeshTransforms::ApplyTransform(*Mesh, ResultTransform);

	return bWeldSuccess;
}


void FMeshSelfUnion::SimplifyAlongNewEdges(TArray<int>& CutBoundaryEdges, TMap<int, int>& FoundMatches)
{
	double DotTolerance = FMathd::Cos(SimplificationAngleTolerance * FMathd::DegToRad);

	TSet<int> CutBoundaryEdgeSet; // set version of CutBoundaryEdges, for faster membership tests
	CutBoundaryEdgeSet.Append(CutBoundaryEdges);

	int NumCollapses = 0, CollapseIters = 0;
	int MaxCollapseIters = 1; // TODO: is there a case where we need more iterations?  Perhaps if we add some triangle quality criteria?
	while (CollapseIters < MaxCollapseIters)
	{
		int LastNumCollapses = NumCollapses;
		for (int EID : CutBoundaryEdges)
		{
			// this can happen if a collapse removes another cut boundary edge
			// (which can happen e.g. if you have a degenerate (colinear) tri flat on the cut boundary)
			if (!Mesh->IsEdge(EID))
			{
				continue;
			}

			FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EID);

			int Matches[2]{ -1, -1 };
			bool bHasMatches = true;
			for (int MatchIdx = 0; MatchIdx < 2; MatchIdx++)
			{
				int* Match = FoundMatches.Find(Edge.Vert[MatchIdx]);
				if (Match)
				{
					Matches[MatchIdx] = *Match;
				}
				else
				{
					bHasMatches = false;
					// TODO: if we switch to allow collapse on unmatched edges, we shouldn't break here
					//        b/c we may be partially matched, and need to track which is matched.
					break;
				}
			}
			if (!bHasMatches)
			{
				continue; // edge wasn't matched up; can't collapse it?
				// TODO: consider supporting collapses in this case?
			}

			// if we have matched vertices, we also need a matched edge to collapse
			int MatchEID = Mesh->FindEdge(Matches[0], Matches[1]);
			if (MatchEID == -1)
			{
				continue;
			}

			// track whether the neighborhood of the vertex is flat (and likewise its matched pair's neighborhood, if present)
			bool Flat[2]{ false, false };
			// normals for each flat vertex, and each "side" (EID side and MatchEID side)
			FVector3d FlatNormals[2][2]{ {FVector3d::Zero(), FVector3d::Zero()}, {FVector3d::Zero(), FVector3d::Zero()} };
			int NumFlat = 0;
			for (int VIdx = 0; VIdx < 2; VIdx++)
			{
				Flat[VIdx] = FLocalPlanarSimplify::IsFlat(*Mesh, Edge.Vert[VIdx], DotTolerance, FlatNormals[VIdx][0]) 
						  && FLocalPlanarSimplify::IsFlat(*Mesh, Matches[VIdx], DotTolerance, FlatNormals[VIdx][1]);

				if (Flat[VIdx])
				{
					NumFlat++;
				}
			}

			if (NumFlat == 0)
			{
				continue;
			}

			// see if we can collapse to remove either vertex
			for (int RemoveVIdx = 0; RemoveVIdx < 2; RemoveVIdx++)
			{
				if (!Flat[RemoveVIdx])
				{
					continue;
				}
				int KeepVIdx = 1 - RemoveVIdx;
				// Note: positions are exactly the same on matched edges because snapping has already happened
				FVector3d RemoveVPos = Mesh->GetVertex(Edge.Vert[RemoveVIdx]);
				FVector3d KeepVPos = Mesh->GetVertex(Edge.Vert[KeepVIdx]);
				FVector3d EdgeDir = KeepVPos - RemoveVPos;
				if (Normalize(EdgeDir) == 0) // 0 is returned as a special case when the edge was too short to normalize
				{
					// collapsing degenerate edges above should prevent this
					ensure(!bCollapseDegenerateEdgesOnCut);
					// Just skip these edges, because in practice we generally have bCollapseDegenerateEdgesOnCut enabled
					break; // break instead of continue to skip the whole edge
				}

				bool bHasBadEdge = false; // will be set if either mesh can't collapse the edge
				for (int WhichEdge = 0; !bHasBadEdge && WhichEdge < 2; WhichEdge++) // same processing on EID and on MatchEID
				{
					int RemoveV = WhichEdge == 0 ? Edge.Vert[RemoveVIdx] : Matches[RemoveVIdx];
					int KeepV = WhichEdge == 0 ? Edge.Vert[KeepVIdx] : Matches[KeepVIdx];
					int SourceEID = WhichEdge == 0 ? EID : MatchEID;

					bHasBadEdge = bHasBadEdge || FLocalPlanarSimplify::CollapseWouldHurtTriangleQuality(
						*Mesh, FlatNormals[RemoveVIdx][WhichEdge], RemoveV, RemoveVPos, KeepV, KeepVPos, TryToImproveTriQualityThreshold);

					bHasBadEdge = bHasBadEdge || FLocalPlanarSimplify::CollapseWouldChangeShapeOrUVs(
						*Mesh, CutBoundaryEdgeSet, DotTolerance,
						SourceEID, RemoveV, RemoveVPos, KeepV, KeepVPos, EdgeDir, bPreserveTriangleGroups,
						true, bPreserveVertexUVs, bPreserveOverlayUVs, UVDistortTolerance * UVDistortTolerance,
						bPreserveVertexNormals, FMathf::Cos(NormalDistortTolerance * FMathf::DegToRad));
				}

				if (bHasBadEdge)
				{
					continue;
				}

				FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
				int RemoveV = Edge.Vert[RemoveVIdx];
				int KeepV = Edge.Vert[KeepVIdx];
				EMeshResult CollapseResult = Mesh->CollapseEdge(KeepV, RemoveV, 0, CollapseInfo);
				if (CollapseResult == EMeshResult::Ok)
				{
					int OtherRemoveV = Matches[RemoveVIdx];
					int OtherKeepV = Matches[KeepVIdx];
					FDynamicMesh3::FEdgeCollapseInfo OtherCollapseInfo;
					EMeshResult OtherCollapseResult = Mesh->CollapseEdge(OtherKeepV, OtherRemoveV, 0, OtherCollapseInfo);
					if (OtherCollapseResult != EMeshResult::Ok)
					{
						// if we get here, we've somehow managed to collapse the first edge but failed on the second (matched) edge
						// which will leave a crack in the result unless we can somehow undo the first collapse, which would require a bunch of extra work
						// but the only case where I could see this happen is if the second edge is on an isolated triangle, which means there is a hole anyway
						// or if the mesh topology is somehow invalid
						ensure(OtherCollapseResult == EMeshResult::Failed_CollapseTriangle);
					}
					else
					{
						FoundMatches.Remove(OtherRemoveV);
						CutBoundaryEdgeSet.Remove(OtherCollapseInfo.CollapsedEdge);
						CutBoundaryEdgeSet.Remove(OtherCollapseInfo.RemovedEdges[0]);
						if (OtherCollapseInfo.RemovedEdges[1] != -1)
						{
							CutBoundaryEdgeSet.Remove(OtherCollapseInfo.RemovedEdges[1]);
						}
					}

					NumCollapses++;
					FoundMatches.Remove(RemoveV);
					CutBoundaryEdgeSet.Remove(CollapseInfo.CollapsedEdge);
					CutBoundaryEdgeSet.Remove(CollapseInfo.RemovedEdges[0]);
					if (CollapseInfo.RemovedEdges[1] != -1)
					{
						CutBoundaryEdgeSet.Remove(CollapseInfo.RemovedEdges[1]);
					}
				}
				break; // if we got through to trying to collapse the edge, don't try to collapse from the other vertex.
			}
		}

		CutBoundaryEdges = CutBoundaryEdgeSet.Array();

		if (NumCollapses == LastNumCollapses)
		{
			break;
		}

		CollapseIters++;
	}
}


bool FMeshSelfUnion::MergeEdges(const TArray<int>& CutBoundaryEdges, const TMap<int, int>& FoundMatches)
{	
	// find "easy" match candidates using the already-made vertex correspondence
	TArray<FIndex2i> CandidateMatches;
	for (int EID : CutBoundaryEdges)
	{
		if (!ensure(Mesh->IsBoundaryEdge(EID)))
		{
			continue;
		}
		FIndex2i VIDs = Mesh->GetEdgeV(EID);
		const int* OtherA = FoundMatches.Find(VIDs.A);
		const int* OtherB = FoundMatches.Find(VIDs.B);
		if (OtherA && OtherB)
		{
			int OtherEID = Mesh->FindEdge(*OtherA, *OtherB);
			// because FoundMatches includes both directions of each mapping
			// only accept the mapping w/ EID < OtherEID (This also excludes OtherEID == InvalidID)
			if (OtherEID > EID)
			{
				checkSlow(OtherEID != FDynamicMesh3::InvalidID);
				CandidateMatches.Add(FIndex2i(EID, OtherEID));
			}
		}
	}

	// merge the easy matches
	for (FIndex2i Candidate : CandidateMatches)
	{
		if (!Mesh->IsEdge(Candidate.A) || !Mesh->IsBoundaryEdge(Candidate.A))
		{
			continue;
		}

		FDynamicMesh3::FMergeEdgesInfo MergeInfo;
		EMeshResult EdgeMergeResult = Mesh->MergeEdges(Candidate.A, Candidate.B, MergeInfo);

		if (EdgeMergeResult == EMeshResult::Ok)
		{
			if (bTrackAllNewEdges)
			{
				AllNewEdges.Add(Candidate.A);
			}
		}
	}

	// try to greedily match remaining unmatched edges within snap tolerance
	double SnapToleranceSq = SnapTolerance * SnapTolerance;
	
	TPointHashGrid3<int32, double> MidpointsHash(FMath::Max(SnapTolerance * 2, FMathd::ZeroTolerance), -1);
	int32 UnmatchedInHash = 0;
	for (int32 EID : CutBoundaryEdges)
	{
		if (!Mesh->IsEdge(EID) || !Mesh->IsBoundaryEdge(EID))
		{
			continue;
		}
		FVector3d A, B;
		Mesh->GetEdgeV(EID, A, B);
		FVector3d Midpoint = (A + B) * .5;
		int32 EIDMatched = INDEX_NONE;
		FVector3d MatchedMidpoint;
		if (UnmatchedInHash > 0)
		{
			MidpointsHash.EnumeratePointsInBall(Midpoint, SnapTolerance,
				[this, &Midpoint](const int32& OtherEID) -> double
				{
					return FVector3d::DistSquared(Midpoint, Mesh->GetEdgePoint(OtherEID, .5));
				},
				[this, &A, &B, EID, SnapToleranceSq, &EIDMatched, &MatchedMidpoint](int32 OtherEID, double MidSqDist) -> bool
				{
					FVector3d OA, OB;
					Mesh->GetEdgeV(OtherEID, OA, OB);

					if ((DistanceSquared(OA, A) < SnapToleranceSq && DistanceSquared(OB, B) < SnapToleranceSq) ||
						(DistanceSquared(OB, A) < SnapToleranceSq && DistanceSquared(OA, B) < SnapToleranceSq))
					{
						FDynamicMesh3::FMergeEdgesInfo MergeInfo;
						EMeshResult EdgeMergeResult = Mesh->MergeEdges(EID, OtherEID, MergeInfo);
						if (EdgeMergeResult == EMeshResult::Ok)
						{
							EIDMatched = OtherEID;
							MatchedMidpoint = (OA + OB) * .5;
							if (bTrackAllNewEdges)
							{
								AllNewEdges.Add(EID);
							}
							return false; // stop iteration on successful merge
						}
					}
					return true; // continue iterating
				},
				[this](int32 OtherEID) -> bool
				{
					return !Mesh->IsEdge(OtherEID) || !Mesh->IsBoundaryEdge(OtherEID);
				}
			);
		}

		if (EIDMatched == INDEX_NONE)
		{
			MidpointsHash.InsertPointUnsafe(EID, Midpoint);
			UnmatchedInHash++;
		}
		else
		{
			MidpointsHash.RemovePointUnsafe(EIDMatched, MatchedMidpoint);
			UnmatchedInHash--;
		}
	}

	// store the failure cases
	bool bAllMatched = true;
	for (int EID : CutBoundaryEdges)
	{
		if (Mesh->IsEdge(EID) && Mesh->IsBoundaryEdge(EID))
		{
			CreatedBoundaryEdges.Add(EID);
			bAllMatched = false;
		}
	}
	
	return bAllMatched;
}


int FMeshSelfUnion::FindNearestEdge(const TArray<int>& EIDs, const TArray<int>& BoundaryNbrEdges, FVector3d Pos)
{
	int NearEID = FDynamicMesh3::InvalidID;
	double NearSqr = SnapTolerance * SnapTolerance;
	FVector3d EdgePts[2];
	for (int EID : EIDs) {
		if (BoundaryNbrEdges.Contains(EID))
		{
			continue;
		}
		Mesh->GetEdgeV(EID, EdgePts[0], EdgePts[1]);

		FSegment3d Seg(EdgePts[0], EdgePts[1]);
		double DSqr = Seg.DistanceSquared(Pos);
		if (DSqr < NearSqr)
		{
			NearEID = EID;
			NearSqr = DSqr;
		}
	}
	return NearEID;
}
