// Copyright Epic Games, Inc. All Rights Reserved.

#include "OrientedBoxTypes.h"

#include "Math/UnrealMath.h"


namespace UE::OrientedBoxTypeLocals
{
	using namespace UE::Math;
	using namespace UE::Geometry;

	/**
	 * Helper to test box intersection in the local space of Box A -- i.e., where Box A is centered at the origin, and axis-aligned.
	 * 
	 * Assumes extents are not negative.
	 * Applies 15 separating axis tests: 3 per axis of the two boxes, and 9 per cross product of the box axes.
	 * If the boxes are aligned on any axis, the 9 cross-edge tests are not needed (and would have robustness issues for the aligned-axis cross product)
	 */
	template<typename RealType>
	static bool BoxIntersectsLocalSpaceHelper(
		const TVector<RealType>& ExtentsA, 
		const TVector<RealType>& CenterB, const TVector<RealType> (&AxesB)[3], const TVector<RealType>& ExtentsB, 
		RealType ParallelAxisTol = TMathUtil<RealType>::ZeroTolerance)
	{
		// Convert zero tolerance to a dot-product tolerance -- i.e., take 1-Tolerance
		RealType DotParallelAxisTol = (RealType)1 - FMath::Max((RealType)0, ParallelAxisTol);

		// Absolute values of all B axes, computed in the loop below.
		// Used to project the farthest distance from box center to corner, along the separating axis test direction
		TVector<RealType> AbsAxesB[3]{};

		bool bHasParallelAxis = false;

		// Test if axes of box A (i.e. the major axes) are separating
		for (int32 AxisA = 0; AxisA < 3; ++AxisA)
		{
			// Projected BoxA radius along its own axis -- just the extent
			RealType ProjRadiusA = ExtentsA[AxisA];

			// BoxB radius along A axis (Dot product of AxisA and largest box corner vector)
			// Compute absolute values of AxesB as well (for reuse in later tests), and test for alignment w/ a major axis
			RealType ProjRadiusB = 0;
			int32 NumZeroes = 0;
			for (int32 AxisB = 0; AxisB < 3; ++AxisB)
			{
				AbsAxesB[AxisB][AxisA] = FMath::Abs(AxesB[AxisB][AxisA]);
				bHasParallelAxis |= (AbsAxesB[AxisB][AxisA] >= DotParallelAxisTol);
				ProjRadiusB += ExtentsB[AxisB] * AbsAxesB[AxisB][AxisA];
			}
			
			// Projected distance between centers
			RealType ProjCenterDist = FMath::Abs(CenterB[AxisA]);

			if (ProjCenterDist > ProjRadiusA + ProjRadiusB)
			{
				return false;
			}
		}

		// Test if axes of box B are separating
		for (int32 AxisB = 0; AxisB < 3; ++AxisB)
		{
			// Projected BoxB radius along its own axis -- just the extent
			RealType ProjRadiusB = ExtentsB[AxisB];

			// BoxA radius along B axis
			RealType ProjRadiusA = ExtentsA.Dot(AbsAxesB[AxisB]);

			// Projected distance between centers
			RealType ProjCenterDist = FMath::Abs(CenterB.Dot(AxesB[AxisB]));

			if (ProjCenterDist > ProjRadiusB + ProjRadiusA)
			{
				return false;
			}
		}

		// if boxes have a parallel axis, we don't need to test cross axes -- since we haven't found a separating axis above, the boxes intersect
		// (this is equivalent to projecting along the shared axis, and doing 2D OBB intersection, where just major axes are sufficient)
		if (bHasParallelAxis)
		{
			return true;
		}

		// Test the 9 cross-product axes
		for (int32 AxisA = 0, OffA1 = 1, OffA2 = 2; AxisA < 3; OffA1 = OffA2, OffA2 = AxisA++)
		{
			for (int32 AxisB = 0, OffB1 = 1, OffB2 = 2; AxisB < 3; OffB1 = OffB2, OffB2 = AxisB++)
			{
				// AxisA x AxisB projections, simplified due to AxisA being a major axis

				// Projected BoxA radius along AxisA x AxisB
				RealType ProjRadiusA = ExtentsA[OffA1] * AbsAxesB[AxisB][OffA2] + ExtentsA[OffA2] * AbsAxesB[AxisB][OffA1];

				// Projected BoxB radius along AxisA x AxisB
				RealType ProjRadiusB = ExtentsB[OffB1] * AbsAxesB[OffB2][AxisA] + ExtentsB[OffB2] * AbsAxesB[OffB1][AxisA];

				// Abs(Dot(Center, Cross(AxisA, AxisB)))
				RealType ProjCenterDist = FMath::Abs(
					CenterB[OffA1] * AxesB[AxisB][OffA2] - CenterB[OffA2] * AxesB[AxisB][OffA1]
				);

				if (ProjCenterDist > ProjRadiusA + ProjRadiusB)
				{
					return false;
				}
			}
		}

		return true;
	}
}


namespace UE::Geometry
{


template<typename RealType>
bool TOrientedBox3<RealType>::Intersects(const TOrientedBox3& OtherBox, RealType ParallelAxesTolerance) const
{
	using namespace UE::OrientedBoxTypeLocals;

	if (!IsValid() || !OtherBox.IsValid())
	{
		return false;
	}

	TQuaternion<RealType> ToFrameRot = Frame.Rotation.Inverse();
	TQuaternion<RealType> LocalRot = ToFrameRot * OtherBox.Frame.Rotation;

	TVector<RealType> LocalAxesB[3];
	LocalRot.GetAxes(LocalAxesB[0], LocalAxesB[1], LocalAxesB[2]);
	
	TVector<RealType> LocalCenterB = ToFrameRot * (OtherBox.Frame.Origin - Frame.Origin);

	return BoxIntersectsLocalSpaceHelper(
		Extents,
		LocalCenterB, LocalAxesB, OtherBox.Extents, ParallelAxesTolerance
	);
}

template<typename RealType>
bool TOrientedBox3<RealType>::Intersects(const TAxisAlignedBox3<RealType>& OtherBox, RealType ParallelAxesTolerance) const
{
	using namespace UE::OrientedBoxTypeLocals;

	if (!IsValid() || OtherBox.IsEmpty())
	{
		return false;
	}

	TVector<RealType> LocalAxesB[3];
	Frame.Rotation.GetAxes(LocalAxesB[0], LocalAxesB[1], LocalAxesB[2]);

	TVector<RealType> LocalCenterB = Frame.Origin - OtherBox.Center();

	return BoxIntersectsLocalSpaceHelper(
		OtherBox.Extents(),
		LocalCenterB, LocalAxesB, Extents, ParallelAxesTolerance
	);
}

template<typename RealType>
TOrientedBox3<RealType> TOrientedBox3<RealType>::Merge(const TOrientedBox3<RealType>& Other, bool bOnlyConsiderExistingAxes) const
{
	TQuaternion<RealType> RotInv = Frame.Rotation.Inverse();
	TQuaternion<RealType> RotInvOtherRot = RotInv * Other.Frame.Rotation;
	TVector<RealType> O2mO1 = Other.Frame.Origin - Frame.Origin;
	TVector<RealType> TranslateIntoFrame1 = RotInv * O2mO1;
	TVector<RealType> TranslateIntoFrame2 = Other.Frame.Rotation.InverseMultiply(-O2mO1);
	TAxisAlignedBox3<RealType> LocalBox1(-Extents, Extents);
	TAxisAlignedBox3<RealType> LocalBox2(-Other.Extents, Other.Extents);
	// Corners of this box, in its local coordinate frame
	TVector<RealType> Corners1[8]
	{
		LocalBox1.Min,
		LocalBox1.Max,
		TVector<RealType>(LocalBox1.Min.X, LocalBox1.Min.Y, LocalBox1.Max.Z),
		TVector<RealType>(LocalBox1.Min.X, LocalBox1.Max.Y, LocalBox1.Min.Z),
		TVector<RealType>(LocalBox1.Max.X, LocalBox1.Min.Y, LocalBox1.Min.Z),
		TVector<RealType>(LocalBox1.Min.X, LocalBox1.Max.Y, LocalBox1.Max.Z),
		TVector<RealType>(LocalBox1.Max.X, LocalBox1.Min.Y, LocalBox1.Max.Z),
		TVector<RealType>(LocalBox1.Max.X, LocalBox1.Max.Y, LocalBox1.Min.Z),
	};
	// Cordinates of Other, in its local coordinate frame
	TVector<RealType> Corners2[8]
	{
		LocalBox2.Min,
		LocalBox2.Max,
		TVector<RealType>(LocalBox2.Min.X, LocalBox2.Min.Y, LocalBox2.Max.Z),
		TVector<RealType>(LocalBox2.Min.X, LocalBox2.Max.Y, LocalBox2.Min.Z),
		TVector<RealType>(LocalBox2.Max.X, LocalBox2.Min.Y, LocalBox2.Min.Z),
		TVector<RealType>(LocalBox2.Min.X, LocalBox2.Max.Y, LocalBox2.Max.Z),
		TVector<RealType>(LocalBox2.Max.X, LocalBox2.Min.Y, LocalBox2.Max.Z),
		TVector<RealType>(LocalBox2.Max.X, LocalBox2.Max.Y, LocalBox2.Min.Z),
	};
	// Expand each local AABB to contain the other box's points
	for (int32 Idx = 0; Idx < 8; ++Idx)
	{
		LocalBox1.Contain(TranslateIntoFrame1 + RotInvOtherRot * Corners2[Idx]);
		LocalBox2.Contain(TranslateIntoFrame2 + RotInvOtherRot.InverseMultiply(Corners1[Idx]));
	}
	RealType CandidateVolumes[3]{ LocalBox1.Volume(), LocalBox2.Volume(), TMathUtil<RealType>::MaxReal };
	TVector<RealType> Axes1[3], Axes2[3], FoundAxes[3];
	TAxisAlignedBox3<RealType> FoundBounds;
	if (!bOnlyConsiderExistingAxes)
	{
		// Compute the two local frames
		Frame.Rotation.GetAxes(Axes1[0], Axes1[1], Axes1[2]);
		Other.Frame.Rotation.GetAxes(Axes2[0], Axes2[1], Axes2[2]);
		// Compute metrics re the size of the boxes and how far apart they are
		RealType MaxExtentsSizeSq = FMath::Max(Extents.SizeSquared(), Other.Extents.SizeSquared());
		RealType CenterDiffSizeSq = O2mO1.SizeSquared();
		// For boxes that are far apart relative to their size, fit a box along the line connecting their centers
		if (CenterDiffSizeSq > MaxExtentsSizeSq)
		{
			// find the most-perpendicular original box axis as our second axis
			// (Note we could alternatively try interpolating axes, or picking the axis with the largest projected extent ...)
			FoundAxes[0] = O2mO1.GetSafeNormal(0.0);
			TVector<RealType> MostPerpendicular;
			RealType SmallestDot = TMathUtil<RealType>::MaxReal;
			for (int32 AxisIdx = 0; AxisIdx < 3; ++AxisIdx)
			{
				RealType Dot1 = FMath::Abs(Axes1[AxisIdx].Dot(FoundAxes[0]));
				RealType Dot2 = FMath::Abs(Axes2[AxisIdx].Dot(FoundAxes[0]));
				if (Dot1 < Dot2)
				{
					if (Dot1 < SmallestDot)
					{
						SmallestDot = Dot1;
						MostPerpendicular = Axes1[AxisIdx];
					}
				}
				else if (Dot2 < SmallestDot)
				{
					SmallestDot = Dot2;
					MostPerpendicular = Axes2[AxisIdx];
				}
			}
			FoundAxes[2] = FoundAxes[0].Cross(MostPerpendicular).GetSafeNormal(0);
			FoundAxes[1] = FoundAxes[2].Cross(FoundAxes[0]);
		}
		else // otherwise, for boxes that are closer together, interpolate their orientations
		{
			// Find which axes on the Other.Frame.Rotation are closest to the Frame.Rotation's X and Y axes
			int32 Axis1X_IdxInAxis2 = 0;
			int32 Axis1Y_IdxInAxis2 = 0;
			RealType BestXAlignment = FMath::Abs(Axes1[0].Dot(Axes2[0]));
			RealType BestYAlignment = FMath::Abs(Axes1[1].Dot(Axes2[0]));
			for (int32 Axis2Idx = 1; Axis2Idx < 3; ++Axis2Idx)
			{
				RealType XAlignment = FMath::Abs(Axes1[0].Dot(Axes2[Axis2Idx]));
				RealType YAlignment = FMath::Abs(Axes1[1].Dot(Axes2[Axis2Idx]));
				if (XAlignment > BestXAlignment)
				{
					Axis1X_IdxInAxis2 = Axis2Idx;
					BestXAlignment = XAlignment;
				}
				if (YAlignment > BestYAlignment)
				{
					Axis1Y_IdxInAxis2 = Axis2Idx;
					BestYAlignment = YAlignment;
				}
			}
			if (Axis1X_IdxInAxis2 == Axis1Y_IdxInAxis2)
			{
				// don't let the closest axis to Y be the same as the closest to X
				// (this could happen in a 45-degree case where two axes are tied; should be very rare)
				Axis1Y_IdxInAxis2 = (1 + Axis1Y_IdxInAxis2) % 3;
			}
			RealType SignX = RealType(Axes1[0].Dot(Axes2[Axis1X_IdxInAxis2]) < 0 ? -1 : 1);
			RealType SignY = RealType(Axes1[1].Dot(Axes2[Axis1Y_IdxInAxis2]) < 0 ? -1 : 1);
			FoundAxes[0] = (Axes1[0] + SignX * Axes2[Axis1X_IdxInAxis2]).GetSafeNormal();
			FoundAxes[1] = (Axes1[1] + SignY * Axes2[Axis1Y_IdxInAxis2]);
			// Make sure the axes form an orthonormal frame
			FoundAxes[2] = (FoundAxes[0].Cross(FoundAxes[1])).GetSafeNormal();
			FoundAxes[1] = FoundAxes[2].Cross(FoundAxes[0]).GetSafeNormal();
		}

		// Fit a local AABB in the Found space
		for (int32 CornerIdx = 0; CornerIdx < 8; ++CornerIdx)
		{
			TVector<RealType> WorldCorner1 = Frame.FromFramePoint(Corners1[CornerIdx]);
			TVector<RealType> AxesCorner1(WorldCorner1.Dot(FoundAxes[0]), WorldCorner1.Dot(FoundAxes[1]), WorldCorner1.Dot(FoundAxes[2]));
			FoundBounds.Contain(AxesCorner1);
			TVector<RealType> WorldCorner2 = Other.Frame.FromFramePoint(Corners2[CornerIdx]);
			TVector<RealType> AxesCorner2(WorldCorner2.Dot(FoundAxes[0]), WorldCorner2.Dot(FoundAxes[1]), WorldCorner2.Dot(FoundAxes[2]));
			FoundBounds.Contain(AxesCorner2);
		}
		CandidateVolumes[2] = FoundBounds.Volume();
	}

	// Choose the coordinate space with the smallest bounding box, by volume
	int BestIdx = 0;
	RealType BestVolume = CandidateVolumes[0];
	int AxesToConsider = bOnlyConsiderExistingAxes ? 2 : 3;
	for (int Idx = 1; Idx < AxesToConsider; ++Idx)
	{
		if (CandidateVolumes[Idx] < BestVolume)
		{
			BestIdx = Idx;
			BestVolume = CandidateVolumes[Idx];
		}
	}
	TOrientedBox3<RealType> Result;
	if (BestIdx == 2)
	{
		// Convert the Found Axes into a new oriented box
		Result.Frame.Rotation.SetFromRotationMatrix(TMatrix3<RealType>(FoundAxes[0], FoundAxes[1], FoundAxes[2], false));
		TVector<RealType> FoundCenter = FoundBounds.Center();
		Result.Frame.Origin = FoundAxes[0] * FoundCenter.X + FoundAxes[1] * FoundCenter.Y + FoundAxes[2] * FoundCenter.Z;
		Result.Extents = FoundBounds.Extents();
	}
	else if (BestIdx == 0)
	{
		Result.Frame.Rotation = Frame.Rotation;
		Result.Frame.Origin = Frame.FromFramePoint(LocalBox1.Center());
		Result.Extents = LocalBox1.Extents();
	}
	else // BestIdx == 1
	{
		Result.Frame.Rotation = Other.Frame.Rotation;
		Result.Frame.Origin = Other.Frame.FromFramePoint(LocalBox2.Center());
		Result.Extents = LocalBox2.Extents();
	}
	// Add a small tolerance so that the input boxes are more likely to be properly contained in the merged result, after floating point error
	Result.Extents += TVector<RealType>(TMathUtil<RealType>::ZeroTolerance);
	return Result;
}

template<typename RealType>
TOrientedBox3<RealType> TOrientedBox3<RealType>::ReparameterizedCloserToWorldFrame() const
{
	TVector<RealType> Axes[3]{ AxisX(), AxisY(), AxisZ() };

	int32 BestZ = FMath::Max3Index(
		FMath::Abs(Axes[0].Z),
		FMath::Abs(Axes[1].Z),
		FMath::Abs(Axes[2].Z));
	TVector<RealType> NewZ = Axes[BestZ] * (Axes[BestZ].Z > 0 ? 1 : -1);

	// Pick the best Y out of the two remaining
	double DotY[3]{
		Axes[0].Y,
		Axes[1].Y,
		Axes[2].Y
	};
	DotY[BestZ] = 0; // don't pick this one

	int32 BestY = FMath::Max3Index(
		FMath::Abs(DotY[0]),
		FMath::Abs(DotY[1]),
		FMath::Abs(DotY[2]));
	TVector<RealType> NewY = Axes[BestY] * (DotY[BestY] > 0 ? 1 : -1);

	// Sum of BestY and BestZ will be either 0+1=1, 0+2=2, or 1+2=3, and the 
	// corresponding leftover index will be 2, 1, or 0.
	int32 BestX = 3 - (BestY + BestZ);
	// Static analyzer probably won't like that, so here's a clamp for safety
	BestX = FMath::Clamp(BestX, 0, 2);

	// Sanity check to make sure we picked unique axes
	if (!ensure(BestX != BestY
		&& BestY != BestZ
		&& BestX != BestZ))
	{
		return *this;
	}

	return TOrientedBox3<RealType>(
		TFrame3<RealType>(Frame.Origin,
			NewY.Cross(NewZ), // better to do this than risk picking the wrong direction for BestX
			NewY,
			NewZ),
		TVector<RealType>(Extents[BestX], Extents[BestY], Extents[BestZ]));
}

template struct TOrientedBox3<float>;
template struct TOrientedBox3<double>;

} // namespace UE::Geometry

