// Copyright Epic Games, Inc. All Rights Reserved.


#include "Parameterization/UVPacking.h"
#include "Parameterization/UVSpaceAllocator.h"

#include "Async/Future.h"
#include "Async/Async.h"

#include "Misc/SecureHash.h"

#include "BoxTypes.h"

using namespace UE::Geometry;

// Hash function to use FMD5Hash in TMap
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}


namespace UE { namespace InternalUVPacking {

	using namespace UE::Geometry;

//
// local representation of a UV island as a set of triangle indices
//
struct FUVIsland
{
	// Store a unique id so that we can come back to the initial Charts ordering when necessary
	int32		Id; 

	// Set of triangles that make up this UV island. Assumption is this is single connected-component, 
	// otherwise multiple islands will be grouped.
	TArray<int32> Triangles;

	// axis-aligned 2D bounding box min/max
	FVector2d	MinUV;
	FVector2d	MaxUV;

	double		UVArea;
	double		ScaleToWorld; // Scale factor that would make 1 texel ~= 1 unit in world space

	FVector2d	UVScale;
	FMatrix2d   UVRotation;
	FMatrix2d   UVPackingScale;

	FVector2d   OrientationBiasShift;
	FVector2d	PackingBias;

	FVector2d   GetPackingBias() const
	{		
		return (UVRotation * (UVPackingScale * OrientationBiasShift)) + 0.5 + PackingBias;
	}
};




//
//
// Chart Packer for UV islands of a FDynamicMesh3
// This code is a port of the FLayoutUV class/implementation to FDynamicMesh3.
// The packing strategy is generally the same, however:
//    - additional control over flips has been added
//    - island merging support was removed, input islands must be externally computed
//    - backwards-compatibility paths were removed
//
class FStandardChartPacker
{
public:

	// Mesh with UV access methods
	FUVPacker::IUVMeshView* Mesh;

	// packing target texture resolution, used to calculate gutter/border size
	uint32 TextureResolution = 512;

	// if true, disallows any change to island rotations while packing. Implicitly disables bAllowFlips, if set.
	bool bPreserveRotation = false;

	// if true, UV islands can be mirrored in X and/or Y to improve packing
	bool bAllowFlips = false;

	double TotalUVArea = 0;

	// Top-level function, packs input charts into positive-unit-square
	bool FindBestPacking(TArray<FUVIsland>& AllCharts);

protected:
	// 
	void ScaleCharts(TArray<FUVIsland>& Charts, double UVScale);
	bool PackCharts(TArray<FUVIsland>& Charts, double UVScale, double& OutEfficiency, TAtomic<bool>& bAbort);
	void OrientChart(FUVIsland& Chart, int32 Orientation);
	void RasterizeChart(const FUVIsland& Chart, uint32 RectW, uint32 RectH, FUVSpaceAllocator& OutChartRaster);
};



bool FStandardChartPacker::FindBestPacking(TArray<FUVIsland>& Charts)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StandardChartPacker_FindBestPacking);
	
	if ( (uint32)Charts.Num() > TextureResolution * TextureResolution)
	{
		// More charts than texels
		return false;
	}

	TotalUVArea = 0.0;
	for (const FUVIsland& Chart : Charts)
	{
		TotalUVArea += Chart.UVArea * Chart.ScaleToWorld * Chart.ScaleToWorld;
	}

	if (TotalUVArea <= 0.0)
	{
		return false;
	}

	// Cleanup uninitialized values to get a stable input hash
	for (FUVIsland& Chart : Charts)
	{
		Chart.PackingBias = FVector2d::Zero();
		Chart.UVScale = FVector2d::Zero();
		Chart.UVPackingScale = FMatrix2d::Identity();
		Chart.UVRotation = FMatrix2d::Identity();
		Chart.OrientationBiasShift = FVector2d::Zero();
	}

	// Those might require tuning, changing them won't affect the outcome and will maintain backward compatibility
	const int32 MultithreadChartsCountThreshold = 100 * 1000;
	const int32 MultithreadTextureResolutionThreshold = 1000;
	const int32 MultithreadAheadWorkCount = 3;

	const double LinearSearchStart = 0.5;
	const double LinearSearchStep = 0.5;
	const int32 BinarySearchSteps = 6;

	double UVScaleFail = TextureResolution * FMathd::Sqrt(1.0 / TotalUVArea);
	double UVScalePass = TextureResolution * FMathd::Sqrt(LinearSearchStart / TotalUVArea);

	// Store successful charts packing to avoid redoing the final step
	TArray<FUVIsland>         LastPassCharts;
	TAtomic<bool>              bAbort(false);

	struct FThreadContext
	{
		TArray<FUVIsland> Charts;
		TFuture<bool>      Result;
		double              Efficiency = 0.0;
	};

	TArray<FThreadContext> ThreadContexts;

	bool bShouldUseMultipleThreads =  
		Charts.Num()>= MultithreadChartsCountThreshold &&
		TextureResolution>= MultithreadTextureResolutionThreshold;

	if (bShouldUseMultipleThreads)
	{
		// Do forward work only when multi-thread activated
		ThreadContexts.SetNum(MultithreadAheadWorkCount);
	}

	// Linear search for first fit
	double LastEfficiency = 0.0f;
	{
		while (!bAbort)
		{
			// Launch forward work in other threads
			for (int32 Index = 0; Index <ThreadContexts.Num(); ++Index)
			{
				ThreadContexts[Index].Charts = Charts;
				double ThreadUVScale = UVScalePass * FMathd::Pow(LinearSearchStep, (double)(Index + 1));
				ThreadContexts[Index].Result =
					Async(
						EAsyncExecution::ThreadPool,
						[this, &ThreadContexts, &bAbort, ThreadUVScale, Index]()
				{
					return PackCharts(ThreadContexts[Index].Charts, ThreadUVScale, ThreadContexts[Index].Efficiency, bAbort);
				}
				);
			}

			// Process the first iteration in this thread
			bool bFit = false;
			{
				bFit = PackCharts(Charts, UVScalePass, LastEfficiency, bAbort);
			}

			// Wait for the work sequentially and cancel everything once we have a first viable solution
			for (int32 Index = 0; Index <ThreadContexts.Num() + 1; ++Index)
			{
				// The first result is not coming from a future
				bFit = Index == 0 ? bFit : ThreadContexts[Index - 1].Result.Get();
				if (bFit && !bAbort)
				{
					// We got a success, cancel other searches
					bAbort = true;

					if (Index> 0)
					{
						Charts = ThreadContexts[Index - 1].Charts;
						LastEfficiency = ThreadContexts[Index - 1].Efficiency;
					}

					LastPassCharts = Charts;
				}

				if (!bAbort)
				{
					UVScaleFail = UVScalePass;
					UVScalePass *= LinearSearchStep;
				}
			}
		}
	}

	// Binary search for best fit
	{
		bAbort = false;
		for (int32 i = 0; i <BinarySearchSteps; i++)
		{
			double UVScale = 0.5f * (UVScaleFail + UVScalePass);

			double Efficiency = 0.0f;
			bool bFit = PackCharts(Charts, UVScale, Efficiency, bAbort);
			if (bFit)
			{
				LastPassCharts = Charts;
				double EfficiencyGainPercent = 100.0f * FMathd::Abs(Efficiency - LastEfficiency);
				LastEfficiency = Efficiency;

				// Early out when we're inside a 1% efficiency range
				if (EfficiencyGainPercent <= 1.0f)
				{
					break;
				}

				UVScalePass = UVScale;
			}
			else
			{
				UVScaleFail = UVScale;
			}
		}
	}

	// In case the last step was a failure, restore from last known good computation
	Charts = LastPassCharts;

	return true;
}






void FStandardChartPacker::ScaleCharts( TArray<FUVIsland>& Charts, double UVScale )
{
	for( int32 i = 0; i <Charts.Num(); i++ )
	{
		FUVIsland& Chart = Charts[i];
		Chart.UVScale = FVector2d::One() * UVScale * Chart.ScaleToWorld;
	}
	
	// Unsort the charts to make sure ScaleCharts always return the same ordering
	Algo::IntroSort( Charts, []( const FUVIsland& A, const FUVIsland& B )
	{
		return A.Id < B.Id;
	});

	// Scale charts such that they all fit and roughly total the same area as before
#if 1
	double UniformScale = 1.0f;
	for (int i = 0; i < 1000; i++)
	{
		uint32 NumMaxedOut = 0;
		double ScaledUVArea = 0.0f;
		for (int32 ChartIndex = 0; ChartIndex < Charts.Num(); ChartIndex++)
		{
			FUVIsland& Chart = Charts[ChartIndex];

			FVector2d ChartSize = Chart.MaxUV - Chart.MinUV;
			FVector2d ChartSizeScaled = ChartSize * Chart.UVScale * UniformScale;

			const double MaxChartEdge = TextureResolution - 1.0;
			const double LongestChartEdge = FMathd::Max(ChartSizeScaled.X, ChartSizeScaled.Y);

			const double Epsilon = 0.01f;
			if (LongestChartEdge + Epsilon > MaxChartEdge)
			{
				// Rescale oversized charts to fit
				Chart.UVScale.X = MaxChartEdge / FMathd::Max(ChartSize.X, ChartSize.Y);
				Chart.UVScale.Y = MaxChartEdge / FMathd::Max(ChartSize.X, ChartSize.Y);
				NumMaxedOut++;
			}
			else
			{
				Chart.UVScale.X *= UniformScale;
				Chart.UVScale.Y *= UniformScale;
			}

			ScaledUVArea += Chart.UVArea * Chart.UVScale.X * Chart.UVScale.Y;
		}

		if (NumMaxedOut == 0)
		{
			// No charts maxed out so no need to rebalance
			break;
		}

		if (NumMaxedOut == Charts.Num())
		{
			// All charts are maxed out
			break;
		}

		// Scale up smaller charts to maintain expected total area
		// Want ScaledUVArea == TotalUVArea * UVScale^2
		double RebalanceScale = UVScale * FMathd::Sqrt(TotalUVArea / ScaledUVArea);
		if (RebalanceScale < 1.01f)
		{
			// Stop if further rebalancing is minor
			break;
		}
		UniformScale = RebalanceScale;
	}
#endif

#if 0 // TODO: re-enable under a boolean flag if we find a case where we want to allow non-uniform scaling
	double NonuniformScale = 1.0f;
	for (int i = 0; i < 1000; i++)
	{
		uint32 NumMaxedOut = 0;
		double ScaledUVArea = 0.0f;
		for (int32 ChartIndex = 0; ChartIndex < Charts.Num(); ChartIndex++)
		{
			FUVIsland& Chart = Charts[ChartIndex];

			for (int k = 0; k < 2; k++)
			{
				const double MaximumChartSize = TextureResolution - 1.0f;
				const double ChartSize = Chart.MaxUV[k] - Chart.MinUV[k];
				const double ChartSizeScaled = ChartSize * Chart.UVScale[k] * NonuniformScale;

				const double Epsilon = 0.01f;
				if (ChartSizeScaled + Epsilon > MaximumChartSize)
				{
					// Scale oversized charts to max size
					Chart.UVScale[k] = MaximumChartSize / ChartSize;
					NumMaxedOut++;
				}
				else
				{
					Chart.UVScale[k] *= NonuniformScale;
				}
			}

			ScaledUVArea += Chart.UVArea * Chart.UVScale.X * Chart.UVScale.Y;
		}

		if (NumMaxedOut == 0)
		{
			// No charts maxed out so no need to rebalance
			break;
		}

		if (NumMaxedOut == Charts.Num() * 2)
		{
			// All charts are maxed out in both dimensions
			break;
		}

		// Scale up smaller charts to maintain expected total area
		// Want ScaledUVArea == TotalUVArea * UVScale^2
		double RebalanceScale = UVScale * FMathd::Sqrt(TotalUVArea / ScaledUVArea);
		if (RebalanceScale < 1.01f)
		{
			// Stop if further rebalancing is minor
			break;
		}
		NonuniformScale = RebalanceScale;
	}
#endif

	// Sort charts from largest to smallest
	struct FCompareCharts
	{
		FORCEINLINE bool operator()( const FUVIsland& A, const FUVIsland& B ) const
		{
			// Rect area
			FVector2d ChartRectA = ( A.MaxUV - A.MinUV ) * A.UVScale;
			FVector2d ChartRectB = ( B.MaxUV - B.MinUV ) * B.UVScale;
			return ChartRectA.X * ChartRectA.Y> ChartRectB.X * ChartRectB.Y;
		}
	};
	Algo::IntroSort( Charts, FCompareCharts() );
}



bool FStandardChartPacker::PackCharts(TArray<FUVIsland>& Charts, double UVScale, double& OutEfficiency, TAtomic<bool>& bAbort)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StandardChartPacker_PackCharts);
	
	ScaleCharts(Charts, UVScale);

	FUVSpaceAllocator BestChartRaster(FUVSpaceAllocator::EMode::UsedSegments, TextureResolution, TextureResolution);
	FUVSpaceAllocator ChartRaster(FUVSpaceAllocator::EMode::UsedSegments, TextureResolution, TextureResolution);
	FUVSpaceAllocator LayoutRaster(FUVSpaceAllocator::EMode::FreeSegments, TextureResolution, TextureResolution);

	uint64 RasterizeCycles = 0;
	uint64 FindCycles = 0;

	OutEfficiency = 0.0f;
	LayoutRaster.Clear();

	// Store the position where we found a spot for each unique raster
	// so we can skip whole sections we know won't work out.
	// This method is obviously more efficient with smaller charts
	// but helps tremendously as the number of charts goes up for
	// the same texture space. This helps counteract the slowdown
	// induced by having more parts to place in the grid and is
	// particularly useful for foliage.
	TMap<FMD5Hash, FVector2d> BestStartPos;

	// Reduce Insights CPU tracing to once per batch
	const int32 BatchSize = 1024;
	for (int32 ChartIndex = 0; ChartIndex <Charts.Num() && !bAbort.Load(EMemoryOrder::Relaxed); )
	{
		for (int32 BatchIndex = 0; BatchIndex <BatchSize && ChartIndex <Charts.Num() && !bAbort.Load(EMemoryOrder::Relaxed); ++ChartIndex, ++BatchIndex)
		{
			FUVIsland& Chart = Charts[ChartIndex];

			// Try different orientations and pick best
			int32				BestOrientation = -1;
			FUVSpaceAllocator::FRect	BestRect = { ~0u, ~0u, ~0u, ~0u };

			// This version focus on minimal surface area giving fairness to both horizontal and vertical chart placement
			// instead of only taking the pixel offset of the lower left corner into account.
			TFunction<bool(const FUVSpaceAllocator::FRect&)> IsBestRect =
				[&BestRect](const FUVSpaceAllocator::FRect& Rect)
			{
				return ((Rect.X + Rect.W) + (Rect.Y + Rect.H)) <((BestRect.X + BestRect.W) + (BestRect.Y + BestRect.H));
			};

			// simpler thing?
			//TFunction<bool(const FUVSpaceAllocator::FRect&)> IsBestRect =
			//	[this, &BestRect](const FUVSpaceAllocator::FRect& Rect)
			//{
			//	return Rect.X + Rect.Y * TextureResolution <BestRect.X + BestRect.Y * TextureResolution;
			//};

			int32 MaxOrientation = bPreserveRotation ? 1 : 8;
			int32 OrientationStep = (bAllowFlips) ? 1 : 2;
			for (int32 Orientation = 0; Orientation < MaxOrientation; Orientation += OrientationStep)
			{
				// TODO If any dimension is less than 1 pixel shrink dimension to zero

				OrientChart(Chart, Orientation);

				FVector2d ChartSize = Chart.MaxUV - Chart.MinUV;
				ChartSize = Chart.UVRotation * Chart.UVPackingScale * ChartSize;

				// Only need half pixel dilate for rects
				FUVSpaceAllocator::FRect	Rect;
				Rect.X = 0;
				Rect.Y = 0;
				Rect.W = FMath::CeilToInt( (float)FMathd::Abs(ChartSize.X) + 1.0f);
				Rect.H = FMath::CeilToInt( (float)FMathd::Abs(ChartSize.Y) + 1.0f);

				// Just in case lack of precision pushes it over
				Rect.W = FMath::Min(TextureResolution, Rect.W);
				Rect.H = FMath::Min(TextureResolution, Rect.H);

				const bool bRectPack = false;

				if (bRectPack)
				{
					if (LayoutRaster.Find(Rect))
					{
						if (IsBestRect(Rect))
						{
							BestOrientation = Orientation;
							BestRect = Rect;
						}
					}
					else
					{
						continue;
					}
				}
				else
				{

					// This block is a little confusing - the intent here is that orientations
					// that are flips always follow orientations that are their flip opposite... 
					// So, we can avoid explicit rasterization for the flip orientations and
					// just flip the previous iteration's raster instead.
					// Technically we can rasterize in all cases and then flip, but it would
					// probably be more expensive?
					if (Orientation % 4 == 1)
					{
						ChartRaster.FlipX(Rect);
					}
					else if (Orientation % 4 == 3)
					{
						ChartRaster.FlipY(Rect);
					}
					else
					{
						RasterizeChart(Chart, Rect.W, Rect.H, ChartRaster);
					}

					bool bFound = false;

					// Use the real raster size for optimal placement
					FUVSpaceAllocator::FRect RasterRect = Rect;
					RasterRect.W = ChartRaster.GetRasterWidth();
					RasterRect.H = ChartRaster.GetRasterHeight();

					// Nothing rasterized, returning 0,0 as fast as possible
					// since this is what the actual algorithm is doing but
					// we might have to flag the entire UV map as invalid since
					// charts are going to overlap
					if (RasterRect.H == 0 && RasterRect.W == 0)
					{
						Rect.X = 0;
						Rect.Y = 0;
						bFound = true;
					}
					else
					{
						FMD5Hash RasterMD5 = ChartRaster.GetRasterMD5();
						FVector2d* StartPos = BestStartPos.Find(RasterMD5);

						if (StartPos)
						{
							RasterRect.X = FMath::FloorToInt32(StartPos->X);
							RasterRect.Y = FMath::FloorToInt32(StartPos->Y);
						}

						bFound = LayoutRaster.FindWithSegments(RasterRect, ChartRaster, IsBestRect);
						if (bFound)
						{
							// Store only the best possible position in the hash table so we can start from there for other identical charts
							BestStartPos.Add(RasterMD5, FVector2d(RasterRect.X, RasterRect.Y));

							// Since the older version stops searching at Width - Rect.W instead of using the raster size,
							// it means a perfect rasterized square of 2,2 won't fit a 2,2 hole at the end of a row if Rect.W = 3.
							// Because of that, we have no choice to worsen our algorithm behavior for backward compatibility.

							// Once we know the best possible position, we'll continue our search from there with the original
							// rect value if it differs from the raster rect to ensure we get the same result as the old algorithm.
							//if (LayoutVersion <ELightmapUVVersion::Segments2D && (Rect.X != RasterRect.X || Rect.Y != RasterRect.Y))
							//{
							//	Rect.X = RasterRect.X;
							//	Rect.Y = RasterRect.Y;

							//	bFound = LayoutRaster.FindWithSegments(Rect, ChartRaster, IsBestRect);
							//}
							//else
							//{
							//	// We can't copy W and H here as they might be different than what we got initially
							//	Rect.X = RasterRect.X;
							//	Rect.Y = RasterRect.Y;
							//}

							// We can't copy W and H here as they might be different than what we got initially
							Rect.X = RasterRect.X;
							Rect.Y = RasterRect.Y;

						}
					}

				
					//if (true)
					//{
					//	UE_LOGF(LogTemp, Log, "[LAYOUTUV_TRACE] Chart %d Orientation %d Found = %d Rect = %d,%d,%d,%d\n", ChartIndex, Orientation, bFound ? 1 : 0, Rect.X, Rect.Y, Rect.W, Rect.H);
					//}

					if (bFound)
					{
						if (IsBestRect(Rect))
						{
							BestChartRaster = ChartRaster;

							BestOrientation = Orientation;
							BestRect = Rect;

							if (BestRect.X == 0 && BestRect.Y == 0)
							{
								// BestRect can't be beat, stop here
								break;
							}
						}
					}
					else
					{
						continue;
					}
				}
			}

			if (BestOrientation>= 0)
			{
				// Add chart to layout
				OrientChart(Chart, BestOrientation);

				LayoutRaster.Alloc(BestRect, BestChartRaster);

				Chart.PackingBias.X += BestRect.X;
				Chart.PackingBias.Y += BestRect.Y;
			}
			else
			{
				//if (true)
				//{
				//	UE_LOGF(LogTemp, Log, "[LAYOUTUV_TRACE] Chart %d Found no orientation that fit\n", ChartIndex);
				//}

				// Found no orientation that fit
				return false;
			}
		}
	}

	if (bAbort)
	{
		return false;
	}

	const uint32 TotalTexels = TextureResolution * TextureResolution;
	const uint32 UsedTexels = LayoutRaster.GetUsedTexels();

	OutEfficiency = double(UsedTexels) / TotalTexels;

	return true;
}






void FStandardChartPacker::OrientChart(FUVIsland& Chart, int32 Orientation)
{
	Chart.PackingBias = FVector2d::Zero();
	Chart.UVPackingScale = FMatrix2d(Chart.UVScale.X, 0.0, 0.0, Chart.UVScale.Y);

	/*
	
	Note: The packing bias is being initialized here to "correct" the positioning of the chart's bounding box after rotation.

	How this works: 

	1) Rotate the box, assuming 0,0 is the upper left hand corner

	2) Shift box, so that the new upper left hand corner of the rotated box lands at 0,0 again.
	
	*/


	switch (Orientation)
	{
	case 0:
		// 0 degrees
		Chart.UVRotation = FMatrix2d(1.0, 0.0, 0.0, 1.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MinUV.X, -Chart.MinUV.Y);
		break;
	case 1:
		// 0 degrees, flip x
		Chart.UVRotation = FMatrix2d(-1.0, 0.0, 0.0, 1.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MaxUV.X, -Chart.MinUV.Y);
		break;
	case 2:
		// 90 degrees
		Chart.UVRotation = FMatrix2d(0.0, 1.0, -1.0, 0.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MaxUV.X, -Chart.MinUV.Y);
		break;
	case 3:
		// 90 degrees, flip x
		Chart.UVRotation = FMatrix2d(0.0, 1.0, 1.0, 0.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MinUV.X, -Chart.MinUV.Y);
		break;
	case 4:
		// 180 degrees
		Chart.UVRotation = FMatrix2d(-1.0, 0.0, 0.0, -1.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MaxUV.X, -Chart.MaxUV.Y);
		break;
	case 5:
		// 180 degrees, flip x
		Chart.UVRotation = FMatrix2d(1.0, 0.0, 0.0, -1.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MinUV.X, -Chart.MaxUV.Y);
		break;
	case 6:
		// 270 degrees
		Chart.UVRotation = FMatrix2d(0.0, -1.0, 1.0, 0.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MinUV.X, -Chart.MaxUV.Y);
		break;
	case 7:
		// 270 degrees, flip x
		Chart.UVRotation = FMatrix2d(0.0, -1.0, -1.0, 0.0);
		Chart.OrientationBiasShift = FVector2d(-Chart.MaxUV.X, -Chart.MaxUV.Y);
		break;
	}
}



// Max of 2048x2048 due to precision
// Dilate in 28.4 fixed point. Half pixel dilation is conservative rasterization.
// Dilation same as Minkowski sum of triangle and square.
template<int32 Dilate>
void InternalRasterizeTriangle(FUVSpaceAllocator& Shader, const FVector2f Points[3], int32 ScissorWidth, int32 ScissorHeight)
{
	const FVector2f HalfPixel(0.5f, 0.5f);
	FVector2f p0 = Points[0] - HalfPixel;
	FVector2f p1 = Points[1] - HalfPixel;
	FVector2f p2 = Points[2] - HalfPixel;

	// Correct winding
	float Facing = (p0.X - p1.X) * (p2.Y - p0.Y) - (p0.Y - p1.Y) * (p2.X - p0.X);
	if (Facing <0.0f)
	{
		Swap(p0, p2);
	}

	// 28.4 fixed point
	const int32 X0 = (int32)(16.0f * p0.X + 0.5f);
	const int32 X1 = (int32)(16.0f * p1.X + 0.5f);
	const int32 X2 = (int32)(16.0f * p2.X + 0.5f);

	const int32 Y0 = (int32)(16.0f * p0.Y + 0.5f);
	const int32 Y1 = (int32)(16.0f * p1.Y + 0.5f);
	const int32 Y2 = (int32)(16.0f * p2.Y + 0.5f);

	// Bounding rect
	int32 MinX = (FMath::Min3(X0, X1, X2) - Dilate + 15) / 16;
	int32 MaxX = (FMath::Max3(X0, X1, X2) + Dilate + 15) / 16;
	int32 MinY = (FMath::Min3(Y0, Y1, Y2) - Dilate + 15) / 16;
	int32 MaxY = (FMath::Max3(Y0, Y1, Y2) + Dilate + 15) / 16;

	// Clip to image
	MinX = FMath::Clamp(MinX, 0, ScissorWidth);
	MaxX = FMath::Clamp(MaxX, 0, ScissorWidth);
	MinY = FMath::Clamp(MinY, 0, ScissorHeight);
	MaxY = FMath::Clamp(MaxY, 0, ScissorHeight);

	// Deltas
	const int32 DX01 = X0 - X1;
	const int32 DX12 = X1 - X2;
	const int32 DX20 = X2 - X0;

	const int32 DY01 = Y0 - Y1;
	const int32 DY12 = Y1 - Y2;
	const int32 DY20 = Y2 - Y0;

	// Half-edge constants
	int32 C0 = DY01 * X0 - DX01 * Y0;
	int32 C1 = DY12 * X1 - DX12 * Y1;
	int32 C2 = DY20 * X2 - DX20 * Y2;

	// Correct for fill convention
	C0 += (DY01 <0 || (DY01 == 0 && DX01> 0)) ? 0 : -1;
	C1 += (DY12 <0 || (DY12 == 0 && DX12> 0)) ? 0 : -1;
	C2 += (DY20 <0 || (DY20 == 0 && DX20> 0)) ? 0 : -1;

	// Dilate edges
	C0 += (abs(DX01) + abs(DY01)) * Dilate;
	C1 += (abs(DX12) + abs(DY12)) * Dilate;
	C2 += (abs(DX20) + abs(DY20)) * Dilate;

	for (int32 y = MinY; y <MaxY; y++)
	{
		for (int32 x = MinX; x <MaxX; x++)
		{
			// same as Edge1>= 0 && Edge2>= 0 && Edge3>= 0
			int32 IsInside;
			IsInside = C0 + (DX01 * y - DY01 * x) * 16;
			IsInside |= C1 + (DX12 * y - DY12 * x) * 16;
			IsInside |= C2 + (DX20 * y - DY20 * x) * 16;

			if (IsInside>= 0)
			{
				Shader.SetBit(x, y);
			}
		}
	}
}




void FStandardChartPacker::RasterizeChart(const FUVIsland& Chart, uint32 RectW, uint32 RectH, FUVSpaceAllocator& OutChartRaster)
{
	// Bilinear footprint is -1 to 1 pixels. If packed geometrically, only a half pixel dilation
	// would be needed to guarantee all charts were at least 1 pixel away, safe for bilinear filtering.
	// Unfortunately, with pixel packing a full 1 pixel dilation is required unless chart edges exactly
	// align with pixel centers.

	OutChartRaster.Clear();

	for (int32 tid : Chart.Triangles)
	{
		FIndex3i UVTriangle = Mesh->GetUVTriangle(tid);
		if (UVTriangle.A < 0 || UVTriangle.B < 0 || UVTriangle.C < 0)
		{
			continue;
		}

		FVector2f Points[3];
		for (int k = 0; k <3; k++)
		{
			FVector2d UV = (FVector2d)Mesh->GetUV(UVTriangle[k]);
			Points[k] = (FVector2f)((Chart.UVRotation * Chart.UVPackingScale * UV) + Chart.GetPackingBias());
		}

		InternalRasterizeTriangle<16>(OutChartRaster, Points, RectW, RectH);
	}

	OutChartRaster.CreateUsedSegments();
}


}} // namespace UE::InternalUVPacking


bool FUVPacker::StandardPack(IUVMeshView* Mesh, int NumIslands, TFunctionRef<void(int, TArray<int32>&)> CopyIsland)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVPacker_StandardPack);
	
	using namespace UE::InternalUVPacking;
	FStandardChartPacker Packer;

	Packer.Mesh = Mesh;

	// Clamp resolution to what the packer can handle --
	// Lower bound of 2 b/c the rebalance step uses MaxChartEdge = TextureResolution - 1
	// Upper bound of 16384 as the largest plausible texture the UVs could address (to avoid OOM; inspired by RHI limits)
	const int32 MaxTextureResolution = 16384;
	const int32 ClampedResolution = FMath::Clamp(this->TextureResolution, 2, MaxTextureResolution);
	if (ClampedResolution != this->TextureResolution)
	{
		UE_LOG(LogGeometry, Warning,
			TEXT("FUVPacker::StandardPack: TextureResolution %d out of supported range [2, %d]; clamping to %d."),
			this->TextureResolution, MaxTextureResolution, ClampedResolution);
	}
	Packer.TextureResolution = ClampedResolution;
	Packer.bPreserveRotation = this->bPreserveRotation;
	Packer.bAllowFlips = this->bAllowFlips;

	int32 NumCharts = NumIslands;
	TArray<FUVIsland> AllCharts;
	AllCharts.SetNum(NumCharts);
	for (int32 ci = 0; ci < NumCharts; ++ci)
	{
		FUVIsland& Chart = AllCharts[ci];

		Chart.Id = ci + 1;

		CopyIsland(ci, Chart.Triangles);

		Chart.MinUV = FVector2d(FLT_MAX, FLT_MAX);
		Chart.MaxUV = FVector2d(-FLT_MAX, -FLT_MAX);
		Chart.UVArea = 0.0f;
		double UVLengthSum = 0, WorldLengthSum = 0;

		FAxisAlignedBox2d IslandBounds;
		GetIslandStats(Mesh, Chart.Triangles, IslandBounds, Chart.ScaleToWorld, Chart.UVArea);
		Chart.MinUV = IslandBounds.Min;
		Chart.MaxUV = IslandBounds.Max;
	}


	bool bPackingFound = Packer.FindBestPacking(AllCharts);
	if (bPackingFound == false)
	{
		return false;
	}

	// Commit chart UVs
	for (int32 i = 0; i <AllCharts.Num(); i++)
	{
		FUVIsland& Chart = AllCharts[i];

		FVector2d PackingBias = Chart.GetPackingBias();
		PackingBias /= (double)Packer.TextureResolution;
		FMatrix2d NormalizedScale = Chart.UVPackingScale * (1.0 / Packer.TextureResolution);

		auto PackUVsWithScaling = [&Mesh, &Chart, &NormalizedScale, &PackingBias](TSet<int32>& IslandElements)
		{
			for (int32 elemid : IslandElements)
			{
				if (elemid >= 0)
				{
					FVector2d UV = (FVector2d)Mesh->GetUV(elemid);
					FVector2d TransformedUV = (Chart.UVRotation * NormalizedScale * UV) + PackingBias;
					Mesh->SetUV(elemid, (FVector2f)TransformedUV);
				}
			}
		};

		auto PackUVsWithoutScaling = [&Mesh, &Chart, &NormalizedScale, &PackingBias](TSet<int32>& IslandElements)
		{
			FMatrix2d InverseScale = NormalizedScale.Inverse();

			for (int32 elemid : IslandElements)
			{
				if (elemid >= 0)
				{
					FVector2d UV = (FVector2d)Mesh->GetUV(elemid);
					FVector2d TransformedUV = InverseScale * ((Chart.UVRotation * NormalizedScale * UV) + PackingBias);
					Mesh->SetUV(elemid, (FVector2f)TransformedUV);
				}
			}
		};

		TSet<int32> IslandElements;

		for (int32 tid : Chart.Triangles)
		{
			FIndex3i Triangle = Mesh->GetUVTriangle(tid);
			IslandElements.Add(Triangle.A);
			IslandElements.Add(Triangle.B);
			IslandElements.Add(Triangle.C);
		}

		if (bPreserveScale)
		{
			PackUVsWithoutScaling(IslandElements);
		}
		else
		{
			PackUVsWithScaling(IslandElements);
		}
	}

	return bPackingFound;
}

bool FUVPacker::StackPack(IUVMeshView* Mesh, int NumIslands, TFunctionRef<void(int, TArray<int32>&)> CopyIsland)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVPacker_StackPack);
	
	double GutterWidth = (double)GutterSize / (double)TextureResolution;

	int32 NumCharts = NumIslands;

	// figure out maximum width and height of existing charts
	TArray<FAxisAlignedBox2d> AllIslandBounds;
	AllIslandBounds.SetNum(NumCharts);
	TArray<double> AllIslandScaleFactors;
	AllIslandScaleFactors.SetNum(NumCharts);
	double MaxWidth = 0, MaxHeight = 0;
	for (int32 ci = 0; ci < NumCharts; ++ci)
	{
		TArray<int32> Island;
		CopyIsland(ci, Island);
		FAxisAlignedBox2d IslandBounds;
		double IslandScaleFactor, UVArea;
		GetIslandStats(Mesh, Island, IslandBounds, IslandScaleFactor, UVArea);
		AllIslandBounds[ci] = IslandBounds;
		AllIslandScaleFactors[ci] = IslandScaleFactor;

		MaxWidth = FMathd::Max(IslandBounds.Width() * IslandScaleFactor, MaxWidth);
		MaxHeight = FMathd::Max(IslandBounds.Height() * IslandScaleFactor, MaxHeight);
	}

	// figure out uniform scale that will make them all fit
	double TargetWidth = 1.0 - 2 * GutterWidth;
	double TargetHeight = 1.0 - 2 * GutterWidth;
	double WidthScale = TargetWidth / MaxWidth;
	double HeightScale = TargetWidth / MaxHeight;
	double UseUniformScale = FMathd::Min(WidthScale, HeightScale);

	// transform them
	TSet<int32> IslandElements;
	for (int32 ci = 0; ci < NumCharts; ++ci)
	{
		IslandElements.Reset();
		FAxisAlignedBox2d IslandBounds = AllIslandBounds[ci];
		TArray<int32> Island;
		CopyIsland(ci, Island);
		for (int32 tid : Island)
		{
			FIndex3i UVTri = Mesh->GetUVTriangle(tid);
			if (UVTri[0] >= 0 && UVTri[1] >= 0 && UVTri[2] >= 0)
			{
				IslandElements.Add(UVTri[0]);
				IslandElements.Add(UVTri[1]);
				IslandElements.Add(UVTri[2]);
			}
		}

		double ScaleFactor = UseUniformScale * AllIslandScaleFactors[ci];
		for (int32 elemid : IslandElements)
		{
			if (elemid >= 0)
			{
				FVector2d CurUV = (FVector2d)Mesh->GetUV(elemid);
				FVector2d NewUV = (CurUV - IslandBounds.Min) * ScaleFactor;
				Mesh->SetUV(elemid, (FVector2f)NewUV);
			}
		}
	}

	return true;
}


void FUVPacker::GetIslandStats(IUVMeshView* Mesh, const TArray<int32>& Island, FAxisAlignedBox2d& IslandBoundsOut, double& IslandScaleFactorOut, double& UVAreaOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVPacker_GetIslandStats);

	IslandBoundsOut = FAxisAlignedBox2d::Empty();
	UVAreaOut = 0.0f;
	double UVLengthSum = 0, WorldLengthSum = 0;

	for (int32 tid : Island)
	{
		FIndex3i Triangle3D = Mesh->GetTriangle(tid);
		FIndex3i TriangleUV = Mesh->GetUVTriangle(tid);

		// skip invalid UV-triangles
		if (TriangleUV.A < 0 || TriangleUV.B < 0 || TriangleUV.C < 0)
		{
			continue;
		}

		FVector3d Positions[3];
		FVector2d UVs[3];

		for (int k = 0; k < 3; k++)
		{
			Positions[k] = Mesh->GetVertex(Triangle3D[k]);
			UVs[k] = (FVector2d)Mesh->GetUV(TriangleUV[k]);

			IslandBoundsOut.Contain(UVs[k]);
		}

		FVector3d Edge1 = Positions[1] - Positions[0];
		FVector3d Edge2 = Positions[2] - Positions[0];
		FVector3d Edge3 = Positions[2] - Positions[1];
		double WorldLength = Edge1.Length() + Edge2.Length() + Edge3.Length();

		FVector2d EdgeUV1 = UVs[1] - UVs[0];
		FVector2d EdgeUV2 = UVs[2] - UVs[0];
		FVector2d EdgeUV3 = UVs[2] - UVs[1];
		double UVLength = EdgeUV1.Length() + EdgeUV2.Length() + EdgeUV3.Length();
		double UVArea = 0.5f * FMathd::Abs(EdgeUV1.X * EdgeUV2.Y - EdgeUV1.Y * EdgeUV2.X);

		UVLengthSum += UVLength;
		WorldLengthSum += WorldLength;
		UVAreaOut += UVArea;
	}

	if (!bScaleIslandsByWorldSpaceTexelRatio || UVLengthSum < FMathd::ZeroTolerance)
	{
		IslandScaleFactorOut = 1;
	}
	else
	{
		// Use ratio of edge lengths instead of areas for robustness
		//  (e.g. to avoid over-scaling a ~zero area UV island, from bad projections or disconnected sliver tris)
		IslandScaleFactorOut = WorldLengthSum / UVLengthSum;
	}
}


