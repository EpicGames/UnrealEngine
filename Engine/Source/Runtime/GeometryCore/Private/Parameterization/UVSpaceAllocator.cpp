// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/UVSpaceAllocator.h"

#include "Misc/SecureHash.h"
#include "ProfilingDebugging/MiscTrace.h"

using namespace UE::Geometry;

FUVSpaceAllocator::FUVSpaceAllocator( FUVSpaceAllocator::EMode InMode, uint32 InWidth, uint32 InHeight)
	: Mode( InMode )
	, Width( InWidth )
	, Height( InHeight )
	, Pitch( ( InWidth + 63 ) / 64 )
	, RasterWidth(0)
	, RasterHeight(0)
{
	check(Width  <= MAX_uint16);
	check(Height <= MAX_uint16);

	Bits.SetNumZeroed(Pitch * Height + 1); // alloc +1 to avoid buffer overrun

	Rows.SetNum( Height );
	SortedRowsIndex.SetNum(Height);
	for (uint32 Index = 0; Index < Height; ++Index)
	{
		SortedRowsIndex[Index] = (uint16)Index;
	}

	Columns.SetNum( Width );
	SortedColumnsIndex.SetNum(Width);
	for (uint32 Index = 0; Index < Width; ++Index)
	{
		SortedColumnsIndex[Index] = (uint16)Index;
	}

	Clear();
}

void FUVSpaceAllocator::Clear()
{
	InitSegments();

	// Only clear the section we used to reduce memory traffic
	FPlatformMemory::Memzero(Bits.GetData(), Pitch * RasterHeight * Bits.GetTypeSize());

	RasterWidth  = 0;
	RasterHeight = 0;
}

void FUVSpaceAllocator::CopyRuns(TArray<FRun>& Runs, const TArray<FRun>& OtherRuns, int32 MaxSize)
{
	Runs.SetNum(OtherRuns.Num(), EAllowShrinking::No);

	MaxSize = FMath::Min(Runs.Num(), MaxSize);

	FRun* RunsPtr = Runs.GetData();
	const FRun* OtherRunsPtr = OtherRuns.GetData();
	for (int32 Index = 0; Index < MaxSize; ++Index, ++RunsPtr, ++OtherRunsPtr)
	{
		*RunsPtr = *OtherRunsPtr;
	}
}

// Optimized version to alloc and copy as less data possible.
FUVSpaceAllocator& FUVSpaceAllocator::operator = (const FUVSpaceAllocator & Other)
{
	if (&Other == this)
	{
		return *this;
	}

	check(Mode == Other.Mode);
	check(Width == Other.Width);
	check(Height == Other.Height);
	check(Pitch == Other.Pitch);

	FPlatformMemory::Memcpy(Bits.GetData(), Other.Bits.GetData(), Pitch * Other.RasterHeight * Bits.GetTypeSize());

	// Zero out anything that was used and not overwritten by the copy
	if (RasterHeight > Other.RasterHeight)
	{
		FPlatformMemory::Memzero(Bits.GetData() + Pitch * Other.RasterHeight, Pitch * (RasterHeight - Other.RasterHeight) * Bits.GetTypeSize());
	}

	RasterWidth  = Other.RasterWidth;
	RasterHeight = Other.RasterHeight;

	CopyRuns(Rows, Other.Rows, Other.RasterHeight);
	SortedRowsIndex = Other.SortedRowsIndex;

	CopyRuns(Columns, Other.Columns, Other.RasterWidth);
	SortedColumnsIndex = Other.SortedColumnsIndex;

	return *this;
}

FMD5Hash FUVSpaceAllocator::GetRasterMD5() const
{
	FMD5 MD5;

	const FRun* RowPtr = Rows.GetData();
	const FRun* RowPtrEnd = RowPtr + RasterHeight;

	for (; RowPtr < RowPtrEnd; ++RowPtr)
	{
		uint32 SegmentsNum = RowPtr->Segments.Num();
		
		// It is important for empty rows to also have a footprint in the MD5
		MD5.Update((const uint8*)&SegmentsNum, sizeof(SegmentsNum));
		MD5.Update((const uint8*)RowPtr->Segments.GetData(), SegmentsNum * RowPtr->Segments.GetTypeSize() );
	}

	FMD5Hash MD5Hash;
	MD5Hash.Set(MD5);
	return MD5Hash;
}

bool FUVSpaceAllocator::Find( FRect& Rect )
{
	FRect TestRect = Rect;
	for( TestRect.X = 0; TestRect.X <= Width - TestRect.W; TestRect.X++ )
	{
		for( TestRect.Y = 0; TestRect.Y <= Height - TestRect.H; TestRect.Y++ )
		{
			if( Test( TestRect ) )
			{
				Rect = TestRect;
				return true;
			}
		}
	}

	return false;
}

FString FUVSpaceAllocator::ToString() const
{
	TArray<TCHAR> Text;
	Text.SetNum(Width+2);

	// Truncate output to last meaningful row
	int32 lastRow = 1;
	for (const FRun& Row : Rows)
	{
		if ((Mode == EMode::FreeSegments && Row.Segments.Num() > 1) ||
			(Mode == EMode::UsedSegments && Row.Segments.Num() != 0))
		{
			lastRow++;
		}
	}

	FString Output;
	Output.Append(TEXT("BEGIN -----------------------\n"));

	const TCHAR FillChar = Mode == EMode::FreeSegments ? TEXT('-') : TEXT(' ');
	const TCHAR UsedChar = Mode == EMode::FreeSegments ? TEXT(' ') : TEXT('+');

	for (const FRun& Row : Rows)
	{
		for (uint32 x = 0; x < Width; ++x)
		{
			Text[x] = FillChar;
		}

		for (const FSegment& Segment : Row.Segments)
		{
			for (uint32 x = Segment.StartPos; x < Segment.StartPos + Segment.Length; ++x)
			{
				Text[x] = UsedChar;
			}
		}

		Text[Text.Num()-2] = TEXT('\n');
		Text[Text.Num()-1] = TEXT('\0');

		Output.Append(Text.GetData());

		if (--lastRow == 0)
		{
			break;
		}
	}

	Output.Appendf(TEXT("MD5 %s\n"), *LexToString(GetRasterMD5()));
	Output.Append(TEXT("END -----------------------\n"));

	return Output;
}

bool FUVSpaceAllocator::FindBitByBit( FRect& Rect, const FUVSpaceAllocator& Other )
{
	FRect TestRect = Rect;
	for( TestRect.X = 0; TestRect.X <= Width - TestRect.W; TestRect.X++ )
	{
		for( TestRect.Y = 0; TestRect.Y <= Height - TestRect.H; TestRect.Y++ )
		{
			if( Test( TestRect, Other ) )
			{
				Rect = TestRect;
				return true;
			}
		}
	}

	return false;
}

bool FUVSpaceAllocator::FindWithSegments( FRect& Rect, const FUVSpaceAllocator& Other, TFunctionRef<bool (const FRect&)> IsBestRect) const
{
	FRect TestRect = Rect;

	const uint32 MaxWidth    = Width - TestRect.W;
	const uint32 MaxHeight   = Height - TestRect.H;

	// For charts that are longer on the Y axis, its a lot faster to use
	// another grid optimized for searching vertically because it will reduce
	// iteration count tremendously by improving strides we make for each iteration.
	// We need a version check to activate this feature because it will favor
	// placement along the Y axis leading to (still efficient) but different results.
	if (TestRect.H > TestRect.W)
	{
		// Perfect fit algorithm will try every position to fit our chart
		for( ; TestRect.X <= MaxWidth; ++TestRect.X)
		{
			// This represents the length we know for sure we won't fit in so we can skip over
			uint32 FailedLength = 1;
			for( TestRect.Y = 0; TestRect.Y <= MaxHeight; TestRect.Y += FailedLength)
			{
				// Do not bother continue if another orientation had a better result
				if( !IsBestRect(Rect) )
				{
					return false;
				}

				if( TestAllColumns( TestRect, Other, FailedLength ) )
				{
					Rect = TestRect;
					return true;
				}
			}
		}
	}
	else
	{
		// Perfect fit algorithm will try every position to fit our chart
		for( ; TestRect.Y <= MaxHeight; ++TestRect.Y)
		{
			// This represents the length we know for sure we won't fit in so we can skip over
			uint32 FailedLength = 1;
			for( TestRect.X = 0; TestRect.X <= MaxWidth; TestRect.X += FailedLength)
			{
				// Do not bother continue if another orientation had a better result
				if( !IsBestRect(Rect) )
				{
					return false;
				}

				if( TestAllRows( TestRect, Other, FailedLength ) )
				{
					Rect = TestRect;
					return true;
				}
			}
		}
	}

	return false;
}

uint32 FUVSpaceAllocator::GetUsedTexels() const
{
	uint32 Texels = 0;
	for (const FRun& Row : Rows)
	{
		for (const FSegment& Segment : Row.Segments)
		{
			Texels += Segment.Length;
		}
	}

	if (Mode == EMode::FreeSegments)
	{
		Texels = Width * Height - Texels;
	}

	return Texels;
}

void FUVSpaceAllocator::Alloc( FRect Rect )
{
	for( uint32 y = Rect.Y; y < Rect.Y + Rect.H; y++ )
	{
		for( uint32 x = Rect.X; x < Rect.X + Rect.W; x++ )
		{
			SetBit( x, y );
		}
	}
}

void FUVSpaceAllocator::Alloc( FRect Rect, const FUVSpaceAllocator& Other )
{
	for( uint32 y = 0; y < Rect.H; y++ )
	{
		for( uint32 x = 0; x < Rect.W; x++ )
		{
			if( Other.GetBit( x, y ) )
			{
				SetBit( x + Rect.X, y + Rect.Y );
			}
		}
	}

	MergeSegments( Rect, Other );
}

bool FUVSpaceAllocator::TestAllRows( const FRect& Rect, const FUVSpaceAllocator& Other, uint32& OutFailedLength ) const
{
	for ( uint32 Index = 0; Index < Other.RasterHeight; ++Index)
	{
		// Longest rows first have a higher chance of giving us a big stride when it fails
		const uint16 y = Other.SortedRowsIndex.GetData()[Index];

		const FRun& ThisRow  = Rows.GetData()[ Rect.Y + y ];
		const FRun& OtherRow = Other.Rows.GetData()[ y ];

		if (!TestOneRun(ThisRow, OtherRow, Rect.X, Rect.W, Width, OutFailedLength))
		{
			return false;
		}
	}

	return true;
}

bool FUVSpaceAllocator::TestAllColumns( const FRect& Rect, const FUVSpaceAllocator& Other, uint32& OutFailedLength ) const
{
	for ( uint32 Index = 0; Index < Other.RasterWidth; ++Index)
	{
		// Longest columns first have a higher chance of giving us a big stride when it fails
		const uint16 x = Other.SortedColumnsIndex.GetData()[Index];

		const FRun& ThisColumn  = Columns.GetData()[ Rect.X + x ];
		const FRun& OtherColumn = Other.Columns.GetData()[ x ];

		if (!TestOneRun(ThisColumn, OtherColumn, Rect.Y, Rect.H, Height, OutFailedLength))
		{
			return false;
		}
	}

	return true;
}

bool FUVSpaceAllocator::TestOneRun(const FRun& ThisRun, const FRun& OtherRun, uint32 RectOffset, uint32 RectLength, uint32 PrimaryResolution, uint32& OutFailedLength) const
{
	// Early out if we can't fit the longest used segment into our longest free segment
	if ( ThisRun.LongestSegment < OtherRun.LongestSegment )
	{
		OutFailedLength = PrimaryResolution;
		return false;
	}

	// Threshold to perform a linear scan until the cache miss induced by the lookup is worth it
	const int32     LoopkupThreshold  = 16;
	const FSegment* FreeSegments      = ThisRun.Segments.GetData();
	const FSegment* FreeSegmentsEnd   = ThisRun.Segments.GetData() + ThisRun.Segments.Num();
	const uint16*   FreeSegmentLookup = ThisRun.Segments.Num() > LoopkupThreshold ? ThisRun.FreeSegmentsLookup.GetData() : nullptr;
	const FSegment* UsedSegments      = OtherRun.Segments.GetData();
	const FSegment* UsedSegmentsEnd   = OtherRun.Segments.GetData() + OtherRun.Segments.Num();

	// Running pointer for used segments
	const FSegment* UsedSegment = UsedSegments;
	for ( ; UsedSegment < UsedSegmentsEnd; ++UsedSegment)
	{
		// #TODO maintain this for backward compatibility but not sure if this could ever happen
		// and if the result is valid if we break instead of returning false when a used segment doesn't fit
		if ( UsedSegment->StartPos >= RectLength )
		{
			break;
		}

		const uint32 StartPos = RectOffset + UsedSegment->StartPos;
		const uint32 EndPos   = RectOffset + FMath::Min( UsedSegment->StartPos + UsedSegment->Length, RectLength );

		// Running pointer for free segments
		const FSegment* FreeSegment = FreeSegments;

		// If lookup exist, advance pointer to the exact spot instead of scanning
		if (FreeSegmentLookup)
		{
			FreeSegment += FreeSegmentLookup[StartPos];
		}

		// Scan the search range to see if the current used segment fits in
		// we use a for loop to support scanning from the beginning if no lookup exists
		bool bFoundSpaceForSegment = false;
		for (; FreeSegment < FreeSegmentsEnd && StartPos >= FreeSegment->StartPos; ++FreeSegment)
		{
			// Check if there's a free segment that can fit the used segment
			if (EndPos <= FreeSegment->StartPos + FreeSegment->Length)
			{
				bFoundSpaceForSegment = true;
				break;
			}
		}

		// If nothing was found in the search range, scan other free segments to hint our
		// caller of how much farther it should advance for next test
		if (!bFoundSpaceForSegment)
		{
			for (; FreeSegment < FreeSegmentsEnd; ++FreeSegment)
			{
				if (UsedSegment->Length <= FreeSegment->Length)
				{
					OutFailedLength = FreeSegment->StartPos - StartPos;

					return false;
				}
			}

			// We couldn't find any free segment big enough
			// set failed length so the whole run will be skipped
			OutFailedLength = PrimaryResolution;

			return false;
		}
	}

	return true;
}

void FUVSpaceAllocator::FlipX( const FRect& Rect )
{
	// If we don't actually have anything in our rasterization, then there's no point in flipping anything. Return Early.
	if (RasterHeight == 0 && RasterWidth == 0)
	{
		return;
	}

	// If we have empty padding around the Rect, keep it there
	uint32 MaxX = 0;
	uint32 MaxY = RasterHeight - 1;

	MaxX = RasterWidth - 1;

	for ( uint32 Y = 0; Y <= MaxY; ++Y )
	{
		for ( uint32 LowX = 0; LowX < ( MaxX + 1 ) / 2; ++LowX )
		{
			uint32 HighX = MaxX - LowX;

			const uint64 BitLow = GetBit( LowX, Y );
			const uint64 BitHigh = GetBit( HighX, Y );

			if ( BitLow != 0ull )
			{
				SetBit( HighX, Y );
			}
			else
			{
				ClearBit( HighX, Y );
			}

			if ( BitHigh != 0ull )
			{
				SetBit( LowX, Y );
			}
			else
			{
				ClearBit( LowX, Y );
			}
		}
	}

	CreateUsedSegments();
}

void FUVSpaceAllocator::FlipY( const FRect& Rect )
{
	// If we don't actually have anything in our rasterization, then there's no point in flipping anything. Return Early.
	if (RasterHeight == 0 && RasterWidth == 0)
	{
		return;
	}

	uint32 MinY = 0;
	uint32 MaxY = RasterHeight - 1;

	for ( uint32 LowY = 0; LowY < ( MaxY + 1 ) / 2; ++LowY )
	{
		for ( uint32 X = 0; X < RasterWidth; ++X )
		{
			uint32 HighY = MaxY - LowY;

			const uint64 BitLow = GetBit( X, LowY );
			const uint64 BitHigh = GetBit( X, HighY );

			if ( BitLow != 0ull )
			{
				SetBit( X, HighY );
			}
			else
			{
				ClearBit( X, HighY );
			}

			if ( BitHigh != 0ull )
			{
				SetBit( X, LowY );
			}
			else
			{
				ClearBit( X, LowY );
			}
		}
	}

	CreateUsedSegments();
}

void FUVSpaceAllocator::InitRuns(TArray<FRun>& Runs, uint32 PrimaryResolution, uint32 PerpendicularRasterSize)
{
	if (Mode == EMode::FreeSegments)
	{
		FSegment FreeSegment;
		FreeSegment.StartPos = 0;
		FreeSegment.Length = PrimaryResolution;

		for ( FRun& Run : Runs )
		{
			Run.Segments.Reset();
			Run.Segments.Add( FreeSegment );
			Run.LongestSegment = PrimaryResolution;
			Run.FreeSegmentsLookup.Reset();
		}
	}
	else
	{
		// This is called inside a hot loop, get rid of RangeCheck.
		FRun* Run = Runs.GetData();
		for ( uint32 Index = 0; Index < PerpendicularRasterSize; ++Index )
		{
			Run[Index].Segments.Reset();
			Run[Index].LongestSegment = 0;
		}
	}
}

void FUVSpaceAllocator::InitSegments()
{
	InitRuns(Rows, Width, RasterHeight);

	InitRuns(Columns, Height, RasterWidth);
}

void FUVSpaceAllocator::CreateUsedSegments()
{
	check(Mode == EMode::UsedSegments);

	SortedRowsIndex.SetNum(RasterHeight, EAllowShrinking::No);

	uint64* BitsData = Bits.GetData();

	// Create segments along the X axis for each rows
	for ( uint32 y = 0; y < RasterHeight; ++y )
	{
		SortedRowsIndex[y] = (uint16)y;

		FRun& CurrentRow = Rows[y];
		CurrentRow.LongestSegment = 0;
		CurrentRow.Segments.Reset();

		int32 FirstUsedX = -1;
		for ( uint32 k = 0; k < Pitch; ++k, ++BitsData )
		{
			const uint32 x = k * 64;
			const uint64 BitsValue = *BitsData;

			// If all bits are set
			if ( BitsValue == ~0ull )
			{
				if ( FirstUsedX < 0 )
				{
					FirstUsedX = x;
				}

				if ( k == Pitch - 1 )
				{
					AddUsedSegment( CurrentRow, FirstUsedX, x + 64 - FirstUsedX );
					FirstUsedX = -1;
				}
			}
			// No bits are set
			else if ( BitsValue == 0ull )
			{
				if ( FirstUsedX >= 0 )
				{
					AddUsedSegment( CurrentRow, FirstUsedX, x - FirstUsedX );
					FirstUsedX = -1;
				}
			}
			// Some bits are set
			else
			{
				for ( uint32 i = 0; i < 64; ++i )
				{
					const uint32 SubX = x + i;

					if ( BitsValue & ( 1ull << i ) )
					{
						if ( FirstUsedX < 0 )
						{
							FirstUsedX = SubX;
						}

						if ( SubX == Width - 1 )
						{
							AddUsedSegment( CurrentRow, FirstUsedX, SubX + 1 - FirstUsedX );
							FirstUsedX = -1;
						}
					}
					else if ( FirstUsedX >= 0 )
					{
						AddUsedSegment( CurrentRow, FirstUsedX, SubX - FirstUsedX );
						FirstUsedX = -1;
					}
				}
			}
		}
	}

	SortedRowsIndex.Sort([this](uint32 a, uint32 b) { return Rows[b].LongestSegment < Rows[a].LongestSegment;});

	// Create segments along the Y axis for each columns
	SortedColumnsIndex.SetNum(RasterWidth, EAllowShrinking::No);
	
	for ( uint32 x = 0; x < RasterWidth; ++x )
	{
		SortedColumnsIndex[x] = (uint16)x;

		FRun& CurrentColumn = Columns[x];
		CurrentColumn.LongestSegment = 0;
		CurrentColumn.Segments.Reset();

		int32 FirstUsedY = -1;
		for ( uint32 y = 0; y < RasterHeight; ++y )
		{
			if (GetBit(x, y))
			{
				if ( FirstUsedY < 0 )
				{
					FirstUsedY = y;
				}

				if ( y == RasterHeight - 1 )
				{
					AddUsedSegment( CurrentColumn, FirstUsedY, y + 1 - FirstUsedY );
					FirstUsedY = -1;
				}
			}
			else if ( FirstUsedY >= 0 )
			{
				AddUsedSegment( CurrentColumn, FirstUsedY, y - FirstUsedY );
				FirstUsedY = -1;
			}
		}
	}

	SortedColumnsIndex.Sort([this](uint32 a, uint32 b) { return Columns[b].LongestSegment < Columns[a].LongestSegment;});
}

void FUVSpaceAllocator::AddUsedSegment( FRun& Row, uint32 StartPos, uint32 Length )
{
	FSegment UsedSegment;
	UsedSegment.StartPos = StartPos;
	UsedSegment.Length = Length;
	Row.Segments.Add(UsedSegment);

	if ( UsedSegment.Length > Row.LongestSegment )
	{
		Row.LongestSegment = UsedSegment.Length;
	}
}

void FUVSpaceAllocator::MergeRun(FRun& ThisRun, const FRun& OtherRun, uint32 RectOffset, uint32 RectLength, uint32 PrimaryResolution, uint32 PerpendicularResolution)
{
	// #TODO
	// Perform a linear scan once for both array at the same time zipping them together
	// Ensure we keep the order of the segments while merging them to avoid costly sort when number of free segments is high
	for ( const FSegment& OtherUsedSegment : OtherRun.Segments )
	{
		for ( int32 Index = 0; Index < ThisRun.Segments.Num(); ++Index )
		{
			FSegment& ThisFreeSegment = ThisRun.Segments[Index];

			uint32 StartPos = RectOffset + OtherUsedSegment.StartPos;

			if ( StartPos >= ThisFreeSegment.StartPos &&
				StartPos < ThisFreeSegment.StartPos + ThisFreeSegment.Length )
			{
				if ( ThisFreeSegment.Length == 1 )
				{
					ThisRun.Segments.RemoveAtSwap(Index);
					break;
				}

				FSegment FirstSegment;
				FirstSegment.StartPos = ThisFreeSegment.StartPos;
				FirstSegment.Length = StartPos - ThisFreeSegment.StartPos;

				uint32 EndPos = RectOffset + FMath::Min( OtherUsedSegment.StartPos + OtherUsedSegment.Length, RectLength ) - 1;

				FSegment SecondSegment;
				SecondSegment.StartPos = EndPos + 1;
				SecondSegment.Length = ThisFreeSegment.StartPos + ThisFreeSegment.Length - SecondSegment.StartPos;

				ThisRun.Segments.RemoveAtSwap(Index);

				if ( FirstSegment.Length > 0 )
				{
					ThisRun.Segments.Add( FirstSegment );
				}

				if ( SecondSegment.Length > 0 )
				{
					ThisRun.Segments.Add( SecondSegment );
				}

				break;
			}
		}
	}

	ThisRun.Segments.Sort();

	// When we're dealing with an Axis, the lookup table is the size of the perpendicular Axis
	ThisRun.FreeSegmentsLookup.SetNum(PerpendicularResolution);
	ThisRun.LongestSegment = 0;

	uint32 LastIndex = 0;

	// Avoid RangeCheck by indexing the pointer directly since we're in a hot loop
	FSegment* SegmentsPtr           = ThisRun.Segments.GetData();
	uint16*   FreeSegmentsLookupPtr = ThisRun.FreeSegmentsLookup.GetData();

	int32 SegmentIndex = 0;
	for ( ; SegmentIndex < ThisRun.Segments.Num(); ++SegmentIndex )
	{
		FSegment& Segment = SegmentsPtr[SegmentIndex];

		if (Segment.Length > ThisRun.LongestSegment)
		{
			ThisRun.LongestSegment = Segment.Length;
		}

		uint32 StopIndex  = Segment.StartPos + Segment.Length;
		for (uint32 LookupIndex = LastIndex; LookupIndex < StopIndex; ++LookupIndex)
		{
			FreeSegmentsLookupPtr[LookupIndex] = uint16(SegmentIndex);
		}

		LastIndex = StopIndex;
	}

	for (uint32 Index = LastIndex; Index < PerpendicularResolution; ++Index)
	{
		FreeSegmentsLookupPtr[Index] = uint16(SegmentIndex);
	}
}

void FUVSpaceAllocator::MergeSegments( const FRect& Rect, const FUVSpaceAllocator& Other )
{
	check(Mode       == EMode::FreeSegments);
	check(Other.Mode == EMode::UsedSegments);

	for ( uint32 y = 0; y < Other.RasterHeight; ++y )
	{
		MergeRun(Rows[Rect.Y + y], Other.Rows[y], Rect.X, Rect.W, Width, Height);
	}

	for ( uint32 x = 0; x < Other.RasterWidth; ++x )
	{
		MergeRun(Columns[Rect.X + x], Other.Columns[x], Rect.Y, Rect.H, Height, Width);
	}
}

