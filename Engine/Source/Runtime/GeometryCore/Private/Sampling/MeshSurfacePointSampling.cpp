// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshSurfacePointSampling.h"

#include "VectorUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicVerticesOctree3.h"
#include "Async/ParallelFor.h"
#include "Tasks/Task.h"
#include "Math/RandomStream.h"

using namespace UE::Geometry;

namespace UELocal
{

// Generate a semi-random point ordering (via shuffling) 
// Note this does create some bias in favor of more-densely
// triangulated regions, which can have more points in the
// initial dense sampling.
// Version 1 and earlier also ignore the RandomSeed, so will tend to draw
// samples from the same triangles across differently-seeded runs.
void GetShuffledPointOrdering(TArray<int32>& PointOrdering, int32 NumDensePoints, int32 RandomSeed, int32 Version)
{
	bool bUseSeed = Version > 1 || Version == INDEX_NONE;

	// as of Version 2, we use the random stream to generate a shuffled order
	if (bUseSeed)
	{
		PointOrdering.SetNumUninitialized(NumDensePoints);
		FRandomStream Stream(RandomSeed);
		for (int32 Index = 0; Index < NumDensePoints; ++Index)
		{
			// Insert Index at a random position via swap
			const int32 SwapIndex = Stream.RandRange(0, Index);
			if (SwapIndex < Index)
			{
				PointOrdering[Index] = PointOrdering[SwapIndex];
			}
			PointOrdering[SwapIndex] = Index;
		}
	}
	else // older version; use modulo iteration w/out random seed
	{
		PointOrdering.Reserve(NumDensePoints);
		FModuloIteration Iter(NumDensePoints);
		uint32 NextIndex;
		while (Iter.GetNextIndex(NextIndex))
		{
			PointOrdering.Add(NextIndex);
		}
	}
}
	

// cache of per-triangle information for a triangle mesh
struct FTriangleInfoCache
{
	TArray<FVector3d> TriNormals;
	TArray<double> TriAreas;
	// Note: The below two arrays are only used for the legacy sampling method 0
	TArray<FFrame3d> Legacy_TriFrames;
	TArray<FTriangle2d> Legacy_UVTriangles;

	double TotalArea;

	template<typename MeshType>
	void InitializeForTriangleSet(const MeshType& SampleMesh, int32 SamplingMethodVersion)
	{
		TriNormals.SetNumUninitialized(SampleMesh.MaxTriangleID());
		TriAreas.SetNumZeroed(SampleMesh.MaxTriangleID()); // note: zero the memory so we can include skipped triangle IDs in the sum below
		if (SamplingMethodVersion == 0)
		{
			Legacy_TriFrames.SetNumUninitialized(SampleMesh.MaxTriangleID());
			Legacy_UVTriangles.SetNumUninitialized(SampleMesh.MaxTriangleID());
		}
				
		ParallelFor(SampleMesh.MaxTriangleID(), [&](int32 tid)
		{
			if (SampleMesh.IsTriangle(tid))
			{
				FVector3d A, B, C;
				SampleMesh.GetTriVertices(tid, A, B, C);
				TriNormals[tid] = VectorUtil::NormalArea(A, B, C, TriAreas[tid]);
				if (SamplingMethodVersion == 0)
				{
					FVector3d Centroid;
					Centroid = (A + B + C) / 3.0;
					Legacy_TriFrames[tid] = FFrame3d(Centroid, TriNormals[tid]);
					Legacy_UVTriangles[tid] = FTriangle2d(Legacy_TriFrames[tid].ToPlaneUV(A), Legacy_TriFrames[tid].ToPlaneUV(B), Legacy_TriFrames[tid].ToPlaneUV(C));
				}
			}
		});
		// Note: This sum includes areas of skipped triangle IDs, which are zero by initialization above
		TotalArea = 0;
		for (double TriArea : TriAreas)
		{
			TotalArea += TriArea;
		}
	}
};


struct FNonUniformSamplingConfig
{
	FMeshSurfacePointSampling::ESizeDistribution SizeDistribution = FMeshSurfacePointSampling::ESizeDistribution::Uniform;
	double SizeDistributionPower = 2.0;

	TOptional<TFunctionRef<double(int TriangleID, FVector3d Position, FVector3d BaryCoords)>> WeightFunction;

	FMeshSurfacePointSampling::EInterpretWeightMode InterpretWeightMode = FMeshSurfacePointSampling::EInterpretWeightMode::WeightedRandom;
};


struct FDenseSamplePointSet
{
	TArray<FVector3d> DensePoints;
	TArray<int> Triangles;
	TArray<double> Weights;

	FAxisAlignedBox3d Bounds;
	// orientation array
	int MaxVertexID() const { return DensePoints.Num(); }
	bool IsVertex(int VertexID) const { return true; }
	FVector3d GetVertex(int Index) const { return DensePoints[Index]; }
	FAxisAlignedBox3d GetBounds() const { return Bounds; }
};


struct FPerTriangleDensePointSampling
{
	struct FTriangleSubArray
	{
		int32 StartIndex;
		int32 NumSamples;
	};
	TArray<FTriangleSubArray> TriSubArrays;


	template<typename MeshType>
	void InitializeForTriangleSet(
		const MeshType& SampleMesh, 
		const FTriangleInfoCache& TriInfo, 
		double DenseSampleArea, 
		int RandomSeed,
		const FNonUniformSamplingConfig& NonUniformConfig,
		FDenseSamplePointSet& PointSetOut,
		int32 SamplingMethodVersion)
	{
		// figure out how many samples in each triangle, and assign each triangle a starting index into PointSetOut arrays
		TriSubArrays.SetNumUninitialized(SampleMesh.MaxTriangleID());
		int32 CurIndex = 0;
		for (int32 tid = 0; tid < SampleMesh.MaxTriangleID(); ++tid)
		{
			if ( SampleMesh.IsTriangle(tid) )
			{
				int NumSamples = FMath::Max( (int)(TriInfo.TriAreas[tid] / DenseSampleArea), 2 );  // a bit arbitrary...
				TriSubArrays[tid] = FTriangleSubArray{CurIndex, NumSamples};
				CurIndex += NumSamples;
			}
			else
			{
				TriSubArrays[tid] = FTriangleSubArray{-1, 0};
			}
		}
		PointSetOut.DensePoints.SetNumUninitialized(CurIndex);
		PointSetOut.Triangles.SetNumUninitialized(CurIndex);

		bool bComputeWeights = (NonUniformConfig.WeightFunction.IsSet());
		if (bComputeWeights)
		{
			PointSetOut.Weights.SetNumUninitialized(CurIndex);
		}

		// This parallel for seems to be quite expensive...maybe contention because all threads are writing to same DensePoints array?
		// Amount of work per TriangleID can vary quite a bit because it depends on triangle size...possibly should pass
		// unbalanced flag if triangle count is small relative to point count
		EParallelForFlags UseFlags = (SampleMesh.MaxTriangleID() < 100) ? EParallelForFlags::Unbalanced : EParallelForFlags::None;
		ParallelFor(TEXT("ComputeDenseSamples"), SampleMesh.MaxTriangleID(), 500, [&](int tid)
		{
			int NumSamples = TriSubArrays[tid].NumSamples;
			if ( NumSamples == 0 ) return;
			int StartIndex = TriSubArrays[tid].StartIndex;
			
			// Legacy sampling method using rejection sampling, to support any application that
			// depends on the sampling pattern not changing (e.g., PCG sampling nodes expect deterministic results)
			// If changing, be sure the sampling pattern remains the same.
			if (SamplingMethodVersion == 0)
			{
				const FFrame3d& ProjectFrame = TriInfo.Legacy_TriFrames[tid];
				const FTriangle2d& TriUV = TriInfo.Legacy_UVTriangles[tid];
				
				// generate uniform random point in 2D quadrilateral  (http://mathworld.wolfram.com/TrianglePointPicking.html)
				// Note: VectorUtil::UniformSampleTrianglePoint has a better (non-rejection) version of this sampling method
				// (but it would generate a different pattern, so we can't use it here)
				FVector2d V1 = TriUV.V[1] - TriUV.V[0];
				FVector2d V2 = TriUV.V[2] - TriUV.V[0];

				int NumGenerated = 0;
				FRandomStream RandomStream(tid + RandomSeed);

				// workaround for rejection sampling method failing for degenerate triangles (sampling method 1 does not have this issue)
				bool bIsDegenerateTri = TriInfo.TriAreas[tid] == 0;
				while (NumGenerated < NumSamples)
				{
					double a1 = RandomStream.GetFraction();
					double a2 = RandomStream.GetFraction();
					if (bIsDegenerateTri && a1 + a2 > 1)
					{
						a1 = 1-a1;
						a2 = 1-a2;
					}
					FVector2d PointUV = TriUV.V[0] + a1 * V1 + a2 * V2;
					if (bIsDegenerateTri || TriUV.IsInside(PointUV))
					{
						FVector3d Position = ProjectFrame.FromPlaneUV(PointUV, 2);
						PointSetOut.DensePoints[StartIndex+NumGenerated] = Position;
						PointSetOut.Triangles[StartIndex+NumGenerated] = tid;
						if (bComputeWeights)
						{
							FVector3d BaryCoords = TriUV.GetBarycentricCoords(PointUV);
							PointSetOut.Weights[StartIndex+NumGenerated] = 
								NonUniformConfig.WeightFunction.GetValue()(tid, Position, BaryCoords);
						}

						NumGenerated++;
					}
				}
			}
			else // Sampling method > 0, sample uniform random barycentric coordinates directly
			{
				FVector3d A, B, C;
				SampleMesh.GetTriVertices(tid, A, B, C);

				int NumGenerated = 0;
				FRandomStream RandomStream(tid + RandomSeed);
				while (NumGenerated < NumSamples)
				{
					double a1 = RandomStream.GetFraction();
					double a2 = RandomStream.GetFraction();

					FVector3d BaryCoords = VectorUtil::UniformSampleTriangleBarycentricCoords(a1, a2);
					FVector3d Position = BaryCoords.X * A + BaryCoords.Y * B + BaryCoords.Z * C;
					PointSetOut.DensePoints[StartIndex + NumGenerated] = Position;
					PointSetOut.Triangles[StartIndex + NumGenerated] = tid;
				
					if (bComputeWeights)
					{
						PointSetOut.Weights[StartIndex+NumGenerated] = 
							NonUniformConfig.WeightFunction.GetValue()(tid, Position, BaryCoords);
					}

					NumGenerated++;
				}
			}
		}, UseFlags);
	}

};

template<typename MeshType>
void ConstructDenseUniformMeshPointSampling(
	const MeshType& SampleMesh,
	double SampleRadius,
	double SubSampleDensity,
	int RandomSeed,
	const FNonUniformSamplingConfig& NonUniformConfig,
	int MaxNumDenseSamples,
	FDenseSamplePointSet& DensePointSetOut,
	int32 SamplingMethodVersion)
{
	FTriangleInfoCache TriInfoCache;
	TriInfoCache.InitializeForTriangleSet<MeshType>(SampleMesh, SamplingMethodVersion);

	// compute mesh bounds in background thread. ParallelFor could make this faster (eg FDynamicMesh3::GetBounds) but it's going to take longer than the other steps below anyway
	UE::Tasks::FTask BoundsTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&DensePointSetOut, &SampleMesh]()
	{
		DensePointSetOut.Bounds = FAxisAlignedBox3d::Empty();
		int MaxVertexID = SampleMesh.MaxVertexID();
		for (int32 k = 0; k < MaxVertexID; ++k)
		{
			if (SampleMesh.IsVertex(k))
			{
				DensePointSetOut.Bounds.Contain(SampleMesh.GetVertex(k));
			}
		}
	});
	
	double DiscArea = (FMathd::Pi * SampleRadius * SampleRadius);
	double ApproxNumUniformSamples = TriInfoCache.TotalArea / DiscArea;		// uniform disc area
	double EstNumDenseSamples = ApproxNumUniformSamples * SubSampleDensity * 2;		// 2 is fudge-factor
	if (MaxNumDenseSamples != 0 && EstNumDenseSamples > MaxNumDenseSamples)
	{
		EstNumDenseSamples = MaxNumDenseSamples;
	}
	double DenseSampleArea = TriInfoCache.TotalArea / EstNumDenseSamples;


	FPerTriangleDensePointSampling DensePerTriangleSampling;
	DensePerTriangleSampling.InitializeForTriangleSet<MeshType>(SampleMesh, TriInfoCache, DenseSampleArea, RandomSeed, 
		NonUniformConfig, DensePointSetOut, SamplingMethodVersion);

	//UE_LOGF(LogTemp, Warning, "TotalArea %f  SampleArea %f EstSamples %d  DenseSamples %d   DenseArea %f  NumDensePoints %d",
	//	TriInfo.TotalArea, DiscArea, (int)ApproxNumUniformSamples, (int)EstNumDenseSamples, DenseSampleArea, NumDensePoints);

	// make sure we have bounds initialized
	BoundsTask.Wait();
}



template<typename MeshType>
void UniformMeshPointSampling(
	const MeshType& SampleMesh,
	TFunctionRef<void(FVector3d, int32, double)> EmitSampleFunc,
	double SampleRadius,
	int32 MaxSamples,
	double SubSampleDensity,
	int RandomSeed,
	int MaxNumDenseSamples,
	int32 SamplingMethodVersion,
	FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UniformMeshPointSampling);

	MaxSamples = (MaxSamples == 0) ? TNumericLimits<int>::Max() : MaxSamples;
	const bool bShuffle = (MaxSamples < TNumericLimits<int>::Max());

	// Guard against requesting extremely small sample radii
	SampleRadius = FMath::Max(UE_DOUBLE_SMALL_NUMBER, SampleRadius);

	//
	// Step 1: generate dense random point sampling of mesh surface. 
	// 
	FDenseSamplePointSet DensePointSet;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeDenseSamples);
		
		ConstructDenseUniformMeshPointSampling<MeshType>(SampleMesh, SampleRadius, SubSampleDensity, RandomSeed, 
			FNonUniformSamplingConfig(), MaxNumDenseSamples, DensePointSet, SamplingMethodVersion);
	}
	int32 NumDensePoints = DensePointSet.MaxVertexID();

	if (Progress && Progress->Cancelled() ) return;

	// 
	// Generate a random point ordering for sampling 
	//

	TArray<int32> PointOrdering;

	if (bShuffle)
	{
		GetShuffledPointOrdering(PointOrdering, NumDensePoints, RandomSeed, SamplingMethodVersion);
	}
	else
	{
		PointOrdering.Reserve(NumDensePoints);
		for (int32 k = 0; k < NumDensePoints; ++k)
		{
			PointOrdering.Add(k);
		}
	}
	int32 CurOrderingIndex = 0;

	// if the bounds are small enough that we'd have at most a single point (i.e., sample radius sphere from any point in bounds will cover bounds), early out here
	// this helps avoid the case of an octree covering very small bounds, which FSparseDynamicPointOctree3 currently doesn't handle well
	if (DensePointSet.Bounds.MaxDim() * FMathd::Sqrt3 < SampleRadius)
	{
		if (NumDensePoints > 0)
		{
			int32 UseVertexID = PointOrdering[0];
			FVector3d SamplePoint = DensePointSet.DensePoints[UseVertexID];
			EmitSampleFunc(SamplePoint, DensePointSet.Triangles[UseVertexID], SampleRadius);
		}
		return;
	}

	//
	// Step 2: store dense point sampling in octree
	// 
	FSparseDynamicPointOctree3 Octree;
	Octree.ConfigureFromPointCountEstimate(DensePointSet.Bounds.MaxDim(), NumDensePoints);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildDenseOctree);
		Octree.ParallelInsertDensePointSet(NumDensePoints, [&](int32 VertexID) { return DensePointSet.GetVertex(VertexID); } );
	}

	if (Progress && Progress->Cancelled() ) return;

	//
	// Step 3: dart throwing. Draw "random" dense point, add it to output set, and then
	// remove all other dense points within radius of this point. 
	// *NOTE* that in this implementation, we are not necessarily drawing random points.
	// We are drawing from a random sampling on the triangles, but the per-triangle points
	// are added to the PointSet in triangle index order. This introduces some bias
	// but makes the algorithm quite a bit faster...


	double QueryRadiusSqr = 4 * SampleRadius * SampleRadius;
	TArray<bool> IsValidPoint;
	IsValidPoint.Init(true, DensePointSet.MaxVertexID());
	TArray<const FSparsePointOctreeCell*> QueryTempBuffer;
	TArray<int> PointsInBall;

	int NumEmittedSamples = 0;
	{ TRACE_CPUPROFILER_EVENT_SCOPE(ExtractPoints);

		while (CurOrderingIndex < NumDensePoints && NumEmittedSamples < MaxSamples)
		{
			if (NumEmittedSamples % 25 == 0 && Progress && Progress->Cancelled() ) return;

			// pick a vertex in the dense point set, ie "throw a dart that is guaranteed to be valid"
			int32 UseVertexID = IndexConstants::InvalidID;
			{ //TRACE_CPUPROFILER_EVENT_SCOPE(FindInitialPoint);

				while (UseVertexID == IndexConstants::InvalidID && CurOrderingIndex < NumDensePoints)
				{
					int VertexID = PointOrdering[CurOrderingIndex++];
					if (IsValidPoint[VertexID])
					{
						UseVertexID = VertexID;
						break;
					}
				}
			}
			if (UseVertexID == IndexConstants::InvalidID)
			{
				continue;
			}

			// found a valid point
			FVector3d SamplePoint = DensePointSet.DensePoints[UseVertexID];
			EmitSampleFunc(SamplePoint, DensePointSet.Triangles[UseVertexID], SampleRadius);
			NumEmittedSamples++;
			Octree.RemovePoint(UseVertexID);
			IsValidPoint[UseVertexID] = false;

			// remove dense points within sample radius from this point
			PointsInBall.Reset();
			FAxisAlignedBox3d QueryBox(SamplePoint, 2*SampleRadius);
			Octree.RangeQuery(QueryBox,		// add SphereQuery to Octree? would save a chunk...
				[&](int32 PointID) {
					return IsValidPoint[PointID] && DistanceSquared(DensePointSet.GetVertex(PointID), SamplePoint) < QueryRadiusSqr;
				},
				PointsInBall, &QueryTempBuffer);

			for (int32 QueryPointID : PointsInBall)
			{
				Octree.RemovePointUnsafe(QueryPointID);
				IsValidPoint[QueryPointID] = false;
			}

		}
	}
}







template<typename MeshType>
void NonUniformMeshPointSampling(
	const MeshType& SampleMesh,
	TFunctionRef<void(FVector3d, int32, double)> EmitSampleFunc,
	double MinSampleRadius,
	double MaxSampleRadius,
	int32 MaxSamples,
	double SubSampleDensity,
	int RandomSeed,
	const FNonUniformSamplingConfig& NonUniformConfig,
	int MaxNumDenseSamples,
	int32 SamplingMethodVersion,
	FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UniformMeshPointSampling);

	MaxSamples = (MaxSamples == 0) ? TNumericLimits<int>::Max() : MaxSamples;

	// Guard against requesting extremely small sample radii
	MinSampleRadius = FMath::Max(UE_DOUBLE_SMALL_NUMBER, MinSampleRadius);
	MaxSampleRadius = FMath::Max(UE_DOUBLE_SMALL_NUMBER, MaxSampleRadius);

	//
	// Step 1: generate dense random point sampling of mesh surface. 
	// 
	FDenseSamplePointSet DensePointSet;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeDenseSamples);
		ConstructDenseUniformMeshPointSampling<MeshType>(SampleMesh, MinSampleRadius, SubSampleDensity, RandomSeed, 
			NonUniformConfig, MaxNumDenseSamples, DensePointSet, SamplingMethodVersion);
	}
	int32 NumDensePoints = DensePointSet.MaxVertexID();
	bool bHaveWeights = DensePointSet.Weights.Num() > 0;

	if (Progress && Progress->Cancelled() ) return;

	// 
	// Generate a random point ordering for sampling 
	//

	// currently only generating a semi-random point ordering (via shuffling) 
	// if a subset of points is requested.  This likely does create some bias, 
	TArray<int32> PointOrdering;
	GetShuffledPointOrdering(PointOrdering, NumDensePoints, RandomSeed, SamplingMethodVersion);
	int32 CurOrderingIndex = 0;

	// if the bounds are small enough that we'd have at most a single point (i.e., min radius sphere from any point in bounds will cover bounds), early out here
	// this helps avoid the case of an octree covering very small bounds, which FSparseDynamicPointOctree3 currently doesn't handle well
	if (DensePointSet.Bounds.MaxDim() * FMathd::Sqrt3 < MinSampleRadius)
	{
		if (NumDensePoints > 0)
		{
			int32 UseVertexID = PointOrdering[0];
			FVector3d SamplePoint = DensePointSet.DensePoints[UseVertexID];
			EmitSampleFunc(SamplePoint, DensePointSet.Triangles[UseVertexID], MinSampleRadius);
		}
		return;
	}


	//
	// Step 2: store dense point sampling in octree
	// 
	FSparseDynamicPointOctree3 Octree;
	Octree.ConfigureFromPointCountEstimate(DensePointSet.Bounds.MaxDim(), NumDensePoints);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildDenseOctree);
		Octree.ParallelInsertDensePointSet(NumDensePoints, [&](int32 VertexID) { return DensePointSet.GetVertex(VertexID); } );
	}

	if (Progress && Progress->Cancelled() ) return;

	//
	// Step 3: dart throwing. Draw "random" dense point, add it to output set, and then
	// remove all other dense points within radius of this point. 
	// *NOTE* that in this implementation, we are not necessarily drawing random points.
	// We are drawing from a random sampling on the triangles, but the per-triangle points
	// are added to the PointSet in triangle index order. This introduces some bias
	// but makes the algorithm quite a bit faster...

	TArray<bool> IsValidPoint;
	IsValidPoint.Init(true, DensePointSet.MaxVertexID());
	TArray<const FSparsePointOctreeCell*> QueryTempBuffer;
	TArray<int> PointsInBall;

	FRandomStream RadiusStream(RandomSeed);

	// likely could benefit from hash grid here? 
	TArray<FVector3d> EmittedSamples;
	TArray<double> EmittedRadius;
	auto FindOverlappingSample = [&EmittedSamples, &EmittedRadius, &PointsInBall, &QueryTempBuffer, MaxSampleRadius](FVector3d Position, double Radius) -> double
	{
		for (int32 k = 0; k < EmittedSamples.Num(); ++k)
		{
			double NeighbourDist = Distance(EmittedSamples[k], Position);
			//if ( NeighbourDist < (Radius + EmittedRadius[k]) )
			if ( Radius > (NeighbourDist - EmittedRadius[k]) )
			{
				return (NeighbourDist - EmittedRadius[k]);
			}
		}
		return TNumericLimits<double>::Max();
	};

	// In weighted sampling, we cannot guarantee that a dense sample point w/ a radius > MinSampleRadius will
	// actually fit without collision. The "correct" way to handle this, by randomly choosing new points until
	// a valid one is found, can take a very long time. So instead we can "decay" the radius down to 
	// MinSampleRadius in multiple steps, trying to find a radius that fits. We are guaranteed that any active
	// dense point will fit with MinSampleRadius, so this significantly accellerates the sampling, at the cost
	// of introducing some bias.
	TArray<double, TInlineAllocator<10>> DecaySteps;
	bool bIsFixedRadiusMethod = false;
	if (NonUniformConfig.InterpretWeightMode == FMeshSurfacePointSampling::EInterpretWeightMode::RadiusInterp)
	{
		DecaySteps = TArray<double, TInlineAllocator<10>>({1.0});
		bIsFixedRadiusMethod = true;
	}
	else
	{
		DecaySteps = TArray<double, TInlineAllocator<10>>({1.0, 0.8, 0.6, 0.4, 0.2, 0.0});
	}

	TArray<double> CurDistances;
	CurDistances.Init(TNumericLimits<double>::Max(), NumDensePoints);

	int NumEmittedSamples = 0;
	{ TRACE_CPUPROFILER_EVENT_SCOPE(ExtractPoints);

		int NumFailures = 0;
		while (PointOrdering.Num() > 0 && NumEmittedSamples < MaxSamples && NumFailures < 1000)
		{
			if (NumEmittedSamples % 25 == 0 && Progress && Progress->Cancelled() ) return;

			int32 UseVertexID = IndexConstants::InvalidID;
			int32 PointOrderingIndex = -1;
			double SampleRadius = MinSampleRadius;

			// try to find a valid (point, radius) pair. This may fail if we cannot find a 
			// valid radius...
			int32 NumRemaining = PointOrdering.Num();
			for ( int32 k = 0; k < NumRemaining; ++k)
			{
				int VertexID = PointOrdering[k];

				if (IsValidPoint[VertexID] == false)		// if point is expired, discard it
				{
					PointOrdering.RemoveAtSwap(k, EAllowShrinking::No);
					NumRemaining--;
					k--;		// reconsider point we just swapped to index k
					continue;
				}

				FVector3d Position = DensePointSet.GetVertex(VertexID);

				// based on the weight/random strategy, generate a parameter in range [0,1] that will
				// be used to interpolate the Min/Max Radius below
				double InterpRadiusT = 0;
				if (bHaveWeights)
				{
					if (NonUniformConfig.InterpretWeightMode == FMeshSurfacePointSampling::EInterpretWeightMode::WeightedRandom)
					{
						double Weight = FMathd::Clamp(DensePointSet.Weights[VertexID], 0, 1);
						double Random = RadiusStream.GetFraction();
						InterpRadiusT = (Weight + Random) / 2.0;		// can parameterize this as ( (N-1)*Weight + Random ) / N
					}
					else  // RadiusInterp
					{
						InterpRadiusT = DensePointSet.Weights[VertexID];
					}
				}
				else 
				{
					InterpRadiusT = RadiusStream.GetFraction();
				}
				if (NonUniformConfig.SizeDistribution == FMeshSurfacePointSampling::ESizeDistribution::Smaller)
				{
					InterpRadiusT = FMathd::Pow(InterpRadiusT, NonUniformConfig.SizeDistributionPower);
				}
				else if (NonUniformConfig.SizeDistribution == FMeshSurfacePointSampling::ESizeDistribution::Larger)
				{
					InterpRadiusT = FMathd::Pow(InterpRadiusT, 1.0 / NonUniformConfig.SizeDistributionPower);
				}

				// try to fit a sample at the selected point, possibly incrementally shrinking the 
				// sample radius down to MinRadius to guarantee a fit
				bool bFound = false;
				double MinNeighbourGap = TNumericLimits<double>::Max();
				for (double DecayStep : DecaySteps)
				{
					double UseRadius = FMathd::Lerp(MinSampleRadius, MaxSampleRadius, InterpRadiusT * DecayStep);
					if (UseRadius > CurDistances[VertexID])
					{
						continue;
					}

					double NeighbourGap = FindOverlappingSample(Position, UseRadius);
					//if ( NeighbourGap == TNumericLimts<double>::Max() )
					if ( UseRadius < NeighbourGap )
					{
						UseVertexID = VertexID;
						SampleRadius = UseRadius;
						PointOrderingIndex = k;
						bFound = true;
						break;
					}
					else
					{
						MinNeighbourGap = FMathd::Min(MinNeighbourGap, NeighbourGap);
					}
				}
				if (bFound)
				{
					break;
				}
				CurDistances[VertexID] = FMathd::Min(CurDistances[VertexID], MinNeighbourGap);

				// if this is a method w/ no random variation or decay, this (point,radius) pair will never fit and can be removed
				if (bIsFixedRadiusMethod)
				{
					PointOrdering.RemoveAtSwap(k, EAllowShrinking::No);
					NumRemaining--;
					k--;		// reconsider point we just swapped to index k
				}
			}
			if (UseVertexID == IndexConstants::InvalidID)
			{
				NumFailures++;
				continue;
			}

			// remove selected point from ordering
			PointOrdering.RemoveAtSwap(PointOrderingIndex, EAllowShrinking::No);

			// emit our valid (point, triangle, radius) sample
			FVector3d SamplePoint = DensePointSet.DensePoints[UseVertexID];
			EmitSampleFunc(SamplePoint, DensePointSet.Triangles[UseVertexID], SampleRadius);
			NumEmittedSamples++;
			Octree.RemovePoint(UseVertexID);
			IsValidPoint[UseVertexID] = false;

			// add point to known samples list
			int32 SampleID = EmittedSamples.Num();
			EmittedSamples.Add(SamplePoint);
			EmittedRadius.Add(SampleRadius);

			// once we add this point, no point can be within it's radius, and any other point closer than
			// MinSampleRadius would collide, so we can decimate all points within the radius sum
			double CombinedRadiusSqr = ((SampleRadius + MinSampleRadius) * (SampleRadius + MinSampleRadius));

			// find all dense points within our query radius
			PointsInBall.Reset();
			FAxisAlignedBox3d QueryBox(SamplePoint, 2*SampleRadius);
			Octree.RangeQuery(QueryBox,		// add SphereQuery to Octree? would save a chunk...
				[&](int32 PointID) {
					return IsValidPoint[PointID] && DistanceSquared(DensePointSet.GetVertex(PointID), SamplePoint) < CombinedRadiusSqr;
				},
				PointsInBall, &QueryTempBuffer);

			// remove all those dense points from the octree and mark them invalid
			for (int32 QueryPointID : PointsInBall)
			{
				Octree.RemovePointUnsafe(QueryPointID);
				IsValidPoint[QueryPointID] = false;
			}
		}
	}
}








} // end namespace UELocal


void FMeshSurfacePointSampling::ComputePoissonSampling(const FDynamicMesh3& Mesh, FProgressCancel* Progress)
{
	Result = FGeometryResult(EGeometryResultType::InProgress);

	Samples.Reset();
	Radii.Reset();
	TriangleIDs.Reset();
	BarycentricCoords.Reset();
	auto AddSampleFunc = [&](FVector3d Position, int TriangleID, double Radius)
	{
		Samples.Add(FFrame3d(Position, Mesh.GetTriNormal(TriangleID)));
		Radii.Add(Radius);
		TriangleIDs.Add(TriangleID);
	};

	if (MaxSampleRadius > SampleRadius)
	{
		UELocal::FNonUniformSamplingConfig NonUniformConfig;
		NonUniformConfig.InterpretWeightMode = this->InterpretWeightMode;
		NonUniformConfig.SizeDistribution = this->SizeDistribution;
		NonUniformConfig.SizeDistributionPower = FMath::Clamp(this->SizeDistributionPower, 1.0, 10.0);

		if (bUseVertexWeights && VertexWeights.Num() == Mesh.MaxVertexID())
		{
			auto ComputeVertexWeightFunc = [&](int TriangleID, FVector3d Position, FVector3d BaryCoords)
				{
					FIndex3i Tri = Mesh.GetTriangle(TriangleID);
					double Weight = BaryCoords.X*VertexWeights[Tri.A] + BaryCoords.Y*VertexWeights[Tri.B] + BaryCoords.Z*VertexWeights[Tri.C];
					if (bInvertWeights)
					{
						Weight = 1.0 - FMathd::Clamp(Weight, 0.0, 1.0);
					}
					return Weight;
				};
			NonUniformConfig.WeightFunction = ComputeVertexWeightFunc;

			UELocal::NonUniformMeshPointSampling<FDynamicMesh3>(Mesh, AddSampleFunc,
				SampleRadius, MaxSampleRadius, MaxSamples, SubSampleDensity, RandomSeed, NonUniformConfig, MaxSubSamplePoints, SamplingMethodVersion, Progress);
		}
		else
		{
			UELocal::NonUniformMeshPointSampling<FDynamicMesh3>(Mesh, AddSampleFunc,
				SampleRadius, MaxSampleRadius, MaxSamples, SubSampleDensity, RandomSeed, NonUniformConfig, MaxSubSamplePoints, SamplingMethodVersion, Progress );
		}
	}
	else
	{
		UELocal::UniformMeshPointSampling<FDynamicMesh3>(Mesh, AddSampleFunc,
			SampleRadius, MaxSamples, SubSampleDensity, RandomSeed, MaxSubSamplePoints, SamplingMethodVersion, Progress );
	}

	if (bComputeBarycentrics)
	{
		FVector3d A,B,C;
		int32 N = Samples.Num();
		BarycentricCoords.SetNum(N);
		for (int32 k = 0; k < N; ++k)
		{
			Mesh.GetTriVertices(TriangleIDs[k], A, B, C);
			BarycentricCoords[k] = VectorUtil::BarycentricCoords( Samples[k].Origin, A, B, C);
		}
	}

	Result.SetSuccess(true, Progress);
}


template<class RealType>
bool TWeightedSamplingAliasTable<RealType>::Init(TConstArrayView<RealType> Weights, RealType SumOfValidWeights, bool bAllowInvalidWeights)
{
	Probability.Reset();
	Alias.Reset();

	const int32 NumWeights = Weights.Num();

	// handle the all-zero weight case specially
	if (SumOfValidWeights <= (RealType)0)
	{
		// in the edge case where we have no positive weights, and some invalid weights, generate valid weights to uniform-sample the zero weight triangles
		if (bAllowInvalidWeights)
		{
			// Make a new weight table where the zero weights are given uniform positive weight,
			// and the invalid weights remain negative
			TArray<RealType> UniformWeights;
			UniformWeights.SetNumUninitialized(NumWeights);
			RealType NewWeightSum = 0;
			bool bHaveInvalidWeights = false;
			for (int32 Idx = 0; Idx < NumWeights; ++Idx)
			{
				if (Weights[Idx] < 0)
				{
					bHaveInvalidWeights = true;
					UniformWeights[Idx] = -1;
				}
				else
				{
					NewWeightSum += 1;
					UniformWeights[Idx] = 1;
				}
			}
			// If there were invalid weights, use our new uniform weights to build the sampling table
			// (otherwise, just fall through to the no-invalid-weight uniform case, where we don't need the alias table)
			if (bHaveInvalidWeights)
			{
				if (NewWeightSum == (RealType)0)
				{
					// no valid indices to sample, cannot build table
					return false;
				}
				return Init(UniformWeights, SumOfValidWeights, true);
			}
		}
		
		// in the case of zero weight sum with no invalid weights, we can use a uniform probability table with no aliases
		Probability.Init(TMathUtilConstants<RealType>::MaxReal, NumWeights);
		// Leave alias empty, since it will never be picked
		return IsValid();
	}

	Probability.SetNumUninitialized(NumWeights);

	TArray<int32> Small, Large;
	Small.Reserve(NumWeights);
	Large.Reserve(NumWeights);

	// Transform weights to initial scaled probabilities, and initialize small, large index arrays
	RealType WeightToProb = (RealType)NumWeights / SumOfValidWeights;
	int32 KnownValidWeightIdx = INDEX_NONE;
	for (int32 Idx = NumWeights - 1; Idx >= 0; --Idx)
	{
		Probability[Idx] = Weights[Idx] * WeightToProb;
		if (bAllowInvalidWeights)
		{
			if (Probability[Idx] < 0)
			{
				Probability[Idx] = 0;
			}
			else
			{
				KnownValidWeightIdx = Idx;
			}
		}
		else
		{
			// Weights must be non-negative if we are not allowing invalid weights
			checkSlow(Probability[Idx] >= (RealType)0);
		}
		if (Probability[Idx] < 1)
		{
			Small.Add(Idx);
		}
		else
		{
			Large.Add(Idx);
		}
	}
	if (bAllowInvalidWeights && KnownValidWeightIdx == INDEX_NONE)
	{
		// No valid weights, clear the table and return failure
		Probability.Empty();
		return false;
	}
	
	// Build aliases
	Alias.SetNumUninitialized(NumWeights);
	while (!Small.IsEmpty() && !Large.IsEmpty())
	{
		int32 SmallIdx = Small.Pop(EAllowShrinking::No);
		int32 LargeIdx = Large.Pop(EAllowShrinking::No);
		Alias[SmallIdx] = LargeIdx;
		Probability[LargeIdx] = (Probability[LargeIdx] + Probability[SmallIdx]) - (RealType)1;
		if (Probability[LargeIdx] < (RealType)1)
		{
			Small.Add(LargeIdx);
		}
		else
		{
			Large.Add(LargeIdx);
		}
	}

	// Remaining unmatched large or small indices are assigned probability 1, don't need aliases
	for (int32 Idx : Large)
	{
		Probability[Idx] = TMathUtilConstants<RealType>::MaxReal;
		// Alias irrelevant
	}
	for (int32 Idx : Small)
	{
		// Theoretically possible due to numerical error to still have an invalid entry in the small list ...
		// In this unlikely event, just redirect to a known valid alias
		if (bAllowInvalidWeights && Weights[Idx] < 0)
		{
			Probability[Idx] = 0;
			Alias[Idx] = KnownValidWeightIdx;
		}
		else
		{
			Probability[Idx] = TMathUtilConstants<RealType>::MaxReal;
			// Alias irrelevant
		}
	}

	return IsValid();
}

namespace UE::Geometry
{
	template class TWeightedSamplingAliasTable<float>;
	template class TWeightedSamplingAliasTable<double>;
}

