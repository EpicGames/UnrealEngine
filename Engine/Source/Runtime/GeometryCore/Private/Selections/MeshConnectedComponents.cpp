// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/MeshConnectedComponents.h"
#include "Algo/Sort.h"

using namespace UE::Geometry;

namespace ConnectedComponentsLocals
{
	// We'll use these enums below instead of raw integers to provide some clarity in the code.
	// Note: To avoid changing the existing public API, the ActiveSet will remain a uint8 array,
	// rather than an array of this enum, even though that would be a more type safe solution.
	enum class EProcessingState : uint8
	{
		Unprocessed = 0,
		InQueue = 1,
		Done = 2,
		Invalid = 255
	};
}

using namespace ConnectedComponentsLocals;

void FMeshConnectedComponents::FindConnectedTriangles(TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedTriangles);
	
	// initial active set contains all valid triangles
	TArray<uint8> ActiveSet;
	int NumTriangles = Mesh->MaxTriangleID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumTriangles);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int tid = 0; tid < NumTriangles; ++tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			ActiveSet[tid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(tid);
		}
	}

	FindTriComponents(ActiveRange, ActiveSet, TrisConnectedPredicate);
}



void FMeshConnectedComponents::FindConnectedTriangles(TFunctionRef<bool(int)> IndexFilterFunc, TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedTriangles_Filtered);
	
	// initial active set contains all valid triangles
	TArray<uint8> ActiveSet;
	int NumTriangles = Mesh->MaxTriangleID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumTriangles);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int tid = 0; tid < NumTriangles; ++tid)
	{
		if (Mesh->IsTriangle(tid) && IndexFilterFunc(tid))
		{
			ActiveSet[tid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(tid);
		}
	}

	FindTriComponents(ActiveRange, ActiveSet, TrisConnectedPredicate);
}



void FMeshConnectedComponents::FindConnectedTriangles(const TArray<int>& TriangleROI, TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedTriangles_TriangleROI);

	// initial active set contains all valid triangles
	TArray<uint8> ActiveSet;
	int NumTriangles = Mesh->MaxTriangleID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumTriangles);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int tid : TriangleROI)
	{
		if (Mesh->IsTriangle(tid))
		{
			ActiveSet[tid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(tid);
		}
	}

	FindTriComponents(ActiveRange, ActiveSet, TrisConnectedPredicate);
}




void FMeshConnectedComponents::FindTrianglesConnectedToSeeds(const TArray<int>& SeedTriangles, TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedTriangles_SeedTriangles);
	
	// initial active set contains all valid triangles
	TArray<uint8> ActiveSet;
	int32 NumTriangles = Mesh->MaxTriangleID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumTriangles);
	for (int32 tid = 0; tid < NumTriangles; ++tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			ActiveSet[tid] = (uint8)EProcessingState::Unprocessed;
		}
	}

	FindTriComponents(SeedTriangles, ActiveSet, TrisConnectedPredicate);
}


void FMeshConnectedComponents::FindConnectedVertices(TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedVertices);

	// initial active set contains all valid vertices
	TArray<uint8> ActiveSet;
	int NumVertices = Mesh->MaxVertexID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumVertices);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int vid = 0; vid < NumVertices; ++vid)
	{
		if (Mesh->IsVertex(vid))
		{
			ActiveSet[vid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(vid);
		}
	}

	FindVertComponents(ActiveRange, ActiveSet, VertsConnectedPredicate);
}



void FMeshConnectedComponents::FindConnectedVertices(TFunctionRef<bool(int)> IndexFilterFunc, TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedVertices_Filtered);

	// initial active set contains all valid vertices
	TArray<uint8> ActiveSet;
	int NumVertices = Mesh->MaxVertexID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumVertices);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int vid = 0; vid < NumVertices; ++vid)
	{
		if (Mesh->IsVertex(vid) && IndexFilterFunc(vid))
		{
			ActiveSet[vid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(vid);
		}
	}

	FindVertComponents(ActiveRange, ActiveSet, VertsConnectedPredicate);
}



void FMeshConnectedComponents::FindConnectedVertices(const TArray<int>& VertexROI, TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedVertices_VertexROI);

	// initial active set contains all valid vertices
	TArray<uint8> ActiveSet;
	int NumVertices = Mesh->MaxVertexID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumVertices);
	FInterval1i ActiveRange = FInterval1i::Empty();
	for (int vid : VertexROI)
	{
		if (Mesh->IsVertex(vid))
		{
			ActiveSet[vid] = (uint8)EProcessingState::Unprocessed;
			ActiveRange.Contain(vid);
		}
	}

	FindVertComponents(ActiveRange, ActiveSet, VertsConnectedPredicate);
}




void FMeshConnectedComponents::FindVerticesConnectedToSeeds(const TArray<int>& SeedVertices, TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshConnectedComponents_FindConnectedVertices_SeedVertices);

	// initial active set contains all valid vertices
	TArray<uint8> ActiveSet;
	int NumVertices = Mesh->MaxVertexID();
	ActiveSet.Init((uint8)EProcessingState::Invalid, NumVertices);
	for (int32 vid = 0; vid < NumVertices; ++vid)
	{
		if (Mesh->IsVertex(vid))
		{
			ActiveSet[vid] = (uint8)EProcessingState::Unprocessed;
		}
	}

	FindVertComponents(SeedVertices, ActiveSet, VertsConnectedPredicate);
}


void FMeshConnectedComponents::FindTriComponents(FInterval1i ActiveRange, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindTriComponents_ActiveRange);
	
	Components.Empty();

	// temporary queue
	TArray<int32> ComponentQueue;
	ComponentQueue.Reserve(256);

	// keep finding valid seed triangles and growing connected components
	// until we are done
	for (int i = ActiveRange.Min; i <= ActiveRange.Max; i++)
	{
		if (ActiveSet[i] != (uint8)EProcessingState::Invalid)
		{
			int SeedTri = i;
			ComponentQueue.Add(SeedTri);
			ActiveSet[SeedTri] = (uint8)EProcessingState::InQueue;      // in ComponentQueue

			FComponent* Component = new FComponent();
			if (TrisConnectedPredicate)
			{
				FindTriComponent(Component, ComponentQueue, ActiveSet, TrisConnectedPredicate);
			}
			else
			{
				FindTriComponent(Component, ComponentQueue, ActiveSet);
			}
			Components.Add(Component);

			RemoveFromActiveSet(Component, ActiveSet);

			ComponentQueue.Reset(0);
		}
	}
}




void FMeshConnectedComponents::FindTriComponents(const TArray<int32>& SeedList, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> TrisConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindTriComponents_SeedList);
	
	Components.Empty();

	// temporary queue
	TArray<int32> ComponentQueue;
	ComponentQueue.Reserve(256);

	// keep finding valid seed triangles and growing connected components
	// until we are done
	for ( int32 SeedTri : SeedList )
	{
		if (ActiveSet.IsValidIndex(SeedTri) && ActiveSet[SeedTri] != (uint8)EProcessingState::Invalid)
		{
			ComponentQueue.Add(SeedTri);
			ActiveSet[SeedTri] = (uint8)EProcessingState::InQueue;      // in ComponentQueue

			FComponent* Component = new FComponent();
			if (TrisConnectedPredicate)
			{
				FindTriComponent(Component, ComponentQueue, ActiveSet, TrisConnectedPredicate);
			}
			else
			{
				FindTriComponent(Component, ComponentQueue, ActiveSet);
			}
			Components.Add(Component);

			RemoveFromActiveSet(Component, ActiveSet);

			ComponentQueue.Reset(0);
		}
	}
}






void FMeshConnectedComponents::FindTriComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindTriComponents_Component);

	while (ComponentQueue.Num() > 0)
	{
		int32 CurTriangle = ComponentQueue.Pop(EAllowShrinking::No);

		ActiveSet[CurTriangle] = (uint8)EProcessingState::Done;   // tri has been processed
		Component->Indices.Add(CurTriangle);

		FIndex3i TriNbrTris = Mesh->GetTriNeighbourTris(CurTriangle);
		for (int j = 0; j < 3; ++j)
		{
			int NbrTri = TriNbrTris[j];
			if (NbrTri != FDynamicMesh3::InvalidID && ActiveSet[NbrTri] == (uint8)EProcessingState::Unprocessed)
			{
				ComponentQueue.Add(NbrTri);
				ActiveSet[NbrTri] = (uint8)EProcessingState::InQueue;           // in ComponentQueue
			}
		}
	}
}


void FMeshConnectedComponents::FindTriComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet, 
	TFunctionRef<bool(int32, int32)> TriConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindTriComponents_Component_Filtered);
	
	while (ComponentQueue.Num() > 0)
	{
		int32 CurTriangle = ComponentQueue.Pop(EAllowShrinking::No);

		ActiveSet[CurTriangle] = (uint8)EProcessingState::Done;   // tri has been processed
		Component->Indices.Add(CurTriangle);

		FIndex3i TriNbrTris = Mesh->GetTriNeighbourTris(CurTriangle);
		for (int j = 0; j < 3; ++j)
		{
			int NbrTri = TriNbrTris[j];
			if (NbrTri != FDynamicMesh3::InvalidID && ActiveSet[NbrTri] == (uint8)EProcessingState::Unprocessed && TriConnectedPredicate(CurTriangle, NbrTri))
			{
				ComponentQueue.Add(NbrTri);
				ActiveSet[NbrTri] = (uint8)EProcessingState::InQueue;           // in ComponentQueue
			}
		}
	}
}



void FMeshConnectedComponents::RemoveFromActiveSet(const FComponent* Component, TArray<uint8>& ActiveSet)
{
	int32 ComponentSize = Component->Indices.Num();
	for (int32 j = 0; j < ComponentSize; ++j)
	{
		ActiveSet[Component->Indices[j]] = (uint8)EProcessingState::Invalid;
	}
}


void FMeshConnectedComponents::FindVertComponents(FInterval1i ActiveRange, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVertComponents_ActiveRange);

	Components.Empty();

	// temporary queue
	TArray<int32> ComponentQueue;
	ComponentQueue.Reserve(256);

	// keep finding valid seed vertices and growing connected components
	// until we are done
	for (int i = ActiveRange.Min; i <= ActiveRange.Max; i++)
	{
		if (ActiveSet[i] != (uint8)EProcessingState::Invalid)
		{
			int SeedVert = i;
			ComponentQueue.Add(SeedVert);
			ActiveSet[SeedVert] = (uint8)EProcessingState::InQueue;      // in ComponentQueue

			FComponent* Component = new FComponent();
			if (VertsConnectedPredicate)
			{
				FindVertComponent(Component, ComponentQueue, ActiveSet, VertsConnectedPredicate);
			}
			else
			{
				FindVertComponent(Component, ComponentQueue, ActiveSet);
			}
			Components.Add(Component);

			RemoveFromActiveSet(Component, ActiveSet);

			ComponentQueue.Reset(0);
		}
	}
}

void FMeshConnectedComponents::FindVertComponents(const TArray<int32>& SeedList, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVertComponents_SeedList);

	Components.Empty();

	// temporary queue
	TArray<int32> ComponentQueue;
	ComponentQueue.Reserve(256);

	// keep finding valid seed vertices and growing connected components
	// until we are done
	for (int32 SeedVert : SeedList)
	{
		if (ActiveSet.IsValidIndex(SeedVert) && ActiveSet[SeedVert] != (uint8)EProcessingState::Invalid)
		{
			ComponentQueue.Add(SeedVert);
			ActiveSet[SeedVert] = (uint8)EProcessingState::InQueue;      // in ComponentQueue

			FComponent* Component = new FComponent();
			if (VertsConnectedPredicate)
			{
				FindVertComponent(Component, ComponentQueue, ActiveSet, VertsConnectedPredicate);
			}
			else
			{
				FindVertComponent(Component, ComponentQueue, ActiveSet);
			}
			Components.Add(Component);

			RemoveFromActiveSet(Component, ActiveSet);

			ComponentQueue.Reset(0);
		}
	}

}

void FMeshConnectedComponents::FindVertComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVertComponents_Component);

	while (ComponentQueue.Num() > 0)
	{
		int32 CurVert = ComponentQueue.Pop(EAllowShrinking::No);

		ActiveSet[CurVert] = (uint8)EProcessingState::Done;   // Vert has been processed
		Component->Indices.Add(CurVert);

		auto ProcessOneRing = [&ActiveSet, &ComponentQueue](int32 NbrVert) mutable
		{
			if (NbrVert != FDynamicMesh3::InvalidID && ActiveSet[NbrVert] == (uint8)EProcessingState::Unprocessed)
			{
				ComponentQueue.Add(NbrVert);
				ActiveSet[NbrVert] = (uint8)EProcessingState::InQueue;           // in ComponentQueue
			}
		};

		Mesh->EnumerateVertexVertices(CurVert, ProcessOneRing);
	}
}

void FMeshConnectedComponents::FindVertComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet,
	 TFunctionRef<bool(int32, int32)> VertsConnectedPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindVertComponents_Component_Filtered);

	while (ComponentQueue.Num() > 0)
	{
		int32 CurVert = ComponentQueue.Pop(EAllowShrinking::No);

		ActiveSet[CurVert] = (uint8)EProcessingState::Done;   // Vert has been processed
		Component->Indices.Add(CurVert);

		auto ProcessOneRing = [&CurVert, &ActiveSet, &VertsConnectedPredicate, &ComponentQueue](int32 NbrVert) mutable
		{
			if (NbrVert != FDynamicMesh3::InvalidID && ActiveSet[NbrVert] == (uint8)EProcessingState::Unprocessed && VertsConnectedPredicate(CurVert, NbrVert))
			{
				ComponentQueue.Add(NbrVert);
				ActiveSet[NbrVert] = (uint8)EProcessingState::InQueue;           // in ComponentQueue
			}
		};

		Mesh->EnumerateVertexVertices(CurVert, ProcessOneRing);
	}
}



int32 FMeshConnectedComponents::GetLargestIndexByCount() const
{
	if (Components.Num() == 0)
	{
		return -1;
	}

	int LargestIdx = 0;
	int LargestCount = Components[LargestIdx].Indices.Num();
	int NumComponents = Components.Num();
	for (int i = 1; i < NumComponents; ++i) {
		if (Components[i].Indices.Num() > LargestCount) {
			LargestCount = Components[i].Indices.Num();
			LargestIdx = i;
		}
	}
	return LargestIdx;
}



void FMeshConnectedComponents::SortByCount(bool bLargestFirst)
{
	TArrayView<FComponent*> View( Components.GetData(), Components.Num() );
	if (bLargestFirst)
	{
		View.StableSort(
			[](const FComponent& A, const FComponent& B) { return (A).Indices.Num() > (B).Indices.Num(); }
		);
	}
	else
	{
		View.StableSort(
			[](const FComponent& A, const FComponent& B) { return (A).Indices.Num() < (B).Indices.Num(); }
		);
	}
}




bool FMeshConnectedComponents::InitializeFromTriangleComponents(const TArray<TArray<int32>>& ComponentLists, bool bValidateIDs)
{
	if (bValidateIDs)
	{
		for (const TArray<int32>& ComponentList : ComponentLists)
		{
			for (int32 tid : ComponentList)
			{
				if (Mesh->IsTriangle(tid) == false)
				{
					return false;
				}
			}
		}
	}

	Components.Reset();
	Components.Reserve(ComponentLists.Num());
	for (const TArray<int32>& ComponentList : ComponentLists)
	{
		// skip any empty lists
		if (ComponentList.Num() == 0)
		{
			continue;
		}

		FComponent* Component = new FComponent();
		Component->Indices.Append(ComponentList);
		Components.Add(Component);
	}

	return true;
}


bool FMeshConnectedComponents::InitializeFromTriangleComponents(TArray<TArray<int32>>& ComponentLists, bool bMoveSubLists, bool bValidateIDs)
{
	if (bValidateIDs)
	{
		for (const TArray<int32>& ComponentList : ComponentLists)
		{
			for (int32 tid : ComponentList)
			{
				if (Mesh->IsTriangle(tid) == false)
				{
					return false;
				}
			}
		}
	}

	Components.Reset();
	Components.Reserve(ComponentLists.Num());
	for (TArray<int32>& ComponentList : ComponentLists)
	{
		// skip any empty lists
		if (ComponentList.Num() == 0)
		{
			continue;
		}

		FComponent* Component = new FComponent();
		if (bMoveSubLists)
		{
			Component->Indices = MoveTemp(ComponentList);
		}
		else
		{
			Component->Indices.Append(ComponentList);
		}

		Components.Add(Component);
	}

	return true;
}

bool FMeshConnectedComponents::InitializeFromVertexComponents(const TArray<TArray<int32>>& ComponentLists, bool bValidateIDs)
{
	if (bValidateIDs)
	{
		for (const TArray<int32>& ComponentList : ComponentLists)
		{
			for (int32 Vid : ComponentList)
			{
				if (Mesh->IsVertex(Vid) == false)
				{
					return false;
				}
			}
		}
	}

	Components.Reset();
	Components.Reserve(ComponentLists.Num());
	for (const TArray<int32>& ComponentList : ComponentLists)
	{
		// skip any empty lists
		if (ComponentList.Num() == 0)
		{
			continue;
		}

		FComponent* Component = new FComponent();
		Component->Indices.Append(ComponentList);
		Components.Add(Component);
	}

	return true;
}


bool FMeshConnectedComponents::InitializeFromVertexComponents(TArray<TArray<int32>>& ComponentLists, bool bMoveSubLists, bool bValidateIDs)
{
	if (bValidateIDs)
	{
		for (const TArray<int32>& ComponentList : ComponentLists)
		{
			for (int32 Vid : ComponentList)
			{
				if (Mesh->IsVertex(Vid) == false)
				{
					return false;
				}
			}
		}
	}

	Components.Reset();
	Components.Reserve(ComponentLists.Num());
	for (TArray<int32>& ComponentList : ComponentLists)
	{
		// skip any empty lists
		if (ComponentList.Num() == 0)
		{
			continue;
		}

		FComponent* Component = new FComponent();
		if (bMoveSubLists)
		{
			Component->Indices = MoveTemp(ComponentList);
		}
		else
		{
			Component->Indices.Append(ComponentList);
		}

		Components.Add(Component);
	}

	return true;
}



void FMeshConnectedComponents::GrowToConnectedTriangles(const FDynamicMesh3* Mesh,
	const TArray<int>& InputROI, TArray<int>& ResultROI,
	TArray<int32>* QueueBuffer, 
	TSet<int32>* DoneBuffer,
	TFunctionRef<bool(int32, int32)> CanGrowPredicate
)
{
	TArray<int32> LocalQueue;
	QueueBuffer = (QueueBuffer == nullptr) ? &LocalQueue : QueueBuffer;
	QueueBuffer->Reset(); 
	QueueBuffer->Append(InputROI);

	TSet<int32> LocalDone;
	DoneBuffer = (DoneBuffer == nullptr) ? &LocalDone : DoneBuffer;
	DoneBuffer->Reset(); 
	DoneBuffer->Append(InputROI);

	while (QueueBuffer->Num() > 0)
	{
		int32 CurTri = QueueBuffer->Pop(EAllowShrinking::No);

		FIndex3i NbrTris = Mesh->GetTriNeighbourTris(CurTri);
		for (int j = 0; j < 3; ++j)
		{
			int32 tid = NbrTris[j];
			if (tid != FDynamicMesh3::InvalidID && DoneBuffer->Contains(tid) == false && CanGrowPredicate(CurTri, tid))
			{
				QueueBuffer->Add(tid);
				DoneBuffer->Add(tid);
			}
		}
	}
	ResultROI.Reserve(ResultROI.Num() + DoneBuffer->Num());
	for (int32 TID : *DoneBuffer)
	{
		ResultROI.Add(TID);
	}
}



void FMeshConnectedComponents::GrowToConnectedTriangles(const FDynamicMesh3* Mesh,
	const TArray<int>& InputROI, TSet<int>& ResultROI,
	TArray<int32>* QueueBuffer,
	TFunctionRef<bool(int32, int32)> CanGrowPredicate
)
{
	TArray<int32> LocalQueue;
	QueueBuffer = (QueueBuffer == nullptr) ? &LocalQueue : QueueBuffer;
	QueueBuffer->Reset();
	QueueBuffer->Append(InputROI);

	ResultROI.Reset();
	ResultROI.Append(InputROI);

	while (QueueBuffer->Num() > 0)
	{
		int32 CurTri = QueueBuffer->Pop(EAllowShrinking::No);

		FIndex3i NbrTris = Mesh->GetTriNeighbourTris(CurTri);
		for (int j = 0; j < 3; ++j)
		{
			int32 tid = NbrTris[j];
			if (tid != FDynamicMesh3::InvalidID && ResultROI.Contains(tid) == false && CanGrowPredicate(CurTri, tid))
			{
				QueueBuffer->Add(tid);
				ResultROI.Add(tid);
			}
		}
	}
}



void FMeshConnectedComponents::GrowToConnectedVertices(const FDynamicMesh3& Mesh,
	const TArray<int>& InputROI, TSet<int>& ResultROI,
	TArray<int32>* QueueBuffer,
	TFunctionRef<bool(int32, int32)> CanGrowPredicate )
{
	TArray<int32> LocalQueue;
	QueueBuffer = (QueueBuffer == nullptr) ? &LocalQueue : QueueBuffer;
	QueueBuffer->Reset();
	QueueBuffer->Append(InputROI);

	ResultROI.Reset();
	ResultROI.Append(InputROI);

	while (QueueBuffer->Num() > 0)
	{
		int32 CurVertexID = QueueBuffer->Pop(EAllowShrinking::No);

		Mesh.EnumerateVertexVertices(CurVertexID, [&](int32 NbrVertexID)
		{
			if (ResultROI.Contains(NbrVertexID) == false && CanGrowPredicate(CurVertexID, NbrVertexID))
			{
				QueueBuffer->Add(NbrVertexID);
				ResultROI.Add(NbrVertexID);
			}
		});

	}
}



void FMeshConnectedComponents::GrowToConnectedEdges(const FDynamicMesh3& Mesh,
	const TArray<int>& InputROI, TSet<int>& ResultROI,
	TArray<int32>* QueueBuffer,
	TFunctionRef<bool(int32, int32)> CanGrowPredicate )
{
	TArray<int32> LocalQueue;
	QueueBuffer = (QueueBuffer == nullptr) ? &LocalQueue : QueueBuffer;
	QueueBuffer->Reset();
	QueueBuffer->Append(InputROI);

	ResultROI.Reset();
	ResultROI.Append(InputROI);

	while (QueueBuffer->Num() > 0)
	{
		int32 CurEdgeID = QueueBuffer->Pop(EAllowShrinking::No);

		FIndex2i EdgeV = Mesh.GetEdgeV(CurEdgeID);
		for (int32 k = 0; k < 2; ++k)
		{
			Mesh.EnumerateVertexEdges(EdgeV[k], [&](int32 NbrEdgeID)
			{
				if (NbrEdgeID != CurEdgeID && ResultROI.Contains(NbrEdgeID) == false && CanGrowPredicate(CurEdgeID, NbrEdgeID))
				{
					QueueBuffer->Add(NbrEdgeID);
					ResultROI.Add(NbrEdgeID);
				}
			});
		}

	}
}