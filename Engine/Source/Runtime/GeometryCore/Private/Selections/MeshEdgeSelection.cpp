// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/MeshEdgeSelection.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Containers/Set.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Math/UnrealMathUtility.h"

using namespace UE::Geometry;

// convert vertex selection to edge selection. Require at least minCount verts of edge to be selected
FMeshEdgeSelection::FMeshEdgeSelection(const FDynamicMesh3* mesh, const FMeshVertexSelection& convertV, int minCount) : Mesh(mesh)
{
	for (int eid : Mesh->EdgeIndicesItr())
	{
		FIndex2i ev = Mesh->GetEdgeV(eid);
		int n = (convertV.IsSelected(ev.A) ? 1 : 0) +
			(convertV.IsSelected(ev.B) ? 1 : 0);
		if (n >= minCount)
		{
			add(eid);
		}
	}
}

// convert face selection to edge selection. Require at least minCount tris of edge to be selected
FMeshEdgeSelection::FMeshEdgeSelection(const FDynamicMesh3* mesh, const FMeshFaceSelection& convertT, int minCount) : Mesh(mesh)
{
	minCount = FMath::Clamp(minCount, 1, 2);

	if (minCount == 1)
	{
		for (int tid : convertT)
		{
			FIndex3i te = Mesh->GetTriEdges(tid);
			add(te.A); add(te.B); add(te.C);
		}
	}
	else
	{
		for (int eid : Mesh->EdgeIndicesItr())
		{
			FIndex2i et = Mesh->GetEdgeT(eid);
			if (convertT.IsSelected(et.A) && convertT.IsSelected(et.B))
			{
				add(eid);
			}
		}
	}
}

void FMeshEdgeSelection::SelectBoundaryTriEdges(const FMeshFaceSelection& Triangles)
{
	for (int tid : Triangles)
	{
		FIndex3i te = Mesh->GetTriEdges(tid);
		for ( int j = 0; j < 3; ++j )
		{
			FIndex2i et = Mesh->GetEdgeT(te[j]);
			int other_tid = (et.A == tid) ? et.B : et.A;
			if (Triangles.IsSelected(other_tid) == false)
			{
				add(te[j]);
			}
		}
	}
}