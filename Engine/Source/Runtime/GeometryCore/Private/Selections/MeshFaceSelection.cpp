// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Math/UnrealMathUtility.h"

using namespace UE::Geometry;

// convert vertex selection to face selection. Require at least minCount verts of
// tri to be selected (valid values are 1,2,3)
FMeshFaceSelection::FMeshFaceSelection(const FDynamicMesh3* mesh, const FMeshVertexSelection& convertV, int minCount) : Mesh(mesh)
{
	minCount = FMath::Clamp(minCount, 1, 3);

	if (minCount == 1)
	{
		for ( int vid : convertV )
		{
			for (int tid : Mesh->VtxTrianglesItr(vid))
			{
				add(tid);
			}
		}
	} else {
		for (int tid : Mesh->TriangleIndicesItr()) {
			FIndex3i tri = Mesh->GetTriangle(tid);
			if (minCount == 3)
			{
				if (convertV.IsSelected(tri.A) && convertV.IsSelected(tri.B) && convertV.IsSelected(tri.C))
				{
					add(tid);
				}
			}
			else
			{
				int n = (convertV.IsSelected(tri.A) ? 1 : 0) +
					(convertV.IsSelected(tri.B) ? 1 : 0) +
					(convertV.IsSelected(tri.C) ? 1 : 0);
				if (n >= minCount)
				{
					add(tid);
				}
			}
		}
	}
}





void FMeshFaceSelection::ExpandToOneRingNeighbours(const TUniqueFunction<bool(int)>& FilterF)
{
	TSet<int> TrisToAdd;

	for (int tid : Selected) 
	{ 
		FIndex3i TriEdges = Mesh->GetTriEdges(tid);
		TArray<int32, TFixedAllocator<3>> VertsToProcess;
		for (int j = 0; j < 3; ++j)
		{
			TPair<bool, bool> BoundaryInfo = IsSelectionBoundaryEdge(TriEdges[j]);
			if ( BoundaryInfo.Key != BoundaryInfo.Value)		// selection boundary but not mesh boundary
			{
				FIndex2i EdgeV = Mesh->GetEdgeV(TriEdges[j]);
				VertsToProcess.AddUnique(EdgeV.A);
				VertsToProcess.AddUnique(EdgeV.B);
			}
		}
		for (int vid : VertsToProcess)
		{
			Mesh->EnumerateVertexTriangles(vid, [&](int nbr_t)
			{
				if (FilterF && FilterF(nbr_t) == false)
				{
					return;
				}
				if (!IsSelected(nbr_t))
				{
					TrisToAdd.Add(nbr_t);
				}
			});
		}
	}

	for (int tid : TrisToAdd)
	{
		add(tid);
	}
}


void FMeshFaceSelection::ExpandToOneRingNeighbours(int nRings, const TUniqueFunction<bool(int)>& FilterF)
{
	if (nRings == 1)
	{
		ExpandToOneRingNeighbours(FilterF);
		return;
	}

	// todo: rewrite to use TSet and mesh edges as in ExpandToOneRingNeighbours above

	TArray<int> triArrays[2];
	int addIdx = 0, checkIdx = 1;
	triArrays[checkIdx] = Selected.Array();

	TBitArray<FDefaultBitArrayAllocator> Bitmap = AsBitArray();

	for (int ri = 0; ri < nRings; ++ri)
	{
		TArray<int>& addTris = triArrays[addIdx];
		TArray<int>& checkTris = triArrays[checkIdx];
		addTris.Empty();

		for (int tid : checkTris)
		{
			FIndex3i tri_v = Mesh->GetTriangle(tid);
			for (int j = 0; j < 3; ++j)
			{
				int vid = tri_v[j];
				Mesh->EnumerateVertexTriangles(vid, [&](int nbr_t)
				{
					if (FilterF && FilterF(nbr_t) == false)
					{
						return;
					}
					if (Bitmap[nbr_t] == false)
					{
						addTris.Add(nbr_t);
						Bitmap[nbr_t] = true;
					}
				});
			}
		}

		for (int TID : addTris)
		{
			add(TID);
		}

		Swap(addIdx, checkIdx); // check in the next iter what we added in this iter
	}
}




void FMeshFaceSelection::ContractBorderByOneRingNeighbours(int NumRings, bool bContractFromMeshBoundary, const TUniqueFunction<bool(int)>& FilterF)
{
	TSet<int> BorderVIDs;   // border vertices

	// TODO: can we track the boundary across iterations? Slightly tricky because the modified tris
	// are no longer in the selection set. However it seems like that is still going to work because
	// all the selection boundary edges will still be adjacent to that set of deselected triangles

	for (int RingIdx = 0; RingIdx < NumRings; ++RingIdx)
	{
		BorderVIDs.Reset();

		// find set of vertices on border
		for (int tid : Selected)
		{
			FIndex3i TriEdges = Mesh->GetTriEdges(tid);
			TArray<int32, TInlineAllocator<3>> BoundaryVertsToProcess;
			for (int j = 0; j < 3; ++j)
			{
				TPair<bool, bool> BoundaryInfo = IsSelectionBoundaryEdge(TriEdges[j]);
				if (BoundaryInfo.Key != false)		// we are some kind of boundary edge
				{
					if (bContractFromMeshBoundary || BoundaryInfo.Value == false)
					{
						FIndex2i EdgeV = Mesh->GetEdgeV(TriEdges[j]);
						BorderVIDs.Add(EdgeV.A);
						BorderVIDs.Add(EdgeV.B);
					}
				}
			}
		}

		for (int VID : BorderVIDs)
		{
			Mesh->EnumerateVertexTriangles(VID, [&](int32 NbrT)
			{
				if (FilterF && !FilterF(NbrT))
				{
					return;
				}
				Deselect(NbrT);
			});
		}
	}

}