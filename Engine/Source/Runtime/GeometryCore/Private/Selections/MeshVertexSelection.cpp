// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/MeshVertexSelection.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshEdgeSelection.h"

#include "DynamicMesh/DynamicMesh3.h"

using namespace UE::Geometry;

// convert face selection to vertex selection. 
FMeshVertexSelection::FMeshVertexSelection(const FDynamicMesh3* mesh, const FMeshFaceSelection& convertT) : Mesh(mesh)
{
	for (int tid : convertT) {
		FIndex3i tv = Mesh->GetTriangle(tid);
		add(tv.A); add(tv.B); add(tv.C);
	}
}

// convert edge selection to vertex selection. 
FMeshVertexSelection::FMeshVertexSelection(const FDynamicMesh3* mesh, const FMeshEdgeSelection& convertE) : Mesh(mesh)
{
	for (int eid : convertE) {
		FIndex2i ev = Mesh->GetEdgeV(eid);
		add(ev.A); add(ev.B);
	}
}

void FMeshVertexSelection::SelectTriangleVertices(const FMeshFaceSelection& Triangles)
{
	for (int tid : Triangles)
	{
		FIndex3i tri = Mesh->GetTriangle(tid);
		add(tri.A); add(tri.B); add(tri.C);
	}
}

void FMeshVertexSelection::SelectInteriorVertices(const FMeshFaceSelection& Triangles)
{
	TSet<int> borderv;
	for (int tid : Triangles) {
		FIndex3i tv = Mesh->GetTriangle(tid);
		for ( int j = 0; j < 3; ++j ) {
			int vid = tv[j];
			if (Selected.Contains(vid) || borderv.Contains(vid))
			{
				continue;
			}
			bool full_ring = true;
			for (int ring_tid : Mesh->VtxTrianglesItr(vid))
			{
				if (Triangles.IsSelected(ring_tid) == false)
				{
					full_ring = false;
					break;
				}
			}
			if (full_ring)
			{
				add(vid);
			}
			else
			{
				borderv.Add(vid);
			}
		}
	}
}

