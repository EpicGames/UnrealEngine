// Copyright Epic Games, Inc. All Rights Reserved.

#include "Spatial/GeometrySet3.h"
#include "Distance/DistRay3Segment3.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

void FGeometrySet3::Reset(bool bPoints, bool bCurves)
{
	if (bPoints)
	{
		Points.Reset();
		PointIDToIndex.Reset();
	}
	if (bCurves)
	{
		Curves.Reset();
		CurveIDToIndex.Reset();
	}
}

void FGeometrySet3::AddPoint(int PointID, const FVector3d& Point)
{
	check(PointIDToIndex.Contains(PointID) == false);
	FPoint NewPoint = { PointID, Point };
	int NewIndex = Points.Add(NewPoint);
	PointIDToIndex.Add(PointID, NewIndex);
}

void FGeometrySet3::AddCurve(int CurveID, const FPolyline3d& Polyline)
{
	check(CurveIDToIndex.Contains(CurveID) == false);
	int NewIndex = Curves.Add(FCurve());
	FCurve& NewCurve = Curves[NewIndex];
	NewCurve.ID = CurveID;
	NewCurve.Geometry = Polyline;
	NewCurve.Bounds = Polyline.GetBounds();
	CurveIDToIndex.Add(CurveID, NewIndex);
}

void FGeometrySet3::RemovePoint(int PointID)
{
	const int* IndexPointer = PointIDToIndex.Find(PointID);
	check(IndexPointer != nullptr);
	int32 Index = *IndexPointer;
	Points.RemoveAt(Index);
	PointIDToIndex.Remove(PointID);

	// Since we store things in a simple array, the indices have shifted
	for (TPair<int,int>& Entry : PointIDToIndex)
	{
		if (Entry.Value > Index)
		{
			--Entry.Value;
		}
	}
}

void FGeometrySet3::RemoveCurve(int CurveID)
{
	const int* IndexPointer = CurveIDToIndex.Find(CurveID);
	check(IndexPointer != nullptr);
	int32 Index = *IndexPointer;
	Curves.RemoveAt(Index);
	CurveIDToIndex.Remove(CurveID);

	// Since we store things in a simple array, the indices have shifted
	for (TPair<int, int>& Entry : CurveIDToIndex)
	{
		if (Entry.Value > Index)
		{
			--Entry.Value;
		}
	}
}

void FGeometrySet3::UpdatePoint(int PointID, const FVector3d& Point)
{
	const int* Index = PointIDToIndex.Find(PointID);
	check(Index != nullptr);
	Points[*Index].Position = Point;
}

void FGeometrySet3::UpdateCurve(int CurveID, const FPolyline3d& Polyline)
{
	const int* Index = CurveIDToIndex.Find(CurveID);
	check(Index != nullptr);
	Curves[*Index].Geometry.SetVertices(Polyline.GetVertices());
	Curves[*Index].Bounds = Polyline.GetBounds();
}



bool FGeometrySet3::FindNearestPointToRay(const FRay3d& Ray, FNearest& ResultOut,
	TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const
{
	double MinRayParamT = TNumericLimits<double>::Max();
	int NearestID = -1;
	int NearestIndex = -1;

	FCriticalSection Critical;

	int NumPoints = Points.Num();
	ParallelFor(NumPoints, [&](int pi)
	{
		const FPoint& Point = Points[pi];
		double RayT = Ray.GetParameter(Point.Position);
		FVector3d RayPt = Ray.ClosestPoint(Point.Position);
		if (PointWithinToleranceTest(RayPt, Point.Position))
		{
			Critical.Lock();
			if (RayT < MinRayParamT)
			{
				MinRayParamT = RayT;
				NearestIndex = pi;
				NearestID = Points[pi].ID;
			}
			Critical.Unlock();
		}
	});

	const bool bFound = NearestID != -1;
	if (bFound)
	{
		ResultOut.ID = NearestID;
		ResultOut.bIsPoint = true;
		ResultOut.NearestRayPoint = Ray.PointAt(MinRayParamT);
		ResultOut.NearestGeoPoint = Points[NearestIndex].Position;
		ResultOut.RayParam = MinRayParamT;
	}

	return bFound;
}

bool FGeometrySet3::CollectPointsNearRay(const FRay3d& Ray, TArray<FNearest>& ResultsOut,
	TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const
{
	// This is used later to see if we added any new results.
	int OriginalResultsLength = ResultsOut.Num();

	// To avoid locking in the ParallelFor below, we let all points have space for their
	// own output even if their point turns out not to be within tolerance.
	int NumPoints = Points.Num();
	TArray<FNearest> NonCompactResults;
	NonCompactResults.AddUninitialized(NumPoints);
	TArray<bool> WithinTolerance;
	WithinTolerance.AddZeroed(NumPoints);

	// Check distances to points in parallel
	ParallelFor(NumPoints, [&](int pi)
	{
		const FPoint& Point = Points[pi];
		double RayT = Ray.GetParameter(Point.Position);
		FVector3d RayPt = Ray.ClosestPoint(Point.Position);
		if (PointWithinToleranceTest(RayPt, Point.Position))
		{
			WithinTolerance[pi] = true;

			// Save result
			NonCompactResults[pi].ID = Points[pi].ID;
			NonCompactResults[pi].bIsPoint = true;
			NonCompactResults[pi].NearestRayPoint = Ray.PointAt(RayT);
			NonCompactResults[pi].NearestGeoPoint = Points[pi].Position;
			NonCompactResults[pi].RayParam = RayT;
		}
	});

	// Output only those results that were within tolerance
	for (int i = 0; i < NonCompactResults.Num(); ++i)
	{
		if (WithinTolerance[i])
		{
			ResultsOut.Add(NonCompactResults[i]);
		}
	}

	return ResultsOut.Num() != OriginalResultsLength;
}


bool FGeometrySet3::FindNearestCurveToRay(const FRay3d& Ray, FNearest& ResultOut,
	TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const
{
	double MinRayParamT = TNumericLimits<double>::Max();
	int NearestID = -1;
	int NearestIndex = -1;
	int NearestSegmentIdx = -1;
	double NearestSegmentParam = 0;
	double NearestCurveMetric = TNumericLimits<double>::Max();

	FCriticalSection Critical;

	int NumCurves = Curves.Num();
	ParallelFor(NumCurves, [&](int ci)
	{
		const FCurve& Curve = Curves[ci];
		const FPolyline3d& Polyline = Curve.Geometry;
		int NumSegments = Polyline.SegmentCount();

		double NearestSegmentMetric = TNumericLimits<double>::Max();
		double CurveNearestSegmentRayParamT = TNumericLimits<double>::Max();
		int CurveNearestSegmentIdx = -1;
		double CurveNearestSegmentParam = 0;
		FVector3d CurveNearestPosition;


		for (int si = 0; si < NumSegments; ++si)
		{
			double SegRayParam; double SegSegParam;
			FDistRay3Segment3d::SquaredDistance(Ray, Polyline.GetSegment(si), SegRayParam, SegSegParam);

			FVector3d RayPosition = Ray.PointAt(SegRayParam);
			FVector3d CurvePosition = Polyline.GetSegmentPoint(si, SegSegParam);
			if (PointWithinToleranceTest(RayPosition, CurvePosition))
			{
				// if multiple segments have a point within the tolerance, try to balance preference for closer-to-ray-origin and closer-to-segment
				const double SegmentDistanceMetric = FMathd::Abs(VectorUtil::Area(Ray.Origin, CurvePosition, Ray.PointAt(SegRayParam)));
				if (SegmentDistanceMetric < NearestSegmentMetric)
				{
					CurveNearestSegmentRayParamT = SegRayParam;
					CurveNearestSegmentIdx = si;
					CurveNearestSegmentParam = SegSegParam;
					CurveNearestPosition = CurvePosition;
					NearestSegmentMetric = SegmentDistanceMetric;
				}
			}
		}

		// if we found a possible point, lock outer structures and merge in
		if (NearestSegmentMetric < TNumericLimits<double>::Max())
		{
			Critical.Lock();
			if (NearestSegmentMetric < NearestCurveMetric)
			{
				MinRayParamT = CurveNearestSegmentRayParamT;
				NearestIndex = ci;
				NearestID = Curve.ID;
				NearestSegmentIdx = CurveNearestSegmentIdx;
				NearestSegmentParam = CurveNearestSegmentParam;
				NearestCurveMetric = NearestSegmentMetric;
			}
			Critical.Unlock();
		}
	});   // end ParallelFor

	if (NearestID != -1)
	{
		ResultOut.ID = NearestID;
		ResultOut.bIsPoint = false;
		ResultOut.NearestRayPoint = Ray.PointAt(MinRayParamT);
		ResultOut.NearestGeoPoint = Curves[NearestIndex].Geometry.GetSegmentPoint(NearestSegmentIdx, NearestSegmentParam);
		ResultOut.RayParam = MinRayParamT;
		ResultOut.PolySegmentIdx = NearestSegmentIdx;
		ResultOut.PolySegmentParam = NearestSegmentParam;
	}

	return NearestID != -1;
}

bool FGeometrySet3::CollectCurvesNearRay(const FRay3d& Ray, TArray<FNearest>& ResultsOut,
	TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const
{
	// This is used later to see if we added any new results.
	int OriginalResultsLength = ResultsOut.Num();

	// To avoid locking in the ParallelFor below, we let all curves have space for their
	// own output even if their point turns out not to be within tolerance.
	TArray<FNearest> NonCompactResults;
	NonCompactResults.AddUninitialized(Curves.Num());
	TArray<bool> WithinTolerance;
	WithinTolerance.AddZeroed(Curves.Num());

	// Find distance to each curve in parallel
	ParallelFor(Curves.Num(), [&](int ci)
	{
		const FCurve& Curve = Curves[ci];
		const FPolyline3d& Polyline = Curve.Geometry;
		int NumSegments = Polyline.SegmentCount();

		// Look for the closest segment in the curve
		double CurveMinRayParamT = TNumericLimits<double>::Max();
		int CurveNearestSegmentIdx = -1;
		double CurveNearestSegmentParam = 0;
		for (int si = 0; si < NumSegments; ++si)
		{
			double SegRayParam; double SegSegParam;
			double SegDistSqr = FDistRay3Segment3d::SquaredDistance(Ray, Polyline.GetSegment(si), SegRayParam, SegSegParam);
			if (SegRayParam < CurveMinRayParamT)
			{
				FVector3d RayPosition = Ray.PointAt(SegRayParam);
				FVector3d CurvePosition = Polyline.GetSegmentPoint(si, SegSegParam);
				if (PointWithinToleranceTest(RayPosition, CurvePosition))
				{
					CurveMinRayParamT = SegRayParam;
					CurveNearestSegmentIdx = si;
					CurveNearestSegmentParam = SegSegParam;
				}
			}
		}

		// See if one of the segments was close enough
		if (CurveMinRayParamT < TNumericLimits<double>::Max())
		{
			WithinTolerance[ci] = true;

			// Save result
			NonCompactResults[ci].ID = Curve.ID;
			NonCompactResults[ci].bIsPoint = false;
			NonCompactResults[ci].NearestRayPoint = Ray.PointAt(CurveMinRayParamT);
			NonCompactResults[ci].NearestGeoPoint = Curves[ci].Geometry.GetSegmentPoint(CurveNearestSegmentIdx, CurveNearestSegmentParam);
			NonCompactResults[ci].RayParam = CurveMinRayParamT;
			NonCompactResults[ci].PolySegmentIdx = CurveNearestSegmentIdx;
			NonCompactResults[ci].PolySegmentParam = CurveNearestSegmentParam;
		}
	});   // end ParallelFor

	// Output only those results that were within tolerance
	for (int i = 0; i < NonCompactResults.Num(); ++i)
	{
		if (WithinTolerance[i])
		{
			ResultsOut.Add(NonCompactResults[i]);
		}
	}

	return ResultsOut.Num() != OriginalResultsLength;
}
