// Copyright Epic Games, Inc. All Rights Reserved.

#include "Spatial/PointSetHashTable.h"
#include "Util/IndexUtil.h"

using namespace UE::Geometry;

void FPointSetHashtable::Build(double CellSizeIn, const FVector3d& OriginIn)
{
	Origin = OriginIn;
	CellSize = CellSizeIn;
	GridIndexer = FShiftGridIndexer3d(Origin, CellSize);

	Grid = TSparseGrid3<PointList>();

	// insert all points into cell lists
	int MaxID = Points->MaxPointID();
	for (int i = 0; i < MaxID; ++i)
	{
		if (Points->IsPoint(i))
		{
			FVector3d Pt = Points->GetPoint(i);
			FVector3i Idx = GridIndexer.ToGrid(Pt);
			PointList* CellList = Grid.Get(Idx, true);
			CellList->Add(i);
		}
	}
}


bool FPointSetHashtable::FindPointsInBall(const FVector3d& QueryPt, double QueryRadius, TArray<int>& ResultOut)
{
	double HalfCellSize = CellSize * 0.5;
	FVector3i QueryCellIdx = GridIndexer.ToGrid(QueryPt);
	FVector3d QueryCellCenter = GridIndexer.FromGrid(QueryCellIdx) + HalfCellSize * FVector3d::One();

	// currently large radius is unsupported...  
	// @todo support large radius!
	// @todo we could alternately clamp this value
	check(QueryRadius <= CellSize);
	double RadiusSqr = QueryRadius * QueryRadius;

	// check all in this cell
	PointList* CenterCellList = Grid.Get(QueryCellIdx, false);
	if (CenterCellList != nullptr)
	{
		for (int vid : *CenterCellList)
		{
			if (Points->IsPoint(vid))		// handle case where points are removed from pointset
			{
				if (DistanceSquared(QueryPt, Points->GetPoint(vid)) < RadiusSqr)
				{
					ResultOut.Add(vid);
				}
			}
		}
	}

	// if we are close enough to cell border we need to check nbrs
	// [TODO] could iterate over fewer cells here, if r is bounded by CellSize,
	// then we should only ever need to look at 3, depending on which octant we are in.
	if (MaxAbsElement(QueryPt - QueryCellCenter) + QueryRadius > HalfCellSize)
	{
		for (int ci = 0; ci < 26; ++ci)
		{
			FVector3i NbrOffset = IndexUtil::GridOffsets26[ci];

			// if we are within r from face, we need to look into it
			FVector3d PtToFaceCenter(
				QueryCellCenter.X + HalfCellSize*NbrOffset.X - QueryPt.X,
				QueryCellCenter.Y + HalfCellSize*NbrOffset.Y - QueryPt.Y,
				QueryCellCenter.Z + HalfCellSize*NbrOffset.Z - QueryPt.Z);
			if (MinAbsElement(PtToFaceCenter) > QueryRadius)
			{
				continue;
			}

			PointList* NbrCellList = Grid.Get(QueryCellIdx + NbrOffset, false);
			if (NbrCellList != nullptr)
			{
				for (int vid : *NbrCellList)
				{
					if (Points->IsPoint(vid))		// handle case where points are removed from pointset
					{
						if (DistanceSquared(QueryPt, Points->GetPoint(vid)) < RadiusSqr)
						{
							ResultOut.Add(vid);
						}
					}
				}
			}
		}
	}

	return true;
};
