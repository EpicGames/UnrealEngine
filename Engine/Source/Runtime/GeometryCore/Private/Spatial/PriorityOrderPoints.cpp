// Copyright Epic Games, Inc. All Rights Reserved.


#include "Spatial/PriorityOrderPoints.h"
#include "Algo/Sort.h"
#include "BoxTypes.h"

using namespace UE::Geometry;


namespace PriorityOrderPointsLocal
{
/**
 * Binary predicate for sorting indices according to a secondary array of values to sort by.
 * Puts values into descending order.
 */
template<class T>
class DescendingPredicate
{
public:
	DescendingPredicate(const TArrayView<const T>& CompValues)
		: CompValues(CompValues)
	{}

	bool operator()(const int i, const int j) const
	{
		// Indices must be in range
		return CompValues[i] > CompValues[j];
	}

private:
	const TArrayView<const T> CompValues;
};

// Spatial ordering algorithm
// Note we support two sets of importance weights because it is sometimes helpful to also sort by an additional metric, such as distance from center
template<typename RealType>
void OrderPoints(TArray<int32>& PointOrder, TArrayView<const TVector<RealType>> Points, TArrayView<const TArrayView<const float>> ImportanceWeights, int32 EarlyStop, int32 SpatialLevels, int32 OffsetResFactor)
{
	OffsetResFactor = FMath::Clamp(OffsetResFactor, 1, 8);

	const int32 NumPoints = Points.Num();
	PointOrder.SetNumUninitialized(NumPoints);
	if (NumPoints == 0)
	{
		return;
	}

	TAxisAlignedBox3<RealType> Bounds;
	for (int32 Idx = 0; Idx < NumPoints; Idx++)
	{
		PointOrder[Idx] = Idx;
		Bounds.Contain(Points[Idx]);
	}

	TVector<RealType> Center = Bounds.Center();
	TArray<TVector<RealType>> LocalPoints;
	LocalPoints.AddUninitialized(NumPoints);
	for (int i = 0; i < NumPoints; i++)
	{
		LocalPoints[i] = Points[i] - Center;
	}
	TAxisAlignedBox3<RealType> LocalBBox(-Bounds.Extents(), Bounds.Extents());
	LocalBBox.Expand(1.0e-3f);
	RealType MaxBBoxDim = LocalBBox.MaxDim();

	auto ToFlatIdx = [&LocalPoints](int32 PointIdx, const TVector<RealType>& Center, RealType CellSizeIn, int64 Res) -> int64
	{
		const TVector<RealType>& Pos = LocalPoints[PointIdx];
		// grid center co-located at bbox center:
		int64 X =
			static_cast<int64>(FMath::Floor((Pos[0] - Center[0]) / CellSizeIn)) + Res / 2;
		int64 Y =
			static_cast<int64>(FMath::Floor((Pos[1] - Center[1]) / CellSizeIn)) + Res / 2;
		int64 Z =
			static_cast<int64>(FMath::Floor((Pos[2] - Center[2]) / CellSizeIn)) + Res / 2;
		return ((X * Res + Y) * Res) + Z;
	};

	constexpr int MaxImportanceWts = 2;
	struct FBestPtData
	{
		int32 Idx[MaxImportanceWts]{ -1, -1 };
		float Wt[MaxImportanceWts]{ -FMathf::MaxReal, -FMathf::MaxReal };
	};
	int32 NumWeights = ImportanceWeights.Num();
	for (int Idx = 0; Idx < NumWeights; Idx++)
	{
		if (ImportanceWeights[Idx].IsEmpty())
		{
			NumWeights = Idx;
			break;
		}
	}

	// Update FBestPointData with a new point; return true if it's the first point in the cell, false otherwise
	auto UpdateBestPtData = [NumWeights, &ImportanceWeights, &PointOrder](FBestPtData& BestData, int32 OrderIdx) -> bool
	{
		int32 PtIdx = PointOrder[OrderIdx];
		if (NumWeights <= 0)
		{
			if (BestData.Idx[0] == -1)
			{
				BestData.Idx[0] = OrderIdx;
				return true;
			}
			return false;
		}

		if (BestData.Idx[0] == -1)
		{
			BestData.Idx[0] = BestData.Idx[1] = OrderIdx;
			BestData.Wt[0] = ImportanceWeights[0][PtIdx];
			if (NumWeights > 1)
			{
				BestData.Wt[1] = ImportanceWeights[1][PtIdx];
			}
			return true;
		}

		if (ImportanceWeights[0][PtIdx] > BestData.Wt[0])
		{
			BestData.Wt[0] = ImportanceWeights[0][PtIdx];
			BestData.Idx[0] = OrderIdx;
		}

		// TODO: if we want to support more than two importance weights, turn this into a loop
		if (NumWeights > 1 && ImportanceWeights[1][PtIdx] > BestData.Wt[1])
		{
			BestData.Wt[1] = ImportanceWeights[1][PtIdx];
			BestData.Idx[1] = OrderIdx;
		}

		return false;
	};

	// points before this offset have already been moved forward, and don't need further consideration
	int32 MovedPtsOffset = 0;
	TArray<uint8> Rank;
	TMap<int64, FBestPtData> BestPoints;
	TSet<int64> CoveredCells, OffsetCoveredCells;
	TArray<int32> MoveToFront;
	int32 MaxRes = 1 << FMath::Clamp(SpatialLevels, 1, 30);
	// traverse power of 2 cell resolutions (~octree levels), moving forward "best" points from the cells at each level
	for (int32 Resolution = 2; MovedPtsOffset < NumPoints && Resolution <= MaxRes; Resolution *= 2)
	{
		BestPoints.Reset();
		CoveredCells.Reset();
		OffsetCoveredCells.Reset();
		const RealType CellSize = MaxBBoxDim / static_cast<RealType>(Resolution);

		// make quarter-sized cells w/ a half-cell offset to help detect points that are close, but across cell boundaries
		int32 OffsetResolution = Resolution * OffsetResFactor;
		const RealType OffsetCellSize = MaxBBoxDim / static_cast<RealType>(OffsetResolution);
		OffsetResolution += 2; // allow for offset center
		TVector<RealType> OffsetCenter = -TVector<RealType>(OffsetCellSize / 2);
		int32 FoundPointsCount = 0;

		// Use the Rank array to mark which points should move to the front of the remaining list in this iteration
		Rank.Reset();
		Rank.AddZeroed(NumPoints);

		auto ToMainFlatIdx = [&ToFlatIdx, &CellSize, &Resolution](int32 PointIdx) -> int64
		{
			return ToFlatIdx(PointIdx, TVector<RealType>::ZeroVector, CellSize, Resolution);
		};
		auto ToOffsetFlatIdx = [&ToFlatIdx, &OffsetCenter, &OffsetCellSize, &OffsetResolution](int32 PointIdx) -> int64
		{
			return ToFlatIdx(PointIdx, OffsetCenter, OffsetCellSize, OffsetResolution);
		};

		// Mark cells that are already covered by points in the already-considered front of the list
		for (int32 OrderIdx = 0; OrderIdx < MovedPtsOffset; OrderIdx++)
		{
			int32 PointIdx = PointOrder[OrderIdx];
			const int64 FlatIdx = ToMainFlatIdx(PointIdx);
			CoveredCells.Add(FlatIdx);
			OffsetCoveredCells.Add(ToOffsetFlatIdx(PointIdx));
		}

		// find the best points in cells that aren't already covered
		bool bAllSeparate = true;
		for (int32 OrderIdx = MovedPtsOffset; OrderIdx < NumPoints; OrderIdx++)
		{
			int32 PointIdx = PointOrder[OrderIdx];
			const int64 FlatIdx = ToMainFlatIdx(PointIdx);
			if (CoveredCells.Contains(FlatIdx) || OffsetCoveredCells.Contains(ToOffsetFlatIdx(PointIdx)))
			{
				bAllSeparate = false;
				continue;
			}
			FBestPtData& Best = BestPoints.FindOrAdd(FlatIdx);
			bAllSeparate = UpdateBestPtData(Best, OrderIdx) & bAllSeparate;
		}

		if (bAllSeparate)
		{
			// all the points were in separate cells, no need to continue promoting the 'best' for each cell beyond here
			break;
		}

		auto ConsiderMovingPt = [&Rank, MovedPtsOffset, &OffsetCoveredCells, &ToOffsetFlatIdx](int32 PointIdx) -> bool
		{
			uint8& R = Rank[PointIdx];
			if (!R)
			{
				// try to avoid clumping by skipping points whose offset cells were already covered
				int64 OffsetFlatIdx = ToOffsetFlatIdx(PointIdx);
				bool bAlreadyInSet = false;
				OffsetCoveredCells.Add(OffsetFlatIdx, &bAlreadyInSet);
				if (bAlreadyInSet)
				{
					return false;
				}

				R = 1;
				return true;
			}
			else
			{
				return false;
			}
		};

		// Decide which points to move into the 'front' section
		int32 NumToMoveToFront = 0;
		MoveToFront.Reset();
		// Add points favored by the first metric
		for (const TPair<int64, FBestPtData>& BestDataPair : BestPoints)
		{
			int PtIdx = PointOrder[BestDataPair.Value.Idx[0]];
			if (ConsiderMovingPt(PtIdx))
			{
				NumToMoveToFront++;
				MoveToFront.Add(BestDataPair.Value.Idx[0]);
			}
		}
		if (NumWeights > 1) // TODO: make this a loop if we support more than two importance weights
		{
			// Add points favored by the second metric *after* adding all the first-metric-favored points
			//  (so they may be skipped where a first-metric-favored point already covered a region)
			for (const TPair<int64, FBestPtData>& BestDataPair : BestPoints)
			{
				int PtIdx = PointOrder[BestDataPair.Value.Idx[1]];
				if (ConsiderMovingPt(PtIdx))
				{
					NumToMoveToFront++;
					MoveToFront.Add(BestDataPair.Value.Idx[1]);
				}
			}
		}

		// Do the move-to-front operations by swapping
		int MoveFrontIdx = 0;
		for (int MoveBackIdx = 0; MoveBackIdx < NumToMoveToFront; MoveBackIdx++)
		{
			if (Rank[PointOrder[MovedPtsOffset + MoveBackIdx]])
			{
				// Point was marked for moving to front, and is already in front; skip it
				continue;
			}
			// Swap point with one that was marked for the front section
			bool bDidSwap = false;
			for (; MoveFrontIdx < MoveToFront.Num(); MoveFrontIdx++)
			{
				if (MoveToFront[MoveFrontIdx] < MovedPtsOffset + NumToMoveToFront)
				{
					// Don't swap this one back because it's already in the 'front' zone
					continue;
				}
				int32 PtIdx = PointOrder[MoveToFront[MoveFrontIdx]];
				checkSlow(Rank[PtIdx]);
				Swap(PointOrder[MovedPtsOffset + MoveBackIdx], PointOrder[MoveToFront[MoveFrontIdx]]);
				bDidSwap = true;
				MoveFrontIdx++;
				break;
			}
			// The above for loop should always find a point to swap back
			checkSlow(bDidSwap);
		}

		// Note the above swapping method could have been done via a Sort() using the Rank array, but swapping is more direct / faster

		// Sort the just-added points by first metric (they were in arbitrary order before)
		if (NumWeights > 0)
		{
			DescendingPredicate<float> DescendingImportancePred(ImportanceWeights[0]);
			Algo::Sort(MakeArrayView(&PointOrder[MovedPtsOffset], NumToMoveToFront), DescendingImportancePred);
		}

		MovedPtsOffset += NumToMoveToFront;

		if (0 <= EarlyStop && EarlyStop <= MovedPtsOffset)
		{
			break;
		}
	}

	// Sort remaining non-coincident points by first metric
	if (NumWeights > 0 && (EarlyStop < 0 || EarlyStop > MovedPtsOffset) && MovedPtsOffset < NumPoints)
	{
		DescendingPredicate<float> DescendingImportancePred(ImportanceWeights[0]);
		Algo::Sort(MakeArrayView(&PointOrder[MovedPtsOffset], NumPoints - MovedPtsOffset), DescendingImportancePred);
	}
}
}


void FPriorityOrderPoints::ComputeUniformSpaced(TArrayView<const FVector3d> Points, TArrayView<const float> ImportanceWeights, int32 EarlyStop, int32 OffsetResFactor)
{
	TArrayView<TArrayView<const float>> ImportanceWeightsWeights(&ImportanceWeights, 1);
	PriorityOrderPointsLocal::OrderPoints<double>(Order, Points, ImportanceWeightsWeights, EarlyStop, SpatialLevels, OffsetResFactor);
}

void FPriorityOrderPoints::ComputeUniformSpaced(TArrayView<const FVector3f> Points, TArrayView<const float> ImportanceWeights, int32 EarlyStop, int32 OffsetResFactor)
{
	TArrayView<TArrayView<const float>> ImportanceWeightsWeights(&ImportanceWeights, 1);
	PriorityOrderPointsLocal::OrderPoints<float>(Order, Points, ImportanceWeightsWeights, EarlyStop, SpatialLevels, OffsetResFactor);
}

void FPriorityOrderPoints::ComputeUniformSpaced(TArrayView<const FVector3d> Points, TArrayView<const float> ImportanceWeights, TArrayView<const float> SecondImportanceWeights, int32 EarlyStop, int32 OffsetResFactor)
{
	TArray<TArrayView<const float>, TFixedAllocator<2>> MultiImportanceWeights;
	MultiImportanceWeights.Add(ImportanceWeights);
	MultiImportanceWeights.Add(SecondImportanceWeights);
	PriorityOrderPointsLocal::OrderPoints<double>(Order, Points, MultiImportanceWeights, EarlyStop, SpatialLevels, OffsetResFactor);
}

void FPriorityOrderPoints::ComputeUniformSpaced(TArrayView<const FVector3f> Points, TArrayView<const float> ImportanceWeights, TArrayView<const float> SecondImportanceWeights, int32 EarlyStop, int32 OffsetResFactor)
{
	TArray<TArrayView<const float>, TFixedAllocator<2>> MultiImportanceWeights;
	MultiImportanceWeights.Add(ImportanceWeights);
	MultiImportanceWeights.Add(SecondImportanceWeights);
	PriorityOrderPointsLocal::OrderPoints<float>(Order, Points, MultiImportanceWeights, EarlyStop, SpatialLevels, OffsetResFactor);
}

void FPriorityOrderPoints::ComputeDescendingImportance(TArrayView<const float> ImportanceWeights)
{
	int32 NumPoints = ImportanceWeights.Num();
	if (NumPoints == 0)
	{
		return;
	}
	Order.SetNum(NumPoints);
	for (int32 Idx = 0; Idx < NumPoints; Idx++)
	{
		Order[Idx] = Idx;
	}
	PriorityOrderPointsLocal::DescendingPredicate<float> DescendingImportancePred(ImportanceWeights);
	Algo::Sort(Order, DescendingImportancePred);
}