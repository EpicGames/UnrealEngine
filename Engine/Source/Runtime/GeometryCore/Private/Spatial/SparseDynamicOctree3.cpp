// Copyright Epic Games, Inc. All Rights Reserved.


#include "Spatial/SparseDynamicOctree3.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

// NB: These have to be here until C++17 allows inline variables
constexpr int32 FDynamicFlagArray::GrowChunkSize;
constexpr uint32 FSparseOctreeCell::InvalidID;
constexpr uint8 FSparseOctreeCell::InvalidLevel;
constexpr uint32 FSparseDynamicOctree3::MaxSupportedTreeDepth;
constexpr uint32 FSparseDynamicOctree3::InvalidCellID;
constexpr uint32 FSparseDynamicOctree3::SpillCellID;


bool FSparseDynamicOctree3::ContainsObject(int32 ObjectID) const
{
	return ValidObjectIDs.Get(ObjectID);
}

void FSparseDynamicOctree3::InsertObject(int32 ObjectID, const FAxisAlignedBox3d& Bounds)
{
	checkSlow(ContainsObject(ObjectID) == false);

	FSparseOctreeCell CurrentCell = FindCurrentContainingCell(Bounds);

	// if we could not find a containing root cell, we spill
	if (CurrentCell.Level == FSparseOctreeCell::InvalidLevel)
	{
		Insert_Spill(ObjectID, Bounds);
		return;
	}
	checkSlow(CanFit(CurrentCell, Bounds));

	// if we found a containing root cell but it doesn't exist, create it and insert
	if (CurrentCell.Level == 0 && CurrentCell.IsExistingCell() == false)
	{
		Insert_NewRoot(ObjectID, Bounds, CurrentCell);
		return;
	}

	// YIKES this currently does max-depth insertion...
	//   desired behavior is that parent cell accumulates and then splits later!

	int PotentialChildIdx = ToChildCellIndex(CurrentCell, Bounds.Center());
	// if current cell does not have this child we might fit there so try it
	if (CurrentCell.HasChild(PotentialChildIdx) == false)
	{
		// todo can we do a fast check based on level and dimensions??
		FSparseOctreeCell NewChildCell = CurrentCell.MakeChildCell(PotentialChildIdx);
		if (NewChildCell.Level <= MaxTreeDepth && CanFit(NewChildCell, Bounds))
		{
			Insert_NewChildCell(ObjectID, Bounds, CurrentCell.CellID, NewChildCell, PotentialChildIdx);
			return;
		}
	}

	// insert into current cell if
	//   1) child cell exists (in which case we didn't fit or FindCurrentContainingCell would have returned)
	//   2) we tried to fit in child cell and failed
	Insert_ToCell(ObjectID, Bounds, CurrentCell);
}

void FSparseDynamicOctree3::Insert_NewChildCell(int32 ObjectID, const FAxisAlignedBox3d& Bounds,
	int ParentCellID, FSparseOctreeCell NewChildCell, int ChildIdx)
{
	FSparseOctreeCell& OrigParentCell = Cells[ParentCellID];
	checkSlow(OrigParentCell.HasChild(ChildIdx) == false);

	NewChildCell.CellID = CellRefCounts.Allocate();
	Cells.InsertAt(NewChildCell, NewChildCell.CellID);

	ObjectIDToCellMap.InsertAt(NewChildCell.CellID, ObjectID);
	ValidObjectIDs.Set(ObjectID, true);

	CellObjectLists.AllocateAt(NewChildCell.CellID);
	CellObjectLists.Insert(NewChildCell.CellID, ObjectID);

	OrigParentCell.SetChild(ChildIdx, NewChildCell);

	checkSlow(CanFit(NewChildCell, Bounds));
	// this check is unstable if the center point is within machine-epsilon of the cell border
	//check(PointToIndex(NewChildCell.Level, Bounds.Center()) == NewChildCell.Index);
}


void FSparseDynamicOctree3::Insert_ToCell(int32 ObjectID, const FAxisAlignedBox3d& Bounds, const FSparseOctreeCell& ExistingCell)
{
	checkSlow(CellRefCounts.IsValid(ExistingCell.CellID));

	ObjectIDToCellMap.InsertAt(ExistingCell.CellID, ObjectID);
	ValidObjectIDs.Set(ObjectID, true);

	CellObjectLists.Insert(ExistingCell.CellID, ObjectID);

	checkSlow(CanFit(ExistingCell, Bounds));
	// this check is unstable if the center point is within machine-epsilon of the cell border
	//check(PointToIndex(ExistingCell.Level, Bounds.Center()) == ExistingCell.Index);
}


void FSparseDynamicOctree3::Insert_NewRoot(int32 ObjectID, const FAxisAlignedBox3d& Bounds, FSparseOctreeCell NewRootCell)
{
	checkSlow(RootCells.Has(NewRootCell.Index) == false);

	NewRootCell.CellID = CellRefCounts.Allocate();
	Cells.InsertAt(NewRootCell, NewRootCell.CellID);

	ObjectIDToCellMap.InsertAt(NewRootCell.CellID, ObjectID);
	ValidObjectIDs.Set(ObjectID, true);

	uint32* RootCellElem = RootCells.Get(NewRootCell.Index, true);
	*RootCellElem = NewRootCell.CellID;

	CellObjectLists.AllocateAt(NewRootCell.CellID);
	CellObjectLists.Insert(NewRootCell.CellID, ObjectID);

	checkSlow(CanFit(NewRootCell, Bounds));
}

void FSparseDynamicOctree3::Insert_Spill(int32 ObjectID, const FAxisAlignedBox3d& Bounds)
{
	SpillObjectSet.Add(ObjectID);
	ObjectIDToCellMap.InsertAt(SpillCellID, ObjectID);
	ValidObjectIDs.Set(ObjectID, true);
}




bool FSparseDynamicOctree3::RemoveObject(int32 ObjectID)
{
	if (ContainsObject(ObjectID) == false)
	{
		return false;
	}

	uint32 CellID = GetCellForObject(ObjectID);
	if (CellID == SpillCellID)
	{
		int32 RemovedCount = SpillObjectSet.Remove(ObjectID);
		checkSlow(RemovedCount > 0);
		ValidObjectIDs.Set(ObjectID, false);
		return (RemovedCount > 0);
	}
	if (CellID == InvalidCellID)
	{
		return false;
	}

	ObjectIDToCellMap[ObjectID] = InvalidCellID;
	ValidObjectIDs.Set(ObjectID, false);

	bool bInList = CellObjectLists.Remove(CellID, ObjectID);
	checkSlow(bInList);
	return true;
}




bool FSparseDynamicOctree3::CheckIfObjectNeedsReinsert(int32 ObjectID, const FAxisAlignedBox3d& NewBounds, uint32& CellIDOut) const
{
	CellIDOut = InvalidCellID;
	if (ContainsObject(ObjectID))
	{
		CellIDOut = GetCellForObject(ObjectID);
		if (CellIDOut != SpillCellID && CellIDOut != InvalidCellID)
		{
			const FSparseOctreeCell& CurrentCell = Cells[CellIDOut];
			if (CanFit(CurrentCell, NewBounds))
			{
				return false;		// everything is fine
			}

		}
	}
	return true;
}


bool FSparseDynamicOctree3::ReinsertObject(int32 ObjectID, const FAxisAlignedBox3d& NewBounds, uint32 CellIDHint)
{
	uint32 CellID = CellIDHint;

	// check if this object still fits in current cell. If so, we can ignore it
	if ( CellID == InvalidCellID && ContainsObject(ObjectID) )
	{
		CellID = GetCellForObject(ObjectID);
		if (CellID != SpillCellID && CellID != InvalidCellID)
		{
			FSparseOctreeCell& CurrentCell = Cells[CellID];
			if (CanFit(CurrentCell, NewBounds))
			{
				return false;		// everything is fine
			}
		}
	}

	// remove object
	if (CellID != InvalidCellID)
	{
		if (CellID == SpillCellID)
		{
			SpillObjectSet.Remove(ObjectID);
		}
		else
		{
			ObjectIDToCellMap[ObjectID] = InvalidCellID;
			CellObjectLists.Remove(CellID, ObjectID);
		}
		ValidObjectIDs.Set(ObjectID, false);
	}

	// reinsert
	InsertObject(ObjectID, NewBounds);

	return true;
}





double FSparseDynamicOctree3::FindNearestRayCellIntersection(const FSparseOctreeCell& Cell, const FRay3d& Ray) const
{
	FAxisAlignedBox3d Box = GetCellBox(Cell, MaxExpandFactor);
	double ray_t = TNumericLimits<double>::Max();
	if (FIntrRay3AxisAlignedBox3d::FindIntersection(Ray, Box, ray_t))
	{
		return ray_t;
	}
	else
	{
		return TNumericLimits<double>::Max();
	}
}



int32 FSparseDynamicOctree3::FindNearestHitObject(const FRay3d& Ray,
	TFunctionRef<FAxisAlignedBox3d(int)> GetObjectBoundsFunc,
	TFunctionRef<double(int, const FRay3d&)> HitObjectDistFunc,
	double MaxDistance) const
{
	// this should take advantage of raster!

	// always test against all spill objects
	int32 HitObjectID = -1;
	for (int ObjectID : SpillObjectSet)
	{
		double HitDist = HitObjectDistFunc(ObjectID, Ray);
		if (HitDist < MaxDistance)
		{
			MaxDistance = HitDist;
			HitObjectID = ObjectID;
		}
	}

	// we use queue instead of recursion
	TArray<const FSparseOctreeCell*> Queue;
	Queue.Reserve(64);

	// push all root cells onto queue if they are hit by ray
	RootCells.AllocatedIteration([&](const uint32* RootCellID)
	{
		const FSparseOctreeCell* RootCell = &Cells[*RootCellID];
		double RayHitParam = FindNearestRayCellIntersection(*RootCell, Ray);
		if (RayHitParam < MaxDistance)
		{
			Queue.Add(&Cells[*RootCellID]);
		}
	});

	// test cells until the queue is empty
	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);
		
		// process elements
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			double HitDist = HitObjectDistFunc(ObjectID, Ray);
			if (HitDist < MaxDistance)
			{
				MaxDistance = HitDist;
				HitObjectID = ObjectID;
			}
		});

		// descend to child cells
		// sort by distance? use DDA?
		for (int k = 0; k < 8; ++k) 
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				double RayHitParam = FindNearestRayCellIntersection(*ChildCell, Ray);
				if (RayHitParam < MaxDistance)
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}

	return HitObjectID;
}



// helper to find the root-level cells that could intersect with a given query object, specialized for range queries
TArray<const FSparseOctreeCell*, TInlineAllocator<32>> FSparseDynamicOctree3::InitializeQueryQueue(
	const FAxisAlignedBox3d& RangeBounds,
	TFunctionRef<bool(const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn,
	bool bCheckOverlapFnInRangeBounds) const
{
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue;

	// Skip range iteration if there are not many root cells -- should be faster to just directly iterate all the root cells in that case
	constexpr int32 MinCountForRangeQuery = 10;
	if (RootCells.GetCount() > MinCountForRangeQuery)
	{
		FVector3d RootBoundExpand(RootDimension * MaxExpandFactor);
		FVector3i RootMinIndex = PointToIndex(0, RangeBounds.Min - RootBoundExpand);
		FVector3i RootMaxIndex = PointToIndex(0, RangeBounds.Max + RootBoundExpand);
		
		// Clamp our iteration to only the range of allocated cells. Do it only in the appropriate
		//  direction for the max/min (e.g. don't clamp min to be <= allocated max, to make it easier
		//  to detect when we were entirely out of allocated bounds).
		FAxisAlignedBox3i IndexBounds = RootCells.GetBoundsInclusive();
		RootMinIndex = Max(RootMinIndex, IndexBounds.Min);
		RootMaxIndex = Min(RootMaxIndex, IndexBounds.Max);
		
		// See if the range bounds were outside of the allocated bounds entirely. This check isn't
		//  strictly necessary since the RangeIteration call below shouldn't do anything if Max < Min,
		//  but we'll be explicit about this.
		for (int i = 0; i < 3; ++i)
		{
			if (RootMinIndex[i] > RootMaxIndex[i])
			{
				return Queue;
			}
		}
		
		// Double-check that there are enough root cells that we are likely to save time by local iteration,
		//  since we could have a huge iteration range but only a couple cells allocated in that range.
		// Use double for the query size here to avoid overflow.
		FVector3d QuerySize = FVector3d(RootMaxIndex) - FVector3d(RootMinIndex) + FVector3d::One();
		if (RootCells.GetCount() > QuerySize.X * QuerySize.Y * QuerySize.Z) // There are enough root cells that we could save time by local iteration
		{
			if (bCheckOverlapFnInRangeBounds)
			{
				RootCells.RangeIteration(RootMinIndex, RootMaxIndex, [&Queue, &BoundsOverlapFn, this](uint32 RootCellID)
				{
					const FSparseOctreeCell* RootCell = &Cells[RootCellID];
					if (BoundsOverlapFn(GetCellBox(*RootCell, MaxExpandFactor)))
					{
						Queue.Add(&Cells[RootCellID]);
					}
				});
			}
			else
			{
				RootCells.RangeIteration(RootMinIndex, RootMaxIndex, [&Queue, this](uint32 RootCellID)
				{
					// No need to check bound intersection since we're only iterating over cells that are in bounds
					Queue.Add(&Cells[RootCellID]);
				});
			}
			return Queue;
		}
	}

	RootCells.AllocatedIteration([&Queue, &BoundsOverlapFn, this](const uint32* RootCellID)
	{
		const FSparseOctreeCell* RootCell = &Cells[*RootCellID];
		if (BoundsOverlapFn(GetCellBox(*RootCell, MaxExpandFactor)))
		{
			Queue.Add(&Cells[*RootCellID]);
		}
	});
	return Queue;
}

void FSparseDynamicOctree3::ContainmentQuery(
	const FVector3d& Point,
	TFunctionRef<void(int)> ObjectIDFunc) const
{
	// always process spill objects
	for (int ObjectID : SpillObjectSet)
	{
		ObjectIDFunc(ObjectID);
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue = InitializeQueryQueue(Point);

	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			ObjectIDFunc(ObjectID);
		});

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (GetCellBox(*ChildCell, MaxExpandFactor).Contains(Point))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}
}




bool FSparseDynamicOctree3::ContainmentQueryCancellable(
	const FVector3d& Point,
	TFunctionRef<bool(int)> ObjectIDFunc) const
{
	// always process spill objects
	for (int ObjectID : SpillObjectSet)
	{
		if (ObjectIDFunc(ObjectID) == false)
		{
			return false;
		}
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue = InitializeQueryQueue(Point);


	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		bool bContinue = true;
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			if ( bContinue && ObjectIDFunc(ObjectID) == false )
			{
				bContinue = false;
			}
		});
		if (!bContinue)
		{
			return false;
		}

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (GetCellBox(*ChildCell, MaxExpandFactor).Contains(Point))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}

	return true;
}




void FSparseDynamicOctree3::RangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TFunctionRef<void(int)> ObjectIDFunc) const
{
	// always process spill objects
	for (int ObjectID : SpillObjectSet)
	{
		ObjectIDFunc(ObjectID);
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue = InitializeQueryQueue(Bounds);

	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			ObjectIDFunc(ObjectID);
		});

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (GetCellBox(*ChildCell, MaxExpandFactor).Intersects(Bounds))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}
}







void FSparseDynamicOctree3::RangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TArray<int>& ObjectIDs) const
{
	// always collect spill objects
	for (int ObjectID : SpillObjectSet)
	{
		ObjectIDs.Add(ObjectID);
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue = InitializeQueryQueue(Bounds);

	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			ObjectIDs.Add(ObjectID);
		});

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (GetCellBox(*ChildCell, MaxExpandFactor).Intersects(Bounds))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}
}




void FSparseDynamicOctree3::ParallelRangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TArray<int>& ObjectIDs) const
{
	// always collect spill objects
	for (int ObjectID : SpillObjectSet)
	{
		ObjectIDs.Add(ObjectID);
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> RelevantRootCells = InitializeQueryQueue(Bounds);

	ParallelCollectObjectsInOverlappingCells(RelevantRootCells, 
		[&Bounds](const FAxisAlignedBox3d& CellBounds) { return CellBounds.Intersects(Bounds); }, 
		ObjectIDs);
}

void FSparseDynamicOctree3::ParallelRangeQuery(
	const FAxisAlignedBox3d& RangeBoundsHint,
	TFunctionRef<bool(const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn, 
	TArray<int32>& ObjectIDs) const
{
	// Same as the box version above, except that (aside from using different overlap
	//  criteria) our root cell initialization still needs to call BoundsOverlapFn
	//  inside RangeBoundsHint.

	// always collect spill objects
	for (int ObjectID : SpillObjectSet)
	{
		ObjectIDs.Add(ObjectID);
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> RelevantRootCells = InitializeQueryQueue(
		RangeBoundsHint, BoundsOverlapFn, true);

	ParallelCollectObjectsInOverlappingCells(RelevantRootCells, BoundsOverlapFn, ObjectIDs);

}

void FSparseDynamicOctree3::ParallelCollectObjectsInOverlappingCells(
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> RelevantRootCells,
	TFunctionRef<bool(const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn,
	TArray<int32>& ObjectIDs) const
{

	// parallel strategy here is to collect each rootcell subtree into a separate
	// array, and then merge them. Although this means we do some dynamic memory
	// allocation, it's much faster than trying to lock ObjectIDs on every Add(),
	// which results in crazy lock contention
	FCriticalSection ObjectIDsLock;
	ParallelFor(RelevantRootCells.Num(), [&](int32 qi)
	{
		TArray<int32> LocalObjectIDs;
		const FSparseOctreeCell* RootCell = RelevantRootCells[qi];
		BranchRangeQuery(RootCell, BoundsOverlapFn, LocalObjectIDs);

		ObjectIDsLock.Lock();
		for (int32 ObjectID : LocalObjectIDs)
		{
			ObjectIDs.Add(ObjectID);
		}
		ObjectIDsLock.Unlock();
	});
}


int FSparseDynamicOctree3::ParallelOverlapAnyQuery(const FAxisAlignedBox3d& ShapeBounds,
	TFunctionRef<bool(int32)> ObjectOverlapFn, TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn) const
{
	FAxisAlignedBox3d EmptyCell = FAxisAlignedBox3d::Empty();
	// first test spill objects
	for (int ObjectID : SpillObjectSet)
	{
		if (ObjectOverlapFn(ObjectID))
		{
			return ObjectID;
		}
	}

	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue = InitializeQueryQueue(ShapeBounds);

	std::atomic<int> FoundID = INDEX_NONE;
	ParallelFor(Queue.Num(), [&](int32 qi)
	{
		if (FoundID != INDEX_NONE)
		{
			return;
		}

		const FSparseOctreeCell* RootCell = Queue[qi];
		int LocalFoundID = BranchCustomOverlapAnyQuery(RootCell, ShapeBounds, ObjectOverlapFn, BoundsOverlapFn);
		if (LocalFoundID != INDEX_NONE)
		{
			FoundID = LocalFoundID;
		}
	});
	return FoundID;
}


void FSparseDynamicOctree3::BranchRangeQuery(
	const FSparseOctreeCell* ParentCell,
	const FAxisAlignedBox3d& BoundsIn,
	TArray<int>& ObjectIDs) const
{
	BranchRangeQuery(ParentCell, 
		[&BoundsIn](const FAxisAlignedBox3d& CellBounds) { return CellBounds.Intersects(BoundsIn); }, 
		ObjectIDs);
}

void FSparseDynamicOctree3::BranchRangeQuery(
	const FSparseOctreeCell* ParentCell, 
	TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn, 
	TArray<int>& ObjectIDs) const
{
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue;
	Queue.Add(ParentCell);

	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		CellObjectLists.Enumerate(CurCell->CellID, [&](int32 ObjectID)
		{
			ObjectIDs.Add(ObjectID);
		});

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (BoundsOverlapFn(GetCellBox(*ChildCell, MaxExpandFactor)))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}
}

int FSparseDynamicOctree3::BranchCustomOverlapAnyQuery(
	const FSparseOctreeCell* ParentCell,
	const FAxisAlignedBox3d& Bounds,
	TFunctionRef<bool(int32)> ObjectOverlapFn, TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn) const
{
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> Queue;
	Queue.Add(ParentCell);

	while (Queue.Num() > 0)
	{
		const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		int32 FoundID = INDEX_NONE;
		bool bNoOverlaps = CellObjectLists.EnumerateEarlyOut(CurCell->CellID, [&](int32 ObjectID)
		{
			if (ObjectOverlapFn(ObjectID))
			{
				FoundID = ObjectID;
				return false;
			}
			return true;
		});

		if (!bNoOverlaps)
		{
			return FoundID;
		}

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				FAxisAlignedBox3d CellBox = GetCellBox(*ChildCell, MaxExpandFactor);
				if (CellBox.Intersects(Bounds) && BoundsOverlapFn(CellBox))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}

	return INDEX_NONE;
}



FSparseOctreeCell FSparseDynamicOctree3::FindCurrentContainingCell(const FAxisAlignedBox3d& Bounds) const
{
	double BoxWidth = Bounds.MaxDim();
	FVector3d BoxCenter = Bounds.Center();

	// look up root cell, which may not exist
	FVector3i RootIndex = PointToIndex(0, BoxCenter);
	const uint32* RootCellID = RootCells.Get(RootIndex);
	if (RootCellID == nullptr)
	{
		FSparseOctreeCell RootCell(0, RootIndex);
		// have to make sure we can fit in this root cell
		if (CanFit(RootCell, Bounds))
		{
			return RootCell;
		}
		else
		{
			return FSparseOctreeCell(FSparseOctreeCell::InvalidLevel, FVector3i::Zero());
		}

	}
	checkSlow(CellRefCounts.IsValid(*RootCellID));

	// check if box is contained in root cell, if not we have to spill
	// (should we do this before checking for existence? we can...)
	const FSparseOctreeCell* RootCell = &Cells[*RootCellID];
	if (CanFit(*RootCell, Bounds) == false)
	{
		return FSparseOctreeCell(FSparseOctreeCell::InvalidLevel, FVector3i::Zero());
	}

	const FSparseOctreeCell* CurrentCell = RootCell;
	do
	{
		int ChildIdx = ToChildCellIndex(*CurrentCell, BoxCenter);
		if (CurrentCell->HasChild(ChildIdx))
		{
			int32 ChildCellID = CurrentCell->GetChildCellID(ChildIdx);
			checkSlow(CellRefCounts.IsValid(ChildCellID));
			const FSparseOctreeCell* ChildCell = &Cells[ChildCellID];
			if (CanFit(*ChildCell, Bounds))
			{
				CurrentCell = ChildCell;
				continue;
			}
		}

		return *CurrentCell;

	} while (true);		// loop will always terminate
}


void FSparseDynamicOctree3::CheckValidity(
	TFunctionRef<bool(int)> IsValidObjectIDFunc,
	TFunctionRef<FAxisAlignedBox3d(int)> GetObjectBoundsFunc,
	EValidityCheckFailMode FailMode,
	bool bVerbose,
	bool bFailOnMissingObjects) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FSparseDynamicOctree3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FSparseDynamicOctree3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}

	TArray<int> CellsAtLevels, ObjectsAtLevel;
	CellsAtLevels.Init(0, 32);
	ObjectsAtLevel.Init(0, 32);
	uint32 SpillObjectCount = 0;
	uint32 MissingObjectCount = 0;
	uint8 MaxLevel = 0;

	// check that all object IDs in per-cell object lists is valid
	for (int32 CellID : CellRefCounts.Indices())
	{
		CellObjectLists.Enumerate(CellID, [&](int32 ObjectID)
		{
			CheckOrFailF(IsValidObjectIDFunc(ObjectID));
		});
	}

	uint32 NumObjectIDs = ObjectIDToCellMap.Num();
	for (uint32 ObjectID = 0; ObjectID < NumObjectIDs; ObjectID++)
	{
		bool IsValidObjectID = IsValidObjectIDFunc(ObjectID);
		if (IsValidObjectID)
		{
			FAxisAlignedBox3d ObjectBounds = GetObjectBoundsFunc(ObjectID);

			CheckOrFailF(ObjectID < ObjectIDToCellMap.Num());
			uint32 CellID = ObjectIDToCellMap[ObjectID];

			if (bFailOnMissingObjects)
			{
				CheckOrFailF(CellID != InvalidCellID);
			}

			if (CellID == SpillCellID)
			{
				// this is a spill node...
				SpillObjectCount++;
				CheckOrFailF(SpillObjectSet.Contains(ObjectID));
			}
			else if (CellID == InvalidCellID)
			{
				MissingObjectCount++;
				CheckOrFailF(SpillObjectSet.Contains(ObjectID) == false);
			}
			else
			{
				CheckOrFailF(CellRefCounts.IsValid(CellID));
				FSparseOctreeCell Cell = Cells[CellID];
				FAxisAlignedBox3d CellBounds = GetCellBox(Cell, MaxExpandFactor);
				CheckOrFailF(CellBounds.Contains(ObjectBounds));
				CheckOrFailF(CellObjectLists.Contains(CellID, ObjectID));

				ObjectsAtLevel[Cell.Level]++;
			}
		}
	}


	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparseOctreeCell& Cell = Cells[CellID];
		CellsAtLevels[Cell.Level]++;
		MaxLevel = FMath::Max(MaxLevel, Cell.Level);
	}

	if (bVerbose)
	{
		UE_LOGF(LogTemp, Warning, "FSparseDynamicOctree3::CheckValidity: MaxLevel %d  SpillCount %d  MissingCount %d", MaxLevel, SpillObjectCount, MissingObjectCount);
		for (uint32 k = 0; k <= MaxLevel; ++k)
		{
			UE_LOGF(LogTemp, Warning, "    Level %4d  Cells %4d  Objects %4d", k, CellsAtLevels[k], ObjectsAtLevel[k]);
		}
	}
}



void FSparseDynamicOctree3::ComputeStatistics(FStatistics& StatsOut) const
{
	StatsOut.SpillObjCount = SpillObjectSet.Num();

	StatsOut.Levels = 0;
	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparseOctreeCell& Cell = Cells[CellID];
		StatsOut.Levels = FMath::Max(StatsOut.Levels, (int32)Cell.Level);
	}
	StatsOut.Levels++;
	StatsOut.LevelObjCounts.Init(0, StatsOut.Levels);
	StatsOut.LevelBoxCounts.Init(0, StatsOut.Levels);
	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparseOctreeCell& Cell = Cells[CellID];
		StatsOut.LevelBoxCounts[Cell.Level]++;
		StatsOut.LevelObjCounts[Cell.Level] += CellObjectLists.GetCount(CellID);
	}
}

FString FSparseDynamicOctree3::FStatistics::ToString() const
{
	FString Result = FString::Printf(
		TEXT("Levels %2d   SpillCount %5d \r\n"), Levels, SpillObjCount);
	for (int k = 0; k < Levels; ++k)
	{
		Result += FString::Printf(TEXT("  Level %2d:  Cells %8d  Tris %8d  Avg %5.3f\r\n"), 
			k, LevelBoxCounts[k], LevelObjCounts[k], ((float)LevelObjCounts[k] / (float)LevelBoxCounts[k])  );
	}
	return Result;
}
