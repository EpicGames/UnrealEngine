// Copyright Epic Games, Inc. All Rights Reserved.


#include "Spatial/SparseDynamicPointOctree3.h"

namespace UE::Geometry
{


void FSparseDynamicPointOctree3::IterateRootsInRadius(FVector3d QueryPt, double InitialRadius, TFunctionRef<double(double, const FSparsePointOctreeCell&)> RootFn) const
{
	double ThresholdSq = InitialRadius * InitialRadius;
	FAxisAlignedBox3d Bounds(QueryPt, InitialRadius);

	ForEachRootCellInBounds(Bounds,
		[this, &QueryPt, &ThresholdSq](const FSparsePointOctreeCell& Cell) { return GetCellBox(Cell).DistanceSquared(QueryPt) < ThresholdSq; },
		[&ThresholdSq, &RootFn](const FSparsePointOctreeCell& Cell) { ThresholdSq = RootFn(ThresholdSq, Cell); });
}

void FSparseDynamicPointOctree3::FindKClosestPoints(FVector3d QueryPt, double DistanceThreshold, int32 NumToFind,
	TArray<TPair<int32, double>>& FoundPoints, TFunctionRef<bool(int32)> PredicateFunc, TFunctionRef<double(int32)> DistSqFunc,
	TArray<const FSparsePointOctreeCell*>* TempBuffer) const
{
	FoundPoints.Reset();
	if (NumToFind <= 0)
	{
		return;
	}
	FoundPoints.Reserve(NumToFind);

	TArray<const FSparsePointOctreeCell*> InternalBuffer;
	TArray<const FSparsePointOctreeCell*>& Queue =
		(TempBuffer == nullptr) ? InternalBuffer : *TempBuffer;
	Queue.Reset(128);

	IterateRootsInRadius(QueryPt, DistanceThreshold, [this, QueryPt, NumToFind, &FoundPoints, &PredicateFunc, &DistSqFunc, &Queue](double CurThresholdSq, const FSparsePointOctreeCell& Cell)
	{
		// Set a heap predicate so we can quickly access the largest point in the array
		auto HeapPredicate = [](const TPair<int32, double>& A, const TPair<int32, double>& B) -> bool
		{
			return A.Value > B.Value;
		};

		Queue.Add(&Cell);

		while (Queue.Num() > 0)
		{
			const FSparsePointOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

			// check if we can skip the cell based on its bounding box distance
			// (since BestDistSq may have been reduced after we added it to the queue)
			if (GetCellBox(*CurCell).DistanceSquared(QueryPt) >= CurThresholdSq)
			{
				continue;
			}

			// process points on the cell
			if (CellPointLists.IsAllocated(CurCell->CellID))
			{
				CellPointLists.Enumerate(CurCell->CellID, [&](int32 PointID)
				{
					if (PredicateFunc(PointID))
					{
						double DSq = DistSqFunc(PointID);
						if (DSq < CurThresholdSq)
						{
							if (FoundPoints.Num() < NumToFind)
							{
								FoundPoints.HeapPush(TPair<int32, double>(PointID, DSq), HeapPredicate);
							}
							else
							{
								// take the farthest point off the heap, and put the new point on
								TPair<int32, double> BiggestItem_Unused;
								FoundPoints.HeapPop(BiggestItem_Unused, HeapPredicate, EAllowShrinking::No);
								FoundPoints.HeapPush(TPair<int32, double>(PointID, DSq), HeapPredicate);
							}

							// once we've found our first K points, we can refine the search radius based on the farthest of them
							if (FoundPoints.Num() == NumToFind)
							{
								CurThresholdSq = FoundPoints[0].Value;
							}
						}
					}
				});
			}

			// add child cells to queue (if within current threshold distance)
			for (int k = 0; k < 8; ++k)
			{
				if (CurCell->HasChild(k))
				{
					const FSparsePointOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
					if (GetCellBox(*ChildCell).DistanceSquared(QueryPt) < CurThresholdSq)
					{
						Queue.Add(ChildCell);
					}
				}
			}
		}

		return CurThresholdSq;
	});
}


int32 FSparseDynamicPointOctree3::FindClosestPoint(FVector3d QueryPt, double DistanceThreshold,
	TFunctionRef<bool(int32)> PredicateFunc, TFunctionRef<double(int32)> DistSqFunc,
	TArray<const FSparsePointOctreeCell*>* TempBuffer) const
{
	TArray<const FSparsePointOctreeCell*> InternalBuffer;
	TArray<const FSparsePointOctreeCell*>& Queue =
		(TempBuffer == nullptr) ? InternalBuffer : *TempBuffer;
	
	double BestDistSq = DistanceThreshold * DistanceThreshold;
	int32 BestPtIdx = INDEX_NONE;

	Queue.Reset(128);

	IterateRootsInRadius(QueryPt, DistanceThreshold, [this, QueryPt, &BestPtIdx, &PredicateFunc, &DistSqFunc, &Queue](double BestDistSq, const FSparsePointOctreeCell& Cell)
	{
		Queue.Add(&Cell);
		while (Queue.Num() > 0)
		{
			const FSparsePointOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

			// check if we can skip the cell based on its bounding box distance
			// (since BestDistSq may have been reduced after we added it to the queue)
			if (GetCellBox(*CurCell).DistanceSquared(QueryPt) >= BestDistSq)
			{
				continue;
			}

			// process points on the cell
			if (CellPointLists.IsAllocated(CurCell->CellID))
			{
				CellPointLists.Enumerate(CurCell->CellID, [&](int32 PointID)
				{
					if (PredicateFunc(PointID))
					{
						double DSq = DistSqFunc(PointID);
						if (DSq < BestDistSq)
						{
							BestPtIdx = PointID;
							BestDistSq = DSq;
						}
					}
				});
			}

			// add child cells to queue (if close enough)
			for (int k = 0; k < 8; ++k)
			{
				if (CurCell->HasChild(k))
				{
					const FSparsePointOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
					if (GetCellBox(*ChildCell).DistanceSquared(QueryPt) < BestDistSq)
					{
						Queue.Add(ChildCell);
					}
				}
			}
		}

		return BestDistSq;
	});

	return BestPtIdx;
}

} // namespace UE::Geometry

