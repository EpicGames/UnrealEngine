// Copyright Epic Games, Inc. All Rights Reserved.


#include "Spatial/ZOrderCurvePoints.h"
#include "BoxTypes.h"

using namespace UE::Geometry;


namespace ZOrderCurvePointsLocal
{

struct FTreeNodeBase // just static helpers, common to both quad- and octree nodes
{
	static constexpr int32 InvalidIndex = -1;

	// By convention, Node indices are encoded as values below InvalidIndex, while valid Point and Bucket indices are encoded above InvalidIndex
	inline static bool IsNodeIndex(int32 Index)
	{
		return Index < InvalidIndex;
	}

	// Transforms to encode/decode data in a Node's 'Children' array to actual indices

	// Note: From and To are the same transform, but kept separate to clarify intent
	inline static int32 FromChildNodeIndex(int32 ChildNodeIndex)
	{
		return -ChildNodeIndex - 3;
	}
	inline static int32 ToChildNodeIndex(int32 ChildValue)
	{
		return -ChildValue - 3;
	}

	// Note: The below are identity transforms, but kept as functions in case we want to do something more with the encoding later / to clarify intent
	inline static int32 FromPointIndex(int32 PointIndex)
	{
		return PointIndex;
	}
	inline static int32 ToPointIndex(int32 ChildValue)
	{
		return ChildValue;
	}
	inline static int32 FromBucketIndex(int32 BucketIndex)
	{
		return BucketIndex;
	}
	inline static int32 ToBucketIndex(int32 ChildValue)
	{
		return ChildValue;
	}
};

struct FQuadTreeNode : FTreeNodeBase
{
	// All node data is stored in the Children array:
	//  - Child node indices on a non-leaf node (by convention: Stored as a negative value, offset by 3)
	//	- Point indices on a leaf node (before max depth)
	//  - Max Depth Bucket indices on a leaf node (at max depth)
	// -1 is reserved as invalid/empty (note: encodes to -2 for child node indices)
	int32 Children[4]{-1,-1,-1,-1};

	static constexpr int32 NumChildren = 4;
	// NumPerLeaf must be <= NumChildren; if less than NumChildren we split leaf nodes earlier to better distribute the points (at cost of deeper trees)
	static constexpr int32 NumPerLeaf = 4;
	
	template<typename RealType>
	static int32 GetSubIdx(const TVector2<RealType>& Center, const TVector2<RealType>& Pt)
	{
		return (Pt.X > Center.X) + (Pt.Y > Center.Y) * 2;
	}

	inline bool IsLeaf() const
	{
		return Children[0] >= InvalidIndex;
	}

	bool AddPtAtFirstZeroIdx(int32 PtIdx)
	{
		for (int32 SubIdx = 0; SubIdx < NumPerLeaf; SubIdx++)
		{
			if (Children[SubIdx] == InvalidIndex)
			{
				Children[SubIdx] = FromPointIndex(PtIdx);
				return true;
			}
		}
		return false;
	}

	template<typename RealType>
	static TVector2<RealType> GetSubCenter(const TAxisAlignedBox2<RealType>& Bounds, const TVector2<RealType>& Center, int32 SubIdx)
	{
		TVector2<RealType> Offset(RealType(SubIdx & 1) - (RealType).5, RealType(SubIdx & 2) - (RealType).5);
		return Center + Offset * Bounds.Extents();
	}

	template<typename RealType>
	static TAxisAlignedBox2<RealType> GetSubBounds(TAxisAlignedBox2<RealType> Bounds, const TVector2<RealType>& Center, int32 SubIdx)
	{
		if (SubIdx & 1)
		{
			Bounds.Min.X = Center.X;
		}
		else
		{
			Bounds.Max.X = Center.X;
		}

		if (SubIdx & 2)
		{
			Bounds.Min.Y = Center.Y;
		}
		else
		{
			Bounds.Max.Y = Center.Y;
		}

		return Bounds;
	}
};

struct FOctreeNode : FTreeNodeBase
{
	// All node data is stored in the Children array:
	//  - Child node indices on a non-leaf node (by convention: Stored as a negative value, offset by 3)
	//	- Point indices on a leaf node (before max depth)
	//  - Max Depth Bucket indices on a leaf node (at max depth)
	// -1 is reserved as invalid/empty (note: encodes to -2 for child node indices)
	int32 Children[8]{ -1,-1,-1,-1, -1,-1,-1,-1 };

	static constexpr int32 NumChildren = 8;
	// NumPerLeaf must be <= NumChildren; if less than NumChildren we split leaf nodes earlier to better distribute the points (at cost of deeper trees)
	static constexpr int32 NumPerLeaf = 8;

	template<typename RealType>
	static int32 GetSubIdx(const TVector<RealType>& Center, const TVector<RealType>& Pt)
	{
		return (Pt.X > Center.X) + (Pt.Y > Center.Y) * 2 + (Pt.Z > Center.Z) * 4;
	}

	inline bool IsLeaf() const
	{
		return Children[0] >= InvalidIndex;
	}

	bool AddPtAtFirstZeroIdx(int32 PtIdx)
	{
		// TODO: if NumPerLeaf is always < NumChildren, we could store the current num indices on the node in the last slot, and skip this iteration
		// (but if NumPerLeaf is very small, maybe not worth it. and if NumPerLeaf is NumChildren, we won't have room / would need to store the num separately)
		for (int32 SubIdx = 0; SubIdx < NumPerLeaf; SubIdx++)
		{
			if (Children[SubIdx] == InvalidIndex)
			{
				Children[SubIdx] = FromPointIndex(PtIdx);
				return true;
			}
		}

		return false;
	}

	template<typename RealType>
	static TVector<RealType> GetSubCenter(const TAxisAlignedBox3<RealType>& Bounds, const TVector<RealType>& Center, int32 SubIdx)
	{
		TVector<RealType> Offset(RealType(SubIdx & 1) - (RealType).5, RealType(SubIdx & 2) - (RealType).5, RealType(SubIdx & 4) - (RealType).5);
		return Center + Offset * Bounds.Extents();
	}

	template<typename RealType>
	static TAxisAlignedBox3<RealType> GetSubBounds(TAxisAlignedBox3<RealType> Bounds, const TVector<RealType>& Center, int32 SubIdx)
	{
		if (SubIdx & 1)
		{
			Bounds.Min.X = Center.X;
		}
		else
		{
			Bounds.Max.X = Center.X;
		}

		if (SubIdx & 2)
		{
			Bounds.Min.Y = Center.Y;
		}
		else
		{
			Bounds.Max.Y = Center.Y;
		}

		if (SubIdx & 4)
		{
			Bounds.Min.Z = Center.Z;
		}
		else
		{
			Bounds.Max.Z = Center.Z;
		}

		return Bounds;
	}
};


// Simple quad- or octree to store indices of points
template<typename NodeT, typename PointT, typename BoxT>
struct TIndexSpatialTree
{
	TArray<NodeT> Nodes;

	// MaxDepthBuckets stores the indices for points that have reached a max-depth leaf node, which may no longer fit in a fixed-size node struct
	// Note: The best allocator here (inline or no) depends a lot on the chosen MaxDepth as well as the distribution of points
	TArray<TArray<int32, TInlineAllocator<4>>> MaxDepthBuckets;

	int32 MaxDepth;
	// Number of points inserted in the tree
	int32 Num = 0;

	void Reset()
	{
		Nodes.Reset();
		MaxDepthBuckets.Reset();
		Num = 0;
	}

	void Init(TArrayView<const PointT> Points, BoxT Bounds, int32 MaxTreeDepth)
	{
		Reset();
		Nodes.Emplace();
		MaxDepth = MaxTreeDepth;

		if (Bounds.IsEmpty())
		{
			for (const PointT& Pt : Points)
			{
				Bounds.Contain(Pt);
			}
		}

		for (int32 PtIdx = 0; PtIdx < Points.Num(); PtIdx++)
		{
			AddPointHelper(Points, PtIdx, Bounds);
		}
	}

	void InitSubset(TArrayView<const PointT> Points, TArrayView<const int32> PointIndices, BoxT Bounds, int32 MaxTreeDepth)
	{
		Reset();
		Nodes.Emplace();
		MaxDepth = MaxTreeDepth;

		if (Bounds.IsEmpty())
		{
			for (int32 PtIdx : PointIndices)
			{
				const PointT& Pt = Points[PtIdx];
				Bounds.Contain(Pt);
			}
		}

		for (int32 PtIdx : PointIndices)
		{
			AddPointHelper(Points, PtIdx, Bounds);
		}
	}
	
protected:

	void PlaceInMaxDepthBucketHelper(int32 PtIdx, const PointT& Pt, NodeT& Node, const PointT& Center)
	{
		int32 SubIdx = NodeT::GetSubIdx(Center, Pt);
		int32& EncodedBucketIndex = Node.Children[SubIdx];
		if (NodeT::ToBucketIndex(EncodedBucketIndex) == NodeT::InvalidIndex)
		{
			EncodedBucketIndex = NodeT::FromBucketIndex(MaxDepthBuckets.Emplace());
		}
		MaxDepthBuckets[NodeT::ToBucketIndex(EncodedBucketIndex)].Add(PtIdx);
	}

	void AddPointHelper(TArrayView<const PointT> Points, int32 PtIdx, const BoxT& Bounds)
	{
		Num++;
		const PointT& Pt = Points[PtIdx];
		int32 CurNodeIdx = 0, CurDepth = 0;
		BoxT CurBox = Bounds;
		while (true)
		{
			PointT CurCenter = CurBox.Center();

			if (CurDepth >= MaxDepth)
			{
				PlaceInMaxDepthBucketHelper(PtIdx, Pt, Nodes[CurNodeIdx], CurCenter);
				break;
			}

			// Node is a leaf that directly stores point indices
			if (Nodes[CurNodeIdx].Children[0] >= NodeT::InvalidIndex)
			{
				// Try to add the point directly to the node
				if (Nodes[CurNodeIdx].AddPtAtFirstZeroIdx(PtIdx))
				{
					break;
				}
				else // didn't have room for the point: switch node to instead store indices to new children, and propagate old children down
				{
					NodeT Old = Nodes[CurNodeIdx];
					// Create an empty non-leaf node
					for (int32 SubIdx = 0; SubIdx < NodeT::NumChildren; SubIdx++)
					{
						Nodes[CurNodeIdx].Children[SubIdx] = NodeT::FromChildNodeIndex(NodeT::InvalidIndex);
					}
					if (CurDepth + 1 >= MaxDepth)
					{
						// the new nodes are final bucket nodes; immediately push the children through to their respective max depth buckets
						for (int32 SubIdx = 0; SubIdx < NodeT::NumPerLeaf; SubIdx++)
						{
							int32 OldPtIdx = NodeT::ToPointIndex(Old.Children[SubIdx]);
							checkSlow(OldPtIdx > NodeT::InvalidIndex);
							PointT OldPt = Points[OldPtIdx];
							int32 OldPtSubIdx = NodeT::GetSubIdx(CurCenter, OldPt);
							int32 NewChildNodeIdx = NodeT::ToChildNodeIndex(Nodes[CurNodeIdx].Children[OldPtSubIdx]);
							if (NewChildNodeIdx == NodeT::InvalidIndex)
							{
								NewChildNodeIdx = Nodes.Emplace();
								Nodes[CurNodeIdx].Children[OldPtSubIdx] = NodeT::FromChildNodeIndex(NewChildNodeIdx);
							}
							NodeT& NewChildNode = Nodes[NewChildNodeIdx];
							PointT ChildCenter = NodeT::GetSubCenter(Bounds, CurCenter, OldPtSubIdx);
							PlaceInMaxDepthBucketHelper(OldPtIdx, OldPt, NewChildNode, ChildCenter);
						}
					}
					else
					{
						// the new nodes are regular leaf nodes; push the children down to the leaves
						for (int32 SubIdx = 0; SubIdx < NodeT::NumPerLeaf; SubIdx++)
						{
							int32 OldPtIdx = NodeT::ToPointIndex(Old.Children[SubIdx]);
							checkSlow(OldPtIdx > NodeT::InvalidIndex);
							PointT OldPt = Points[OldPtIdx];
							int32 OldPtSubIdx = NodeT::GetSubIdx(CurCenter, OldPt);
							int32 NewChildNodeIdx = NodeT::ToChildNodeIndex(Nodes[CurNodeIdx].Children[OldPtSubIdx]);
							if (NewChildNodeIdx == NodeT::InvalidIndex)
							{
								NewChildNodeIdx = Nodes.Emplace();
								Nodes[CurNodeIdx].Children[OldPtSubIdx] = NodeT::FromChildNodeIndex(NewChildNodeIdx);
							}
							NodeT& NewChildNode = Nodes[NewChildNodeIdx];
							bool bPlaced = NewChildNode.AddPtAtFirstZeroIdx(OldPtIdx);
							checkSlow(bPlaced); // should always succeed
						}
					}
				}
			}

			check(NodeT::IsNodeIndex(Nodes[CurNodeIdx].Children[0])); // should only reach here if we need to iterate further down the tree
			int32 PtSubIdx = NodeT::GetSubIdx(CurCenter, Pt);
			int32 OldNodeIdx = CurNodeIdx;
			CurNodeIdx = NodeT::ToChildNodeIndex(Nodes[CurNodeIdx].Children[PtSubIdx]);
			if (CurNodeIdx == NodeT::InvalidIndex) // if child didn't exist yet, create it here
			{
				CurNodeIdx = Nodes.Emplace();
				Nodes[OldNodeIdx].Children[PtSubIdx] = NodeT::FromChildNodeIndex(CurNodeIdx);
			}
			CurBox = NodeT::GetSubBounds(CurBox, CurCenter, PtSubIdx);
			CurDepth++;
		}
	}
	
public:

	// Add indices of all points in the quad/octree to the PointOrder array, following a z order curve when extracting them.
	void ExtractZOrderCurve(TArray<int32>& PointOrder, int32 ExpectedNumberPoints = 0) const
	{
		PointOrder.Reserve(PointOrder.Num() + ExpectedNumberPoints);
		ExtractZOrderCurveHelper(PointOrder, Nodes[0], 0);
	}

	// Overwrite the values in the OrderView array view with the indices in the quad/octree, extracted in a z-order curve
	// The OrderView should have the same size as the number of points in the tree
	void InPlaceExtractZOrderCurve(TArrayView<int32> OrderView, bool bReverse) const
	{
		if (!ensure(Num == OrderView.Num()))
		{
			return;
		}
		int32 Idx = 0;
		InPlaceExtractZOrderCurveHelper(OrderView, Nodes[0], 0, bReverse, Idx);
		check(Idx == OrderView.Num());
	}

protected:

	void InPlaceExtractZOrderCurveHelper(TArrayView<int32> OrderView, const NodeT& Node, int32 CurDepth, bool bReverse, int& CurIdx) const
	{
		// at max depth: append all the non-empty buckets
		if (CurDepth >= MaxDepth)
		{
			for (int32 SubIdx = 0; SubIdx < NodeT::NumChildren; SubIdx++)
			{
				int32 ChildValue = Node.Children[bReverse ? NodeT::NumChildren - 1 - SubIdx : SubIdx];
				if (ChildValue == NodeT::InvalidIndex)
				{
					continue;
				}

				for (int32 Val : MaxDepthBuckets[NodeT::ToBucketIndex(ChildValue)])
				{
					OrderView[CurIdx++] = Val;
				}
			}
			return;
		}

		// at leaf: add all the attached points
		if (Node.IsLeaf())
		{
			// Note: bReverse doesn't affect this ordering, because leaf points are in arbitrary order
			for (int32 SubIdx = 0; SubIdx < NodeT::NumPerLeaf; SubIdx++)
			{
				int32 ChildValue = Node.Children[SubIdx];
				if (ChildValue == NodeT::InvalidIndex)
				{
					break;
				}

				OrderView[CurIdx++] = NodeT::ToPointIndex(ChildValue);
			}
			return;
		}

		// recurse to children
		for (int32 SubIdx = 0; SubIdx < NodeT::NumChildren; SubIdx++)
		{
			int32 ChildIndex = NodeT::ToChildNodeIndex(Node.Children[bReverse ? NodeT::NumChildren - 1 - SubIdx : SubIdx]);
			if (ChildIndex == NodeT::InvalidIndex)
			{
				continue;
			}

			InPlaceExtractZOrderCurveHelper(OrderView, Nodes[ChildIndex], CurDepth + 1, bReverse, CurIdx);
		}
	}

	// recursive helper for following a z order curve to list the points
	void ExtractZOrderCurveHelper(TArray<int32>& PointOrder, const NodeT& Node, int32 CurDepth) const
	{
		// at max depth: append all the non-empty buckets
		if (CurDepth >= MaxDepth)
		{
			for (int32 SubIdx = 0; SubIdx < NodeT::NumChildren; SubIdx++)
			{
				int32 ChildValue = Node.Children[SubIdx];
				if (ChildValue == NodeT::InvalidIndex)
				{
					continue;
				}

				PointOrder.Append(MaxDepthBuckets[NodeT::ToBucketIndex(ChildValue)]);
			}
			return;
		}

		// at leaf: add all the attached points
		if (Node.IsLeaf())
		{
			for (int32 SubIdx = 0; SubIdx < NodeT::NumPerLeaf; SubIdx++)
			{
				int32 ChildValue = Node.Children[SubIdx];
				if (ChildValue == NodeT::InvalidIndex)
				{
					break;
				}

				PointOrder.Add(NodeT::ToPointIndex(ChildValue));
			}
			return;
		}

		// recurse to children
		for (int32 SubIdx = 0; SubIdx < NodeT::NumChildren; SubIdx++)
		{
			int32 ChildIndex = NodeT::ToChildNodeIndex(Node.Children[SubIdx]);
			if (ChildIndex == NodeT::InvalidIndex)
			{
				continue;
			}

			ExtractZOrderCurveHelper(PointOrder, Nodes[ChildIndex], CurDepth + 1);
		}
	}
};

template<typename RealType>
void ZOrderCurve2(TArray<int32>& PointOrder, TArrayView<const TVector2<RealType>> Points, TAxisAlignedBox2<RealType> Bounds, int32 MaxTreeDepth)
{
	TIndexSpatialTree<FQuadTreeNode, TVector2<RealType>, TAxisAlignedBox2<RealType>> Tree;
	Tree.Init(Points, Bounds, MaxTreeDepth);
	Tree.ExtractZOrderCurve(PointOrder, Points.Num());
}

template<typename RealType>
void ZOrderCurve3(TArray<int32>& PointOrder, TArrayView<const TVector<RealType>> Points, TAxisAlignedBox3<RealType> Bounds, int32 MaxTreeDepth)
{
	TIndexSpatialTree<FOctreeNode, TVector<RealType>, TAxisAlignedBox3<RealType>> Tree;
	Tree.Init(Points, Bounds, MaxTreeDepth);
	Tree.ExtractZOrderCurve(PointOrder, Points.Num());
}

template<typename RealType, typename VectorType, typename BoxType, typename NodeType>
void ApplyBRIO(TArray<int32>& PointOrder, TArrayView<const VectorType> Points, BoxType Bounds, int32 MaxTreeDepth, const FRandomStream& Random)
{
	if (Bounds.IsEmpty())
	{
		Bounds.Contain(Points);
	}

	int32 NumPoints = Points.Num();

	int32 OrderInitialNum = PointOrder.Num();
	PointOrder.SetNumUninitialized(PointOrder.Num() + NumPoints);
	TArrayView<int32> OrderSlice(PointOrder.GetData() + OrderInitialNum, NumPoints);
	for (int32 Idx = 0; Idx < Points.Num(); ++Idx)
	{
		OrderSlice[Idx] = Idx;
	}

	if (NumPoints < 2) // not enough points to need any BRIO phases
	{
		return;
	}

	int32 NumPhases = (int32)FMath::Log2((double)NumPoints);
	TArray<int32> PhaseEnds;
	PhaseEnds.SetNumUninitialized(NumPhases);
	int32 CurEnd = NumPoints;
	PhaseEnds[NumPhases - 1] = CurEnd;
	for (int32 Phase = NumPhases - 1; Phase > 0; --Phase)
	{
		for (int32 Idx = 0; Idx < CurEnd; ++Idx)
		{
			if (Random.GetFraction() < .5)
			{
				CurEnd--;
				Swap(OrderSlice[Idx], OrderSlice[CurEnd]);
				Idx--;
			}
		}
		PhaseEnds[Phase - 1] = CurEnd;
	}

	TIndexSpatialTree<NodeType, VectorType, BoxType> Tree;
	for (int32 Phase = 0, PhaseStart = 0; Phase < NumPhases; PhaseStart = PhaseEnds[Phase++])
	{
		int32 PhaseEnd = PhaseEnds[Phase];
		if (PhaseEnd == PhaseStart)
		{
			continue;
		}
		Tree.Reset();
		TArrayView<int32> PhaseSlice(OrderSlice.GetData() + PhaseStart, PhaseEnd - PhaseStart);
		Tree.InitSubset(Points, PhaseSlice, Bounds, MaxTreeDepth);
		Tree.InPlaceExtractZOrderCurve(PhaseSlice, bool(Phase & 1) /*alternate phases have reversed order*/);
	}
}

} // end of local helper function namespace for ZOrderCurvePoints functions


void FZOrderCurvePoints::Compute(TArrayView<const FVector2d> Points, const FAxisAlignedBox2d& Bounds)
{
	ZOrderCurvePointsLocal::ZOrderCurve2<double>(Order, Points, Bounds, MaxTreeDepth);
}
void FZOrderCurvePoints::Compute(TArrayView<const FVector2f> Points, const FAxisAlignedBox2f& Bounds)
{
	ZOrderCurvePointsLocal::ZOrderCurve2<float>(Order, Points, Bounds, MaxTreeDepth);
}
void FZOrderCurvePoints::Compute(TArrayView<const FVector3d> Points, const FAxisAlignedBox3d& Bounds)
{
	ZOrderCurvePointsLocal::ZOrderCurve3<double>(Order, Points, Bounds, MaxTreeDepth);
}
void FZOrderCurvePoints::Compute(TArrayView<const FVector3f> Points, const FAxisAlignedBox3f& Bounds)
{
	ZOrderCurvePointsLocal::ZOrderCurve3<float>(Order, Points, Bounds, MaxTreeDepth);
}

void FBRIOPoints::Compute(TArrayView<const FVector2d> Points, const FAxisAlignedBox2d& Bounds, const FRandomStream& Random)
{
	ZOrderCurvePointsLocal::ApplyBRIO<double, FVector2d, FAxisAlignedBox2d, ZOrderCurvePointsLocal::FQuadTreeNode>(Order, Points, Bounds, MaxTreeDepth, Random);
}
void FBRIOPoints::Compute(TArrayView<const FVector2f> Points, const FAxisAlignedBox2f& Bounds, const FRandomStream& Random)
{
	ZOrderCurvePointsLocal::ApplyBRIO<float, FVector2f, FAxisAlignedBox2f, ZOrderCurvePointsLocal::FQuadTreeNode>(Order, Points, Bounds, MaxTreeDepth, Random);
}
void FBRIOPoints::Compute(TArrayView<const FVector3d> Points, const FAxisAlignedBox3d& Bounds, const FRandomStream& Random)
{
	ZOrderCurvePointsLocal::ApplyBRIO<double, FVector3d, FAxisAlignedBox3d, ZOrderCurvePointsLocal::FOctreeNode>(Order, Points, Bounds, MaxTreeDepth, Random);
}
void FBRIOPoints::Compute(TArrayView<const FVector3f> Points, const FAxisAlignedBox3f& Bounds, const FRandomStream& Random)
{
	ZOrderCurvePointsLocal::ApplyBRIO<float, FVector3f, FAxisAlignedBox3f, ZOrderCurvePointsLocal::FOctreeNode>(Order, Points, Bounds, MaxTreeDepth, Random);
}