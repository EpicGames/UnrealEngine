// Copyright Epic Games, Inc. All Rights Reserved.

#include "Splines/PolyBezierSpline.h"
#include "HAL/IConsoleManager.h"

namespace UE::Geometry::Spline
{
	namespace Private
	{
		static bool bFindNearestUseSplineCulling = true;
		static FAutoConsoleVariableRef CVarFindNearestUseSplineCulling(
			TEXT("Spline.PolyBezier.FindNearestUseSplineCulling"),
			bFindNearestUseSplineCulling,
			TEXT("If enabled, FindNearest uses spline culling to optimize."),
			ECVF_Default);

		bool UseFindNearestUseSplineCulling()
		{
			return bFindNearestUseSplineCulling;
		}
	}
}

