// Copyright Epic Games, Inc. All Rights Reserved.

#include "Splines/SplineTypeId.h"

namespace UE::Geometry::Spline
{
DEFINE_LOG_CATEGORY(LogSpline);
}

