// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tessellation/Regularization.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Containers/Array.h"
#include "Util/IndexUtil.h"
#include "VectorUtil.h"
#include "VectorTypes.h"
#include "HAL/IConsoleManager.h"
#include "Async/ParallelFor.h"
#include "MathUtil.h"

namespace UE {
namespace Geometry {

/**
 * A binary heap data structure with external swap tracking capability.
 *
 * This class implements a min-heap (based on operator<) and allows external systems
 * to track element movement via a swap policy. This is useful in scenarios like
 * priority queues where an external index mapping needs to be updated on swaps.
 *
 * @tparam T               The type of elements stored in the heap.
 * @tparam SwapPolicyType  A policy class that defines Swap(int32, int32, T&, T&) to track swaps.
 */
template <typename T, typename SwapPolicyType>
class THeapWithTracking
{
public:
	using SizeType = typename TArray<T>::SizeType;
	
	/**
	 * @param InSwapPolicy Reference to a swap policy instance used to track heap element swaps.
	 */
	THeapWithTracking(SwapPolicyType& InSwapPolicy)
	: SwapPolicy(InSwapPolicy)
	{
	}

	/**
	 * Adds an element to the heap and reorders to maintain the heap property.
	 *
	 * @param Element The element to add.
	 */
	void Add(const T& Element)
	{
		HeapUp(Heap.Add(Element));
	}
	/**
	 * Removes and returns the root (minimum) element from the heap.
	 *
	 * @return The removed root element.
	 */
	T Pop()
	{
		check(!Heap.IsEmpty());
		const T Root = Heap[0];
		if (Heap.Num() > 1)
		{
			Swap(0, Heap.Num()-1);
			Heap.Pop();
			HeapDown(0);
		}
		else
		{
			Heap.Pop();
		}
		return Root;
	}

	/**
	 * Updates an element at a given index and reorders the heap accordingly.
	 *
	 * @param Idx     Index of the element to update.
	 * @param Element The new element value.
	 */
	void Update(const SizeType Idx, const T& Element)
	{
		Heap[Idx] = Element;
		HeapUp(Idx);
		HeapDown(Idx);
	}

	/**
	 * Removes the element at the specified index and reorders the heap to maintain
	 * heap property.
	 *
	 * @param Idx Index of the element to remove.
	 */
	void Remove(const SizeType Idx)
	{
		check(Idx < Heap.Num());
		if (Heap.Num() > 1)
		{
			Swap(Idx, Heap.Num()-1);
			Heap.Pop();
			HeapDown(Idx);
		}
		else
		{
			Heap.Pop();
		}
	}

	/**
	 * Checks if the heap is empty.
	 *
	 * @return true if the heap is empty; false otherwise.
	 */
	bool IsEmpty() const
	{
		return Heap.IsEmpty();
	}

	/**
	 * Returns the number of elements currently in the heap.
	 *
	 * @return The number of heap elements.
	 */
	SizeType Num() const 
	{ 
		return Heap.Num(); 
	}

	/**
	 * Provides read-only access to an element by index.
	 *
	 * @param Idx Index to access.
	 * @return Reference to the element at the specified index.
	 */
	const T& operator[](const SizeType Idx) const
	{
		return Heap[Idx];
	}

private:

	/**
	 * Bubbles an element up the heap to restore the heap property.
	 *
	 * @param Idx Index of the element to heapify up.
	 */
	void HeapUp(SizeType Idx)
	{
		while (Idx > 0)
		{
			const SizeType Parent = (Idx - 1) >> 1;
			if (!(Heap[Idx] < Heap[Parent]))
			{
				break;
			}

			Swap(Idx, Parent);
			Idx = Parent;
		};
	}

	/**
	 * Pushes an element down the heap to restore the heap property.
	 *
	 * @param Idx Index of the element to heapify down.
	 */
	void HeapDown(SizeType Idx)
	{
		const int32 Num = Heap.Num();

		while (true)
		{
			const SizeType Child0 = (Idx << 1) + 1;
			const SizeType Child1 = Child0 + 1;

			SizeType MinIdx = Idx;
			if (Child0 < Num && Heap[Child0] < Heap[MinIdx])
			{
				MinIdx = Child0;
			}
			if (Child1 < Num && Heap[Child1] < Heap[MinIdx])
			{
				MinIdx = Child1;
			}

			if (MinIdx == Idx)
			{
				break;
			}

            Swap(Idx, MinIdx);
            Idx = MinIdx;
		}
	}

	/**
	 * Swaps two elements in the heap and informs the swap policy.
	 *
	 * @param A Index of the first element.
	 * @param B Index of the second element.
	 */
	void Swap(const SizeType A, const SizeType B)
	{
		SwapPolicy.Swap(A, B, Heap[A], Heap[B]);
		::Swap(Heap[A], Heap[B]);
	}

	// The underlying array representing the heap.
	TArray<T> Heap;

	// Reference to the swap policy for tracking element movements.
	SwapPolicyType& SwapPolicy;
};

// [ Baensch et al. "A Finite Element Method for Surface Diffusion", 2004]
void RegularizeVolumePreserving(FDynamicMesh3& Mesh, bool bPreserveVolume, const TOptional<TBitArray<>>& ActiveVertices)
{
	constexpr double ClampT = 0.5;
	TArray<FVector3d> Vertices;

	Vertices.SetNum(Mesh.MaxVertexID());

	for (int VID : Mesh.VertexIndicesItr())
	{
		Vertices[VID] = Mesh.GetVertex(VID);		
	}

	ParallelFor(Mesh.MaxVertexID(), [&](int32 VID)
	{
		if (!Mesh.IsVertex(VID))
		{
			return;
		}

		if (ActiveVertices && !(*ActiveVertices)[VID])
		{
			return;
		}

		const FVector3d Z = Vertices[VID];

		FVector3d ZHat(0.);
		int TriCount = 0;
		FVector3d NormalDirection(0.);

		for (int TID : Mesh.VtxTrianglesItr(VID))
		{
			const FIndex3i Tri = Mesh.GetTriangle(TID);
			const FVector3d V[3] = { Vertices[Tri[0]], Vertices[Tri[1]], Vertices[Tri[2]] };			

			FVector3d Edge1(V[1] - V[0]);
			FVector3d Edge2(V[2] - V[0]);
			
			FVector3d vCross(Edge2.Cross(Edge1));
			NormalDirection += vCross;

			ZHat += (V[0] + V[1] + V[2]) * 1./3.;
			TriCount++;
		}

		if (TriCount == 0) 
		{
			return;
		}

		ZHat /= static_cast<double>(TriCount);


		const double NormalDirectionLength = NormalDirection.Length();
		if (NormalDirectionLength < UE_DOUBLE_SMALL_NUMBER)
		{
			return;
		}
        // scale the normal direction such that t becomes invariant of mesh scaling (note, this is not unit normal, but proportional to edge length) 
		NormalDirection /= FMath::Sqrt(NormalDirectionLength);

		double t = 0.;

		if (bPreserveVolume)
		{
			double nom(0.), denom(0.);
			for (int TID : Mesh.VtxTrianglesItr(VID))
			{
				const FIndex3i Tri = Mesh.GetTriangle(TID);
				const int CID = IndexUtil::FindTriIndex(VID, Tri);

				const FVector3d Z2 = Vertices[Tri[(CID+1)%3]];
				const FVector3d Z3 = Vertices[Tri[(CID+2)%3]];

				nom   += ((Z-ZHat) ^ (Z2-ZHat)) | (Z3-ZHat);
				denom += (NormalDirection   ^ (Z2-ZHat)) | (Z3-ZHat);
			}
			t = nom / denom;
		}

		if (TMathUtil<double>::IsFinite(t)) 
		{
			Mesh.SetVertex(VID, ZHat + NormalDirection * FMath::Clamp(t, -ClampT, ClampT), false);
		}
		else
		{
			Mesh.SetVertex(VID, ZHat, false);
		}
	});
}

// Applies edge splits to edges for which |longest edge| > SplitThreshold (|other edge a| + |other edge b|)
// in order of importance.
//
void SplitLongEdges(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double SplitThreshold)
{
	check(Displacements.IsEmpty() || Displacements.Num() == Mesh.MaxVertexID());

	const bool bHasDisplacements = !Displacements.IsEmpty();

	check(!bHasDisplacements || DisplacementFunctor);

	const auto ComputeWeight = [&](const int32 EID) -> double
	{
		const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);
	
		if (Edge.Tri[1] == IndexConstants::InvalidID)
		{
			return 0.;
		}

		const FIndex3i Tri0 = Mesh.GetTriangle(Edge.Tri[0]);
		const FIndex3i Tri1 = Mesh.GetTriangle(Edge.Tri[1]);
		
		const int SharedEdge0 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri0);
		const int SharedEdge1 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri1);

		const int VID0 = Tri0[SharedEdge0];
		const int VID1 = Tri1[SharedEdge1];

		check(VID1 == Tri0[(SharedEdge0+1)%3]);
		check(VID0 == Tri1[(SharedEdge1+1)%3]);

		const int VIDA = Tri0[(SharedEdge0+2)%3];
		const int VIDB = Tri1[(SharedEdge1+2)%3];

		// Edge (V0, V1) is the split candidate
		//
		//       o VA
		//      / \
		//     /   \
		// V0 o=====o V1
		//     \   /
		//      \ /
	    //       o VB
		// 

		FVector3d V0 = Mesh.GetVertex(VID0);
		FVector3d V1 = Mesh.GetVertex(VID1);
		FVector3d VA = Mesh.GetVertex(VIDA);
		FVector3d VB = Mesh.GetVertex(VIDB);

		if (bHasDisplacements)
		{
			V0 += Displacements[VID0];
			V1 += Displacements[VID1];
			VA += Displacements[VIDA];
			VB += Displacements[VIDB];
		}

		double E01 = (V1 - V0).Length();
		double E0A = (VA - V0).Length();
		double E0B = (VB - V0).Length();
		double E1A = (VA - V1).Length();
		double E1B = (VB - V1).Length();
		
		return E01 / FMath::Max(E0A + E1A, E0B + E1B);		
	};

	// Entry of the heap
	struct FEntry
	{
		double Weight;
		int32 EID;

		bool operator<(const FEntry& Other) const
		{
			return Weight < Other.Weight;
		}
	};

	// keeps track of changed heap indices
	struct FSwapPolicy
	{
		FSwapPolicy(TArray<int32>& InHeapIndices)
			: HeapIndices(InHeapIndices)
		{
		}

		void Swap(const int32 HeapIdxA, const int32 HeapIdxB, const FEntry& EntryA, const FEntry& EntryB)
		{
			check(HeapIndices[EntryA.EID] == HeapIdxA);
			check(HeapIndices[EntryB.EID] == HeapIdxB);

			::Swap(HeapIndices[EntryA.EID], HeapIndices[EntryB.EID]);
		}

		TArray<int32>& HeapIndices;
	};

	TArray<int32> HeapIndices;
	FSwapPolicy SwapPolicy(HeapIndices);
	THeapWithTracking<FEntry, FSwapPolicy> Heap(SwapPolicy);

	int32 SplitEdgeCount = 0;
	for (int EID : Mesh.EdgeIndicesItr())
	{
		const double Weight = ComputeWeight(EID);
		if (Weight > SplitThreshold)
		{
			HeapIndices.Add(Heap.Num());
			Heap.Add({ -Weight, EID });
		}
		else
		{
			HeapIndices.Add(IndexConstants::InvalidID);
		}
	}

	check(HeapIndices.Num() == Mesh.MaxEdgeID());
	
	while(!Heap.IsEmpty())
	{
		const FEntry Entry = Heap.Pop();
		HeapIndices[Entry.EID] = IndexConstants::InvalidID;

		if (Entry.Weight < -SplitThreshold)
		{
			int EdgesToUpdate[4];
			{
				const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(Entry.EID);
				const FIndex3i Tri0 = Mesh.GetTriangle(Edge.Tri[0]);
				const FIndex3i Tri1 = Mesh.GetTriangle(Edge.Tri[1]);

				const int SharedEdge0 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri0);
				const int SharedEdge1 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri1);

				EdgesToUpdate[0] = Mesh.GetTriEdge(Edge.Tri[0], (SharedEdge0+1)%3);
				EdgesToUpdate[1] = Mesh.GetTriEdge(Edge.Tri[0], (SharedEdge0+2)%3);
				EdgesToUpdate[2] = Mesh.GetTriEdge(Edge.Tri[1], (SharedEdge1+1)%3);
				EdgesToUpdate[3] = Mesh.GetTriEdge(Edge.Tri[1], (SharedEdge1+2)%3);
			}

			FDynamicMesh3::FEdgeSplitInfo SplitInfo;
			auto Result = Mesh.SplitEdge(Entry.EID, SplitInfo);
		
			if (Result == EMeshResult::Ok)
			{
				if (bHasDisplacements)
				{
					const FVector3d SplitVertexDisplacement = DisplacementFunctor(SplitInfo.NewVertex);
					if (SplitInfo.NewVertex == Displacements.Num())
					{
						Displacements.Add(SplitVertexDisplacement);
					}
					else
					{
						Displacements.SetNum(FMath::Max(Displacements.Num(), SplitInfo.NewVertex + 1));
						Displacements[SplitInfo.NewVertex] = SplitVertexDisplacement;
					}
				}

				HeapIndices[Entry.EID] = IndexConstants::InvalidID;

				for (int32 I=0; I<3; ++I)
				{
					const int32 EID = SplitInfo.NewEdges[I];
					HeapIndices.Add(IndexConstants::InvalidID);
				}

				for (const int EID : EdgesToUpdate)
				{
					check(EID < Mesh.MaxEdgeID());

					if (EID == IndexConstants::InvalidID || HeapIndices[EID] == IndexConstants::InvalidID)
					{
						continue;
					}

					const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);

					if (Edge.Tri[1] != IndexConstants::InvalidID)
					{
						Heap.Update(HeapIndices[EID], { -ComputeWeight(EID), EID });
					}
				}
			}
		}
	}
}

// Apply edge flips on non-Delaunay edges in order of importance.
// Intrinsic edge-flips would split the flipped edge and produce edges that are on the same surface.
// This function relaxes this and applies honest flips when the surface is planar enough (when the
// change in the angle is smaller than FlipAngleThreshold)
// 
// [Fisher, Springborn, Bobenko, Schroeder, "An Algorithm for the Construction of Intrinsic Delaunay Triangulations
// with Applications to Digital Geometry Processing", 2006]
//
void IntrinsicDelaunayFlipEdge(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double FlipAngleThreshold)
{
	check(Displacements.IsEmpty() || Displacements.Num() == Mesh.MaxVertexID());

	const bool bHasDisplacements = !Displacements.IsEmpty();

	check(!bHasDisplacements || DisplacementFunctor);

	const auto ComputeWeight = [&](const int32 EID) -> std::tuple<double, bool>
	{
		const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);
		
		const FIndex3i Tri0 = Mesh.GetTriangle(Edge.Tri[0]);
		const FIndex3i Tri1 = Mesh.GetTriangle(Edge.Tri[1]);
		
		const int SharedEdge0 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri0);
		const int SharedEdge1 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri1);

		const int VID0 = Tri0[SharedEdge0];
		const int VID1 = Tri1[SharedEdge1];

		check(VID1 == Tri0[(SharedEdge0+1)%3]);
		check(VID0 == Tri1[(SharedEdge1+1)%3]);

		const int VIDA = Tri0[(SharedEdge0+2)%3];
		const int VIDB = Tri1[(SharedEdge1+2)%3];

		FVector3d V0 = Mesh.GetVertex(VID0);
		FVector3d V1 = Mesh.GetVertex(VID1);
		FVector3d VA = Mesh.GetVertex(VIDA);
		FVector3d VB = Mesh.GetVertex(VIDB);

		if (bHasDisplacements)
		{
			V0 += Displacements[VID0];
			V1 += Displacements[VID1];
			VA += Displacements[VIDA];
			VB += Displacements[VIDB];
		}

		const FVector3d EA0 = V0 - VA;
		const FVector3d EA1 = V1 - VA;
		const FVector3d EB0 = V0 - VB;
		const FVector3d EB1 = V1 - VB;
		const FVector3d E01 = V1 - V0;

		const double CotanA = Geometry::VectorUtil::VectorCot(EA0, EA1);
		const double CotanB = Geometry::VectorUtil::VectorCot(EB0, EB1);

		const double LaplacianWeight  = 0.5 * (CotanA + CotanB);

		// unflipped
		FVector3d N0 = Normalized(EA1 ^ EA0);
		FVector3d N1 = Normalized(EB0 ^ EB1);
		FVector3d E  = Normalized(E01);

		// flipped
		FVector3d N0f = Normalized(EA0 ^ EB0);
		FVector3d N1f = Normalized(EB1 ^ EA1);
		FVector3d Ef  = Normalized(VA - VB); 

		const double OrientedDihedral  = Geometry::VectorUtil::OrientedDihedralAngle(N0 , N1 , E );
		const double OrientedDihedralf = Geometry::VectorUtil::OrientedDihedralAngle(N0f, N1f, Ef);

		if (LaplacianWeight > 0.)
		{
			return { -LaplacianWeight, false };
		}

		const bool bNeedsSplit = (FMath::Abs(OrientedDihedralf - OrientedDihedral) > FlipAngleThreshold);

		return { -LaplacianWeight, bNeedsSplit };
	};

	struct FEntry
	{
		double Weight;
		bool bNeedsSplit { false };
		int32 EID;

		bool operator<(const FEntry& Other) const
		{
			return Weight < Other.Weight;
		}
	};

	struct FSwapPolicy
	{
		FSwapPolicy(TArray<int32>& InHeapIndices)
			: HeapIndices(InHeapIndices)
		{
		}

		void Swap(const int32 HeapIdxA, const int32 HeapIdxB, const FEntry& EntryA, const FEntry& EntryB)
		{
			check(HeapIndices[EntryA.EID] == HeapIdxA);
			check(HeapIndices[EntryB.EID] == HeapIdxB);

			::Swap(HeapIndices[EntryA.EID], HeapIndices[EntryB.EID]);
		}

		TArray<int32>& HeapIndices;
	};

	TArray<int32> HeapIndices;
	FSwapPolicy SwapPolicy(HeapIndices);
	THeapWithTracking<FEntry, FSwapPolicy> Heap(SwapPolicy);

	for (int EID : Mesh.EdgeIndicesItr())
	{
		const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);
		if (Edge.Tri[1] == IndexConstants::InvalidID)
		{
			HeapIndices.Add(IndexConstants::InvalidID);
		}
		else
		{
			double Weight;
			bool bNeedsSplit;
			std::tie(Weight, bNeedsSplit) = ComputeWeight(EID);

			if (Weight > 0)
			{
				const int32 Idx = Heap.Num();
				HeapIndices.Add(Idx);
				Heap.Add({ -Weight, bNeedsSplit, EID });
			}
			else
			{
				HeapIndices.Add(IndexConstants::InvalidID);
			}
		}
	}

	check(HeapIndices.Num() == Mesh.MaxEdgeID());

	while(!Heap.IsEmpty())
	{
		const FEntry Entry = Heap.Pop();
		HeapIndices[Entry.EID] = IndexConstants::InvalidID;

		if (Entry.Weight < 0.)
		{
			int EdgesToUpdate[4];
			{
				const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(Entry.EID);
				const FIndex3i Tri0 = Mesh.GetTriangle(Edge.Tri[0]);
				const FIndex3i Tri1 = Mesh.GetTriangle(Edge.Tri[1]);

				const int SharedEdge0 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri0);
				const int SharedEdge1 = IndexUtil::FindEdgeIndexInTri(Edge.Vert[0], Edge.Vert[1], Tri1);

				EdgesToUpdate[0] = Mesh.GetTriEdge(Edge.Tri[0], (SharedEdge0+1)%3);
				EdgesToUpdate[1] = Mesh.GetTriEdge(Edge.Tri[0], (SharedEdge0+2)%3);
				EdgesToUpdate[2] = Mesh.GetTriEdge(Edge.Tri[1], (SharedEdge1+1)%3);
				EdgesToUpdate[3] = Mesh.GetTriEdge(Edge.Tri[1], (SharedEdge1+2)%3);
			}

			if (Entry.bNeedsSplit)
			{
				FDynamicMesh3::FEdgeSplitInfo SplitInfo;
				EMeshResult Result = Mesh.SplitEdge(Entry.EID, SplitInfo);

				if (Result == EMeshResult::Ok)
				{
					if (bHasDisplacements)
					{
						const FVector3d SplitVertexDisplacement = DisplacementFunctor(SplitInfo.NewVertex);
						if (SplitInfo.NewVertex == Displacements.Num())
						{
							Displacements.Add(SplitVertexDisplacement);
						}
						else
						{
							Displacements.SetNum(FMath::Max(Displacements.Num(), SplitInfo.NewVertex+1));
							Displacements[SplitInfo.NewVertex] = SplitVertexDisplacement;
						}
					}

					HeapIndices[Entry.EID] = IndexConstants::InvalidID;

					for (int32 I = 0; I < 3; ++I)
					{
						const int32 EID = SplitInfo.NewEdges[I];
						HeapIndices.Add(IndexConstants::InvalidID);
					}

					for (const int EID : EdgesToUpdate)
					{
						check(EID < Mesh.MaxEdgeID());

						if (EID == IndexConstants::InvalidID || HeapIndices[EID] == IndexConstants::InvalidID)
						{
							continue;
						}

						const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);

						if (Edge.Tri[1] != IndexConstants::InvalidID)
						{
							double Weight;
							bool bNeedsSplit;
							std::tie(Weight, bNeedsSplit) = ComputeWeight(EID);

							Heap.Update(HeapIndices[EID], { Weight, bNeedsSplit, EID });
						}
					}
				}
			}
			else
			{
				// Flip Only
				FDynamicMesh3::FEdgeFlipInfo EdgeFlipInfo;
				const EMeshResult Result = Mesh.FlipEdge(Entry.EID, EdgeFlipInfo);

				if (Result == EMeshResult::Ok)
				{
					for (const int EID : EdgesToUpdate)
					{
						check(EID < Mesh.MaxEdgeID());

						if (HeapIndices[EID] == IndexConstants::InvalidID)
						{
							continue;
						}

						const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);

						if (Edge.Tri[1] != IndexConstants::InvalidID)
						{
							double Weight;
							bool bNeedsSplit;
							std::tie(Weight, bNeedsSplit) = ComputeWeight(EID);
							
							Heap.Update(HeapIndices[EID], { Weight, bNeedsSplit, EID });
						}
					}
				}
			}
		}
	}
}

} // namespace Geometry
} // namespace UE