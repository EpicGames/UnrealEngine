// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TestHarnessAdapter.h"
#include "AutoRTFM.h"
#include "CompGeom/ConvexHull3.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"

#if WITH_DEV_AUTOMATION_TESTS

TEST_CASE_NAMED(FAutoRTFMConvexHullTests, "AutoRTFM.TConvexHull3", "[EngineFilter][ClientContext][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	using Vec3 = UE::Math::TVector<float>;

	static const Vec3 Pyramid[] = 
	{
		{ 0.f,  0.f,  0.f},
		{20.f,  0.f,  0.f},
		{ 0.f, 20.f,  0.f},
		{20.f, 20.f,  0.f},
		{10.f, 10.f,  3.f},
	};

	UE::Geometry::TConvexHull3<float> Hull;
	double Volume = Hull.ComputeVolume(Pyramid);

	TestTrueExpr(FMath::IsNearlyEqual(Volume, 400.)); 
}

#endif // WITH_DEV_AUTOMATION_TESTS
