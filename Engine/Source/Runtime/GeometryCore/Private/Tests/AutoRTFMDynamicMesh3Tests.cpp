// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AutoRTFM.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "OrientedBoxTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Generators/GridBoxMeshGenerator.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMDynamicMesh3Test, "AutoRTFM + FDynamicMesh3", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMDynamicMesh3Test::RunTest(const FString& Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMDynamicMesh3Test' test. AutoRTFM disabled.")));
		return true;
	}

	// Setup code is adapted from TEST_CASE("Remove Vertices") in DynamicMeshTests.cpp.
	UE::Geometry::FGridBoxMeshGenerator Gen;
	Gen.Box = UE::Geometry::FOrientedBox3d{ FVector3d{ 0,0,0 }, FVector3d{ 1.0,1.0,1.0 } };
	Gen.EdgeVertices = UE::Geometry::FIndex3i{ 3, 3, 3 };
	UE::Geometry::FDynamicMesh3 Mesh(&Gen.Generate());

	Mesh.SetShapeChangeStampEnabled(true);
	Mesh.SetTopologyChangeStampEnabled(true);

	bool bSuccess = true;
	uint32 StartingShapeChangeStamp = Mesh.GetShapeChangeStamp();
	uint32 StartingTopologyChangeStamp = Mesh.GetTopologyChangeStamp();
	constexpr int VertexID = 10;  // any vertex ID will do

	// Transaction abort after `UpdateChangeStamps(true, false)` restores the starting ShapeChangeStamp.
	AutoRTFM::Transact([&]
	{
		Mesh.UpdateChangeStamps(true, false);
		check(Mesh.GetShapeChangeStamp()    >  StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);
		AutoRTFM::AbortTransaction();
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    == StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);

	// Transaction abort after `UpdateChangeStamps(true, true)` restores the starting ShapeChangeStamp and TopologyChangeStamp.
	AutoRTFM::Transact([&]
	{
		Mesh.UpdateChangeStamps(true, true);
		check(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);
		AutoRTFM::AbortTransaction();
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    == StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);

	// Transaction abort after `RemoveVertex` restores the starting ShapeChangeStamp and TopologyChangeStamp.
	AutoRTFM::Transact([&]
	{
		check(Mesh.RemoveVertex(VertexID, /*bPreserveManifold=*/ false) == UE::Geometry::EMeshResult::Ok);
		check(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);
		AutoRTFM::AbortTransaction();
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    == StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);

	// Transaction commit after `UpdateChangeStamps(true, false)` keeps the new ShapeChangeStamp.
	AutoRTFM::Transact([&]
	{
		Mesh.UpdateChangeStamps(true, false);
		check(Mesh.GetShapeChangeStamp()    >  StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    >  StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() == StartingTopologyChangeStamp);

	// Transaction commit after `UpdateChangeStamps(true, true)` keeps the new ShapeChangeStamp and TopologyChangeStamp.
	StartingShapeChangeStamp = Mesh.GetShapeChangeStamp();
	StartingTopologyChangeStamp = Mesh.GetTopologyChangeStamp();

	AutoRTFM::Transact([&]
	{
		Mesh.UpdateChangeStamps(true, true);
		check(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);

	// Transaction commit after `RemoveVertex` keeps the new ShapeChangeStamp and TopologyChangeStamp.
	StartingShapeChangeStamp = Mesh.GetShapeChangeStamp();
	StartingTopologyChangeStamp = Mesh.GetTopologyChangeStamp();

	AutoRTFM::Transact([&]
	{
		check(Mesh.RemoveVertex(VertexID, /*bPreserveManifold=*/ false) == UE::Geometry::EMeshResult::Ok);
		check(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
		check(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);
	});

	bSuccess &= TestTrueExpr(Mesh.GetShapeChangeStamp()    > StartingShapeChangeStamp);
	bSuccess &= TestTrueExpr(Mesh.GetTopologyChangeStamp() > StartingTopologyChangeStamp);

	return bSuccess;
}

#endif //WITH_DEV_AUTOMATION_TESTS

