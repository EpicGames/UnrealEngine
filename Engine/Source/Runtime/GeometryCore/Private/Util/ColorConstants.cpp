// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ColorConstants.h"

FColor UE::Geometry::LinearColors::SelectFColor(int32 Index)
{
	static const FColor ColorMap[] = {
		SpringGreen3b(), Plum3b(), Khaki3b(),
		PaleGreen3b(), LightSteelBlue3b(), Aquamarine3b(),
		Salmon3b(), Goldenrod3b(), LightSeaGreen3b(),
		IndianRed3b(), DarkSalmon3b(), Coral3b(),
		Burlywood3b(), GreenYellow3b(), Lavender3b(),
		MediumAquamarine3b(), Thistle3b(), Wheat3b(),
		LightSkyBlue3b(), LightPink3b(), MediumSpringGreen3b()
	};
	return (Index <= 0) ? White3b() : ColorMap[(Index-1) % (7*3)];
}