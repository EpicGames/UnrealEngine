// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

void FProgressCancel::StartWorkScope(FProgressData& SaveProgressFrameOut, float StepSize, const FText& Message)
{
	SaveProgressFrameOut = Progress;
	Progress.ScopeDepth++;
	float SavedMax = Progress.CurrentMax;
	float SavedRange = Progress.Range();
	// Note: New min/max are clamped to the saved max to prevent going outside the active scope
	// The clamp on CurrentMin should do nothing as the ProgressFraction should already be prevented from exceeding the range max
	Progress.CurrentMin = FMathf::Min(ProgressFraction, SavedMax);
	Progress.CurrentMax = FMathf::Min(ProgressFraction + SavedRange * StepSize, SavedMax);
	if (!Message.IsEmpty()) // if message is empty, let any parent-scope message persist to this scope
	{
		SetProgressMessage(Message);
	}
}

void FProgressCancel::EndWorkScope(const FProgressData& SavedProgressFrame)
{
	if (!ensureMsgf(Progress.ScopeDepth == SavedProgressFrame.ScopeDepth + 1,
		TEXT("ScopeDepth (%d) not 1 less than saved ScopeDepth (%d): Likely incorrect use of FProgressScope"),
		Progress.ScopeDepth, SavedProgressFrame.ScopeDepth))
	{
		if (SavedProgressFrame.ScopeDepth + 1 > Progress.ScopeDepth || Progress.CurrentMax > SavedProgressFrame.CurrentMax)
		{
			return; // ignore invalid scope rather than potentially allow the active range to go backward
		}
	}

	ProgressFraction = Progress.CurrentMax;
	{
		bool bWasEmpty = Progress.Message.IsEmpty();
		Progress = SavedProgressFrame;
		if (!bWasEmpty || !Progress.Message.IsEmpty())
		{
			FScopeLock MessageLock(&MessageCS);
			ProgressMessage = Progress.Message;
		}
	}
}

FProgressCancel::FProgressScope::FProgressScope(FProgressCancel* ProgressCancel, float ProgressAmount, const FText& Message) : ProgressCancel(ProgressCancel)
{
	if (ProgressCancel)
	{
		ProgressCancel->StartWorkScope(SavedProgressData, ProgressAmount, Message);
	}
}

FProgressCancel::FProgressScope FProgressCancel::CreateScopeTo(FProgressCancel* ProgressCancel, float ProgressTo, const FText& Message)
{
	if (ProgressCancel)
	{
		float Range = ProgressCancel->Progress.Range();
		float ProgressBy = 0;
		if (Range > 0)
		{
			float TargetProgress = FMathf::Max(ProgressCancel->ProgressFraction, ProgressCancel->Progress.CurrentMin + Range * ProgressTo);
			ProgressBy = (TargetProgress - ProgressCancel->ProgressFraction) / Range;
		}
		return FProgressScope(ProgressCancel, ProgressBy, Message);
	}
	return FProgressScope();
}



void FProgressCancel::FProgressScope::Done()
{
	if (ensure(!bEnded) && ProgressCancel)
	{
		ProgressCancel->EndWorkScope(SavedProgressData);
	}
	bEnded = true;
}
