// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/RevolveUtil.h"

using namespace UE::Geometry;

void RevolveUtil::GenerateSweepCurve(const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection, 
	double DegreesOffset, double DegreesPerStep, double DownAxisOffset, int TotalNumFrames, TArray<FFrame3d> &SweepCurveOut)
{
	// For a revolve, we need to sweep along a circular path around the axis of rotation. While we could pick
	// any arbitrary frame to create the curve, we choose the standard world frame since that is what the 
	// profile curve is specified in.
	FVector3d WorldOriginWrtAxisOrigin = -RevolutionAxisOrigin;
	for (int i = 0; i < TotalNumFrames; ++i)
	{
		// Revolve the origin frame relative the axis
		FQuaterniond Rotation(RevolutionAxisDirection, DegreesOffset + DegreesPerStep * i, true);
		FVector3d NewOrigin = Rotation * (WorldOriginWrtAxisOrigin + i * DownAxisOffset * RevolutionAxisDirection) + RevolutionAxisOrigin;
		FFrame3d NewFrame(NewOrigin, Rotation);
		SweepCurveOut.Add(NewFrame);
	}
}

void RevolveUtil::WeldPointsOnAxis(TArray<FVector3d>& ProfileCurve, const FVector3d& RevolutionAxisOrigin,
	const FVector3d& RevolutionAxisDirection, double Tolerance, TSet<int32>& ProfileVerticesToWeldOut)
{
	double ToleranceSquared = Tolerance * Tolerance;
	for (int32 ProfileIndex = 0; ProfileIndex < ProfileCurve.Num(); ++ProfileIndex)
	{
		FVector3d VectorToPoint = ProfileCurve[ProfileIndex] - RevolutionAxisOrigin;
		FVector3d VectorProjectedToAxis = RevolutionAxisDirection.Dot(VectorToPoint) * RevolutionAxisDirection;

		double dist = DistanceSquared(VectorToPoint, VectorProjectedToAxis);
		if (dist < ToleranceSquared)
		{
			// Move the point directly onto the axis and mark it as welded
			ProfileCurve[ProfileIndex] = VectorProjectedToAxis + RevolutionAxisOrigin;
			ProfileVerticesToWeldOut.Add(ProfileIndex);
		}
	}
}

bool RevolveUtil::ProfileIsCCWRelativeRevolve(TArray<FVector3d>& ProfileCurve, const FVector3d& RevolutionAxisOrigin,
	const FVector3d& RevolutionAxisDirection, bool bProfileCurveIsClosed)
{
	if (ProfileCurve.Num() < 2)
	{
		return true;
	}

	// We're going to imagine the profile curve as given in cylindrical coordinates with respect to the revolution axis
	// and then project everything down to angle 0. In other words, we use the distance from the axis and height along
	// the axis to get all the points of the profile curve rotatated into a half plane to the side of the axis, and then
	// check that the profile curve is clockwise in this half plane (this corresponds to being counterclockwise relative
	// to the rotation direction, which is out of the plane here).
	// To check for clockwise orientation, we look at the lowest point along the axis, which will be on the convex hull
	// and should be representative of the orientation. This can fail if there are self intersections, or in a "lollipop"
	// case where the lower edges are coincident.

	// Find the point lowest along the revolution axis
	double MinAlongAxis = (ProfileCurve[0] - RevolutionAxisOrigin).Dot(RevolutionAxisDirection);
	int32 MinAlongAxisIndex = 0;
	for (int32 ProfileIndex = 1; ProfileIndex < ProfileCurve.Num(); ++ProfileIndex)
	{
		double AlongAxis = (ProfileCurve[ProfileIndex] - RevolutionAxisOrigin).Dot(RevolutionAxisDirection);
		if (AlongAxis < MinAlongAxis)
		{
			MinAlongAxis = AlongAxis;
			MinAlongAxisIndex = ProfileIndex;
		}
	}

	// If the curve is not closed, we'll be doing the same check, but we may not be able to do so if the lowest
	// point was an endpoint. In such a case, we want the curve to be going down.
	if (!bProfileCurveIsClosed)
	{
		if (MinAlongAxisIndex == 0)
		{
			return false;
		}
		else if (MinAlongAxisIndex == ProfileCurve.Num() - 1)
		{
			return true;
		}
		//otherwise do the regular check
	}

	// Check that direction is clockwise by looking at the minimal point and its neighbors in the 2D 
	// coordinate space of "(distance from axis, height along axis)"
	double PointDistance = Distance( (ProfileCurve[MinAlongAxisIndex] - RevolutionAxisOrigin), (MinAlongAxis * RevolutionAxisDirection) );
	FVector2d Point(PointDistance, MinAlongAxis);

	int32 PreviousIndex = (MinAlongAxisIndex + ProfileCurve.Num() - 1) % ProfileCurve.Num();
	FVector3d PreviousWrtAxis = ProfileCurve[PreviousIndex] - RevolutionAxisOrigin;
	double PreviousAlongAxis = PreviousWrtAxis.Dot(RevolutionAxisDirection);
	double PreviousDistance = Distance(PreviousWrtAxis, PreviousAlongAxis * RevolutionAxisDirection);
	FVector2d Previous(PreviousDistance, PreviousAlongAxis);

	int32 NextIndex = (MinAlongAxisIndex + 1) % ProfileCurve.Num();
	FVector3d NextWrtAxis = ProfileCurve[NextIndex] - RevolutionAxisOrigin;
	double NextAlongAxis = NextWrtAxis.Dot(RevolutionAxisDirection);
	double NextDistance = Distance(NextWrtAxis, NextAlongAxis * RevolutionAxisDirection);
	FVector2d Next(NextDistance, NextAlongAxis);
	
	return DotPerp((Previous - Point), (Next - Point)) > 0;
}

void RevolveUtil::MakeProfileCurveMidpointOfFirstStep(TArray<FVector3d>& ProfileCurve, double DegreesPerStep,
	const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection)
{
	double AdjustmentScale = tan(TMathUtil<double>::DegToRad * DegreesPerStep / 2);
	for (FVector3d& Point : ProfileCurve)
	{
		FVector3d PointWrtAxisOrigin = Point - RevolutionAxisOrigin;
		FVector3d AdjustmentDirection = -Normalized(RevolutionAxisDirection.Cross(PointWrtAxisOrigin));
		double DistToAxis = Distance(PointWrtAxisOrigin, PointWrtAxisOrigin.Dot(RevolutionAxisDirection)*RevolutionAxisDirection);
		
		Point += AdjustmentScale * DistToAxis * AdjustmentDirection;
	}
}

