// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/SmallListSet.h"

#include <UObject/UE5MainStreamObjectVersion.h>

using namespace UE::Geometry;

void FSmallListSet::Resize(int32 NewSize)
{
	int32 CurSize = (int32)ListHeads.GetLength();
	if (NewSize > CurSize)
	{
		ListHeads.Resize(NewSize);
		for (int32 k = CurSize; k < NewSize; ++k)
		{
			ListHeads[k] = NullValue;
		}
	}
}

void FSmallListSet::ResizeAndAllocateBlocks(int32 NewSize)
{
	Reset();
	Resize(NewSize);
	ListBlocks.Resize(NewSize * (BLOCK_LIST_OFFSET+1));
	for (int32 i(0); i<NewSize; ++i)
	{
		int32 ListHead = i * (BLOCK_LIST_OFFSET+1);
		ListHeads[i] = ListHead;
		ListBlocks[ListHead] = 0;
		ListBlocks[ListHead + BLOCK_LIST_OFFSET] = NullValue;
	}

	for (int32 i(0); i < NewSize; ++i)
	{
		checkSlow(ListBlocks[ListHeads[i]] == 0);
		checkSlow(ListBlocks[ListHeads[i] + BLOCK_LIST_OFFSET] == NullValue);
	}

	AllocatedCount = NewSize;
}

void FSmallListSet::AllocateAt(int32 ListIndex)
{
	checkSlow(ListIndex >= 0);
	if (ListIndex >= (int)ListHeads.GetLength())
	{
		int32 j = (int32)ListHeads.GetLength();
		ListHeads.InsertAt(NullValue, ListIndex);
		// need to set intermediate values to null! 
		while (j < ListIndex)
		{
			ListHeads[j] = NullValue;
			j++;
		}
	}
	else
	{
		checkf(ListHeads[ListIndex] == NullValue, TEXT("FSmallListSet: list at %d is not empty!"), ListIndex);
	}
}

void FSmallListSet::Compact(int32 MaxListIndex)
{
	checkSlow(MaxListIndex >= 0);
	int32 CurSize = (int32)ListHeads.GetLength();
	if (MaxListIndex < CurSize)
	{
		// We just resize w/out book-keeping what we cleared, since we rebuild the blocks/etc below
		ListHeads.Resize(MaxListIndex);
	}

	AllocatedCount = 0;
	TDynamicVector<int32> NewBlocks{};
	TDynamicVector<int32> NewLinkedListElements{};
	for (int32 Idx = 0, Num = (int32)ListHeads.GetLength(), CurBlockIdx = 0; Idx < Num; ++Idx, CurBlockIdx += BLOCK_LIST_OFFSET + 1)
	{
		int32 OrigHead = ListHeads[Idx];
		if (OrigHead == NullValue)
		{
			continue;
		}
		AllocatedCount++;
		ListHeads[Idx] = CurBlockIdx;
		NewBlocks.InsertAt(NullValue, CurBlockIdx + BLOCK_LIST_OFFSET);
		for (int32 SubIdx = 0; SubIdx < BLOCK_LIST_OFFSET; ++SubIdx)
		{
			NewBlocks[CurBlockIdx + SubIdx] = ListBlocks[OrigHead + SubIdx];
		}
		
		int32 OrigLinkStart = ListBlocks[OrigHead + BLOCK_LIST_OFFSET];
		if (OrigLinkStart == NullValue)
		{
			NewBlocks[CurBlockIdx + BLOCK_LIST_OFFSET] = NullValue;
		}
		else
		{
			int32 CurPtr = OrigLinkStart;
			NewBlocks[CurBlockIdx + BLOCK_LIST_OFFSET] = NewLinkedListElements.GetLength();
			while (true)
			{
				NewLinkedListElements.Add(LinkedListElements[CurPtr]);
				CurPtr = LinkedListElements[CurPtr + 1];
				if (CurPtr == NullValue)
				{
					break;
				}
				NewLinkedListElements.Add(NewLinkedListElements.GetLength() + 1);
			}
			NewLinkedListElements.Add(NullValue);
		}
	}

	ListBlocks = MoveTemp(NewBlocks);
	LinkedListElements = MoveTemp(NewLinkedListElements);
	FreeHeadIndex = NullValue;
	FreeBlocks.Clear();
}


void FSmallListSet::AppendWithElementOffset(const FSmallListSet& Other, int32 ElementOffset)
{
	int32 OrigListBlocksNum = ListBlocks.Num();
	int32 OrigListHeadsNum = ListHeads.Num();
	int32 OrigLinkedListElementsNum = LinkedListElements.Num();
	int32 OrigFreeHeadIndex = FreeHeadIndex;

	// Append ListHeads indices
	ListHeads.Add(Other.ListHeads);
	for (int32 Idx = OrigListHeadsNum, N = ListHeads.Num(); Idx < N; ++Idx)
	{
		// Offset appended non-null indices to point to appended ListBlock indices
		if (ListHeads[Idx] != NullValue)
		{
			ListHeads[Idx] += (int32)OrigListBlocksNum;
		}
	}

	// Append LinkedListElements entries
	LinkedListElements.Add(Other.LinkedListElements);
	for (int32 Idx = OrigLinkedListElementsNum, N = LinkedListElements.Num(); Idx < N; Idx += 2)
	{
		// Offset the element data
		LinkedListElements[Idx] += ElementOffset;
		// Offset appended non-null pointer indices to refer to appended indices
		int32& Link = LinkedListElements[Idx + 1];
		if (Link != NullValue)
		{
			Link += OrigLinkedListElementsNum;
		}
	}

	// Append ListBlocks entries
	ListBlocks.Add(Other.ListBlocks);
	for (int32 Idx = OrigListBlocksNum, N = ListBlocks.Num(); Idx < N; Idx += BLOCKSIZE + 2)
	{
		int32 BlockNumEls = ListBlocks[Idx];
		for (int32 SubIdx = 0, SubIdxNum = FMath::Min(BLOCKSIZE, BlockNumEls); SubIdx < SubIdxNum; ++SubIdx)
		{
			ListBlocks[Idx + 1 + SubIdx] += ElementOffset;
		}
		if (ListBlocks[Idx + BLOCKSIZE + 1] != NullValue)
		{
			ListBlocks[Idx + BLOCKSIZE + 1] += OrigLinkedListElementsNum;
		}
	}

	// If we had a non-empty free list on Other, need to transfer it too
	if (Other.FreeHeadIndex != NullValue)
	{
		FreeHeadIndex = Other.FreeHeadIndex + OrigLinkedListElementsNum;
		// If both were non-empty, need to walk Other's free list to attach its tail to our original head
		if (OrigFreeHeadIndex != NullValue)
		{
			int32 WalkListIndex = FreeHeadIndex;
			while (true)
			{
				int32 NextIndex = LinkedListElements[WalkListIndex + 1];
				if (NextIndex == NullValue)
				{
					break;
				}
				WalkListIndex = NextIndex;
			}
			checkSlow(LinkedListElements[WalkListIndex + 1] == NullValue);
			LinkedListElements[WalkListIndex + 1] = OrigFreeHeadIndex;
		}
	}

	AllocatedCount += Other.AllocatedCount;
}


void FSmallListSet::Insert(int32 ListIndex, int32 Value)
{
	checkSlow(0 <= ListIndex && ListIndex < ListHeads.Num());
	int32 block_ptr = ListHeads[ListIndex];
	if (block_ptr == NullValue)
	{
		block_ptr = AllocateBlock();
		ListBlocks[block_ptr] = 0;
		ListHeads[ListIndex] = block_ptr;
	}

	int32 N = ListBlocks[block_ptr];
	if (N < BLOCKSIZE) 
	{
		ListBlocks[block_ptr + N + 1] = Value;
	}
	else 
	{
		// spill to linked list
		int32 cur_head = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];

		if (FreeHeadIndex == NullValue)
		{
			// allocate linkedlist node
			int32 new_ptr = (int32)LinkedListElements.GetLength();
			LinkedListElements.Add(Value);
			LinkedListElements.Add(cur_head);
			ListBlocks[block_ptr + BLOCK_LIST_OFFSET] = new_ptr;
		}
		else 
		{
			// pull from free list
			int32 free_ptr = FreeHeadIndex;
			FreeHeadIndex = LinkedListElements[free_ptr + 1];
			LinkedListElements[free_ptr] = Value;
			LinkedListElements[free_ptr + 1] = cur_head;
			ListBlocks[block_ptr + BLOCK_LIST_OFFSET] = free_ptr;
		}
	}

	// count element
	ListBlocks[block_ptr] += 1;
}




bool FSmallListSet::Remove(int32 ListIndex, int32 Value)
{
	checkSlow(ListIndex >= 0);
	int32 block_ptr = ListHeads[ListIndex];
	int32 N = ListBlocks[block_ptr];


	int32 iEnd = block_ptr + FMath::Min(N, BLOCKSIZE);
	for (int32 i = block_ptr + 1; i <= iEnd; ++i) 
	{

		if (ListBlocks[i] == Value) 
		{
			// TODO since this is a set and order doesn't matter, shouldn't you just move the last thing 
			// to the empty spot rather than shifting the whole list left?
			for (int32 j = i + 1; j <= iEnd; ++j)     // shift left
			{
				ListBlocks[j - 1] = ListBlocks[j];
			}
			//ListBlocks[iEnd] = -2;     // OPTIONAL

			if (N > BLOCKSIZE) 
			{
				int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
				ListBlocks[block_ptr + BLOCK_LIST_OFFSET] = LinkedListElements[cur_ptr + 1];  // point32 to cur->next
				ListBlocks[iEnd] = LinkedListElements[cur_ptr];
				AddFreeLink(cur_ptr);
			}

			ListBlocks[block_ptr] -= 1;
			return true;
		}

	}

	// search list
	if (N > BLOCKSIZE) 
	{
		if (RemoveFromLinkedList(block_ptr, Value)) 
		{
			ListBlocks[block_ptr] -= 1;
			return true;
		}
	}

	return false;
}




void FSmallListSet::Move(int32 FromIndex, int32 ToIndex)
{
	checkSlow(FromIndex >= 0);
	checkSlow(ToIndex >= 0);
	checkSlow(ListHeads[ToIndex] == NullValue);
	ListHeads[ToIndex] = ListHeads[FromIndex];
	ListHeads[FromIndex] = NullValue;
}




void FSmallListSet::Clear(int32 ListIndex)
{
	checkSlow(ListIndex >= 0);
	int32 block_ptr = ListHeads[ListIndex];
	if (block_ptr != NullValue)
	{
		int32 N = ListBlocks[block_ptr];

		// if we have spilled to linked-list, free nodes
		if (N > BLOCKSIZE) 
		{
			int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
			while (cur_ptr != NullValue)
			{
				int32 free_ptr = cur_ptr;
				cur_ptr = LinkedListElements[cur_ptr + 1];
				AddFreeLink(free_ptr);
			}
			ListBlocks[block_ptr + BLOCK_LIST_OFFSET] = NullValue;
		}

		// free our block
		ListBlocks[block_ptr] = 0;
		FreeBlocks.Add(block_ptr);
		ListHeads[ListIndex] = NullValue;
	}
}




bool FSmallListSet::Contains(int32 ListIndex, int32 Value) const
{
	checkSlow(ListIndex >= 0);
	int32 block_ptr = ListHeads[ListIndex];
	if (block_ptr != NullValue)
	{
		int32 N = ListBlocks[block_ptr];
		if (N < BLOCKSIZE) 
		{
			int32 iEnd = block_ptr + N;
			for (int32 i = block_ptr + 1; i <= iEnd; ++i) 
			{
				if (ListBlocks[i] == Value)
				{
					return true;
				}
			}
		}
		else 
		{
			// we spilled to linked list, have to iterate through it as well
			int32 iEnd = block_ptr + BLOCKSIZE;
			for (int32 i = block_ptr + 1; i <= iEnd; ++i) 
			{
				if (ListBlocks[i] == Value)
				{
					return true;
				}
			}
			int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
			while (cur_ptr != NullValue)
			{
				if (LinkedListElements[cur_ptr] == Value)
				{
					return true;
				}
				cur_ptr = LinkedListElements[cur_ptr + 1];
			}
		}
	}
	return false;
}




















bool FSmallListSet::EnumerateEarlyOut(int32 ListIndex, TFunctionRef<bool(int32)> ApplyFunc) const
{
	int32 block_ptr = ListHeads[ListIndex];
	if (block_ptr != NullValue)
	{
		int32 N = ListBlocks[block_ptr];
		if (N < BLOCKSIZE)
		{
			int32 iEnd = block_ptr + N;
			for (int32 i = block_ptr + 1; i <= iEnd; ++i)
			{
				if (!ApplyFunc(ListBlocks[i]))
				{
					return false;
				}
			}
		}
		else
		{
			// we spilled to linked list, have to iterate through it as well
			int32 iEnd = block_ptr + BLOCKSIZE;
			for (int32 i = block_ptr + 1; i <= iEnd; ++i)
			{
				if (!ApplyFunc(ListBlocks[i]))
				{
					return false;
				}
			}
			int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
			while (cur_ptr != NullValue)
			{
				if (!ApplyFunc(LinkedListElements[cur_ptr]))
				{
					return false;
				}
				cur_ptr = LinkedListElements[cur_ptr + 1];
			}
		}
	}
	return true;
}


int32 FSmallListSet::AllocateBlock()
{
	int32 nfree = (int32)FreeBlocks.GetLength();
	if (nfree > 0)
	{
		int32 ptr = FreeBlocks[nfree - 1];
		FreeBlocks.PopBack();
		return ptr;
	}
	int32 nsize = (int32)ListBlocks.GetLength();
	ListBlocks.InsertAt(NullValue, nsize + BLOCK_LIST_OFFSET);
	ListBlocks[nsize] = 0;
	AllocatedCount++;
	return nsize;
}



bool FSmallListSet::RemoveFromLinkedList(int32 block_ptr, int32 val)
{
	int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
	int32 prev_ptr = NullValue;
	while (cur_ptr != NullValue)
	{
		if (LinkedListElements[cur_ptr] == val)
		{
			int32 next_ptr = LinkedListElements[cur_ptr + 1];
			if (prev_ptr == NullValue)
			{
				ListBlocks[block_ptr + BLOCK_LIST_OFFSET] = next_ptr;
			}
			else
			{
				LinkedListElements[prev_ptr + 1] = next_ptr;
			}
			AddFreeLink(cur_ptr);
			return true;
		}
		prev_ptr = cur_ptr;
		cur_ptr = LinkedListElements[cur_ptr + 1];
	}
	return false;
}

void FSmallListSet::Serialize(FArchive& Ar, bool bCompactData, bool bUseCompression)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
	{
		Ar << ListHeads;
		Ar << ListBlocks;
		Ar << FreeBlocks;
		Ar << AllocatedCount;
		Ar << LinkedListElements;
		Ar << FreeHeadIndex;
	}
	else
	{
		Ar << bCompactData;
		Ar << bUseCompression;

		auto SerializeVector = [](FArchive& Ar, auto& Vector, bool bUseCompression)
		{
			if (bUseCompression)
			{
				Vector.template Serialize<true, true>(Ar);
			}
			else
			{
				Vector.template Serialize<true, false>(Ar);
			}
		};

		// Compact the data into a flat buffer if either bCompactData or bUseCompression is enabled.
		// Considering the significant time overhead for compression, it makes sense to just compact the data as well even though it is not requested.
		if (bCompactData || bUseCompression)
		{
			// We are compacting the data by flattening the lists into one tightly packed buffer.
			// The first value in the buffer is the number of allocated lists even if they are empty. After that each list consists of the number of values
			// followed by the actual values.
			// Note that due to values beyond the BLOCKSIZE are inserted into the linked list, we reverse the order for these values then loading in the data to
			// restore the original order.

			// Temporary buffer to flatten all lists during save or restore them during load.
			TDynamicVector<int32> Buffer;

			if (Ar.IsLoading())
			{
				Reset();

				SerializeVector(Ar, Buffer, bUseCompression);

				// Read a value from the buffer while making sure there are no buffer overruns due to corrupted data.
				const size_t BufferNum = Buffer.Num();
				auto GetBufferValue = [&Buffer, &BufferNum](const size_t Index) -> int32
				{
					return Index < BufferNum ? Buffer[Index] : 0;
				};
				
				size_t BufferIndex = 0;
				const int32 ListCount = GetBufferValue(BufferIndex++);
				Resize(ListCount);

				for (int32 ListIndex = 0; ListIndex < ListCount; ++ListIndex)
				{
					const int32 ListValueCount = GetBufferValue(BufferIndex++);
					if (ListValueCount > 0)
					{
						AllocateAt(ListIndex);

						// The first BLOCKSIZE values get inserted in order.
						const int32 ListValueCountInBlock = FMath::Min(BLOCKSIZE, ListValueCount);
						for (int32 ListValueIndex = 0; ListValueIndex < ListValueCountInBlock; ++ListValueIndex)
						{
							Insert(ListIndex, GetBufferValue(BufferIndex + ListValueIndex));
						}

						// Any values beyond BLOCKSIZE get inserted in reversed order. This is due to the fact that any values inserted into the linked list
						// effectively get reversed in order, and we want to restore the original order before serializing out the data.
						for (int32 ListValueIndex = ListValueCount - 1; ListValueIndex >= BLOCKSIZE; --ListValueIndex)
						{
							Insert(ListIndex, GetBufferValue(BufferIndex + ListValueIndex));
						}

						BufferIndex += ListValueCount;
					}
				}

				if (BufferIndex > BufferNum)
				{
					UE_LOGF(LogGeometry, Warning,
					       "Encountered corrupted data when deserializing FSMallListSet; tried to read %llu values from buffer with %llu elements.",
					       BufferIndex - 1, BufferNum);
					Ar.SetError();
				}
			}
			else
			{
				const int32 ListCount = ListHeads.Num();
				size_t BufferSize = 0;
				size_t BufferIndex = 0;

				// Store number of lists.
				Buffer.Resize(++BufferSize);
				Buffer[BufferIndex++] = ListCount;

				for (int32 ListIndex = 0; ListIndex < ListCount; ++ListIndex)
				{
					const int32 ListValueCount = GetCount(ListIndex);

					// Store number of values in the current list. 
					Buffer.Resize(BufferSize += 1 + ListValueCount);
					Buffer[BufferIndex++] = ListValueCount;

					if (ListValueCount > 0)
					{
						// Store values in the current list.
						for (const int32 Value : Values(ListIndex))
						{
							Buffer[BufferIndex++] = Value;
						}
					}
				}

				SerializeVector(Ar, Buffer, bUseCompression);
			}
		}
		else
		{
			// Naively serialize all of the underlying data.
			SerializeVector(Ar, ListHeads, false);
			SerializeVector(Ar, ListBlocks, false);
			SerializeVector(Ar, FreeBlocks, false);
			SerializeVector(Ar, LinkedListElements, false);
			Ar << AllocatedCount;
			Ar << FreeHeadIndex;
		}
	}
}
