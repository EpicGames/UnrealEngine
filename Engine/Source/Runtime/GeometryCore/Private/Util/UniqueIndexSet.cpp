// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/UniqueIndexSet.h"

using namespace UE::Geometry;

FUniqueIndexSet::~FUniqueIndexSet()
{
	if (Bits != nullptr)
	{
		delete[] Bits;
	}
}
