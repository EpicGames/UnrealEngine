// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "MathUtil.h"

namespace UE {
namespace Geometry {

	//
	// Index interface (similar to ParallelFor)
	// 	   TransformFuncT should be a function pointer-like object with signature: T(IntType )
	// 	   ReduceFuncT should be a function pointer-like object with signature: T(T,T)

	template<typename IntType, typename T, typename TransformFuncT, typename ReduceFuncT>
	T ParallelTransformReduce(IntType Num,
							  const T& Init,
							  TransformFuncT Transform,
							  ReduceFuncT Reduce,
							  int64 InNumTasks)
	{
		check(InNumTasks > 0);
		// ParallelFor doesn't yet support int64 NumTasks, so cap it
		const int32 NumTasks = (int32) FMath::Min(InNumTasks, MAX_int32);

		const IntType NumPerTask = (IntType)FMathf::Ceil((float)Num / (float)NumTasks);

		TArray<T> PerTaskResults;
		PerTaskResults.SetNum(NumTasks);

		ParallelFor(NumTasks, [&](int32 TaskIndex)
		{
			T LocalResult{ Init };

			IntType End = FMath::Min((TaskIndex + 1) * NumPerTask, Num);
			for (IntType Index = TaskIndex * NumPerTask; Index < End; ++Index)
			{
				T Transformed{ Transform(Index) };
				LocalResult = Reduce(Transformed, LocalResult);
			}

			PerTaskResults[TaskIndex] = LocalResult;
		});

		T FinalResult{ Init };
		for (const T& PartialResult : PerTaskResults)
		{
			FinalResult = Reduce(PartialResult, FinalResult);
		}
		return FinalResult;
	}



	//
	// Index interface for non-copyable types
	// 	   InitFuncT should be a function-like object with signature void(T&)
	// 	   TransformFuncT should be a function-like object with signature void(IntType,T&)
	// 	   ReduceFuncT should be a function-like object with signature void(T,T&)

	template<typename IntType, typename T, typename InitFuncT, typename TransformFuncT, typename ReduceFuncT>
	void ParallelTransformReduce(IntType Num,
								 InitFuncT InitFunc,
								 TransformFuncT Transform,
								 ReduceFuncT Reduce,
								 T& Out,
								 int64 InNumTasks)
	{
		check(InNumTasks > 0);

		// ParallelFor doesn't yet support int64 NumTasks, so cap it
		const int32 NumTasks = (int32)FMath::Min(InNumTasks, MAX_int32);

		IntType NumPerTask = (IntType)FMathf::Ceil((float)Num / (float)NumTasks);

		TArray<T> PerTaskResults;
		PerTaskResults.SetNum(NumTasks);

		ParallelFor(NumTasks, [&](int32 TaskIndex)
		{
			InitFunc(PerTaskResults[TaskIndex]);

			IntType End = FMath::Min((TaskIndex + 1) * NumPerTask, Num);
			for (IntType Index = TaskIndex * NumPerTask; Index < End; ++Index)
			{
				T Transformed;
				Transform(Index, Transformed);
				Reduce(Transformed, PerTaskResults[TaskIndex]);
			}
		});

		InitFunc(Out);
		for (const T& PartialResult : PerTaskResults)
		{
			Reduce(PartialResult, Out);
		}
	}


	//
	// Iterator interface helpers
	//

	// TODO: Specialize this for for random-access iterators
	template<typename IterT>
	int64 IteratorDistance(IterT Begin, IterT End)
	{
		int64 Count = 0;
		for (IterT Iter = Begin; Iter != End; ++Iter)
		{
			++Count;
		}
		return Count;
	}


	// TODO: Specialize this for random-access iterators
	template<typename IterT>
	IterT AdvanceIterator(IterT Start, int64 N)
	{
		IterT I = Start;
		for (int64 Inc = 0; Inc < N; ++Inc)
		{
			++I;
		}
		return I;
	}

	template<typename ContainerType, typename ElementType, typename SizeType>
	int64 IteratorDistance(TIndexedContainerIterator<ContainerType, ElementType, SizeType> Begin,
						   TIndexedContainerIterator<ContainerType, ElementType, SizeType> End)
	{
		return End.GetIndex() - Begin.GetIndex();
	}

	template<typename ContainerType, typename ElementType, typename SizeType>
	TIndexedContainerIterator<ContainerType, ElementType, SizeType>
		AdvanceIterator(TIndexedContainerIterator<ContainerType, ElementType, SizeType> Begin,
						SizeType N)
	{
		return Begin + N;
	}

	// Iterator interface
	// 	   TransformFuncT should be a function-like object with signature: T(U), where U is the type referred to by IterT
	// 	   ReduceFuncT should be a function-like object with signature: T(T,T)

	template<typename T, typename IterT, typename TransformFuncT, typename ReduceFuncT>
	T ParallelTransformReduce(IterT BeginIterator,
							  IterT EndIterator,
							  const T& Init,
							  TransformFuncT Transform,
							  ReduceFuncT Reduce,
							  int64 InNumTasks)
	{
		int64 Num = IteratorDistance(BeginIterator, EndIterator);

		check(InNumTasks > 0);

		// ParallelFor doesn't yet support int64 NumTasks, so cap it
		const int32 NumTasks = (int32)FMath::Min(InNumTasks, MAX_int32);

		int64 NumPerTask = (int64)FMathf::Ceil((float)Num / (float)NumTasks);

		TArray<T> PerTaskResults;
		PerTaskResults.SetNum(NumTasks);

		ParallelFor(NumTasks, [&](int32 TaskIndex)
		{
			T LocalResult{ Init };

			IterT LocalIter = AdvanceIterator(BeginIterator, FMath::Min(Num, int64(TaskIndex) * NumPerTask));
			IterT LocalEndIter = AdvanceIterator(BeginIterator, FMath::Min(Num, (int64(TaskIndex) + 1) * NumPerTask));

			while (LocalIter != LocalEndIter)
			{
				T Transformed{ Transform(*LocalIter) };
				LocalResult = Reduce(Transformed, LocalResult);

				++LocalIter;
			}

			PerTaskResults[TaskIndex] = LocalResult;
		});

		T FinalResult{ Init };
		for (const T& PartialResult : PerTaskResults)
		{
			FinalResult = Reduce(PartialResult, FinalResult);
		}
		return FinalResult;
	}

	//
	// Iterator interface for non-copyable types
	// 	   InitFuncT should be a function-like object with signature: void(T&)
	// 	   TransformFuncT should be a function-like object with signature: void(U,T&), where U is the type referred to by IterT
	// 	   ReduceFuncT should be a function-like object with signature: void(T,T&)

	template<typename T, typename IterT, typename InitFuncT, typename TransformFuncT, typename ReduceFuncT>
	void ParallelTransformReduce(IterT BeginIterator,
								 IterT EndIterator,
								 InitFuncT InitFunc,
								 TransformFuncT Transform,
								 ReduceFuncT Reduce,
								 T& Out,
								 int64 InNumTasks)
	{
		int64 Num = IteratorDistance(BeginIterator, EndIterator);

		check(InNumTasks > 0);
		// ParallelFor doesn't yet support int64 NumTasks, so cap it
		const int32 NumTasks = (int32)FMath::Min(InNumTasks, MAX_int32);

		int64 NumPerTask = (int64)FMathf::Ceil((float)Num / (float)NumTasks);

		TArray<T> PerTaskResults;
		PerTaskResults.SetNum(NumTasks);

		ParallelFor(NumTasks, [&](int32 TaskIndex)
		{
			InitFunc(PerTaskResults[TaskIndex]);

			IterT LocalIter = AdvanceIterator(BeginIterator, FMath::Min(Num, int64(TaskIndex) * NumPerTask));
			IterT LocalEndIter = AdvanceIterator(BeginIterator, FMath::Min(Num, (int64(TaskIndex) + 1) * NumPerTask));

			while (LocalIter != LocalEndIter)
			{
				T Transformed;
				Transform(*LocalIter, Transformed);
				Reduce(Transformed, PerTaskResults[TaskIndex]);
				++LocalIter;
			}
		});

		InitFunc(Out);
		for (const T& PartialResult : PerTaskResults)
		{
			Reduce(PartialResult, Out);
		}
	}

} // end namespace UE::Geometry
} // end namespace UE
