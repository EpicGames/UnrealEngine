// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Math/Box2D.h"
#include "VectorTypes.h"
#include "TransformTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template <typename RealType>
struct TInterval1
{
	RealType Min;
	RealType Max;

	TInterval1() :
		TInterval1(Empty())
	{
	}

	TInterval1(const RealType& Min, const RealType& Max)
	{
		this->Min = Min;
		this->Max = Max;
	}

	static TInterval1<RealType> Empty()
	{
		return TInterval1(TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max());
	}

	static TInterval1<RealType> MakeFromUnordered(const RealType& A, const RealType& B)
	{
		TInterval1<RealType> Result(A, B);
		if (A > B)
		{
			Swap(Result.Min, Result.Max);
		}
		return Result;
	}

	RealType Center() const
	{
		return (Min + Max) * (RealType)0.5;
	}

	RealType Extent() const
	{
		return (Max - Min)*(RealType).5;
	}
	RealType Length() const
	{
		return Max - Min;
	}

	RealType MaxAbsExtrema() const
	{
		return TMathUtil<RealType>::Max(TMathUtil<RealType>::Abs(Min), TMathUtil<RealType>::Abs(Max));
	}

	void Contain(const RealType& V)
	{
		if (V < Min)
		{
			Min = V;
		}
		if (V > Max)
		{
			Max = V;
		}
	}

	void Contain(const TInterval1<RealType>& O)
	{
		if (O.Min < Min)
		{
			Min = O.Min;
		}
		if (O.Max > Max)
		{
			Max = O.Max;
		}
	}

	bool Contains(RealType D) const
	{
		return D >= Min && D <= Max;
	}

	bool Contains(const TInterval1<RealType>& O) const
	{
		return Contains(O.Min) && Contains(O.Max);
	}

	bool Overlaps(const TInterval1<RealType>& O) const
	{
		return !(O.Min > Max || O.Max < Min);
	}

	RealType SquaredDist(const TInterval1<RealType>& O) const
	{
		if (Max < O.Min)
		{
			return (O.Min - Max) * (O.Min - Max);
		}
		else if (Min > O.Max)
		{
			return (Min - O.Max) * (Min - O.Max);
		}
		else
		{
			return 0;
		}
	}
	RealType Dist(const TInterval1<RealType>& O) const
	{
		if (Max < O.Min)
		{
			return O.Min - Max;
		}
		else if (Min > O.Max)
		{
			return Min - O.Max;
		}
		else
		{
			return 0;
		}
	}

	TInterval1<RealType> IntersectionWith(const TInterval1<RealType>& O) const
	{
		if (O.Min > Max || O.Max < Min)
		{
			return TInterval1<RealType>::Empty();
		}
		return TInterval1<RealType>(TMathUtil<RealType>::Max(Min, O.Min), TMathUtil<RealType>::Min(Max, O.Max));
	}

	/**
	 * clamp Value f to interval [Min,Max]
	 */
	RealType Clamp(RealType f) const
	{
		return (f < Min) ? Min : (f > Max) ? Max : f;
	}

	/**
	 * interpolate between Min and Max using Value T in range [0,1]
	 */
	RealType Interpolate(RealType T) const
	{
		return (1 - T) * Min + (T)*Max;
	}

	/**
	 * Convert Value into (clamped) T Value in range [0,1]
	 */
	RealType GetT(RealType Value) const
	{
		if (Value <= Min)
		{
			return 0;
		}
		else if (Value >= Max)
		{
			return 1;
		}
		else
		{
			return GetUnclampedT(Value);
		}
	}

	/**
	 * Convert Value into T Value.
	 */
	RealType GetUnclampedT(RealType Value) const
	{
		if (Min == Max)
		{
			return 0.5;
		}
		else
		{
			return (Value - Min) / (Max - Min);
		}
	}

	void Set(TInterval1 O)
	{
		Min = O.Min;
		Max = O.Max;
	}

	void Set(RealType A, RealType B)
	{
		Min = A;
		Max = B;
	}

	TInterval1 operator-(TInterval1 V) const
	{
		return TInterval1(-V.Min, -V.Max);
	}

	TInterval1 operator+(RealType f) const
	{
		return TInterval1(Min + f, Max + f);
	}

	TInterval1 operator-(RealType f) const
	{
		return TInterval1(Min - f, Max - f);
	}

	TInterval1 operator*(RealType f) const
	{
		return TInterval1(Min * f, Max * f);
	}

	inline bool IsEmpty() const
	{
		return Max < Min;
	}

	void Expand(RealType Radius)
	{
		Max += Radius;
		Min -= Radius;
	}
};

typedef TInterval1<float> FInterval1f;
typedef TInterval1<double> FInterval1d;


template <typename RealType>
struct TAxisAlignedBox3
{
	TVector<RealType> Min;
	TVector<RealType> Max;

	TAxisAlignedBox3() : 
		TAxisAlignedBox3(TAxisAlignedBox3<RealType>::Empty())
	{
	}

	TAxisAlignedBox3(const TVector<RealType>& Min, const TVector<RealType>& Max)
	{
		this->Min = Min;
		this->Max = Max;
	}

	TAxisAlignedBox3(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C)
	{
		// TMathUtil::MinMax could be used here, but it generates worse code because the Min3's below will be
		// turned into SSE instructions by the optimizer, while MinMax will not
		Min = TVector<RealType>(
			TMathUtil<RealType>::Min3(A.X, B.X, C.X),
			TMathUtil<RealType>::Min3(A.Y, B.Y, C.Y),
			TMathUtil<RealType>::Min3(A.Z, B.Z, C.Z));
		Max = TVector<RealType>(
			TMathUtil<RealType>::Max3(A.X, B.X, C.X),
			TMathUtil<RealType>::Max3(A.Y, B.Y, C.Y),
			TMathUtil<RealType>::Max3(A.Z, B.Z, C.Z));
	}

	template<typename OtherRealType>
	explicit TAxisAlignedBox3(const TAxisAlignedBox3<OtherRealType>& OtherBox)
	{
		this->Min = TVector<RealType>(OtherBox.Min);
		this->Max = TVector<RealType>(OtherBox.Max);
	}

	TAxisAlignedBox3(const TVector<RealType>& Center, RealType HalfWidth)
	{
		this->Min = TVector<RealType>(Center.X-HalfWidth, Center.Y-HalfWidth, Center.Z-HalfWidth);
		this->Max = TVector<RealType>(Center.X+HalfWidth, Center.Y+HalfWidth, Center.Z+HalfWidth);
	}


	TAxisAlignedBox3(const TAxisAlignedBox3& Box, const TFunction<TVector<RealType>(const TVector<RealType>&)> TransformF)
	{
		if (TransformF == nullptr)
		{
			Min = Box.Min;
			Max = Box.Max;
			return;
		}

		TVector<RealType> C0 = TransformF(Box.GetCorner(0));
		Min = C0;
		Max = C0;
		for (int i = 1; i < 8; ++i)
		{
			Contain(TransformF(Box.GetCorner(i)));
		}
	}

	TAxisAlignedBox3(const TAxisAlignedBox3& Box, const FTransformSRT3d& Transform)
	{
		if (Box.IsEmpty())
		{
			Min = Box.Min;
			Max = Box.Max;
			return;
		}

		TVector<RealType> C0 = Transform.TransformPosition(Box.GetCorner(0));
		Min = C0;
		Max = C0;
		for (int i = 1; i < 8; ++i)
		{
			Contain(Transform.TransformPosition(Box.GetCorner(i)));
		}
	}

	TAxisAlignedBox3(TArrayView<const TVector<RealType>> Pts) :
		TAxisAlignedBox3(TAxisAlignedBox3<RealType>::Empty())
	{
		Contain(Pts);
	}

	TAxisAlignedBox3(const TArray<TVector<RealType>>& Pts) :
		TAxisAlignedBox3(TAxisAlignedBox3<RealType>::Empty())
	{
		Contain(Pts);
	}

	bool operator==(const TAxisAlignedBox3<RealType>& Other) const
	{
		return Max == Other.Max && Min == Other.Min;
	}
	bool operator!=(const TAxisAlignedBox3<RealType>& Other) const
	{
		return Max != Other.Max || Min != Other.Min;
	}


	template<typename OutRealType>
	explicit operator TBox<OutRealType>() const
	{
		TBox<OutRealType> ToRet((TVector<OutRealType>)Min, (TVector<OutRealType>)Max);
		ToRet.IsValid = !IsEmpty();
		return ToRet;
	}
	template<typename InRealType>
	TAxisAlignedBox3(const TBox<InRealType>& Box)
	{
		if (Box.IsValid)
		{
			Min = TVector<RealType>(Box.Min);
			Max = TVector<RealType>(Box.Max);
		}
		else
		{
			*this = Empty();
		}
	}

	/**
	* @param Index corner index in range 0-7
	* @return Corner point on the box identified by the given index. See diagram in OrientedBoxTypes.h for index/corner mapping.
	*/
	TVector<RealType> GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 7);
		RealType X = (((Index & 1) != 0) ^ ((Index & 2) != 0)) ? (Max.X) : (Min.X);
		RealType Y = ((Index / 2) % 2 == 0) ? (Min.Y) : (Max.Y);
		RealType Z = (Index < 4) ? (Min.Z) : (Max.Z);
		return TVector<RealType>(X, Y, Z);
	}

	static TAxisAlignedBox3<RealType> Empty()
	{
		return TAxisAlignedBox3(
			TVector<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max()),
			TVector<RealType>(-TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max()));
	}

	static TAxisAlignedBox3<RealType> Infinite()
	{
		return TAxisAlignedBox3(
			TVector<RealType>(-TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max()),
			TVector<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max()) );
	}

	/**
	 * Compute bounding box of 3D points returned by GetPoint(Index) for indices in range [0...MaxIndex)
	 */
	template<typename PointFunc>
	static TAxisAlignedBox3<RealType> MakeBoundsFromIndices(int32 MaxIndex, PointFunc GetPoint)
	{
		TAxisAlignedBox3<RealType> Result = TAxisAlignedBox3<RealType>::Empty();
		for (int32 Index = 0; Index < MaxIndex; ++Index)
		{
			Result.Contain(GetPoint(Index));
		}
		return Result;
	}

	/**
	 * Compute bounding box of 3D points returned by GetPoint(Index) for indices in range for_each(IndexEnumerable)
	 */
	template<typename EnumerableIntType, typename PointFunc>
	static TAxisAlignedBox3<RealType> MakeBoundsFromIndices(EnumerableIntType IndexEnumerable, PointFunc GetPoint)
	{
		TAxisAlignedBox3<RealType> Result = TAxisAlignedBox3<RealType>::Empty();
		for (int32 Index : IndexEnumerable)
		{
			Result.Contain(GetPoint(Index));
		}
		return Result;
	}


	TVector<RealType> Center() const
	{
		return TVector<RealType>(
			(Min.X + Max.X) * (RealType)0.5,
			(Min.Y + Max.Y) * (RealType)0.5,
			(Min.Z + Max.Z) * (RealType)0.5);
	}

	TVector<RealType> Extents() const
	{
		return (Max - Min) * (RealType).5;
	}

	void Contain(const TVector<RealType>& V)
	{
		if (V.X < Min.X)
		{
			Min.X = V.X;
		}
		if (V.X > Max.X)
		{
			Max.X = V.X;
		}
		if (V.Y < Min.Y)
		{
			Min.Y = V.Y;
		}
		if (V.Y > Max.Y)
		{
			Max.Y = V.Y;
		}
		if (V.Z < Min.Z)
		{
			Min.Z = V.Z;
		}
		if (V.Z > Max.Z)
		{
			Max.Z = V.Z;
		}
	}

	void Contain(const TAxisAlignedBox3<RealType>& Other)
	{
		Min.X = Min.X < Other.Min.X ? Min.X : Other.Min.X;
		Min.Y = Min.Y < Other.Min.Y ? Min.Y : Other.Min.Y;
		Min.Z = Min.Z < Other.Min.Z ? Min.Z : Other.Min.Z;
		Max.X = Max.X > Other.Max.X ? Max.X : Other.Max.X;
		Max.Y = Max.Y > Other.Max.Y ? Max.Y : Other.Max.Y;
		Max.Z = Max.Z > Other.Max.Z ? Max.Z : Other.Max.Z;
	}

	void Contain(TArrayView<const TVector<RealType>> Pts)
	{
		for (const TVector<RealType>& Pt : Pts)
		{
			Contain(Pt);
		}
	}

	void Contain(const TArray<TVector<RealType>>& Pts)
	{
		for (const TVector<RealType>& Pt : Pts)
		{
			Contain(Pt);
		}
	}

	bool Contains(const TVector<RealType>& V) const
	{
		return (Min.X <= V.X) && (Min.Y <= V.Y) && (Min.Z <= V.Z) && (Max.X >= V.X) && (Max.Y >= V.Y) && (Max.Z >= V.Z);
	}

	bool Contains(const TAxisAlignedBox3<RealType>& Box) const
	{
		return Contains(Box.Min) && Contains(Box.Max);
	}

	TVector<RealType> Clamp(const TVector<RealType>& V) const
	{
		return TVector<RealType>(
			FMath::Clamp(V.X, Min.X, Max.X),
			FMath::Clamp(V.Y, Min.Y, Max.Y),
			FMath::Clamp(V.Z, Min.Z, Max.Z)
		);
	}

	TAxisAlignedBox3<RealType> Intersect(const TAxisAlignedBox3<RealType>& Box) const
	{
		TAxisAlignedBox3<RealType> Intersection(
			TVector<RealType>(TMathUtil<RealType>::Max(Min.X, Box.Min.X), TMathUtil<RealType>::Max(Min.Y, Box.Min.Y), TMathUtil<RealType>::Max(Min.Z, Box.Min.Z)),
			TVector<RealType>(TMathUtil<RealType>::Min(Max.X, Box.Max.X), TMathUtil<RealType>::Min(Max.Y, Box.Max.Y), TMathUtil<RealType>::Min(Max.Z, Box.Max.Z)));
		if (Intersection.Height() <= 0 || Intersection.Width() <= 0 || Intersection.Depth() <= 0)
		{
			return TAxisAlignedBox3<RealType>::Empty();
		}
		else
		{
			return Intersection;
		}
	}

	bool Intersects(TAxisAlignedBox3 Box) const
	{
		return !((Box.Max.X <= Min.X) || (Box.Min.X >= Max.X) || (Box.Max.Y <= Min.Y) || (Box.Min.Y >= Max.Y) || (Box.Max.Z <= Min.Z) || (Box.Min.Z >= Max.Z));
	}

	RealType DistanceSquared(const TVector<RealType>& V) const
	{
		RealType dx = (V.X < Min.X) ? Min.X - V.X : (V.X > Max.X ? V.X - Max.X : 0);
		RealType dy = (V.Y < Min.Y) ? Min.Y - V.Y : (V.Y > Max.Y ? V.Y - Max.Y : 0);
		RealType dz = (V.Z < Min.Z) ? Min.Z - V.Z : (V.Z > Max.Z ? V.Z - Max.Z : 0);
		return dx * dx + dy * dy + dz * dz;
	}

	RealType DistanceSquared(const TAxisAlignedBox3<RealType>& Box) const
	{
		// compute lensqr( max(0, abs(center1-center2) - (extent1+extent2)) )
		RealType delta_x = TMathUtil<RealType>::Abs((Box.Min.X + Box.Max.X) - (Min.X + Max.X))
			- ((Max.X - Min.X) + (Box.Max.X - Box.Min.X));
		if (delta_x < 0)
		{
			delta_x = 0;
		}
		RealType delta_y = TMathUtil<RealType>::Abs((Box.Min.Y + Box.Max.Y) - (Min.Y + Max.Y))
			- ((Max.Y - Min.Y) + (Box.Max.Y - Box.Min.Y));
		if (delta_y < 0)
		{
			delta_y = 0;
		}
		RealType delta_z = TMathUtil<RealType>::Abs((Box.Min.Z + Box.Max.Z) - (Min.Z + Max.Z))
			- ((Max.Z - Min.Z) + (Box.Max.Z - Box.Min.Z));
		if (delta_z < 0)
		{
			delta_z = 0;
		}
		return (RealType)0.25 * (delta_x * delta_x + delta_y * delta_y + delta_z * delta_z);
	}

	RealType Dimension(int32 Index) const
	{
		return TMathUtil<RealType>::Max(Max[Index] - Min[Index], (RealType)0);
	}

	RealType Width() const
	{
		return TMathUtil<RealType>::Max(Max.X - Min.X, (RealType)0);
	}

	RealType Height() const
	{
		return TMathUtil<RealType>::Max(Max.Y - Min.Y, (RealType)0);
	}

	RealType Depth() const
	{
		return TMathUtil<RealType>::Max(Max.Z - Min.Z, (RealType)0);
	}

	RealType Volume() const
	{
		return Width() * Height() * Depth();
	}

	RealType SurfaceArea() const
	{
		return (RealType)2. * ( Width() * ( Height() + Depth() ) + Height() * Depth() );
	}

	RealType DiagonalLength() const
	{
		return TMathUtil<RealType>::Sqrt((Max.X - Min.X) * (Max.X - Min.X) + (Max.Y - Min.Y) * (Max.Y - Min.Y) + (Max.Z - Min.Z) * (Max.Z - Min.Z));
	}

	RealType MaxDim() const
	{
		return TMathUtil<RealType>::Max(Width(), TMathUtil<RealType>::Max(Height(), Depth()));
	}

	RealType MinDim() const
	{
		return TMathUtil<RealType>::Min(Width(), TMathUtil<RealType>::Min(Height(), Depth()));
	}

	TVector<RealType> Diagonal() const
	{
		return TVector<RealType>(Max.X - Min.X, Max.Y - Min.Y, Max.Z - Min.Z);
	}

	inline bool IsEmpty() const
	{
		return Max.X < Min.X || Max.Y < Min.Y || Max.Z < Min.Z;
	}

	void Expand(RealType Radius)
	{
		Max.X += Radius;
		Max.Y += Radius;
		Max.Z += Radius;
		Min.X -= Radius;
		Min.Y -= Radius;
		Min.Z -= Radius;
	}

	void Expand(const TVector<RealType>& ExpandByVec)
	{
		Max += ExpandByVec;
		Min -= ExpandByVec;
	}
};

template <typename RealType>
struct TAxisAlignedBox2
{
	TVector2<RealType> Min;
	TVector2<RealType> Max;

	TAxisAlignedBox2() : 
		TAxisAlignedBox2(Empty())
	{
	}

	TAxisAlignedBox2(const TVector2<RealType>& Min, const TVector2<RealType>& Max)
		: Min(Min), Max(Max)
	{
	}

	template<typename OtherRealType>
	explicit TAxisAlignedBox2(const TAxisAlignedBox2<OtherRealType>& OtherBox)
	{
		this->Min = TVector2<RealType>(OtherBox.Min);
		this->Max = TVector2<RealType>(OtherBox.Max);
	}

	TAxisAlignedBox2(RealType SquareSize)
		: Min((RealType)0, (RealType)0), Max(SquareSize, SquareSize)
	{
	}
	TAxisAlignedBox2(RealType Width, RealType Height)
		: Min((RealType)0, (RealType)0), Max(Width, Height)
	{
	}

	TAxisAlignedBox2(const TArray<TVector2<RealType>>& Pts) :
		TAxisAlignedBox2(Empty())
	{
		Contain(Pts);
	}

	TAxisAlignedBox2(TArrayView<const TVector2<RealType>> Pts) :
		TAxisAlignedBox2(Empty())
	{
		Contain(Pts);
	}

	TAxisAlignedBox2(const TVector2<RealType>& Center, RealType HalfWidth)
	{
		this->Min = TVector2<RealType>(Center.X - HalfWidth, Center.Y - HalfWidth);
		this->Max = TVector2<RealType>(Center.X + HalfWidth, Center.Y + HalfWidth);
	}

	template<typename OutRealType>
	explicit operator TBox2<OutRealType>() const
	{
		TBox2<OutRealType> ToRet((TVector2<OutRealType>)Min, (TVector2<OutRealType>)Max);
		ToRet.bIsValid = !IsEmpty();
		return ToRet;
	}
	template<typename InRealType>
	TAxisAlignedBox2(const TBox2<InRealType>& Box)
	{
		if (Box.bIsValid)
		{
			Min = TVector2<RealType>(Box.Min);
			Max = TVector2<RealType>(Box.Max);
		}
		else
		{
			*this = Empty();
		}
	}

	static TAxisAlignedBox2<RealType> Empty()
	{
		return TAxisAlignedBox2(
			TVector2<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max()),
			TVector2<RealType>(-TNumericLimits<RealType>::Max(), -TNumericLimits<RealType>::Max()));
	}

	TVector2<RealType> Center() const
	{
		return TVector2<RealType>(
			(Min.X + Max.X) * (RealType)0.5,
			(Min.Y + Max.Y) * (RealType)0.5);
	}

	TVector2<RealType> Extents() const
	{
		return (Max - Min) * (RealType).5;
	}

	/**
	 * Corners are ordered to follow the perimeter of the bounding rectangle, starting from the (Min.X, Min.Y) corner and ending at (Min.X, Max.Y)
	 * @param Index which corner to return, must be in range [0,3]
	 * @return Corner of the bounding rectangle
	 */
	TVector2<RealType> GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 3);
		RealType X = ((Index % 3) == 0) ? (Min.X) : (Max.X);
		RealType Y = ((Index & 2) == 0) ? (Min.Y) : (Max.Y);
		return TVector2<RealType>(X, Y);
	}

	inline void Contain(const TVector2<RealType>& V)
	{
		if (V.X < Min.X)
		{
			Min.X = V.X;
		}
		if (V.X > Max.X)
		{
			Max.X = V.X;
		}
		if (V.Y < Min.Y)
		{
			Min.Y = V.Y;
		}
		if (V.Y > Max.Y)
		{
			Max.Y = V.Y;
		}
	}

	inline void Contain(const TAxisAlignedBox2<RealType>& Other)
	{
		Min.X = Min.X < Other.Min.X ? Min.X : Other.Min.X;
		Min.Y = Min.Y < Other.Min.Y ? Min.Y : Other.Min.Y;
		Max.X = Max.X > Other.Max.X ? Max.X : Other.Max.X;
		Max.Y = Max.Y > Other.Max.Y ? Max.Y : Other.Max.Y;
	}

	void Contain(const TArray<TVector2<RealType>>& Pts)
	{
		for (const TVector2<RealType>& Pt : Pts)
		{
			Contain(Pt);
		}
	}

	void Contain(TArrayView<const TVector2<RealType>> Pts)
	{
		for (const TVector2<RealType>& Pt : Pts)
		{
			Contain(Pt);
		}
	}

	bool Contains(const TVector2<RealType>& V) const
	{
		return (Min.X <= V.X) && (Min.Y <= V.Y) && (Max.X >= V.X) && (Max.Y >= V.Y);
	}

	bool Contains(const TAxisAlignedBox2<RealType>& Box) const
	{
		return Contains(Box.Min) && Contains(Box.Max);
	}

	TVector<RealType> Clamp(const TVector2<RealType>& V)
	{
		return TVector2<RealType>(
			FMath::Clamp(V.X, Min.X, Max.X),
			FMath::Clamp(V.Y, Min.Y, Max.Y)
		);
	}

	bool Intersects(const TAxisAlignedBox2<RealType>& Box) const
	{
		return !((Box.Max.X < Min.X) || (Box.Min.X > Max.X) || (Box.Max.Y < Min.Y) || (Box.Min.Y > Max.Y));
	}

	TAxisAlignedBox2<RealType> Intersect(const TAxisAlignedBox2<RealType> &Box) const
	{
		TAxisAlignedBox2<RealType> Intersection(
			TVector2<RealType>(TMathUtil<RealType>::Max(Min.X, Box.Min.X), TMathUtil<RealType>::Max(Min.Y, Box.Min.Y)),
			TVector2<RealType>(TMathUtil<RealType>::Min(Max.X, Box.Max.X), TMathUtil<RealType>::Min(Max.Y, Box.Max.Y)));
		if (Intersection.Height() <= 0 || Intersection.Width() <= 0)
		{
			return TAxisAlignedBox2<RealType>::Empty();
		}
		else
		{
			return Intersection;
		}
	}

	RealType DistanceSquared(const TVector2<RealType>& V) const
	{
		RealType dx = (V.X < Min.X) ? Min.X - V.X : (V.X > Max.X ? V.X - Max.X : 0);
		RealType dy = (V.Y < Min.Y) ? Min.Y - V.Y : (V.Y > Max.Y ? V.Y - Max.Y : 0);
		return dx * dx + dy * dy;
	}

	inline RealType Width() const
	{
		return TMathUtil<RealType>::Max(Max.X - Min.X, (RealType)0);
	}

	inline RealType Height() const
	{
		return TMathUtil<RealType>::Max(Max.Y - Min.Y, (RealType)0);
	}

	inline RealType Area() const
	{
		return Width() * Height();
	}

	inline RealType Perimeter() const
	{
		return (Width() + Height()) * 2;
	}

	RealType DiagonalLength() const
	{
		return (RealType)TMathUtil<RealType>::Sqrt((Max.X - Min.X) * (Max.X - Min.X) + (Max.Y - Min.Y) * (Max.Y - Min.Y));
	}

	inline RealType MaxDim() const
	{
		return TMathUtil<RealType>::Max(Width(), Height());
	}

	inline RealType MinDim() const
	{
		return TMathUtil<RealType>::Min(Width(), Height());
	}

	inline bool IsEmpty() const
	{
		return Max.X < Min.X || Max.Y < Min.Y;
	}

	void Expand(RealType Radius)
	{
		Max.X += Radius;
		Max.Y += Radius;
		Min.X -= Radius;
		Min.Y -= Radius;
	}
	
	void Expand(const TVector2<RealType>& ExpandByVec)
	{
		Max += ExpandByVec;
		Min -= ExpandByVec;
	}
};

typedef TAxisAlignedBox2<float> FAxisAlignedBox2f;
typedef TAxisAlignedBox2<double> FAxisAlignedBox2d;
typedef TAxisAlignedBox3<float> FAxisAlignedBox3f;
typedef TAxisAlignedBox3<double> FAxisAlignedBox3d;

} // end namespace UE::Geometry
} // end namespace UE