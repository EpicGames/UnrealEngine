// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "SegmentTypes.h"
#include "BoxTypes.h"
#include "OrientedBoxTypes.h"


namespace UE 
{
namespace Geometry 
{

using namespace UE::Math;

/*
 * 3D Capsule stored as Line Segment and Radius
 */
template<typename T>
struct TCapsule3
{
public:
	/** Line segment of the capsule */
	TSegment3<T> Segment;
	/** Radius of the capsule */
	T Radius = (T)0;

	TCapsule3() = default;

	TCapsule3(const TSegment3<T>& SegmentIn, T RadiusIn)
		: Segment(SegmentIn), Radius(RadiusIn) {}

	TCapsule3(const TVector<T>& StartPoint, const TVector<T>& EndPoint, T RadiusIn)
		: Segment(StartPoint, EndPoint), Radius(RadiusIn) {}


	/** @return Center of capsule line segment */
	const TVector<T>& Center() const
	{
		return Segment.Center;
	}

	/** @return Direction of capsule line segment */
	const TVector<T>& Direction() const
	{
		return Segment.Direction;
	}

	/** @return Length of capsule line segment */
	T Length() const
	{
		return (T)2 * Segment.Extent;
	}

	/** @return Extent (half-length) of capsule line segment */
	T Extent() const
	{
		return Segment.Extent;
	}

	/** @return Volume of Capsule */
	T Volume() const
	{
		return Volume(Radius, Segment.Extent);
	}

	/** @return Bounding Box of Capsule */
	TAxisAlignedBox3<T> GetBounds() const
	{
		return Segment.GetBounds(Radius);
	}

	/** @return Oriented Bounding Box of Capsule (orthogonal axes are arbitrary) */
	TOrientedBox3<T> GetOrientedBounds() const
	{
		return TOrientedBox3<T>(
			TFrame3<T>(Segment.Center, Segment.Direction),
			TVector<T>(Radius, Radius, Segment.Extent));
	}

	/** @return true if Capsule contains the given Point */
	bool Contains(const TVector<T>& Point) const
	{
		T DistSqr = Segment.DistanceSquared(Point);
		return DistSqr <= Radius * Radius;
	}


	/**
	 * @return minimum squared distance from Point to Capsule surface for points outside capsule, 0 for points inside
	 */
	inline T DistanceSquared(const TVector<T>& Point) const
	{
		const T PosDistance = TMathUtil<T>::Max(SignedDistance(Point), (T)0);
		return PosDistance * PosDistance;
	}

	/**
	 * @return signed distance from Point to Capsule surface. Points inside capsule return negative distance.
	 */
	inline T SignedDistance(const TVector<T>& Point) const
	{
		T SqrDist = Segment.DistanceSquared(Point);
		return TMathUtil<T>::Sqrt(SqrDist) - Radius;
	}

	//
	// Sphere utility functions
	//

	/** @return Volume of Capsule with given Radius and Extent (half-length) */
	static T Volume(T Radius, T Extent)
	{
		T PiRadSqr = TMathUtil<T>::Pi * Radius * Radius;
		return 
			(PiRadSqr * ((T)2 * Extent)) +				// cylinder vol = (pi*r^2)*length
			((T)(4.0 / 3.0) * PiRadSqr * Radius);
	}


};


typedef TCapsule3<float> FCapsule3f;
typedef TCapsule3<double> FCapsule3d;


} // end namespace UE::Geometry
} // end namespace UE
