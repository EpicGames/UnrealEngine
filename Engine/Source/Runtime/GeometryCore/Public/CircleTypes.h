// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "VectorUtil.h"
#include "BoxTypes.h"
#include "OrientedBoxTypes.h"
#include "FrameTypes.h"

namespace UE 
{
namespace Geometry 
{

using namespace UE::Math;

template<typename RealType>
struct TCircle2
{
	TVector2<RealType> Center = TVector2<RealType>(0, 0);
	RealType Radius = (RealType)1;
	bool bIsReversed = false;

	TCircle2() {}

	explicit TCircle2(const RealType& RadiusIn)
	{
		Center = TVector2<RealType>(0, 0);
		Radius = RadiusIn;
	}

	TCircle2(const TVector2<RealType>& CenterIn, const RealType& RadiusIn)
	{
		Center = CenterIn;
		Radius = RadiusIn;
	}


	RealType GetCircumference() const
	{
		return TMathUtil<RealType>::TwoPi * Radius;
	}
	void SetCircumference(RealType NewCircumference)
	{
		Radius = NewCircumference / TMathUtil<RealType>::TwoPi;
	}


	RealType GetDiameter() const
	{
		return (RealType)2 * Radius;
	}
	void SetDiameter(RealType NewDiameter)
	{
		Radius = NewDiameter * (RealType)0.5;
	}

	RealType GetArea() const
	{
		return TMathUtil<RealType>::Pi * Radius * Radius;
	}
	void SetArea(RealType NewArea)
	{
		Radius = TMathUtil<RealType>::Sqrt(NewArea / TMathUtil<RealType>::Pi);
	}


	RealType GetCurvature() const
	{
		return (RealType)1 / Radius;
	}

	RealType GetSignedCurvature() const
	{
		return (bIsReversed) ? ((RealType)-1 / Radius) : ((RealType)1 / Radius);
	}


	TVector2<RealType> GetPointFromAngleD(RealType AngleDeg) const
	{
		return GetPointFromAngleR(AngleDeg * TMathUtil<RealType>::DegToRad);
	}

	TVector2<RealType> GetPointFromAngleR(RealType AngleRad) const
	{
		RealType c = TMathUtil<RealType>::Cos(AngleRad), s = TMathUtil<RealType>::Sin(AngleRad);
		return TVector2<RealType>(Center.X + c*Radius, Center.Y + s*Radius);
	}


	TVector2<RealType> GetPointFromUnitParameter(RealType UnitParam) const
	{
		RealType AngleRad = ((bIsReversed) ? (-UnitParam) : (UnitParam)) * TMathUtil<RealType>::TwoPi;
		return GetPointFromAngleR(AngleRad);
	}


	bool IsInside(const TVector2<RealType>& Point) const
	{
		return Center.DistanceSquared(Point) < Radius*Radius;
	}

	bool IsInsideOrOn(const TVector2<RealType>& Point) const
	{
		return TVector2<RealType>::DistSquared(Center, Point) <= Radius*Radius;
	}

	RealType SignedDistance(const TVector2<RealType>& Point) const
	{
		return Center.Distance(Point) - Radius;
	}

	RealType Distance(const TVector2<RealType>& Point) const
	{
		return TMathUtil<RealType>::Abs(Center.Distance(Point) - Radius);
	}


	TAxisAlignedBox2<RealType> GetBoundingBox() const
	{
		return TAxisAlignedBox2<RealType>(
			TVector2<RealType>(Center.X - Radius, Center.Y - Radius),
			TVector2<RealType>(Center.X + Radius, Center.Y + Radius));
	}


	RealType GetBoundingPolygonRadius(int NumSides) const
	{
		RealType DeltaAngle = (TMathUtil<RealType>::TwoPi / (RealType)NumSides) / (RealType)2;
		return Radius / TMathUtil<RealType>::Cos(DeltaAngle);
	}

};

typedef TCircle2<float> FCircle2f;
typedef TCircle2<double> FCircle2d;







template<typename RealType>
struct TCircle3
{
	TFrame3<RealType> Frame;
	RealType Radius = (RealType)1;
	bool bIsReversed = false;

	TCircle3() {}

	explicit TCircle3(const RealType& RadiusIn)
	{
		Radius = RadiusIn;
	}

	TCircle3(const TVector<RealType>& CenterIn, const RealType& RadiusIn)
	{
		Frame.Origin = CenterIn;
		Radius = RadiusIn;
	}

	TCircle3(const TFrame3<RealType>& FrameIn, const RealType& RadiusIn)
	{
		Frame = FrameIn;
		Radius = RadiusIn;
	}


	const TVector<RealType>& GetCenter() const
	{
		return Frame.Origin;
	}

	TVector<RealType> GetNormal() const
	{
		return Frame.Z();
	}


	RealType GetCircumference() const
	{
		return TMathUtil<RealType>::TwoPi * Radius;
	}
	void SetCircumference(RealType NewCircumference)
	{
		Radius = NewCircumference / TMathUtil<RealType>::TwoPi;
	}


	RealType GetDiameter() const
	{
		return (RealType)2 * Radius;
	}
	void SetDiameter(RealType NewDiameter)
	{
		Radius = NewDiameter * (RealType)0.5;
	}

	RealType GetArea() const
	{
		return TMathUtil<RealType>::Pi * Radius * Radius;
	}
	void SetArea(RealType NewArea)
	{
		Radius = TMathUtil<RealType>::Sqrt(NewArea / TMathUtil<RealType>::Pi);
	}


	RealType GetCurvature() const
	{
		return (RealType)1 / Radius;
	}

	RealType GetSignedCurvature() const
	{
		return (bIsReversed) ? ((RealType)-1 / Radius) : ((RealType)1 / Radius);
	}


	TVector<RealType> GetPointFromAngleD(RealType AngleDeg) const
	{
		return GetPointFromAngleR(AngleDeg * TMathUtil<RealType>::DegToRad);
	}

	TVector<RealType> GetPointFromAngleR(RealType AngleRad) const
	{
		RealType c = TMathUtil<RealType>::Cos(AngleRad), s = TMathUtil<RealType>::Sin(AngleRad);
		return Frame.FromPlaneUV(TVector2<RealType>(Radius*c, Radius*s), 2);
	}

	TVector<RealType> GetPointFromUnitParameter(RealType UnitParam) const
	{
		RealType AngleRad = ((bIsReversed) ? (-UnitParam) : (UnitParam)) * TMathUtil<RealType>::TwoPi;
		return GetPointFromAngleR(AngleRad);
	}




	TVector<RealType> ClosestPoint(const TVector<RealType>& QueryPoint) const
	{
		const TVector<RealType>& Center = Frame.Origin;
		TVector<RealType> Normal = Frame.GetAxis(2);

		TVector<RealType> PointDelta = QueryPoint - Center;
		TVector<RealType> DeltaInPlane = PointDelta - Normal.Dot(PointDelta)*Normal;
		RealType OriginDist = DeltaInPlane.Length();
		if (OriginDist > (RealType)0)
		{
			return Center + (Radius / OriginDist)*DeltaInPlane;
		}
		else    // all points equidistant, use any one
		{
			return Frame.Origin + Radius * Frame.GetAxis(0);
		}
	}


	RealType DistanceSquared(const TVector<RealType>& Point) const
	{
		return Point.DistanceSquared(ClosestPoint(Point));
	}
	
	RealType Distance(const TVector<RealType>& Point) const
	{
		return TMathUtil<RealType>::Sqrt(DistanceSquared(Point));
	}

	TOrientedBox3<RealType> GetBoundingBox() const
	{
		return TOrientedBox3<RealType>(Frame, TVector<RealType>(Radius, Radius, 0));
	}

};

typedef TCircle3<float> FCircle3f;
typedef TCircle3<double> FCircle3d;


} // end namespace UE::Geometry
} // end namespace UE
