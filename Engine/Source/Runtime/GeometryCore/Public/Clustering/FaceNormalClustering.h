// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"

namespace UE::Geometry
{

class FDynamicMesh3;

namespace FaceNormalClustering
{
	struct FClusterOptions
	{
		// If one minus the dot product of normals is greater than this value, they are similar enough to be joined together.
		double NormalOneMinusCosTolerance = .01;
		// The area threshold below which we ignore the computed triangle normal, and always merge to an adjacent group. Helps avoid tiny triangles failing to cluster due to unreliable normals.
		double SmallFaceAreaThreshold = FMathd::ZeroTolerance;
		// The minimum number of cluster groups below which to stop merging clusters
		int32 TargetMinGroups = 0;
		// Whether to consider the average normal of the whole cluster before merging. If false, will only consider the normals of the original triangles within a group.
		bool bApplyNormalToleranceToClusters = true;

		void SetNormalAngleToleranceInDegrees(double ToleranceInDegrees)
		{
			NormalOneMinusCosTolerance = 1 - FMathd::Cos(FMathd::DegToRad * ToleranceInDegrees);
		}
	};

	/**
	 * Find Mesh PolyGroups by clustering neighboring faces (or polygroups) with similar normals.
	 * 
	 * @param Mesh				The Mesh to operate on
	 * @param OutPolyGroups		An array holding the array-of-triangle-IDs for each found PolyGroup
	 * @param Options			Options controlling the tolerances used for clustering
	 * @param IgnoreEdges		Optional set of edges that clusters should not be merged across
	 */
	void GEOMETRYCORE_API ComputeMeshPolyGroupsFromClusters(FDynamicMesh3& Mesh, TArray<TArray<int32>>& OutPolyGroups, const FClusterOptions& Options, TSet<int32>* IgnoreEdges = nullptr);

	/**
	 * Find the vertices at the corners of face clusters -- i.e., vertices which touch faces from three or more clusters
	 *
	 * @param Mesh				The Mesh to operate on
	 * @param OutCornerVertices	All vertices which touch at least three clusters
	 * @param Options			Options controlling the tolerances used for clustering
	 * @param IgnoreEdges		Optional set of edges that clusters should not be merged across
	 */
	 void GEOMETRYCORE_API ComputeClusterCornerVertices(FDynamicMesh3& Mesh, TArray<int32>& OutCornerVertices, const FClusterOptions& Options, TSet<int32>* IgnoreEdges = nullptr);
}

} // end namespace UE::Geometry