// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/RandomStream.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "Spatial/PriorityOrderPoints.h"
#include "Containers/StaticArray.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

// Not all platforms support c++20 or have their standard library containing floating atomic operations, 
// which is required for using std::atomic::+= opertor on floating types
#define PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD (PLATFORM_WINDOWS && (__cplusplus >= 202002L))

// Abstract array of positions for running the KMeans algorithm in serial or parallel
template<typename TVectorType, bool bUseAtomicType>
struct TClusterCenterArray
{
private:
	// Atomic values for parallel algorithm
	TStaticArray<TArray<std::atomic<typename TVectorType::FReal>>, TVectorType::NumComponents> AtomicValues;

	// Regular values for serial algorithm
	TArray<TVectorType> Values;
public:
	void SetNumZeroed(uint32 Num)
	{ 
		#if PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD
		if (bUseAtomicType)
		{
			for (uint32 ComponentIndex = 0; ComponentIndex < TVectorType::NumComponents; ++ComponentIndex)
			{
				AtomicValues[ComponentIndex].SetNumZeroed(Num);
			}
		}
		else
		#endif
		{
			Values.SetNumZeroed(Num);
		}
	}

	void Assign(int32 Index, const TVectorType& In)
	{ 
		#if PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD
		if (bUseAtomicType)
		{
			for (uint32 ComponentIndex = 0; ComponentIndex < TVectorType::NumComponents; ++ComponentIndex)
			{
				AtomicValues[ComponentIndex][Index] = In[ComponentIndex];
			}
		}
		else
		#endif
		{
			Values[Index] = In;
		}
	}

	TVectorType DivideBy(int32 Index, int32 InDivider)
	{ 
		typedef typename TVectorType::FReal NumericType;

		#if PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD
		if (bUseAtomicType)
		{
			TVectorType Out;
			for (uint32 ComponentIndex = 0; ComponentIndex < TVectorType::NumComponents; ++ComponentIndex)
			{
				Out[ComponentIndex] = NumericType(AtomicValues[ComponentIndex][Index]) / NumericType(InDivider);
			}
			return Out;
		}
		else
		#endif
		{
			return Values[Index] / NumericType(InDivider);
		}
	}

	void Accumulate(int32 Index, const TVectorType& In)
	{ 
		#if PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD
		if (bUseAtomicType)
		{
			for (uint32 ComponentIndex = 0; ComponentIndex < TVectorType::NumComponents; ++ComponentIndex)
			{
				AtomicValues[ComponentIndex][Index].fetch_add(In[ComponentIndex]);
			}
		}
		else
		#endif
		{
			Values[Index] += In;
		}
	}

	void RemoveAtSwap(size_t Index, EAllowShrinking AllowShrinking)
	{
		#if PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD
		if (bUseAtomicType)
		{
			for (uint32 ComponentIndex = 0; ComponentIndex < TVectorType::NumComponents; ++ComponentIndex)
			{
				AtomicValues[ComponentIndex].RemoveAtSwap(Index, AllowShrinking);
			}
		}
		else
		#endif
		{
			Values.RemoveAtSwap(Index, AllowShrinking);
		}
	}
};

struct FClusterKMeans
{
	/// Parameters

	// Max iterations of K-Means clustering. Will use fewer iterations if the clustering method converges.
	int32 MaxIterations = 500;

	// Random Seed used to initialize clustering (if InitialCenters are not provided)
	int32 RandomSeed = 0;

	/// Outputs

	// Mapping from input points to cluster IDs
	TArray<int32> ClusterIDs;

	// Number of points in each cluster
	TArray<std::atomic<int32>> ClusterSizes;

	/**
	 * Compute the K-Means clustering of FVector points
	 * 
	 * @param PointsToCluster	Points to partition into clusters
	 * @param NumClusters		Target number of clusters to create, if InitialCenters is not provided
	 * @param InitialCenters	If non-empty, these positions will be used to initialize the cluster locations
	 * @param OutClusterCenters	If non-null, will be filled with the cluster centers
	 * @param bRunParallel		Runs a parallel algorithm (faster) if possible. The output might be non-deterministic when running in parallel
	 * @return number of clusters found
	 */
	template<typename TVectorType = FVector, bool bRunParallel=false>
	int32 ComputeClusters(
		TArrayView<const TVectorType> PointsToCluster, int32 NumClusters,
		TArrayView<const TVectorType> InitialCenters = TArrayView<const TVectorType>(), 
		TArray<TVectorType>* OutClusterCenters = nullptr)
	{
		constexpr bool bRunParallelIfCompatible = PLATFORM_SUPPORT_FLOATING_INTERLOCKED_ADD && bRunParallel && std::atomic<typename TVectorType::FReal>::is_always_lock_free;

		if (PointsToCluster.IsEmpty())
		{
			// nothing to cluster
			return 0;
		}

		// Initialize cluster centers
		TArray<TVectorType> LocalCenters;
		TArray<TVectorType>* UseCenters = &LocalCenters;
		if (OutClusterCenters)
		{
			UseCenters = OutClusterCenters;
			UseCenters->Reset();
		}
		if (InitialCenters.IsEmpty())
		{
			TArray<int32> Ordering;
			const int32 OrderingNum = PointsToCluster.Num();
			Ordering.SetNumUninitialized(OrderingNum);
			for (int32 Idx = 0; Idx < OrderingNum; ++Idx)
			{
				Ordering[Idx] = Idx;
			}
			// Shuffle the first NumClusters indices and use them as the initial centers
			NumClusters = FMath::Min(PointsToCluster.Num(), NumClusters);
			UseCenters->Reserve(NumClusters);
			FRandomStream RandomStream(RandomSeed);
			for (int32 Idx = 0; Idx < NumClusters; ++Idx)
			{
				Swap(Ordering[Idx], Ordering[Idx + RandomStream.RandHelper(OrderingNum - Idx)]);
				UseCenters->Add(PointsToCluster[Ordering[Idx]]);
			}
		}
		else
		{
			// Note: We intentionally do not check if more centers were provided than points here.
			// The excess centers will instead be removed below when no points are assigned to them.
			UseCenters->Append(InitialCenters);
		}

		NumClusters = UseCenters->Num();
		ClusterIDs.Init(-1, PointsToCluster.Num());
		ClusterSizes.SetNumZeroed(NumClusters);

		if (NumClusters == 0)
		{
			return NumClusters;
		}
		else if (NumClusters == 1)
		{
			for (int32 PointIdx = 0; PointIdx < PointsToCluster.Num(); ++PointIdx)
			{
				ClusterIDs[PointIdx] = 0;
			}
			ClusterSizes[0] = PointsToCluster.Num();
			return NumClusters;
		}

		
		TClusterCenterArray<TVectorType, bRunParallelIfCompatible> NextCenters;
		NextCenters.SetNumZeroed(NumClusters);

		TArray<int32> ClusterIDRemap; // array to use if we need to remap cluster IDs due to an empty cluster

		int32 UseMaxIterations = FMath::Max(1, MaxIterations); // always use at least one iteration to make sure the initial assignment happens
		for (int32 Iterations = 0; Iterations < UseMaxIterations; ++Iterations)
		{
			bool bClustersChanged = Iterations == 0; // clusters always change on first iteration; otherwise we must check
			if (bRunParallelIfCompatible)
			{
				ParallelFor(PointsToCluster.Num(), 
				[
					this,
					Iterations,
					&PointsToCluster,
					&NextCenters,
					&UseCenters,
					&bClustersChanged
				] (uint32 PointIdx)
				{
					const TVectorType& Point = PointsToCluster[PointIdx];
					double ClosestDistSq = (double)UE::Geometry::DistanceSquared(Point, (*UseCenters)[0]);
					int32 ClosestCenter = 0;
					for (int32 CenterIdx = 1; CenterIdx < UseCenters->Num(); ++CenterIdx)
					{
						double DistSq = UE::Geometry::DistanceSquared(PointsToCluster[PointIdx], (*UseCenters)[CenterIdx]);
						if (DistSq < ClosestDistSq)
						{
							ClosestDistSq = DistSq;
							ClosestCenter = CenterIdx;
						}
					}

					ClusterSizes[ClosestCenter]++;
					if (Iterations > 0)
					{
						int32 OldClusterID = ClusterIDs[PointIdx];
						ClusterSizes[OldClusterID]--;
						if (OldClusterID != ClosestCenter)
						{
							bClustersChanged = true;
						}
					}
				
					ClusterIDs[PointIdx] = ClosestCenter;
					NextCenters.Accumulate(ClosestCenter, Point);
				});
			}
			else
			{
			for (int32 PointIdx = 0; PointIdx < PointsToCluster.Num(); ++PointIdx)
			{
				const TVectorType& Point = PointsToCluster[PointIdx];
				double ClosestDistSq = (double)DistanceSquared(Point, (*UseCenters)[0]);
				int32 ClosestCenter = 0;
				for (int32 CenterIdx = 1; CenterIdx < UseCenters->Num(); ++CenterIdx)
				{
					double DistSq = DistanceSquared(PointsToCluster[PointIdx], (*UseCenters)[CenterIdx]);
					if (DistSq < ClosestDistSq)
					{
						ClosestDistSq = DistSq;
						ClosestCenter = CenterIdx;
					}
				}

				ClusterSizes[ClosestCenter]++;
				if (Iterations > 0)
				{
					int32 OldClusterID = ClusterIDs[PointIdx];
					ClusterSizes[OldClusterID]--;
					if (OldClusterID != ClosestCenter)
					{
						bClustersChanged = true;
					}
				}
				
				ClusterIDs[PointIdx] = ClosestCenter;
				NextCenters.Accumulate(ClosestCenter, Point);
			}
			} // bParallel

			// Stop iterating if clusters are unchanged
			if (!bClustersChanged)
			{
				break;
			}

			// Update cluster centers and detect/delete any empty clusters
			bool bDeletedClusters = false;
			for (int32 ClusterIdx = 0; ClusterIdx < UseCenters->Num(); ++ClusterIdx)
			{
				if (ClusterSizes[ClusterIdx] > 0)
				{
					(*UseCenters)[ClusterIdx] = NextCenters.DivideBy(ClusterIdx, int32(ClusterSizes[ClusterIdx]));
					NextCenters.Assign(ClusterIdx, TVectorType::ZeroVector);
				}
				else
				{
					if (!bDeletedClusters)
					{
						ClusterIDRemap.SetNumUninitialized(UseCenters->Num(), EAllowShrinking::No);
						for (int32 Idx = 0; Idx < UseCenters->Num(); ++Idx)
						{
							ClusterIDRemap[Idx] = Idx;
						}
					}
					bDeletedClusters = true;
					ClusterIDRemap[ClusterSizes.Num() - 1] = ClusterIdx;
					ClusterSizes.RemoveAtSwap(ClusterIdx, EAllowShrinking::No);
					UseCenters->RemoveAtSwap(ClusterIdx, EAllowShrinking::No);
					NextCenters.RemoveAtSwap(ClusterIdx, EAllowShrinking::No);
				}
			}
			if (bDeletedClusters)
			{
				checkSlow(!ClusterSizes.IsEmpty());
				checkSlow(ClusterSizes.Num() == UseCenters->Num());
				for (int32& ClusterID : ClusterIDs)
				{
					ClusterID = ClusterIDRemap[ClusterID];
				}
			}
		}

		return ClusterSizes.Num();
	}

	/**
	 * Helper function to generate (approximately) uniform-spaced initial clusters centers, which can be passed to ComputeClusters.
	 */
	template<typename TVectorType = FVector>
	void GetUniformSpacedInitialCenters(TArrayView<const TVectorType> PointsToCluster, int32 NumClusters, TArray<TVectorType>& OutCenters)
	{
		FPriorityOrderPoints OrderPoints;
		OrderPoints.ComputeUniformSpaced(PointsToCluster, TArrayView<const float>(), NumClusters);
		int32 NumOut = FMath::Min(NumClusters, OrderPoints.Order.Num());
		OutCenters.Reset(NumOut);
		for (int32 OrderIdx = 0; OrderIdx < NumOut; ++OrderIdx)
		{
			OutCenters.Add(PointsToCluster[OrderPoints.Order[OrderIdx]]);
		}
	}


	void GEOMETRYCORE_API GetClusters(TArray<TArray<int32>>& OutClusters);
};

} // end namespace UE::Geometry
} // end namespace UE