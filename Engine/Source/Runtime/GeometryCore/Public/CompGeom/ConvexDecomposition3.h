// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "IndexTypes.h"
#include "Math/Sphere.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"
#include "PlaneTypes.h"
#include "TransformTypes.h"
#include "VectorTypes.h"

namespace UE {
namespace Geometry {

// Forward declare fast winding tree for negative space sampling
template<class TriangleMeshType>
class TFastWindingTree;

// TODO: To support meshes where volume is not well defined (e.g., open boundaries or many self-intersecting parts), we'll need alternative error metrics
enum class EConvexErrorMethod
{
	//GeometryToHullRayDistance,
	//HullToGeometryDistance,
	//BothDistances,
	RelativeVolume
};

struct FNegativeSpaceSampleSettings
{
	FNegativeSpaceSampleSettings() = default;
	FNegativeSpaceSampleSettings(int32 TargetNumSamples, double MinSpacing, double ReduceRadiusMargin)
		: TargetNumSamples(TargetNumSamples), MinSpacing(MinSpacing), ReduceRadiusMargin(ReduceRadiusMargin)
	{}

	enum class ESampleMethod : uint8
	{
		// Place sample spheres in a uniform grid pattern
		Uniform,
		// Use voxel-based subtraction and offsetting methods to specifically target concavities
		VoxelSearch,
		// Use a variant of VoxelSearch that aims to limit negative space to the space that can be accessed by a ball of radius >= MinRadius
		NavigableVoxelSearch
	};

	// Enum to manage settings configurations for FNegativeSpaceSampleSettings
	enum class EConfigDefaults : uint8
	{
		Latest
	};

	// Note: Class member default values are held fixed to avoid changing results for existing callers
	// Use this method to request newer, recommended defaults
	void ApplyDefaults(EConfigDefaults Settings = EConfigDefaults::Latest)
	{
		SampleMethod = ESampleMethod::NavigableVoxelSearch;
		MarchingCubesGridScale = 1.0;
		MaxVoxelsPerDim = 1024;
		MinSpacing = 0.0;
		TargetNumSamples = 0;
		VoxelExpandBoundsFactor = UE_DOUBLE_KINDA_SMALL_NUMBER;
	}

	// Method used to place samples
	ESampleMethod SampleMethod = ESampleMethod::Uniform;

	// Target number of spheres to use to cover negative space. Intended as a rough guide; actual number used may be larger or smaller.
	int32 TargetNumSamples = 30;

	// Minimum desired spacing between sampling spheres; not a strictly enforced bound.
	double MinSpacing = 3.0;
	
	// Whether to allow samples w/ center inside other negative space spheres
	bool bAllowSamplesInsideSpheres = false;

	// Space to allow between spheres and actual surface
	double ReduceRadiusMargin = 3.0;

	// Ignore spheres with smaller radius than this
	double MinRadius = 10.0;

	// Below options currently only apply to VoxelSearch.
	
	// Whether to require that all candidate sample locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing.
	// Note: This takes priority over TargetNumSamples if the TargetNumSamples did not achieve the required coverage.
	bool bRequireSearchSampleCoverage = false;
	// Whether to only consider negative space that is connected to the bounding convex hull, i.e., to ignore hollow inner pockets of negative space that cannot be reached from the outside (for VoxelSearch and Navigable methods)
	bool bOnlyConnectedToHull = false;
	// Maximum number of voxels to use per dimension, when performing VoxelSearch
	int32 MaxVoxelsPerDim = 128;
	// Attempt to keep negative space computation deterministic, at some additional runtime cost
	bool bDeterministic = true;
	// Scale factor for marching cubes grid used for VoxelSearch-based methods
	double MarchingCubesGridScale = .5;

	// Whether to allow samples to be added inside the mesh, based on winding number. Can enabled for non-solid meshes; note the convex decomposition should then set bTreatAsSolid to false as well.
	bool bAllowSamplesInsideMesh = false;

	// How much to expand the bounding box used for voxel search algorithms
	double VoxelExpandBoundsFactor = 1.0;

	// Optional function to define an obstacle SDF which the negative space should also stay out of. Can be used for example to ignore anything below a ground plane.
	// Note: Assumed to be in world space
	TFunction<double(FVector Pos)> OptionalObstacleSDF;

	// Optional function to define a bias (scalar multiplier) for the ReduceRadiusMargin value, to support scaling the tolerance based on location or direction to surfaces.
	// The custom method takes the reference mesh, nearest triangle ID on that mesh, and the query position,
	// and returns a scale factor between 0 and 1 to apply to the ReduceRadiusMargin
	TFunction<double(const FDynamicMesh3& Mesh, int32 NearTID, FVector3d QueryPos)> CustomRadiusMarginScale;

	// Margin scales should be between this value and 1.0. When the MinCustomMarginScale is 1.0, no custom scaling is applied.
	// Should be greater than 0. Note that the sampling will be more expensive to compute for smaller minimum scale values.
	double MinCustomMarginScale = 1.0;

	// @return the scale factor that has been applied by Rescale()
	double GetAppliedScaleFactor() const
	{
		return AppliedScaleFactor;
	}

	UE_DEPRECATED(5.4, "Use SetResultTransform instead")
	void Rescale(double ScaleFactor)
	{
		RescaleSettings(ScaleFactor);
	}
	
	void SetResultTransform(FTransform InResultTransform)
	{
		RescaleSettings(ResultTransform.GetScale3D().X / InResultTransform.GetScale3D().X);
		ResultTransform = InResultTransform;
	}
	FTransform GetResultTransform() const
	{
		return ResultTransform;
	}

	// Make sure the settings values are in valid ranges
	void Sanitize()
	{
		TargetNumSamples = FMath::Max(1, TargetNumSamples);
		MinSpacing = FMath::Max(0.0, MinSpacing);
		ReduceRadiusMargin = FMath::Max(0.0, ReduceRadiusMargin);
		MinRadius = FMath::Max(0.0, MinRadius);
		MaxVoxelsPerDim = FMath::Clamp(MaxVoxelsPerDim, 4, 1290);
		MinCustomMarginScale = FMath::Clamp(MinCustomMarginScale, .1, 1.0);
	}

	// helper to evaluate OptionalObstacleSDF for local positions
	inline double ObstacleDistance(const FVector3d& LocalPos) const
	{
		checkSlow(OptionalObstacleSDF);
		FVector3d WorldPos = ResultTransform.TransformPosition(LocalPos);
		double WorldSD = OptionalObstacleSDF(WorldPos);
		return WorldSD * AppliedScaleFactor;
	}
	
private:

	// Track how the settings have been rescaled
	double AppliedScaleFactor = 1;

	// Track how to transform back to the original space
	FTransform ResultTransform = FTransform::Identity;
	
	void RescaleSettings(double ScaleFactor)
	{
		MinSpacing *= ScaleFactor;
		ReduceRadiusMargin *= ScaleFactor;
		MinRadius *= ScaleFactor;
		AppliedScaleFactor *= ScaleFactor;
	}
};

// Define a volume with a set of spheres
class FSphereCovering
{
public:

	inline int32 Num() const
	{
		return Position.Num();
	}

	inline FVector3d GetCenter(int32 Idx) const
	{
		return Position[Idx];
	}

	inline double GetRadius(int32 Idx) const
	{
		return Radius[Idx];
	}

	inline void SetRadius(int32 Idx, double UpdateRadius)
	{
		Radius[Idx] = UpdateRadius;
	}

	// Add spheres covering the negative space of the given fast winding tree
	// @param Spatial	Fast winding tree of the reference mesh, used to compute negative space
	// @param Settings	Settings controlling how negative space is computed
	// @param bHasFlippedTriangles	Whether the mesh referenced by Spatial has reversed triangle windings
	// @return true if any spheres were added
	GEOMETRYCORE_API bool AddNegativeSpace(const TFastWindingTree<FDynamicMesh3>& Spatial, const FNegativeSpaceSampleSettings& Settings, bool bHasFlippedTriangles);
	
	// Note: This version of AddNegativeSpace assumed the input had flipped triangle orientations
	UE_DEPRECATED(5.4, "Use the version of this function with a bHasFlippedTriangles parameter")
	bool AddNegativeSpace(const TFastWindingTree<FDynamicMesh3>& Spatial, const FNegativeSpaceSampleSettings& Settings)
	{
		return AddNegativeSpace(Spatial, Settings, true);
	}

	void Reset()
	{
		Position.Reset();
		Radius.Reset();
	}

	void Append(const FSphereCovering& Other)
	{
		Position.Append(Other.Position);
		Radius.Append(Other.Radius);
	}

	// Remove spheres with radius below a threshold
	void RemoveSmaller(double MinRadius)
	{
		for (int32 Idx = 0; Idx < Radius.Num(); ++Idx)
		{
			if (Radius[Idx] < MinRadius)
			{
				Radius.RemoveAtSwap(Idx, EAllowShrinking::No);
				Position.RemoveAtSwap(Idx, EAllowShrinking::No);
				--Idx;
			}
		}
	}

	void AppendSpheres(TArrayView<const UE::Math::TSphere<double>> Spheres)
	{
		const int32 AddCount = Spheres.Num();
		const int32 OrigCount = Position.Num();
		Position.SetNum(AddCount + OrigCount);
		Radius.SetNum(AddCount + OrigCount);
		for (int32 Idx = 0; Idx < AddCount; ++Idx)
		{
			Position[Idx + OrigCount] = Spheres[Idx].Center;
			Radius[Idx + OrigCount] = Spheres[Idx].W;
		}
	}

	void AddSphere(FVector3d InCenter, double InRadius)
	{
		Position.Add(InCenter);
		Radius.Add(InRadius);
	}

	// Apply transform to spheres
	// Note: Non-uniform scales will scale the radius by the smallest scale axis's magnitude (matching how physics collision transforms spheres)
	void ApplyTransform(const FTransform& Transform)
	{
		TransformRange(0, Position.Num(), Transform);
	}

	// Append spheres from Other w/ the given Transform
	// Note: Non-uniform scales will scale the radius by the smallest scale axis's magnitude (matching how physics collision transforms spheres)
	void AppendTransformed(const FSphereCovering& Other, const FTransform& Transform)
	{
		int32 AppendStart = Position.Num();
		Position.Append(Other.Position);
		Radius.Append(Other.Radius);
		TransformRange(AppendStart, Position.Num(), Transform);
	}

private:
	// Sphere centers
	TArray<FVector3d> Position;
	// Sphere radii
	TArray<double> Radius;

	void TransformRange(const int32 Start, const int32 Max, const FTransform& Transform)
	{
		for (int32 Idx = Start; Idx < Max; ++Idx)
		{
			Position[Idx] = Transform.TransformPosition(Position[Idx]);
		}
		double RadiusScale = Transform.GetScale3D().GetAbsMin();
		for (int32 Idx = Start; Idx < Max; ++Idx)
		{
			Radius[Idx] *= RadiusScale;
		}
	}
};



class FConvexDecomposition3
{

public:

	FConvexDecomposition3() : ResultTransform(FTransformSRT3d::Identity())
	{
	}

	FConvexDecomposition3(const FDynamicMesh3& SourceMesh, bool bMergeEdges = true)
	{
		InitializeFromMesh(SourceMesh, bMergeEdges);
	}

	struct FPreprocessMeshOptions
	{
		// Whether to attempt to merge boundary edges before processing
		bool bMergeEdges = true;

		// If greater than zero, we can 'inflate' the input shape by this amount along any degenerate axes in cases where the hull failed to construct due to the input being coplanar/colinear
		// Note: If passed to the FConvexDecomposition3 constructor, will also set the similar ThickenAfterHullFailure 
		// member, which applies to generated parts of the decomposition.
		double ThickenInputAfterHullFailure = 0.0;

		// CustomPreprocess takes the mesh to process, and that mesh's bounding box, and updates the mesh in place
		// Applied after discarding attributes and applying Merge Edges (if enabled)
		// Can be used e.g. to simplify/repair/solidify the input
		TUniqueFunction<void(FDynamicMesh3& Mesh, const FAxisAlignedBox3d& Bounds)> CustomPreprocess = nullptr;
	};

	FConvexDecomposition3(const FDynamicMesh3& SourceMesh, const FPreprocessMeshOptions& Options)
	{
		ThickenAfterHullFailure = Options.ThickenInputAfterHullFailure;
		InitializeFromMesh(SourceMesh, Options);
	}

	/**
	 * Test whether the input mesh (after preprocessing) is solid. Must be called after initialization, but before decomposition is computed.
	 */
	bool IsInputSolid()
	{
		if (!ensureMsgf(Decomposition.Num() == 1 && !Decomposition[0].IsCompact(), TEXT("This method must be called after initializing with the input mesh, and before the decomposition has been computed.")))
		{
			return false;
		}
		return Decomposition[0].InternalGeo.IsClosed();
	}

	/**
	 * Initialize from convex hulls allows the caller to only use the hull merging phase of the algorithm
	 * @param NumHulls			Number of convex hulls in the initial decomposition
	 * @param HullVolumes		Function from Hull Index -> Hull Volume
	 * @param HullNumVertices	Function from Hull Index -> Hull Vertex Count
	 * @param HullVertices		Function from Hull Index, Vertex Index -> Hull Vertex Position
	 * @param Proximity			All the local proximities for the hulls. Hulls will not be merged unless they are connected by this proximity graph.
	 */
	GEOMETRYCORE_API void InitializeFromHulls(int32 NumHulls, TFunctionRef<double(int32)> HullVolumes, TFunctionRef<int32(int32)> HullNumVertices, TFunctionRef<FVector3d(int32, int32)> HullVertices, TArrayView<const TPair<int32, int32>> Proximity);

	/**
	 * Create the proximity graph from the current decomposition, using bounding box overlaps. To consider non-overlapping proximity, the bounding boxes can be expanded by a factor of their own size or by an absolute amount.
	 * @param BoundsExpandByMinDimFactor		Part bounds will be expanded by at least this factor of their own min dimension, before finding overlaps
	 * @param BoundsExpandByMaxDimFactor		Part bounds will be expanded by at least this factor of their own max dimension, before finding overlaps
	 * @param MinBoundsExpand					Part bounds will be expanded by at least this fixed amount, before finding overlaps
	 */
	GEOMETRYCORE_API void InitializeProximityFromDecompositionBoundingBoxOverlaps(double BoundsExpandByMinDimFactor, double BoundsExpandByMaxDimFactor, double MinBoundsExpand);

	GEOMETRYCORE_API void InitializeFromMesh(const FDynamicMesh3& SourceMesh, bool bMergeEdges);
	GEOMETRYCORE_API void InitializeFromMesh(const FDynamicMesh3& SourceMesh, const FPreprocessMeshOptions& Options);

	/**
	 * Initialize convex decomposition with a triangle index mesh
	 * @param Vertices			Vertex buffer for mesh to decompose
	 * @param Faces				Triangle buffer for mesh to decompose
	 * @param bMergeEdges		Whether to attempt to weld matching edges before computing the convex hull; this can help the convex decomposition find better cutting planes for meshes that have boundaries e.g. due to seams
	 * @param FaceVertexOffset	Indices from the Faces array are optionally offset by this value.  Useful e.g. to take slices of the multi-geometry vertex and face buffers of FGeometryCollection.
	 */
	GEOMETRYCORE_API void InitializeFromIndexMesh(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector> Faces, bool bMergeEdges, int32 FaceVertexOffset = 0);
	/**
	 * Initialize convex decomposition with a triangle index mesh
	 * @param Vertices			Vertex buffer for mesh to decompose
	 * @param Faces				Triangle buffer for mesh to decompose
	 * @param Options			Options for pre-processing the mesh before decomposition (e.g., to weld matching edges, thicken degenerate inputs, or to run a general pre-process lambda function)
	 * @param FaceVertexOffset	Indices from the Faces array are optionally offset by this value.  Useful e.g. to take slices of the multi-geometry vertex and face buffers of FGeometryCollection.
	 */
	GEOMETRYCORE_API void InitializeFromIndexMesh(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector> Faces, const FPreprocessMeshOptions& Options, int32 FaceVertexOffset = 0);

	/**
	 * Find and set a new negative space that should be protected (clearing any existing). Uses the mesh passed on construction or to InitializeFromMesh or InitializeFromIndexMesh.
	 * @param Settings	Settings to use to find the negative space
	 * @param RequestedSamples Optional points where we should sample the negative space, in addition to spheres automatically added via the Settings. Settings still apply to filter (bAllowSamplesInsideMesh) and to determine sphere radii.
	 * @return			False on failure -- e.g., if there was no mesh available
	 */
	GEOMETRYCORE_API bool InitializeNegativeSpace(const FNegativeSpaceSampleSettings& Settings, TArrayView<const FVector3d> RequestedSamples = TArrayView<const FVector3d>());

	/**
	 * Add a sphere covering to existing negative space. Light-weight compared to InitializeNegativeSpace: Does not filter spheres that overlap or are inside the mesh geometry.
	 * @param Covering The spheres to add.
	 * @param CoveringToLocalSpace Matrix transform from input covering space to the local space of input convex decomposition geometry
	 * @param RadiusToLocalScale Scale to transform sphere radii to the local space of the convex decomposition. Note spheres cannot be non-uniformly scaled in the local space.
	 */
	GEOMETRYCORE_API bool AddCoveringToInitialNegativeSpace(const FSphereCovering& Covering, FMatrix CoveringToLocalSpace = FMatrix::Identity, double RadiusToLocalScale = 1.);

	//
	// Settings
	//

	// Note we automatically transform the input to a centered-unit-cube space, and tolerances are measured in that space
	// In other words, distance-based tolerances are expressed as a fraction of the bounding box max axis
	
	// Threshold for considering two convex parts close enough to consider merging
	double ProximityTolerance = 1e-3;
	// Threshold for considering two components to be touching
	double ConnectedComponentTolerance = 1e-3;
	// Maximum 'on plane' distance for points during planar cuts
	double OnPlaneTolerance = 1e-7;

	// Scale factor for error on the largest axis; slightly favoring cutting on the largest axis helps get more consistent result when no initial cut will reduce error (e.g. a torus)
	double CutLargestAxisErrorScale = .99;

	// Angle at edge above which we take 5 sample planes instead of 3, to better cover the wider range (expressed in degrees)
	double ConvexEdgeAngleMoreSamplesThreshold = 30; 
	// Angle at edge above which we don't sample it as a convex edge (expressed in degrees)
	double ConvexEdgeAngleThreshold = 170;

	// A bias term to favor merging away 'too thin' parts before merging other parts, expressed as a volume in units s.t. the longest axis of the original input bounding box is 1 unit long
	// Because these merges *must* be done, it's generally safer to do them earlier, rather than hoping the error associated with them will go down after other merges are performed
	double BiasToRemoveTooThinParts = .1;

	// Maximum number of convex edges to sample for possible cutting planes when calling SplitWorst() to generate an initial convex decomposition
	// Larger values will cost more to run but can let the algorithm find a cleaner decomposition
	int32 MaxConvexEdgePlanes = 50;

	// Whether to, after each split, also detect whether a part is made up of disconnected pieces and further split them if so
	bool bSplitDisconnectedComponents = true;

	// Whether to treat the input mesh as a solid shape -- controls whether, e.g., a hole-fill is performed after splitting the mesh to close off the resulting parts
	bool bTreatAsSolid = true;

	// If greater than zero, we can 'inflate' the input shape by this amount along any degenerate axes in cases where the hull failed to construct due to the input being coplanar/colinear.
	// Note: This should be less than the smallest tolerance used negative space spheres. Do not use this parameter to control minimum thickness of the output shapes; that can be better enforced later (after merges).
	double ThickenAfterHullFailure = 0;


	// If > 0, search for best merges will be restricted to a greedy local search after testing this many connections. Helpful when there are many potential merges.
	int32 RestrictMergeSearchToLocalAfterTestNumConnections = -1;

	// TODO: Provide hull approximation options?


	/**
	 * Compute a decomposition with the desired number of hulls
	 * Note: A future version of this function may replace NumOutputHulls with MaxOutputHulls, but this version keeps both parameters for compatibility / consistent behavior.
	 * 
	 * @param NumOutputHulls		Number of convex hulls to use in the final convex decomposition
	 * @param NumAdditionalSplits	How far to go beyond the target number of hulls when initially the mesh into pieces -- larger values will require more computation but can find better convex decompositions
	 * @param ErrorTolerance		Stop splitting when hulls have error less than this (expressed in cm; will be cubed for volumetric error). Overrides NumOutputHulls if specified
	 * @param MinThicknessTolerance	Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Overrides NumOutputHulls and ErrorTolerance when needed
	 * @param MaxOutputHulls		If > 0, maximum number of convex hulls to generate. Overrides ErrorTolerance and TargetNumParts when needed
	 * @param bOnlySplitIfOverlapNegativeSpace	If true, use NegativeSpace to guide splits, and only split parts that overlap negative space
	 */
	GEOMETRYCORE_API void Compute(int32 NumOutputHulls, int32 NumAdditionalSplits = 10, double ErrorTolerance = 0.0, double MinThicknessTolerance = 0, int32 MaxOutputHulls = -1, bool bOnlySplitIfNegativeSpaceCovered = false);

	// Split the worst convex part, and return the increase in the total number of convex parts after splitting (can be more than 1 if result has multiple separate connected components)
	// Note: could return 0 if no splits were possible
	// @param bCanSkipUnreliableGeoVolumes		if true, don't split hulls where we have questionable geometry volume results, unless there is no hull with good geometry volume results
	// @param bOnlySplitIfNegativeSpaceCovered	if true, don't split hulls unless they overlap with some covered Negative Space (stored in the corresponding member variable)
	// @param MinSplitSize						if > 0, don't split hulls with max bounds dimension lower than this
	GEOMETRYCORE_API int32 SplitWorst(bool bCanSkipUnreliableGeoVolumes = false, double ErrorTolerance = 0.0, bool bOnlySplitIfNegativeSpaceCovered = false, double MinSplitSizeInWorldSpace = -1);
	
	struct FConvexPart;

	struct FMergeSettings
	{
		// Desired number of output parts
		int32 TargetNumParts = 1;
		// If > 0, allow merging parts with error less than this tolerance
		double ErrorTolerance = 0;
		// Whether ErrorTolerance is allowed to keep merging below the TargetNumParts
		bool bErrorToleranceOverridesNumParts = true;
		// If > 0, maximum number of output parts; overrides ErrorTolerance and TargetNumParts if they would create more parts than this
		int32 MaxOutputHulls = -1;
		
		// Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Overrides TargetNumParts and ErrorTolerance when needed.
		double MinThicknessTolerance = 0;
		// Allow the algorithm to discard underlying geometry once it will no longer be used, resulting in a smaller representation and faster merges
		bool bAllowCompact = true;
		// Require all hulls to have associated triangles after MergeBest is completed. (If using InitializeFromHulls, will need to triangulate any un-merged hulls.)
		bool bRequireHullTriangles = false;
		
		// Optional representation of negative space that should not be covered by merges
		const FSphereCovering* OptionalNegativeSpace = nullptr;
		// Optional transform from space of the convex hulls into the space of the sphere covering; if not provided, assume spheres are in the same coordinate space as the hulls
		const FTransform* OptionalTransformIntoNegativeSpace = nullptr;

		// Optional callback, to be called when two parts are merged. Takes the original (pre-merge) convex decomposition part indices as input.
		TFunction<void(int32, int32)> MergeCallback = nullptr;

		// Optional custom function to control whether parts are allowed to merge
		TFunction<bool(const FConvexDecomposition3::FConvexPart& A, const FConvexDecomposition3::FConvexPart& B)> CustomAllowMergeParts = nullptr;
	};

	// Merge the pairs of convex hulls in the decomposition that will least increase the error.  Intermediate results can be used across merges, so it is best to do all merges in one call.
	// @param TargetNumParts		The target number of parts for the decomposition; will be overriden by non-default ErrorTolerance or MinPartThickness
	// @param ErrorTolerance		If > 0, continue to merge (if there are possible merges) until the resulting error would be greater than this value. Overrides TargetNumParts as the stopping condition.
	//								Note: ErrorTolerance is expressed in cm, and will be cubed for volumetric error.
	// @param MinPartThickness		Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Overrides TargetNumParts and ErrorTolerance when needed.
	//								Note: These parts may be further split so they can be merged into multiple hulls
	// @param bAllowCompact			Allow the algorithm to discard underlying geometry once it will no longer be used, resulting in a smaller representation & faster merges
	// @param bRequireHullTriangles	Require all hulls to have associated triangles after MergeBest is completed. (If using InitializeFromHulls, will need to triangulate any un-merged hulls.)
	// @param MaxOutputHulls		If > 0, maximum number of convex hulls to generate. Overrides ErrorTolerance and TargetNumParts when needed
	// @param OptionalNegativeSpace Optional representation of negative space that should not be covered by merges
	// @param OptionalNegativeSpaceTransform Optional transform from space of the convex hulls into the space of the sphere covering; if not provided, assume spheres are in the same coordinate space as the hulls
	// @return						The number of merges performed
	GEOMETRYCORE_API int32 MergeBest(int32 TargetNumParts, double ErrorTolerance = 0, double MinThicknessTolerance = 0, bool bAllowCompact = true, bool bRequireHullTriangles = false, int32 MaxOutputHulls = -1,
		const FSphereCovering* OptionalNegativeSpace = nullptr, const FTransform* OptionalTransformIntoNegativeSpace = nullptr);

	// Merge the pairs of convex hulls in the decomposition that will least increase the error.  Intermediate results can be used across merges, so it is best to do all merges in one call.
	// @return The number of merges performed
	GEOMETRYCORE_API int32 MergeBest(const FMergeSettings& Settings);

	// simple helper to convert an error tolerance expressed in world space to a local-space volume tolerance
	inline double ConvertDistanceToleranceToLocalVolumeTolerance(double DistTolerance) const
	{
		double LocalDist = ConvertDistanceToleranceToLocalSpace(DistTolerance);
		double LocalVolume = LocalDist * LocalDist * LocalDist;
		return LocalVolume;
	}

	// simple helper to convert an error tolerance expressed in world space to local space
	inline double ConvertDistanceToleranceToLocalSpace(double DistTolerance) const
	{
		double ScaleFactor = ResultTransform.GetScale().GetMax();
		if (!ensure(ScaleFactor >= FMathd::Epsilon)) // If ScaleFactor is zero, ResultTransform is trying to collapse all the hulls to a degenerate space -- likely indicates an incorrect ResultTransform
		{
			ScaleFactor = FMathd::Epsilon;
		}
		double LocalDist = DistTolerance / ScaleFactor;
		return LocalDist;
	}

	// Compact the decomposition representation to the minimal needed for output: Just the hull vertices and triangles, no internal geometry
	// This will prevent some further processing on the representation; to ensure it is called only when the geometry is no longer needed,
	// instead of calling this directly, call MergeBest with bAllowCompact=true
	void Compact()
	{
		for (FConvexPart& Part : Decomposition)
		{
			Part.Compact();
		}
	}

	//
	// Results
	//

	int32 NumHulls() const
	{
		return Decomposition.Num();
	}

	/** @return convex hull triangles */
	TArray<FIndex3i> const& GetTriangles(int32 HullIdx) const
	{
		return Decomposition[HullIdx].HullTriangles;
	}

	template<typename RealType>
	TArray<TVector<RealType>> GetVertices(int32 HullIdx, bool bTransformedToOutput = true) const
	{
		TArray<TVector<RealType>> Vertices;
		Vertices.SetNum(Decomposition[HullIdx].InternalGeo.MaxVertexID());
		for (int32 VID = 0; VID < Decomposition[HullIdx].InternalGeo.MaxVertexID(); VID++)
		{
			if (Decomposition[HullIdx].InternalGeo.IsVertex(VID))
			{
				Vertices[VID] = (TVector<RealType>) ResultTransform.TransformPosition(Decomposition[HullIdx].InternalGeo.GetVertex(VID));
			}
		}
		return Vertices;
	}

	FDynamicMesh3 GetHullMesh(int32 HullIdx) const
	{
		FDynamicMesh3 HullMesh;
		for (int32 VID = 0; VID < Decomposition[HullIdx].InternalGeo.MaxVertexID(); VID++)
		{
			if (Decomposition[HullIdx].InternalGeo.IsVertex(VID))
			{
				HullMesh.AppendVertex(ResultTransform.TransformPosition(Decomposition[HullIdx].InternalGeo.GetVertex(VID)));
			}
			else
			{
				HullMesh.AppendVertex(FVector3d::ZeroVector);
			}
		}
		for (const FIndex3i& Tri : Decomposition[HullIdx].HullTriangles)
		{
			HullMesh.AppendTriangle(Tri);
		}
		return HullMesh;
	}

	// Get the non-hull geometry underlying a part of the convex decomposition
	const FDynamicMesh3& GetInternalMesh(int32 HullIdx) const
	{
		return Decomposition[HullIdx].InternalGeo;
	}

	const int32 GetHullSourceID(int32 HullIdx) const
	{
		return Decomposition[HullIdx].HullSourceID;
	}

	const int32 CountMergedParts() const
	{
		int32 Count = 0;
		for (int32 HullIdx = 0; HullIdx < Decomposition.Num(); ++HullIdx)
		{
			Count += int32(Decomposition[HullIdx].HullSourceID < 0);
		}
		return Count;
	}

	// Representation of a convex hull in the decomposition + associated information to help further split or merge
	struct FConvexPart
	{
		FConvexPart() {}
		FConvexPart(const FDynamicMesh3& SourceMesh, bool bMergeEdges, FTransformSRT3d& TransformOut);
		FConvexPart(const FDynamicMesh3& SourceMesh, const FConvexDecomposition3::FPreprocessMeshOptions& Options, FTransformSRT3d& TransformOut);
		FConvexPart(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector3> Faces, const FConvexDecomposition3::FPreprocessMeshOptions& Options, FTransformSRT3d& TransformOut, int32 FaceVertexOffset = 0);
		FConvexPart(TArrayView<const FVector3f> Vertices, TArrayView<const FIntVector3> Faces, bool bMergeEdges, FTransformSRT3d& TransformOut, int32 FaceVertexOffset = 0);

		// Allow direct construction of a compact part (e.g. to allow construction of a pre-existing convex hull)
		FConvexPart(bool bIsCompact) : bIsCompact(bIsCompact)
		{}

		void Reset()
		{
			InternalGeo.Clear();
			HullTriangles.Reset();
			HullPlanes.Reset();
			HullSourceID = -1;
			OverlapsNegativeSpace.Reset();
			HullVolume = 0;
			GeoVolume = 0;
			SumHullsVolume = -FMathd::MaxReal;
			GeoCenter = FVector3d::ZeroVector;
			Bounds = FAxisAlignedBox3d::Empty();
			HullError = FMathd::MaxReal;
			bIsCompact = false;
			bFailed = false;
			bGeometryVolumeUnreliable = false;
			bMustMerge = false;
		}

		// Underlying geometry represented by the convex part
		// Note: If IsCompact(), this will only store vertices
		FDynamicMesh3 InternalGeo;

		inline bool IsCompact() const
		{
			return bIsCompact;
		}

		inline bool IsFailed() const
		{
			return bFailed;
		}

		TArray<FIndex3i> HullTriangles;
		TArray<FPlane3d> HullPlanes; // 1:1 with HullTriangles

		int32 HullSourceID = -1; // Optional ID, cleared on merge, to track the origin of un-merged hulls

		// Indices of NegativeSpace spheres that are overlapped by the part
		TArray<int32> OverlapsNegativeSpace;

		// Measurements of the geo and hull, to be used when evaluating potential further splits
		double HullVolume = 0, GeoVolume = 0;
		double SumHullsVolume = -FMathd::MaxReal; // Sum of volume of any hulls that have been merged to form this hull; only valid if the FConvexPart was created by merging
		FVector3d GeoCenter = FVector3d::ZeroVector; // Some central point of the geometry (e.g., an average of vertices)
		FAxisAlignedBox3d Bounds;
		bool bGeometryVolumeUnreliable = false;
		bool bSplitFailed = false; // flag indicating that cutting has been attempted and failed, so the part should not be considered for splitting

		// Flag indicating the hull should be merged into another part during a MergeBest() call
		bool bMustMerge = false;

		// Approximate value indicating how badly the hull mismatches the input geometry
		double HullError = FMathd::MaxReal;

		void ComputeHullPlanes();

		// get the depth of a point inside the hull (positive values are inside the hull, negative outside)
		double GetHullDepth(FVector3d Pt)
		{
			int32 NumPlanes = HullPlanes.Num();
			ensure(HullTriangles.Num() == NumPlanes);
			if (NumPlanes == 0)
			{
				return 0;
			}
			double MaxDist = -FMathd::MaxReal;
			for (int32 Idx = 0; Idx < NumPlanes; Idx++)
			{
				const FPlane3d& Plane = HullPlanes[Idx];
				if (Plane.Normal == FVector3d::ZeroVector)
				{
					continue;
				}
				double Dist = Plane.DistanceTo(Pt);
				MaxDist = FMath::Max(Dist, MaxDist);
			}
			return MaxDist == -FMathd::MaxReal ? 0 : -MaxDist;
		}

		// Helper to create hull after InternalGeo is set
		bool ComputeHull(bool bComputePlanes = true, double ThickenAfterHullFailureInLocalSpace = 0.0);

		// Helper to compute volumes, centroid, bounds, etc after the convex part is initialized
		void ComputeStats();

		void Compact();

	protected:
		bool bIsCompact = false;
		bool bFailed = false;

	private:
		// helper to initialize the part from the already-set InternalGeo member; used to support constructors
		void InitializeFromInternalGeo(const FConvexDecomposition3::FPreprocessMeshOptions& Preprocess, FTransformSRT3d& TransformOut);
	};

	// All convex hulls parts in the convex decomposition
	// Note this must be an indirect array, not a TArray, because FConvexPart holds an FDynamicMesh3 mesh, which is not trivially movable
	TIndirectArray<FConvexPart> Decomposition;

	/// Transform taking the result hull vertices back to the original space of the inputs; automatically applied by GetVertices()
	FTransformSRT3d ResultTransform = FTransformSRT3d::Identity();

	// Edge in a graph of merge candidates, indicating which Decomposition parts could be merged together
	// Note a proximity link does not guarantee that the geometry (or convex hulls) are touching
	struct FProximity
	{
		FProximity() = default;
		FProximity(const FIndex2i& Link, const FPlane3d& Plane, bool bPlaneSeparates) : Link(Link), Plane(Plane), bPlaneSeparates(bPlaneSeparates)
		{}

		FIndex2i Link{ -1, -1 };
		
		// the shared plane between the two hulls (because adjacent hulls are formed by planar cuts in the splitting step)
		// Note: Plane becomes invalid in the merge-up phase; it is used to accelerate filtering in the split-down phase
		FPlane3d Plane;
		
		// Whether the geometry is separated by Plane
		bool bPlaneSeparates = false;

		// Whether this merge can be allowed (e.g. has not been ruled out by negative space)
		bool bIsValidLink = true;
		
		// What the volume of the convex hull would be if we merged these two parts
		double MergedVolume = VolumeNotComputed();

		double GetMergeCost(const TIndirectArray<FConvexPart>& DecompositionIn) const
		{
			if (!HasMergedVolume() || !DecompositionIn.IsValidIndex(Link.A) || !DecompositionIn.IsValidIndex(Link.B))
			{
				return FMathd::MaxReal;
			}
			ensure(DecompositionIn[Link.A].SumHullsVolume != -FMathd::MaxReal && DecompositionIn[Link.B].SumHullsVolume != -FMathd::MaxReal);
			return MergedVolume - DecompositionIn[Link.A].SumHullsVolume - DecompositionIn[Link.B].SumHullsVolume;
		}

		bool HasMergedVolume() const
		{
			return MergedVolume != VolumeNotComputed();
		}
		void ClearMergedVolume()
		{
			MergedVolume = VolumeNotComputed();
		}

		constexpr double VolumeNotComputed() const { return FMathd::MaxReal; }
		
	};

	// All potential merge edges between Decomposition FConvexParts
	TArray<FProximity> Proximities;

	// Mapping from Decomposition indices to Proximity indices
	TMultiMap<int32, int32> DecompositionToProximity;

	// Helper function to delete a proximity relationship, and fix index references to account for the new array ordering
	// @param ToRemove					Indices of proximities that will be removed; note this function is destructive to this array
	// @param bDeleteMapReferences		If true, also update references in the DecompositionToProximity map
	GEOMETRYCORE_API void DeleteProximity(TArray<int32>&& ToRemove, bool bDeleteMapReferences);

	// Helper function to update all proximity data after splitting an FConvexPart apart
	// @param SplitIdx				The index of the FConvexPart that has been split into multiple parts.  Now contains the first new part.
	// @param NewIdxStart			The first index in the Decomposition array of new FConvexParts aside from SplitIdx.  Everything from this to the end of the Decomposition array must be new.
	// @param CutPlane				The plane used for the split
	// @param SecondSideIdxStart	The first index in the Decomposition array of new FConvexParts that were on the opposite side of the plane from SplitIdx.  Will be different from NewIdxStart only if the first part was split into multiple components.
	// @param OrigHullVolume		The volume of the convex hull *before* the split was performed -- used to precompute the merge cost.
	GEOMETRYCORE_API void UpdateProximitiesAfterSplit(int32 SplitIdx, int32 NewIdxStart, FPlane3d CutPlane, int32 SecondSideIdxStart, double OrigHullVolume);

	// Fix overlaps between current Decomposition and associated NegativeSpace, using the saved associations in the FConvexPart::OverlapsNegativeSpace
	// Note the tolerance and min radius may not be the same as originally used to generate the negative space; they should just be large enough to
	// avoid 'locking' the merge step for any hulls that still overlap negative space after splitting
	GEOMETRYCORE_API void FixHullOverlapsInNegativeSpace(double NegativeSpaceTolerance = UE_DOUBLE_KINDA_SMALL_NUMBER, double NegativeSpaceMinRadius = UE_DOUBLE_KINDA_SMALL_NUMBER);
	
	// Tests if a convex part overlaps a sphere
	// @param Part				Convex part to test for overlap with sphere
	// @param Center			Center of sphere
	// @param Radius			Radius of sphere
	// @param TransformIntoSphereSpace	If non-null, transform from convex hull to sphere coordinate space. Otherwise, hulls and spheres are assumed to be in the same space.
	// @param OutDistanceSq		If non-null, and there is an overlap, will be filled with the squared distance from the sphere center to the convex hull (0 if the center is inside the hull). If no overlap is found, value is not meaningful.
	// @return true if the Part overlaps the sphere, false otherwise
	GEOMETRYCORE_API static bool ConvexPartVsSphereOverlap(const FConvexPart& Part, FVector3d Center, double Radius, const FTransform* TransformIntoSphereSpace = nullptr, double* OutDistanceSq = nullptr);

	// Get the current negative space tracked by the convex decomposition
	// Note: Does not include externally-managed negative space passed to MergeBest.
	// Note: Direct access is const-only; use InitializeNegativeSpace() to set the negative space.
	const FSphereCovering& GetNegativeSpace() const
	{
		return NegativeSpace;
	}

private:
	
	// Initialize mappings from Decomposition parts to NegativeSpace indices (FConvexPart::OverlapsNegativeSpace)
	void InitNegativeSpaceConvexPartMapping(int32 StartSphereIndex = 0);

	// Negative space to attempt to protect; can be referenced by the Decomposition parts
	FSphereCovering NegativeSpace;

	// Helper to implement SplitWorst
	bool SplitWorstHelper(bool bCanSkipUnreliableGeoVolumes, double ErrorTolerance, bool bOnlySplitIfNegativeSpaceCovered, double MinSplitSizeInWorldSpace);
};

} // end namespace UE::Geometry
} // end namespace UE

