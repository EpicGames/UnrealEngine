// Copyright Epic Games, Inc. All Rights Reserved.

// Adapted from GteConvexHull2.h from GTEngine; see Private/ThirdParty/GTEngine/Mathematics/GteConvexHull2.h

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "IndexTypes.h"
#include "LineTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"
#include "PlaneTypes.h"
#include "Polygon2.h"
#include "Templates/Function.h"


namespace UE {
namespace Geometry {

using namespace UE::Math;


template<typename RealType>
class TConvexHull2
{
public:

	/**
	 * Generate convex hull for a simple polygon
	 * If input has fewer than 3 points, will return false
	 * If input is not a simple polygon (e.g. if it has self-intersections), may not find a correct hull
	 * 
	 * @param NumPolygonPoints	Number of points in the polygon
	 * @param GetPointFunc		Function providing array-style access into the polygon points
	 * @param bIsKnownCCW		If true, will save some processing time by assuming input is wound counter-clockwise
	 * @return true if a hull was generated
	 */
	GEOMETRYCORE_API bool SolveSimplePolygon(int32 NumPolygonPoints, TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, bool bIsKnownCCW = false);

	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were collinear, or all the same point
	 *
	 * @param NumPoints Number of points to consider
	 * @param GetPointFunc Function providing array-style access into points
	 * @param FilterFunc Optional filter to include only a subset of the points in the output hull
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	GEOMETRYCORE_API bool Solve(int32 NumPoints, TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc = [](int32 Idx) {return true;});

	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were collinear, or all the same point
	 *
	 * @param Points Array of points to consider
	 * @param Filter Optional filter to include only a subset of the points in the output hull
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	bool Solve(TArrayView<const TVector2<RealType>> Points, TFunctionRef<bool(int32)> FilterFunc)
	{
		return Solve(Points.Num(), [&Points](int32 Idx)
			{
				return Points[Idx];
			}, FilterFunc);
	}

	// default FilterFunc version of the above Solve(); workaround for clang bug https://bugs.llvm.org/show_bug.cgi?id=25333
	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were collinear, or all the same point
	 *
	 * @param Points Array of points to consider
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	bool Solve(TArrayView<const TVector2<RealType>> Points)
	{
		return Solve(Points.Num(), [&Points](int32 Idx)
			{
				return Points[Idx];
			}, [](int32 Idx) {return true;});
	}

	/** @return true if convex hull is available */
	bool IsSolutionAvailable() const
	{
		return Dimension == 2;
	}

	/**
	 * Empty any previously-computed convex hull data.  Frees the hull memory.
	 * Note: You do not need to call this before calling Solve() with new data.
	 */
	void Empty()
	{
		Dimension = 0;
		NumUniquePoints = 0;
		Hull.Empty();
	}

	/** @return Number of dimensions spanned by the input points. */
	inline int32 GetDimension() const
	{
		return Dimension;
	}

	/** Number of unique points considered by convex hull construction (excludes exact duplicate points and filtered-out points) */
	inline int32 GetNumUniquePoints() const
	{
		return NumUniquePoints;
	}

	/** @return the calculated polygon vertices, as indices into the point set passed to Solve() */
	TArray<int32> const& GetPolygonIndices() const
	{
		ensure(IsSolutionAvailable());
		return Hull;
	}

	/** @return the line spanned by a collinear input.  Only meaningful if GetDimension() == 1 */
	TLine2<RealType> GetLine(TArrayView<const TVector2<RealType>> Points) const
	{
		ensure(Dimension == 1);
		if (ensure(Hull.Num() > 1))
		{
			return TLine2<RealType>::FromPoints(Points[Hull[0]], Points[Hull[1]]);
		}
		return TLine2<RealType>(); // not enough points to compute a valid line, return default
	}

protected:

	// divide-and-conquer algorithm
	void GetHull(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32& IdxFirst, int32& IdxLast);
	void Merge(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32 j0, int32 j1, int32 j2, int32 j3, int32& i0, int32& i1);
	void GetTangent(TFunctionRef<TVector2<RealType>(int32)> GetPointFunc, TArray<int32>& Merged, int32 j0, int32 j1, int32 j2, int32 j3, int32& i0, int32& i1);

	int32 Dimension = 0;
	int32 NumUniquePoints = 0;
	TArray<int32> Hull;
};

typedef TConvexHull2<float> FConvexHull2f;
typedef TConvexHull2<double> FConvexHull2d;



} // end namespace UE::Geometry
} // end namespace UE
