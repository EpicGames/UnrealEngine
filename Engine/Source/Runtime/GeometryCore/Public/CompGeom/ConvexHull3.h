// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HalfspaceTypes.h"
#include "IndexTypes.h"
#include "LineTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "MathUtil.h"
#include "PlaneTypes.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Util/ProgressCancel.h"
#include "VectorTypes.h"

class FProgressCancel;

namespace UE {
namespace Geometry {

using namespace UE::Math;

/**
 * Helper class to find the dimensions spanned by a point cloud
 * and (if it spans 3 dimensions) the indices of four 'extreme' points
 * forming a (non-degenerate, volume > 0) tetrahedron
 *
 * The extreme points are chosen to be far apart, and are used as the starting point for
 * incremental convex hull construction.
 */
template<typename RealType>
struct TExtremePoints3
{
	int Dimension = 0;
	int Extreme[4]{ 0, 0, 0, 0 };

	// Coordinate frame spanned by input points
	TVector<RealType> Origin{ 0,0,0 };
	TVector<RealType> Basis[3]{ {0,0,0}, {0,0,0}, {0,0,0} };

	TExtremePoints3(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc = [](int32 Idx) {return true; }, RealType Epsilon = TMathUtil<RealType>::Epsilon)
	{
		Init(NumPoints, GetPointFunc, FilterFunc, Epsilon);
	}

private:
	GEOMETRYCORE_API void Init(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc, RealType Epsilon = TMathUtil<RealType>::Epsilon);
};

template<typename RealType>
struct TConvexHullSimplificationSettings
{
	/// Points will not be added to the hull if doing so would create an edge smaller than this distance tolerance.
	/// If zero, no points will be skipped.
	RealType DegenerateEdgeTolerance = (RealType)0;

	/// If positive, hulls generated will only have at most this many points.
	int32 MaxHullVertices = -1;

	/// If positive, skip adding points that are closer than this threshold to the in-progress hull.
	RealType SkipAtHullDistanceAbsolute = -TMathUtil<RealType>::MaxReal;
	/// If positive, skip adding points that are closer than this threshold to the in-progress hull -- expressed as a fraction of the overall hull extent.
	/// Note if both the Absolute and Fraction MinPlaneDistance thresholds are set, they will both apply (i.e., the larger of the two will be used)
	RealType SkipAtHullDistanceAsFraction = -TMathUtil<RealType>::MaxReal;
};

/**
 * Calculate the Convex Hull of a 3D point set as a Triangle Mesh
 */
template<typename RealType>
class TConvexHull3
{
public:

	/// Whether neighbors for the hull triangles should be computed/saved.
	/// If true, can call GetTriangleNeighbors() after Solve().
	bool bSaveTriangleNeighbors = false;

	// Settings controlling whether and how to generate a simpler hull
	TConvexHullSimplificationSettings<RealType> SimplificationSettings;

	// Helper to compute the a convex hull and return only its volume. If the hull cannot be constructed, a volume of 0 will be returned.
	GEOMETRYCORE_API static double ComputeVolume(const TArrayView<const TVector<RealType>> Vertices);

	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were coplanar, collinear, or all the same point
	 *
	 * @param NumPoints Number of points to consider
	 * @param GetPointFunc Function providing array-style access into points
	 * @param FilterFunc Optional filter to include only a subset of the points in the output hull
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	GEOMETRYCORE_API bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc);
	bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc)
	{
		return Solve(NumPoints, GetPointFunc, [](int32 Idx) {return true;});
	}

	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were collinear, or all the same point
	 *
	 * @param Points Array of points to consider
	 * @param FilterFunc Optional filter to include only a subset of the points in the output hull
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	template<typename VectorType>
	bool Solve(TArrayView<const VectorType> Points, TFunctionRef<bool(int32)> FilterFunc)
	{
		return Solve(Points.Num(), [&Points](int32 Idx)
			{
				return Points[Idx];
			}, FilterFunc);
	}

	// default FilterFunc version of the above Solve(); workaround for clang bug https://bugs.llvm.org/show_bug.cgi?id=25333
	/**
	 * Generate convex hull as long as input is not degenerate
	 * If input is degenerate, this will return false, and caller can call GetDimension()
	 * to determine whether the points were collinear, or all the same point
	 *
	 * @param Points Array of points to consider
	 * @return true if hull was generated, false if points span < 2 dimensions
	 */
	template<typename VectorType>
	bool Solve(TArrayView<const VectorType> Points)
	{
		return Solve(Points.Num(), [&Points](int32 Idx)
			{
				return Points[Idx];
			}, [](int32 Idx) {return true;});
	}

	/** @return true if convex hull is available */
	bool IsSolutionAvailable() const
	{
		return Dimension == 3;
	}

	/**
	 * Call TriangleFunc for each triangle of the Convex Hull. The triangles index into the point set passed to Solve()
	 */
	void GetTriangles(TFunctionRef<void(FIndex3i)> TriangleFunc)
	{
		for (FIndex3i Triangle : Hull)
		{
			TriangleFunc(Triangle);
		}
	}

	/** @return convex hull triangles */
	TArray<FIndex3i> const& GetTriangles() const
	{
		return Hull;
	}

	/**
	 * Get faces of the convex hull as convex polygons, merging faces that are exactly coplanar
	 * @param PolygonFunc	Callback to be called for each polygon, with the array of vertex indices and the face normal
	 * @param GetPointFunc	Function providing array-style access into points
	 */
	GEOMETRYCORE_API void GetFaces(TFunctionRef<void(TArray<int32>&, TVector<RealType>)> PolygonFunc, TFunctionRef<TVector<RealType>(int32)> GetPointFunc) const;

	/**
	 * Get faces of the convex hull as convex polygons, simplifying the hull by merging near-coplanar faces and only keeping vertices that are on the corner of at least three merged faces
	 * 
	 * @param PolygonFunc					Callback to be called for each polygon, with the array of vertex indices and the face normal
	 * @param GetPointFunc					Function providing array-style access into points
	 * @param FaceAngleToleranceInDegrees	The hull will be simplified by merging faces with less than this dihedral angle between them
	 * @param PlaneDistanceTolerance		Faces will not merge unless all points on the face are within this distance of the combined (average) face plane
	 */
	GEOMETRYCORE_API void GetSimplifiedFaces(TFunctionRef<void(TArray<int32>&, TVector<RealType>)> PolygonFunc, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,
		RealType FaceAngleToleranceInDegrees = (RealType)1.0, RealType PlaneDistanceTolerance = (RealType)1.0) const;

	// Polygon face array type: A nested array with inline allocator per face, optimizing for the case where most faces have less than 8 vertices
	using FPolygonFace = TArray<int32, TInlineAllocator<8>>;
	/**
	 * Get faces of the convex hull as convex polygons, simplifying the hull by merging near-coplanar faces and only keeping vertices that are on the corner of at least three merged faces
	 * 
	 * @param OutPolygons					Polygons of the convex hull faces, as arrays of indices into the original points
	 * @param GetPointFunc					Function providing array-style access into points
	 * @param FaceAngleToleranceInDegrees	The hull will be simplified by merging faces with less than this dihedral angle between them
	 * @param PlaneDistanceTolerance		Faces will not merge unless all points on the face are within this distance of the combined (average) face plane
	 * @param OutPolygonNormals				Optional array of normals for each polygon
	 */
	GEOMETRYCORE_API void GetSimplifiedFaces(TArray<FPolygonFace>& OutPolygons, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,
		RealType FaceAngleToleranceInDegrees = (RealType)1.0, RealType PlaneDistanceTolerance = (RealType)1.0, TArray<TVector<RealType>>* OutPolygonNormals = nullptr) const;

	/**
	 * Only valid if bSaveTriangleNeighbors was true when Solve() was called
	 * @return Neighbors of each hull triangle, in edge order -- i.e., Nbr.A is the triangle across edge(Tri.A, Tri.B)
	 */
	TArray<FIndex3i> const& GetTriangleNeighbors() const
	{
		ensure(bSaveTriangleNeighbors && Hull.Num() == HullNeighbors.Num());
		return HullNeighbors;
	}

	/** @return convex hull triangles by a move */
	TArray<FIndex3i>&& MoveTriangles()
	{
		return MoveTemp(Hull);
	}

	/** 
	 * Convert an already-computed convex hull into a halfspace representation
	 * Following the logic of ContainmentQueries3.h, all halfspaces are oriented "outwards,"
	 * so a point is inside the convex hull if it is outside of all halfspaces in the array
	 * 
	 * @param Points Array of points to consider
	 * @return Array of halfspaces
	 */
	template<typename VectorType>
	TArray<THalfspace3<RealType>> GetAsHalfspaces(TArrayView<const VectorType> Points) const
	{
		TArray<THalfspace3<RealType>> Halfspaces;
		for (FIndex3i Tri : Hull)
		{
			THalfspace3<RealType> TriHalfspace(Points[Tri.A], Points[Tri.B], Points[Tri.C]);
			Halfspaces.Add(TriHalfspace);
		}
		return Halfspaces;
	}

	/**
	 * Convert an already-computed convex hull into a halfspace representation
	 * Following the logic of ContainmentQueries3.h, all halfspaces are oriented "outwards,"
	 * so a point is inside the convex hull if it is outside of all halfspaces in the array
	 *
	 * @param GetPointFunc Function providing array-style access into points
	 * @return Array of halfspaces
	 */
	template<typename VectorType>
	TArray<THalfspace3<RealType>> GetAsHalfspaces(TFunctionRef<VectorType(int32)> GetPointFunc) const
	{
		TArray<THalfspace3<RealType>> Halfspaces;
		for (FIndex3i Tri : Hull)
		{
			THalfspace3<RealType> TriHalfspace(GetPointFunc(Tri.A), GetPointFunc(Tri.B), GetPointFunc(Tri.C));
			Halfspaces.Add(TriHalfspace);
		}
		return Halfspaces;
	}

	/**
	 * Empty any previously-computed convex hull data.  Frees the hull memory.
	 * Note: You do not need to call this before calling Generate() with new data.
	 */
	void Empty()
	{
		Dimension = 0;
		NumHullPoints = 0;
		Hull.Empty();
	}

	/**
	 * Number of dimensions spanned by the input points.
	 */
	inline int GetDimension() const
	{
		return Dimension;
	}
	/** @return If GetDimension() returns 1, this returns the line the data was on.  Otherwise result is not meaningful. */
	inline TLine3<RealType> const& GetLine() const
	{
		return Line;
	}
	/** @return If GetDimension() returns 2, this returns the plane the data was on.  Otherwise result is not meaningful. */
	inline TPlane3<RealType> const& GetPlane() const
	{
		return Plane;
	}

	/** @return Number of points on the output hull */
	int GetNumHullPoints() const
	{
		return NumHullPoints;
	}


	/** Set this to be able to cancel running operation */
	FProgressCancel* Progress = nullptr;

	// Useful helper for walking the border of a convex region, or of a QHull horizon. Will also work for extracting a group boundary for any closed triangle mesh.
	// Note: Assumes TriangleNeighbors are listed such that the first neighbor is the tri across from the (Tri.A, Tri.B) edge
	static void WalkBorder(const TArray<FIndex3i>& Triangles, const TArray<FIndex3i>& TriangleNeighbors, TFunctionRef<bool(int32)> InGroupFunc, int32 StartIdx, TArray<int32>& OutBorderVertexIndices);

protected:

	int32 Dimension;
	TLine3<RealType> Line;
	TPlane3<RealType> Plane;

	int NumHullPoints = 0;
	TArray<FIndex3i> Hull;
	// We can optionally also return the hull triangle adjacencies
	TArray<FIndex3i> HullNeighbors;
	
};

typedef TExtremePoints3<float> FExtremePoints3f;
typedef TExtremePoints3<double> FExtremePoints3d;
typedef TConvexHull3<float> FConvexHull3f;
typedef TConvexHull3<double> FConvexHull3d;


} // end namespace UE::Geometry
} // end namespace UE

