// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Curve/GeneralPolygon2.h"
#include "HAL/PlatformCrt.h"
#include "IndexTypes.h"
#include "IndexTypes.h"
#include "LineTypes.h"
#include "Math/RandomStream.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MathUtil.h"
#include "PlaneTypes.h"
#include "Polygon2.h"
#include "Templates/PimplPtr.h"
#include "Templates/UnrealTemplate.h"
#include "VectorTypes.h"


namespace UE {
namespace Geometry {
template <typename T> class TGeneralPolygon2;
template <typename T> class TPolygon2;

using namespace UE::Math;

// Internal representation of mesh connectivity; not exposed to interface
struct FDelaunay2Connectivity;

class FDelaunay2
{
public:

	// Indicates result of triangulation
	enum class EResult
	{
		Success,
		// Has not yet tried to compute a triangulation
		NotComputed,
		// No input points
		EmptyInput,
		// Input did not span a 2D basis (was 1D or 0D) so could not form a triangle
		Collinear,
		// Could not create triangulation containing all requested constraint edges, typically due to intersections between requested edges
		MissingEdges,
		// Uncategorized failure
		Unknown
	};

	// Options for selecting what triangles to include in the output, for constrained Delaunay triangulation of polygons
	enum class EFillMode
	{
		// Fastest/simplest option: Keep all triangles if you'd have to cross a constrained edge to reach it from outside, ignoring edge orientation
		Solid,
		// Winding-number based fill options: Keep triangles based on the winding number
		PositiveWinding,
		NonZeroWinding,
		NegativeWinding,
		OddWinding
	};


	//
	// Inputs
	//

	// Source for random permutations, used internally in the triangulation algorithm
	FRandomStream RandomStream;

	// Option to keep extra vertex->edge adjacency data; useful if you will call ConstrainEdges many times on the same triangulation
	bool bKeepFastEdgeAdjacencyData = false;

	// Option to validate that edges remain in the triangulation after multiple constraint edges passed in
	// Only validates for the edges in the current call; if separate calls constrain additional edges, set this to 'false' and call HasEdges() on the full edge set
	// TODO: Consider having the internal mesh remember what edges have been constrained, so we can set an error flag when constrained edges cross previous constrained edges
	bool bValidateEdges = true;

	// Option to automatically track duplicate vertices and treat edges that reference them as if they referenced the instance that was actually used in the triangulation
	// Note: Cannot change this to 'true' *after* triangulation and then call ConstrainEdges; duplicate vertices will only be detected on their initial insertion
	bool bAutomaticallyFixEdgesToDuplicateVertices = false;

	// TODO: it would often be useful to pass in sparse vertex data
	//// Optional function to allow Triangulate to skip vertices
	//TFunction<bool(int32)> SkipVertexFn = nullptr;

	/**
	 * Compute an (optionally constrained) Delaunay triangulation.
	 * Note this clears any previously-held triangulation data, and triangulates the passed-in vertices (and optional edges) from scratch
	 *
	 * @return false if triangulation failed
	 */
	GEOMETRYCORE_API bool Triangulate(TArrayView<const TVector2<double>> Vertices, TArrayView<const FIndex2i> Edges = TArrayView<const FIndex2i>());
	GEOMETRYCORE_API bool Triangulate(TArrayView<const TVector2<float>> Vertices, TArrayView<const FIndex2i> Edges = TArrayView<const FIndex2i>());

	// Triangulate a polygon, and optionally pass back the resulting triangles
	// Uses the 'solid' fill mode, and fills the polygon regardless of the edge orientation
	// Note: TrianglesOut may be empty or incomplete if the input is self-intersecting.
	// Note this clears any previously-held triangulation data, and triangulates the passed-in polygon from scratch
	template<typename RealType>
	bool Triangulate(const TPolygon2<RealType>& Polygon, TArray<FIndex3i>* TrianglesOut = nullptr)
	{
		int32 NumVertices = Polygon.VertexCount();
		TArray<FIndex2i> Edges;
		Edges.Reserve(NumVertices - 1);
		for (int32 Last = NumVertices - 1, Idx = 0; Idx < NumVertices; Last = Idx++)
		{
			Edges.Add(FIndex2i(Last, Idx));
		}
		bool bSuccess = Triangulate(Polygon.GetVertices(), Edges);
		if (TrianglesOut)
		{
			*TrianglesOut = GetFilledTriangles(Edges, EFillMode::Solid);
		}
		return bSuccess;
	}

	// Triangulate a polygon, and optionally pass back the resulting triangles
	// Uses a winding-number-based fill mode, and relies on the general polygon having correct orientations
	// (Uses TGeneralPolygon2's OuterIsClockwise() to automatically choose between positive or negative winding)
	// Note: TrianglesOut may be empty or incomplete if the input is self-intersecting.
	// Note this clears any previously-held triangulation data, and triangulates the passed-in general polygon from scratch
	template<typename RealType>
	bool Triangulate(const TGeneralPolygon2<RealType>& GeneralPolygon, TArray<FIndex3i>* TrianglesOut = nullptr, TArray<TVector2<RealType>>* VerticesOut = nullptr, bool bFallbackToGeneralizedWinding = false)
	{
		TArray<TVector2<RealType>> AllVertices;
		TArray<FIndex2i> AllEdges;

		auto AppendVertices = [&AllVertices, &AllEdges](const TArray<TVector2<RealType>>& Vertices)
		{
			if (Vertices.IsEmpty())
			{
				return;
			}

			int32 StartIdx = AllVertices.Num();
			AllVertices.Append(Vertices);
			AllEdges.Reserve(AllEdges.Num() + Vertices.Num() - 1);
			int32 NumVertices = Vertices.Num();
			for (int32 Last = NumVertices - 1, Idx = 0; Idx < NumVertices; Last = Idx++)
			{
				AllEdges.Add(FIndex2i(StartIdx + Last, StartIdx + Idx));
			}
		};
		AppendVertices(GeneralPolygon.GetOuter().GetVertices());
		for (int32 HoleIdx = 0; HoleIdx < GeneralPolygon.GetHoles().Num(); HoleIdx++)
		{
			AppendVertices(GeneralPolygon.GetHoles()[HoleIdx].GetVertices());
		}
		
		bool bSuccess = Triangulate(AllVertices, AllEdges);
		if (TrianglesOut)
		{
			EFillMode FillMode = GeneralPolygon.OuterIsClockwise() ? EFillMode::NegativeWinding : EFillMode::PositiveWinding;
			if (bSuccess || !bFallbackToGeneralizedWinding)
			{
				*TrianglesOut = GetFilledTriangles(AllEdges, FillMode);
			}
			else
			{
				GetFilledTrianglesGeneralizedWinding(*TrianglesOut, AllVertices, AllEdges, FillMode);
			}
		}
		if (VerticesOut)
		{
			*VerticesOut = MoveTemp(AllVertices);
		}
		return bSuccess;
	}

	/**
	 * Update an already-computed triangulation so the given edges are in the triangulation.
	 * Note: Assumes the edges do not intersect other constrained edges OR existing vertices in the triangulation
	 * @return true if edges were successfully inserted.
	 * Note: Will not detect failure due to intersection of constrained edges unless member bValidateEdges == true.
	 */
	GEOMETRYCORE_API bool ConstrainEdges(TArrayView<const TVector2<double>> Vertices, TArrayView<const FIndex2i> Edges);
	GEOMETRYCORE_API bool ConstrainEdges(TArrayView<const TVector2<float>> Vertices, TArrayView<const FIndex2i> Edges);

	// Update the triangulation incrementally, assuming Vertices are unchanged before FirstNewIndex, and nothing after FirstNewIndex has been inserted yet
	// Note that updating with new vertices *after* constraining edges may remove previously-constrained edges, unless we also add a way to tag constrained edges
	GEOMETRYCORE_API bool Update(TArrayView<const TVector2<double>> Vertices, int32 FirstNewIdx);

	// Get the triangulation as an array of triangles
	// Note: This creates a new array each call, because the internal data structure does not have a triangle array
	GEOMETRYCORE_API TArray<FIndex3i> GetTriangles() const;

	// Get the triangulation as an array with a corresponding adjacency array, indicating the adjacent triangle on each triangle edge (-1 if no adjacent triangle)
	GEOMETRYCORE_API void GetTrianglesAndAdjacency(TArray<FIndex3i>& Triangles, TArray<FIndex3i>& Adjacency) const;

	/**
	 * Return the triangles that are inside the given edges, removing the outer boundary triangles
	 * If a Winding-Number-based fill mode is used, assumes edges are oriented and tracks the winding number across edges
	 * 
	 * @param Edges						The array of edges that were already constrained in the triangulation (by a previous call to Triangulate or ConstrainEdges)
	 * @param FillMode					Strategy to use to define which triangles to include in the output
	 * @return							A subset of the triangulation that is 'inside' the given edges, as defined by the FillMode
	 */
	TArray<FIndex3i> GetFilledTriangles(TArrayView<const FIndex2i> Edges, EFillMode FillMode = EFillMode::PositiveWinding)
	{
		TArray<FIndex3i> Triangles;
		GetFilledTriangles(Triangles, Edges, FillMode);
		return Triangles;
	}

	/**
	 * Get (by reference) the triangles that are inside the given edges, removing the outer boundary triangles
	 * If a Winding-Number-based fill mode is used, assumes edges are oriented and tracks the winding number across edges
	 *
	 * @param TrianglesOut				Will be filled with a subset of the triangulation that is 'inside' the given edges, as defined by the FillMode
	 * @param Edges						The array of edges that were already constrained in the triangulation (by a previous call to Triangulate or ConstrainEdges)
	 * @param FillMode					Strategy to use to define which triangles to include in the output
	 * @return							true if the result was well defined and consistent; false otherwise. Solid fill mode always has a well defined result; Winding-number-based fills do not if the edges have open spans.
	 *									Note the triangulation will still be filled by best-effort even if the function returns false.
	 */
	GEOMETRYCORE_API bool GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> Edges, EFillMode FillMode = EFillMode::PositiveWinding) const;

	/**
	 * Get (by reference) the triangles that are inside the given edges, using a generalized winding number method to determine which triangles are inside
	 * Not valid for EFillMode::Solid, will fall back to the above GetFilledTriangles method in that case.
	 * 
	 * This method is for triangulations where the edges did not define a fully-closed shape, either by construction or due to un-resolved edge intersections in the input.
	 * Note that it is slower than the standard GetFilledTriangles, and is worse for closed polygons where all edges were successfully constrained.
	 *
	 * @param TrianglesOut				Will be filled with a subset of the triangulation that is 'inside' the given edges, as defined by the FillMode
	 * @param Vertices					The array of vertices that were used in the triangulation (by the previous call to Triangulate)
	 * @param Edges						The array of edges that were already constrained in the triangulation (by a previous call to Triangulate or ConstrainEdges)
	 * @param FillMode					Strategy to use to define which triangles to include in the output
	 * @return							true if the result was well defined and consistent; false otherwise. Solid fill mode always has a well defined result; Winding-number-based fills do not if the edges have open spans.
	 *									Note the triangulation will still be filled by best-effort even if the function returns false.
	 */
	GEOMETRYCORE_API bool GetFilledTrianglesGeneralizedWinding(TArray<FIndex3i>& TrianglesOut, TArrayView<const TVector2<double>> Vertices, TArrayView<const FIndex2i> Edges, EFillMode FillMode = EFillMode::PositiveWinding) const;
	GEOMETRYCORE_API bool GetFilledTrianglesGeneralizedWinding(TArray<FIndex3i>& TrianglesOut, TArrayView<const TVector2<float>> Vertices, TArrayView<const FIndex2i> Edges, EFillMode FillMode = EFillMode::PositiveWinding) const;

	/**
	 * Get (by reference) the triangles that are inside the given edges, removing the outside-boundary triangles and the inside-hole triangles
	 * 
	 * @param TrianglesOut					Will be filled with a subset of the triangulation that is 'inside' the given edges, as defined by the FillMode
	 * @param BoundaryEdges					Constrained edges in the triangulation that define the boundary of the desired shape
	 * @param HoleEdges						Constrained edges in the triangulation that define inner holes of the desired shape
	 * @return								true if any result was successfully computed (including an empty result).
	 *										Currently only returns false if Triangulate() has not been called yet.
	 */
	GEOMETRYCORE_API bool GetFilledTriangles(TArray<FIndex3i>& TrianglesOut, TArrayView<const FIndex2i> BoundaryEdges, TArrayView<const FIndex2i> HoleEdges) const;

	// @return true if this is a constrained Delaunay triangulation
	bool IsConstrained() const
	{
		return bIsConstrained;
	}

	// @return true if triangulation is Delaunay, useful for validating results (note: likely to be false if edges are constrained)
	GEOMETRYCORE_API bool IsDelaunay(TArrayView<const FVector2f> Vertices, TArrayView<const FIndex2i> SkipEdges = TArrayView<const FIndex2i>()) const;
	GEOMETRYCORE_API bool IsDelaunay(TArrayView<const FVector2d> Vertices, TArrayView<const FIndex2i> SkipEdges = TArrayView<const FIndex2i>()) const;

	// @return true if triangulation has the given edges, useful for validating results
	GEOMETRYCORE_API bool HasEdges(TArrayView<const FIndex2i> Edges) const;

	// Remap any references to duplicate vertices to only reference the vertices used in the triangulation
	GEOMETRYCORE_API void FixDuplicatesOnEdge(FIndex2i& Edge);

	// @return true if the edge is in the triangulation
	GEOMETRYCORE_API bool HasEdge(const FIndex2i& Edge, bool bRemapDuplicates);

	// @return true if the input had duplicate vertices, and so not all vertices could be used in the triangulation.
	GEOMETRYCORE_API bool HasDuplicates() const;

	// @return if bAutomaticallyFixEdgesToDuplicateVertices was set during triangulation, will return the remapped vertex index for any duplicate vertices, or Index if the vertex was not remapped.
	GEOMETRYCORE_API int32 RemapIfDuplicate(int32 Index) const;



	/**
	 * Get Voronoi diagram cells as dual of the Delaunay triangulation.  You must call Triangulate() before calling this.
	 * @param Vertices			Vertices of the Delaunay triangulation, to be used as the sites of the Voronoi diagram
	 * @param bIncludeBoundary	If true, include the cells on the boundary of the diagram.  These cells are conceptually infinite, but will be clipped to a bounding box.
	 * @param ClipBounds		If non-empty, all Voronoi diagram cells will be clipped to this rectangle.
	 * @param ExpandBounds		Amount to expand the clipping bounds beyond the Bounds argument (or the bounding box of non-boundary Voronoi cells, if Bounds was empty)
	 * @return					The cells of each Voronoi site, or an empty array if the Triangulation was not yet computed
	 */
	GEOMETRYCORE_API TArray<TArray<FVector2d>> GetVoronoiCells(TArrayView<const FVector2d> Vertices, bool bIncludeBoundary = false, FAxisAlignedBox2d ClipBounds = FAxisAlignedBox2d::Empty(), double ExpandBounds = 0.0) const;
	GEOMETRYCORE_API TArray<TArray<FVector2f>> GetVoronoiCells(TArrayView<const FVector2f> Vertices, bool bIncludeBoundary = false, FAxisAlignedBox2f ClipBounds = FAxisAlignedBox2f::Empty(), float ExpandBounds = 0.0f) const;

	/**
	 * Compute Voronoi diagram cells
	 * @param Sites				Positions to use as the sites of the Voronoi diagram
	 * @param bIncludeBoundary	If true, include the cells on the boundary of the diagram.  These cells are conceptually infinite, but will be clipped to a bounding box.
	 * @param ClipBounds		If non-empty, all Voronoi diagram cells will be clipped to this rectangle.
	 * @param ExpandBounds		Amount to expand the clipping bounds beyond the Bounds argument (or the bounding box of non-boundary Voronoi cells, if Bounds was empty)
	 * @return					The cells of each Voronoi site, or an empty array if the Triangulation was not yet computed
	 */
	template<typename RealType>
	static TArray<TArray<TVector2<RealType>>> ComputeVoronoiCells(TArrayView<const TVector2<RealType>> Sites, bool bIncludeBoundary = false, TAxisAlignedBox2<RealType> ClipBounds = FAxisAlignedBox2f::Empty(), RealType ExpandBounds = (RealType)0)
	{
		FDelaunay2 Delaunay;
		Delaunay.Triangulate(Sites);
		return Delaunay.GetVoronoiCells(Sites, bIncludeBoundary, ClipBounds, ExpandBounds);
	}

	/** @return Last triangulation result -- either Success, NotComputed, or a reason for failure */
	EResult GetResult() const
	{
		return Result;
	}
	
	/** @return Whether the triangulation result supports computing a Voronoi diagram -- i.e., if it was triangulated, or a collinear input */
	bool CanComputeVoronoiCells() const
	{
		return Result == EResult::Collinear || Result == EResult::Success;
	}

protected:
	TPimplPtr<FDelaunay2Connectivity> Connectivity;

	bool bIsConstrained = false;

	// helper to perform standard validation on results after Triangulate or ConstrainEdges calls
	UE_DEPRECATED(5.5, "No longer used by the implementation and may be removed")
	bool ValidateResult(TArrayView<const FIndex2i> Edges) const
	{
		return !bValidateEdges || HasEdges(Edges);
	}

private:
	EResult Result = EResult::NotComputed;

	bool ValidateEdgesResult(TArrayView<const FIndex2i> Edges)
	{
		if (bValidateEdges && !HasEdges(Edges))
		{
			Result = EResult::MissingEdges;
			return false;
		}
		return true;
	}
};

} // end namespace UE::Geometry
} // end namespace UE
