// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "IndexTypes.h"
#include "Math/RandomStream.h"
#include "Math/Vector.h"
#include "MathUtil.h"
#include "Templates/PimplPtr.h"


namespace UE {
namespace Geometry {

using namespace UE::Math;

// Internal representation of mesh connectivity; not exposed to interface
struct FDelaunay3Connectivity;

class FDelaunay3
{
public:

	//
	// Inputs
	//

	// Source for random permutations, used internally in the triangulation algorithm
	FRandomStream RandomStream;

	/**
	 * Compute a 3D Delaunay triangulation.
	 * Note this clears any previously-held triangulation data, and triangulates the passed-in vertices from scratch
	 *
	 * @return false if triangulation failed
	 */
	GEOMETRYCORE_API bool Triangulate(TArrayView<const TVector<double>> Vertices);
	GEOMETRYCORE_API bool Triangulate(TArrayView<const TVector<float>> Vertices);

	// Get the result as an array of tetrahedra
	// Note: This creates a new array each call, because the internal data structure does not have a tetrahedra array
	GEOMETRYCORE_API TArray<FIndex4i> GetTetrahedraAsFIndex4i(bool bReverseOrientation = false) const;
	GEOMETRYCORE_API TArray<FIntVector4> GetTetrahedra(bool bReverseOrientation = false) const;

	// @return true if triangulation is Delaunay, useful for validating results (note: likely to be false if edges are constrained)
	GEOMETRYCORE_API bool IsDelaunay(TArrayView<const FVector3f> Vertices) const;
	GEOMETRYCORE_API bool IsDelaunay(TArrayView<const FVector3d> Vertices) const;

protected:
	TPimplPtr<FDelaunay3Connectivity> Connectivity;
};

} // end namespace UE::Geometry
} // end namespace UE
