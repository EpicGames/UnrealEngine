// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Vector.h"
#include "OrientedBoxTypes.h"
#include "VectorTypes.h"

template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Math { template <typename T> struct TVector; }

namespace Geometry
{

	/**
	* selects the number of extremal vertices and directional vector family for the DiTO - K algorithm, with implementations for K = {12, 14, 20, 26}
	*/
	enum EDiTO
	{
		DiTO_12 = 0,
		DiTO_14 = 1,
		DiTO_20 = 2,
		DiTO_26 = 3
	};

	/**
	* @return OrientedBox3 that contains the points provided by the given GetPointFunc
	* 
	* Heuristic-based computation of an object oriented bounding box that utilizes 
	* a small number of extremal vertices and an internal Di-Tetrahedron to generate the box orientation.
	* Based on "Fast Computation of Tight-Fitting Oriented Bounding Boxes" by Larson and Kallberg ( cf., Game Engine Gems 2)
	*
	* @param DiTO_K       - selects the number of extremal vertices used by the method.  More extremal vertices may produces a better result, but at a slightly higher cost
	* @param NumPoints    - number points used in constructing the OrientedBox
	* @param GetPointFunc - returns a point for every index i = [0, NumPoints)  
	*/
	template <typename RealType>
	TOrientedBox3<RealType>  GEOMETRYCORE_API ComputeOrientedBBox(const EDiTO DiTO_K, const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)>  GetPointFunc);


	/**
	* @return OrientedBox3 that contains the points provided by the given GetPointFunc
	* 
	* Heuristic-based computation of an object oriented bounding box that utilizes 
	* a small number of extremal vertices and an internal Di-Tetrahedron to generate the box orientation.
	* Based on "Fast Computation of Tight-Fitting Oriented Bounding Boxes" by Larson and Kallberg ( cf., Game Engine Gems 2)
	*
	* @param SampleDirections       - array of sample directions, ideally the direction vectors should be of the same length but no length checks are done internally. 
	* @param NumPoints              - number points used in constructing the OrientedBox
	* @param GetPointFunc           - returns a point for every index i = [0, NumPoints)  
	* 
	* The number of extreme vertices used by the method will be twice the number of sample directions. 
	*
	*/
	template <typename RealType>
	TOrientedBox3<RealType>  GEOMETRYCORE_API ComputeOrientedBBox(const TArray<TVector<RealType>>& SampleDirections, const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)>  GetPointFunc);

} // end namespace Geometry
}// end namespace UE

