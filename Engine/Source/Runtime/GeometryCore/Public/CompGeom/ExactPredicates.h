// Copyright Epic Games, Inc. All Rights Reserved.

// Interface for exact predicates w/ Unreal Engine vector types

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "VectorTypes.h"

#include "Misc/CoreMiscDefines.h" // for UE_DEPRECATED

namespace UE {
namespace Math { template <typename T> struct TVector2; }
namespace Math { template <typename T> struct TVector; }

namespace Geometry {
namespace ExactPredicates {

using namespace UE::Math;

/**
 * Must be called once for exact predicates to work.
 * Will be called by GeometryAlgorithmsModule startup, so you don't need to manually call this.
 */
void GEOMETRYCORE_API GlobalInit();

double GEOMETRYCORE_API Orient2DInexact(const double* PA, const double* PB, const double* PC);
double GEOMETRYCORE_API Orient2D(const double* PA, const double* PB, const double* PC);

// Specialized version of Orient2D for the case when PC is the origin
double GEOMETRYCORE_API Orient2DOrigin(double ax, double ay, double bx, double by);

double GEOMETRYCORE_API Orient3DInexact(const double* PA, const double* PB, const double* PC, const double* PD);
double GEOMETRYCORE_API Orient3D(const double* PA, const double* PB, const double* PC, const double* PD);

double GEOMETRYCORE_API Facing3D(const double* PA, const double* PB, const double* PC, const double* Direction);
double GEOMETRYCORE_API Facing2D(const double* PA, const double* PB, const double* Direction);

double GEOMETRYCORE_API InCircleInexact(const double* PA, const double* PB, const double* PC, const double* PD);
double GEOMETRYCORE_API InCircle(const double* PA, const double* PB, const double* PC, const double* PD);

double GEOMETRYCORE_API InSphereInexact(const double* PA, const double* PB, const double* PC, const double* PD, const double* PE);
double GEOMETRYCORE_API InSphere(const double* PA, const double* PB, const double* PC, const double* PD, const double* PE);

// Note: The float versions of these functions can be marginally faster in some cases,
// but also are often orders of magnitude slower, and are more likely to fail due to underflow or overflow
// Consider calling the double versions even for float inputs.
float GEOMETRYCORE_API Orient2DInexact(const float* PA, const float* PB, const float* PC);
float GEOMETRYCORE_API Orient2D(const float* PA, const float* PB, const float* PC);

// Specialized version of Orient2D for the case when PC is the origin
float GEOMETRYCORE_API Orient2DOrigin(float ax, float ay, float bx, float by);

float GEOMETRYCORE_API Orient3DInexact(const float* PA, const float* PB, const float* PC, const float* PD);
float GEOMETRYCORE_API Orient3D(const float* PA, const float* PB, const float* PC, const float* PD);

float GEOMETRYCORE_API Facing3D(const float* PA, const float* PB, const float* PC, const float* Direction);
float GEOMETRYCORE_API Facing2D(const float* PA, const float* PB, const float* PC);

float GEOMETRYCORE_API InCircleInexact(const float* PA, const float* PB, const float* PC, const float* PD);
float GEOMETRYCORE_API InCircle(const float* PA, const float* PB, const float* PC, const float* PD);

// Note: float version of InSphere is not exposed here; instead, convert to and use the double version

/**
 * TVector2-only version that can run in float or double
 * @return value indicating which side of line AB point C is on, or 0 if ABC are collinear
 */
template<typename RealType>
RealType Orient2(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C)
{
	RealType PA[2]{ A.X, A.Y };
	RealType PB[2]{ B.X, B.Y };
	RealType PC[2]{ C.X, C.Y };
	return Orient2D(PA, PB, PC);
}

template<typename RealType>
inline RealType Orient2Origin(const TVector2<RealType>& A, const TVector2<RealType>& B)
{
	return Orient2DOrigin(A.X, A.Y, B.X, B.Y);
}

/**
 * TVector-only version that can run in float or double
 * @return value indicating which side of triangle ABC point D is on, or 0 if ABCD are coplanar
 */
template<typename RealType>
RealType Orient3(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C, const TVector<RealType>& D)
{
	// Note: Though we support computing exact predicates in float precision directly, in practice
	// it is generally faster and more reliable to compute in double precision
	double PA[3]{ (double)A.X, (double)A.Y, (double)A.Z };
	double PB[3]{ (double)B.X, (double)B.Y, (double)B.Z };
	double PC[3]{ (double)C.X, (double)C.Y, (double)C.Z };
	double PD[3]{ (double)D.X, (double)D.Y, (double)D.Z };
	if constexpr (std::is_same_v<RealType, double>)
	{
		return Orient3D(PA, PB, PC, PD);
	}
	else
	{
		// make sure the sign is not lost in casting
		return (RealType)FMath::Sign(Orient3D(PA, PB, PC, PD));
	}
}

/**
 * @return value w/ sign indicating whether the normal of line AB is facing Direction (and 0 if parallel)
 *  (i.e. the sign of -PerpCW(B-A) dot Dir)
 */
template<typename RealType>
RealType Facing2(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& Direction)
{
	RealType PA[2]{ A.X, A.Y };
	RealType PB[2]{ B.X, B.Y };
	RealType Dir[2]{ Direction.X, Direction.Y };
	return Facing2D(PA, PB, Dir);
}

/**
 * TVector-only version that can run in float or double
 * @return value w/ sign indicating whether triangle ABC is facing Direction, or 0 if it is parallel to Direction
 */
template<typename RealType>
RealType Facing3(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C, const TVector<RealType>& Direction)
{
	// Note: Though we support computing exact predicates in float precision directly, in practice
	// it is generally faster and more reliable to compute in double precision
	double PA[3]{ (double)A.X, (double)A.Y, (double)A.Z };
	double PB[3]{ (double)B.X, (double)B.Y, (double)B.Z };
	double PC[3]{ (double)C.X, (double)C.Y, (double)C.Z };
	double Dir[3]{ (double)Direction.X, (double)Direction.Y, (double)Direction.Z };
	if constexpr (std::is_same_v<RealType, double>)
	{
		return Facing3D(PA, PB, PC, Dir);
	}
	else
	{
		// make sure the sign is not lost in casting
		return (RealType)FMath::Sign(Facing3D(PA, PB, PC, Dir));
	}
}

/**
 * TVector2-only version that can run in float or double
 * @return value indicating whether point D is inside, outside, or exactly on the circle passing through ABC
 * Note: Sign of the result depends on the orientation of triangle ABC --
 *	Counterclockwise: Inside is positive
 *	Clockwise: Inside is negative
 */
template<typename RealType>
RealType InCircle2(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C, const TVector2<RealType>& D)
{
	// Note: Though we support computing exact predicates in float precision directly, in practice
	// it is generally faster and more reliable to compute in double precision
	double PA[2]{ (double)A.X, (double)A.Y };
	double PB[2]{ (double)B.X, (double)B.Y };
	double PC[2]{ (double)C.X, (double)C.Y };
	double PD[2]{ (double)D.X, (double)D.Y };
	if constexpr (std::is_same_v<RealType, double>)
	{
		return InCircle(PA, PB, PC, PD);
	}
	else
	{
		// make sure the sign is not lost in casting
		return (RealType)FMath::Sign(InCircle(PA, PB, PC, PD));
	}
}

/**
 * TVector-only version that can run in float or double
 * @return value indicating whether point E is inside, outside, or exactly on the sphere passing through ABCD
 * Note: Sign of the result depends on the orientation of tetrahedron ABCD
 */
template<typename RealType>
RealType InSphere3(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C, const TVector<RealType>& D, const TVector<RealType>& E)
{
	// Note: Though we could compute exact predicates in float precision directly, in practice
	// it is more reliable and often faster to compute in double precision
	double PA[3]{ (double)A.X, (double)A.Y, (double)A.Z };
	double PB[3]{ (double)B.X, (double)B.Y, (double)B.Z };
	double PC[3]{ (double)C.X, (double)C.Y, (double)C.Z };
	double PD[3]{ (double)D.X, (double)D.Y, (double)D.Z };
	double PE[3]{ (double)E.X, (double)E.Y, (double)E.Z };
	if constexpr (std::is_same_v<RealType, double>)
	{
		return InSphere(PA, PB, PC, PD, PE);
	}
	else
	{
		// make sure the sign is not lost in casting
		return (RealType)FMath::Sign(InSphere(PA, PB, PC, PD, PE));
	}
}


}}} // namespace UE::Geometry::ExactPredicates
