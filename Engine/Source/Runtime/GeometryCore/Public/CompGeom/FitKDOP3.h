// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "MathUtil.h"
#include "Math/Vector.h"
#include "Math/Plane.h"
#include "VectorTypes.h"

template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Math { template <typename T> struct TVector; }

namespace Geometry
{


	/**
	 * Compute the vertices (and optionally planes) of a k-DOP bounding convex hull containing the points returned by GetPointFunc where FilterFunc returns true
	 */
	template <typename RealType>
	bool GEOMETRYCORE_API FitKDOPVertices3(
		TArrayView<const UE::Math::TVector<RealType>> PlaneDirections,
		const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> FilterFunc,
		TArray<UE::Math::TVector<RealType>>& OutVertices, TArray<UE::Math::TPlane<RealType>>* OptionalOutPlanes = nullptr,
		RealType Epsilon = TMathUtil<RealType>::Epsilon, RealType VertexSnapDistance = (RealType)0.1, RealType Inflate = (RealType)0.1);

	/**
	 * Compute the vertices (and optionally planes) of a k-DOP bounding convex hull containing the points returned by GetPointFunc
	 */
	template <typename RealType>
	bool FitKDOPVertices3(
		TArrayView<const UE::Math::TVector<RealType>> PlaneDirections,
		const int32 NumPoints, TFunctionRef<UE::Math::TVector<RealType>(int32)> GetPointFunc,
		TArray<UE::Math::TVector<RealType>>& OutVertices, TArray<UE::Math::TPlane<RealType>>* OptionalOutPlanes = nullptr,
		RealType Epsilon = TMathUtil<RealType>::Epsilon, RealType VertexSnapDistance = (RealType)0.1, RealType Inflate = (RealType)0.1)
	{
		return FitKDOPVertices3<RealType>(PlaneDirections, NumPoints, GetPointFunc, [](int32)->bool {return true;}, OutVertices, OptionalOutPlanes, Epsilon, VertexSnapDistance, Inflate);
	}

	/**
	 * Compute the vertices (and optionally planes) of a k-DOP bounding convex hull containing Points
	 */
	template <typename RealType>
	bool FitKDOPVertices3(
		TArrayView<const UE::Math::TVector<RealType>> PlaneDirections,
		const TArray<UE::Math::TVector<RealType>>& Points,
		TArray<UE::Math::TVector<RealType>>& OutVertices, TArray<UE::Math::TPlane<RealType>>* OptionalOutPlanes = nullptr,
		RealType Epsilon = TMathUtil<RealType>::Epsilon, RealType VertexSnapDistance = (RealType)0.1, RealType Inflate = (RealType)0.1)
	{
		return FitKDOPVertices3<RealType>(PlaneDirections, Points.Num(), [&Points](int32 Idx) {return Points[Idx];}, [](int32)->bool {return true;}, OutVertices, OptionalOutPlanes, Epsilon, VertexSnapDistance, Inflate);
	}

} // end namespace Geometry
}// end namespace UE

