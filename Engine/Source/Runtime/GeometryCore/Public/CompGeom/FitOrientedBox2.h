// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "VectorTypes.h"
#include "OrientedBoxTypes.h"

namespace UE
{
namespace Geometry
{

	enum EBox2FitCriteria
	{
		Area,
		Perimeter
	};


	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a point set.
	 * Note it internally computes a 2D convex hull of a point set, and is currently O(n log n) in the number of points.
	 *
	 * @param Points		The points to fit
	 * @param FitFn			Function from (Width, Height) -> Fit metric; The box with the lowest fit metric will be chosen
	 * @return				A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> GEOMETRYCORE_API FitOrientedBox2Points(TArrayView<const TVector2<RealType>> Points, TFunctionRef<RealType(RealType, RealType)> FitFn);

	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a point set.
	 * Note it internally computes a 2D convex hull of a point set, and is currently O(n log n) in the number of points.
	 * 
	 * @param Points		The points to fit
	 * @param FitMethod		What criteria to optimize
	 * @return				A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> FitOrientedBox2Points(TArrayView<const TVector2<RealType>> Points, EBox2FitCriteria FitMethod = EBox2FitCriteria::Area)
	{
		if (FitMethod == EBox2FitCriteria::Area)
		{
			return FitOrientedBox2Points(Points, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W * H;
				});
		}
		else // perimeter
		{
			return FitOrientedBox2Points(Points, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W + H; // Note: ok to skip constant factor
				});
		}
	}


	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a simple polygon.
	 * Note the polygon should not be self-intersecting.  O(n) in the number of points.
	 *
	 * @param Polygon	The vertices of the simple polygon to fit
	 * @param FitFn		Function from (Width, Height) -> Fit metric; The box with the lowest fit metric will be chosen
	 * @return			A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> GEOMETRYCORE_API FitOrientedBox2SimplePolygon(TArrayView<const TVector2<RealType>> Polygon, TFunctionRef<RealType(RealType, RealType)> FitFn);

	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a simple polygon.
	 * Note the polygon should not be self-intersecting.  O(n) in the number of points.
	 *
	 * @param Polygon	The vertices of the simple polygon to fit
	 * @param FitMethod	What criteria to optimize
	 * @return			A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> FitOrientedBox2SimplePolygon(TArrayView<const TVector2<RealType>> Polygon, EBox2FitCriteria FitMethod = EBox2FitCriteria::Area)
	{
		if (FitMethod == EBox2FitCriteria::Area)
		{
			return FitOrientedBox2SimplePolygon(Polygon, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W * H;
				});
		}
		else // perimeter
		{
			return FitOrientedBox2SimplePolygon(Polygon, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W + H; // Note: ok to skip constant factor
				});
		}
	}



	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a convex hull.
	 *
	 * @param NumPts		Number of points in the convex hull
	 * @param GetHullPt		Function(hull point index) -> hull point
	 * @param FitFn			Function from (Width, Height) -> Fit metric; The box with the lowest fit metric will be chosen
	 * @return				A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> GEOMETRYCORE_API FitOrientedBox2ConvexHull(int32 NumPts, TFunctionRef<TVector2<RealType>(int32)> GetHullPt, TFunctionRef<RealType(RealType, RealType)> FitFn);

	/**
	 * Use the rotating calipers algorithm to find a best-fit oriented bounding box of a convex hull.
	 *
	 * @param NumPts		Number of points in the convex hull
	 * @param GetHullPt		Function(hull point index) -> hull point
	 * @param FitMethod		What criteria to optimize
	 * @return				A best-fit TOrientedBox2 that contains the points
	 */
	template <typename RealType>
	TOrientedBox2<RealType> FitOrientedBox2ConvexHull(int32 NumPts, TFunctionRef<TVector2<RealType>(int32)> GetHullPt, EBox2FitCriteria FitMethod = EBox2FitCriteria::Area)
	{
		if (FitMethod == EBox2FitCriteria::Area)
		{
			return FitOrientedBox2ConvexHull(NumPts, GetHullPt, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W * H;
				});
		}
		else // perimeter
		{
			return FitOrientedBox2ConvexHull(NumPts, GetHullPt, (TFunctionRef<RealType(RealType, RealType)>)[](RealType W, RealType H) -> RealType
				{
					return W + H; // Note: ok to skip constant factor
				});
		}
	}

	// TODO: also have versions for a convex polygon input (+ a near-convex polygon version, that tolerates/fixes 'small' defects?)

} // end namespace Geometry
}// end namespace UE

