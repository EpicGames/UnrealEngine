// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "VectorTypes.h"
#include "OrientedBoxTypes.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

	enum EBox3FitCriteria
	{
		Volume,
		SurfaceArea
	};

	/**
	 * Compute a close-fitting oriented bounding box to the given points.
	 * Relatively expensive; for a faster approximation use DiTOrientedBox.h's ComputeOrientedBBox
	 *
	 * @param NumPoints				Number of points to consider
	 * @param GetPointFunc			Function providing array-style access into points
	 * @param Filter				Optional filter to include only a subset of the points in the output hull
	 * @param FitMethod				What criteria to optimize
	 * @param SameNormalTolerance	Tolerance for considering normals the same when choosing projection directions.  If > 0, can save some computation if the convex hull is very large.
	 * @return						A best-fit TOrientedBox3 that contains the points
	 */
	template <typename RealType>
	TOrientedBox3<RealType> GEOMETRYCORE_API FitOrientedBox3Points(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> Filter,
		EBox3FitCriteria FitMethod = EBox3FitCriteria::Volume, RealType SameNormalTolerance = 0, FProgressCancel* ProgressCancel = nullptr);

	/**
	 * Compute a close-fitting oriented bounding box to the given points.
	 * Relatively expensive; for a faster approximation use DiTOrientedBox.h's ComputeOrientedBBox
	 * 
	 * @param Points				The points to fit
	 * @param FitMethod				What criteria to optimize
	 * @param SameNormalTolerance	Tolerance for considering normals the same when choosing projection directions.  If > 0, can save some computation if the convex hull is very large.
	 * @return						A best-fit TOrientedBox3 that contains the points
	 */
	template <typename RealType>
	TOrientedBox3<RealType> FitOrientedBox3Points(TArrayView<const TVector<RealType>> Points, EBox3FitCriteria FitMethod = EBox3FitCriteria::Volume, RealType SameNormalTolerance = 0, FProgressCancel* ProgressCancel = nullptr)
	{
		auto GetPtFn = [&Points](int32 Idx)
		{
			return Points[Idx];
		};
		return FitOrientedBox3Points(Points.Num(), (TFunctionRef<TVector<RealType>(int32)>)GetPtFn, [](int32 Idx) {return true;}, FitMethod, SameNormalTolerance, ProgressCancel);
	}


	/**
	 * Improve an oriented bounding box fit iteratively
	 *
	 * @param InitialBox		An initial guess at an oriented bounding box fitting the points
	 * @param NumIterations		How many passes to take to improve the bounding 
	 * @param NumPoints			Number of points to consider
	 * @param GetPointFunc		Function providing array-style access into points
	 * @param Filter			Optional filter to include only a subset of the points in the output hull
	 * @param FitMethod			What criteria to optimize
	 * @return					A best-fit TOrientedBox3 that contains the points
	 */
	template <typename RealType>
	TOrientedBox3<RealType> GEOMETRYCORE_API OptimizeOrientedBox3Points(const TOrientedBox3<RealType>& InitialBox, int32 NumIterations, int32 NumPoints, 
		TFunctionRef<TVector<RealType>(int32)> GetPointFunc, TFunctionRef<bool(int32)> Filter, EBox3FitCriteria FitMethod = EBox3FitCriteria::Volume, FProgressCancel* ProgressCancel = nullptr);

} // end namespace Geometry
}// end namespace UE

