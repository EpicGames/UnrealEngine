// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "GeometryBase.h"
#include "IndexTypes.h"
#include "Math/MathFwd.h"
#include "VectorTypes.h"

namespace UE { namespace Geometry { struct FIndex3i; } }
namespace UE { namespace Math { template <typename T> struct TVector2; } }
namespace UE { namespace Math { template <typename T> struct TVector; } }

namespace PolygonTriangulation
{
	using namespace UE::Geometry;
	using namespace UE::Math;

	/**
	 * Compute triangulation of simple 2D polygon using ear-clipping
	 * @param VertexPositions ordered vertices of 2D polygon
	 * @param OutTriangles computed triangulation. Each triangle is a tuple of indices into VertexPositions.
	 * @param bOrientAsHoleFill if true, output triangles are wound opposite of the input polygon; this is typically desired when the polygon comes from the boundary of a hole
	 */
	template<typename RealType>
	void GEOMETRYCORE_API TriangulateSimplePolygon(const TArray<TVector2<RealType>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill = true);

	/**
	* Computes a plane centroid and plane normal from an input 3D polygon.
	* @param VertexPositions ordered vertices of 3D polygon.
	* @param PlaneNormalOut normal of the computed plane. Will be normalized.
	* @param PlanePointOut centroid of the computed plane.
	* @return Area of the polygon.
	*/
	template<typename RealType>
	RealType GEOMETRYCORE_API ComputePolygonPlane(const TArray<TVector<RealType>>& VertexPositions, TVector<RealType>& PlaneNormalOut, TVector<RealType>& PlanePointOut );


	/**
	 * Compute triangulation of 3D simple polygon using ear-clipping
	 * @param VertexPositions ordered vertices of 3D polygon
	 * @param OutTriangles computed triangulation. Each triangle is a tuple of indices into VertexPositions.
	 * @param bOrientAsHoleFill if true, output triangles are wound opposite of the input polygon; this is typically desired when the polygon comes from the boundary of a hole
	 */
	template<typename RealType>
	void GEOMETRYCORE_API TriangulateSimplePolygon(const TArray<TVector<RealType>>& VertexPositions, TArray<FIndex3i>& OutTriangles, bool bOrientAsHoleFill = true);

}