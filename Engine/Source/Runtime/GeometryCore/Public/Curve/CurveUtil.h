// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "SegmentTypes.h"
#include "LineTypes.h"
#include "BoxTypes.h"

#include "Math/NumericLimits.h"

namespace UE
{
namespace Geometry
{
namespace CurveUtil
{

using namespace UE::Math;

/**
 * Curve utility functions
 */

	/**
	 * Get (by reference) the vertices surrounding the given vertex index
	 * If bLoop is false and the Prev or Next vertex would be out of bounds, the vertex at Idx is used instead.
	 */
	template<typename RealType, typename VectorType, bool bLoop>
	inline void GetPrevNext(const TArrayView<const VectorType>& Vertices, int32 Idx, VectorType& OutPrev, VectorType& OutNext)
	{
		int32 NextIdx = Idx + 1;
		int32 PrevIdx = Idx - 1;
		int32 NV = Vertices.Num();
		if constexpr (bLoop)
		{
			NextIdx = NextIdx % NV;
			PrevIdx = (PrevIdx + NV) % NV;
		}
		else
		{
			NextIdx = FMath::Min(NV - 1, NextIdx);
			PrevIdx = FMath::Max(0, PrevIdx);
		}
		OutNext = Vertices[NextIdx];
		OutPrev = Vertices[PrevIdx];
	}

	/**
	 * Get (by reference) vectors pointing toward the given vertex index, from its surrounding vertices
	 * If bLoop is false and the Prev or Next vertex would be out of bounds, a zero vector is used instead
	 */
	template<typename RealType, typename VectorType, bool bLoop>
	inline void GetVectorsToPrevNext(const TArrayView<const VectorType>& Vertices, int32 VertexIndex, VectorType& OutToPrev, VectorType& OutToNext, bool bNormalize)
	{
		GetPrevNext<RealType, VectorType, bLoop>(Vertices, VertexIndex, OutToPrev, OutToNext);
		OutToNext -= Vertices[VertexIndex];
		OutToPrev -= Vertices[VertexIndex];
		if (bNormalize)
		{
			Normalize(OutToNext);
			Normalize(OutToPrev);
		}
	}

	/**
	 * Get (by reference) vectors pointing toward the given vertex index, from its surrounding vertices
	 * If bLoop is false and the Prev or Next vertex would be out of bounds, a zero vector is used instead
	 */
	template<typename RealType, typename VectorType>
	inline void GetVectorsToPrevNext(const TArrayView<const VectorType>& Vertices, int32 VertexIndex, VectorType& OutToPrev, VectorType& OutToNext, bool bNormalize, bool bLoop)
	{
		if (bLoop)
		{
			GetVectorsToPrevNext<RealType, VectorType, true>(Vertices, VertexIndex, OutToPrev, OutToNext, bNormalize);
		}
		else
		{
			GetVectorsToPrevNext<RealType, VectorType, false>(Vertices, VertexIndex, OutToPrev, OutToNext, bNormalize);
		}
	}
	
	/**
	 * @return the tangent direction (normalized) of the path at Idx, using the surrounding vertices (as found by GetPrevNext())
	 */
	template<typename RealType, typename VectorType, bool bLoop>
	inline VectorType Tangent(const TArrayView<const VectorType>& Vertices, int32 Idx)
	{
		VectorType Prev, Next;
		GetPrevNext<RealType, VectorType, bLoop>(Vertices, Idx, Prev, Next);
		return Normalized(Next - Prev);
	}

	/**
	 * @return the tangent direction (normalized) of the path at Idx, using the surrounding vertices (as found by GetPrevNext())
	 */
	template<typename RealType, typename VectorType>
	inline VectorType Tangent(const TArrayView<const VectorType>& Vertices, int32 Idx, bool bLoop = false)
	{
		return bLoop ?
			Tangent<RealType, VectorType, true>(Vertices, Idx) :
			Tangent<RealType, VectorType, false>(Vertices, Idx);
	}

	/**
	 * Construct a normal at a vertex of the Polygon by averaging the adjacent face normals. 
	 * This vector is independent of the lengths of the adjacent segments.
	 * Points "inward" for a Clockwise Polygon, and outward for CounterClockwise
	 * Note: Specific to 2D curves; normals computed by rotating tangents in the XY plane
	 */
	template<typename RealType, typename VectorType, bool bLoop>
	VectorType GetNormal_FaceAvg2(const TArrayView<const VectorType>& Vertices, int VertexIndex)
	{
		TVector2<RealType> ToPrev, ToNext;
		GetVectorsToPrevNext<RealType, VectorType, bLoop>(Vertices, VertexIndex, ToPrev, ToNext, true);

		TVector2<RealType> N = (PerpCW(ToNext) - PerpCW(ToPrev));
		RealType Len = Normalize(N);
		if (Len == 0)
		{
			return Normalized(ToNext + ToPrev);   // this gives right direction for degenerate angle
		}
		else
		{
			return N;
		}
	}

	/** 
	 * Compute the signed area of a closed curve, assuming vertices in the XY plane
	 */
	template<typename RealType, typename VectorType>
	RealType SignedArea2(const TArrayView<const VectorType>& Vertices)
	{
		RealType Area = 0;
		int N = Vertices.Num();
		if (N == 0)
		{
			return 0;
		}
		for (int Idx = 0, PrevIdx = N-1; Idx < N; PrevIdx=Idx++)
		{
			const TVector2<RealType>& V1 = Vertices[PrevIdx];
			const TVector2<RealType>& V2 = Vertices[Idx];
			Area += V1.X * V2.Y - V1.Y * V2.X;
		}
		return static_cast<RealType>(Area * 0.5);
	}

	/**
	 * Compute the winding of a point relative to a closed curve, assuming vertices in the XY plane
	 */
	template<typename RealType, typename VectorType>
	RealType WindingIntegral2(const TArrayView<const VectorType>& Vertices, const VectorType& QueryPoint)
	{
		RealType Sum = 0;
		int N = Vertices.Num();
		VectorType A = Vertices[N-1] - QueryPoint, B = VectorType::Zero();
		for (int Idx = 0; Idx < N; ++Idx)
		{
			B = Vertices[Idx] - QueryPoint;
			// TODO: Consider computing closed curve winding w/out trig functions, i.e. see Contains2 below
			Sum += TMathUtil<RealType>::Atan2(A.X * B.Y - A.Y * B.X, A.X * B.X + A.Y * B.Y);
			A = B;
		}
		return Sum / FMathd::TwoPi;
	}
	
	/**
	 * Project ToProject to a segment connecting points SegA,SegB, and return the unit interpolation parameter
	 * 
	 * @param SegA First point of the segment
	 * @param SegB Second point of the segment
	 * @param OutSegLenSq Will be set to the squared length of the segment
	 * @param ToProject Point to project
	 * @param ToleranceSquared If the squared segment length is less than this, will fall back to using the segment midpoint. (Defaults to 1e-16, which is FMathd::ZeroTolerance squared.)
	 * @return Unit interpolation parameter, s.t. Lerp(SegA, SegB, ReturnValue) will give the projected point
	 */
	template<typename VectorType, typename RealType = VectorType::FReal>
	inline RealType ProjectToSegment(const VectorType& SegA, const VectorType& SegB, RealType& OutSegLenSq, const VectorType& ToProject, RealType ToleranceSquared = (RealType)1e-16)
	{
		VectorType ABVec = SegB - SegA;
		OutSegLenSq = ABVec.SquaredLength();
		return FMath::Clamp(OutSegLenSq > ToleranceSquared ? ((ToProject - SegA) | ABVec) / OutSegLenSq : .5, (RealType)0., (RealType)1.);
	}

	/**
	 * Project the QueryPoint to the given curve, reporting squared distance, segment, and closest point
	 * 
	 * @param Vertices Vertices of the curve
	 * @param bLoop Whether the curve is a loop (i.e., whether we should include a segment from the last vertex to the first)
	 * @param QueryPoint Point to query for distance from curve
	 * @param OutDistanceSq Squared distance to query point
	 * @param OutNearestSegIndex Index of the nearest segment on the curve, or INDEX_NONE if the curve has no segments (has length < 2)
	 * @param OutNearestPoint The nearest point on the curve
	 * @param OutNearestSegmentUnitParam The 'unit parameter' along the closest segment -- i.e. the parameter from 0 to 1 that you would pass to a Lerp to get the closest point on that segment
	 * @param OutNearestSegmentParam If non-null, will be set to the 'segment parameter' along the closest segment -- i.e. the parameter from -SegmentExtent to +SegmentExtent where 0 is the segment midpoint
	 * @return true if a closest point was found (i.e. if the curve had vertices)
	 */
	template<typename RealType, typename VectorType>
	bool ProjectToCurve(const TArrayView<const VectorType>& Vertices, bool bLoop, const VectorType& QueryPoint, RealType& OutDistanceSq, int& OutNearestSegIndex, VectorType& OutNearestPoint, RealType& OutNearestSegmentUnitParam, RealType* OutNearestSegmentParam = nullptr)
	{
		OutDistanceSq = TNumericLimits<RealType>::Max();
		OutNearestSegIndex = INDEX_NONE;
		OutNearestSegmentUnitParam = TNumericLimits<RealType>::Max();
		if (OutNearestSegmentParam)
		{
			*OutNearestSegmentParam = TNumericLimits<RealType>::Max();
		}
		
		// handle if we don't have a curve
		if (Vertices.Num() < 2)
		{
			if (Vertices.Num() == 1)
			{
				OutNearestPoint = Vertices[0];
				OutDistanceSq = VectorType::DistSquared(OutNearestPoint, QueryPoint);
				return true;
			}
			else
			{
				return false;
			}
		}

		auto ProjectToSegmentHelper = [&Vertices, &QueryPoint, &OutNearestSegIndex, &OutNearestPoint, &OutNearestSegmentUnitParam, OutNearestSegmentParam, &OutDistanceSq](int32 SegIdx, int32 EndIdx)
		{
			VectorType A = Vertices[SegIdx], B = Vertices[EndIdx];
			RealType SegLenSq;
			RealType LerpParam = ProjectToSegment(A, B, SegLenSq, QueryPoint);
			VectorType ProjectedSegmentPoint = FMath::Lerp(A, B, LerpParam);
			RealType DSq = VectorType::DistSquared(ProjectedSegmentPoint, QueryPoint);
			if (DSq < OutDistanceSq)
			{
				OutDistanceSq = DSq;
				OutNearestSegIndex = SegIdx;
				OutNearestPoint = ProjectedSegmentPoint;
				OutNearestSegmentUnitParam = LerpParam;
				if (OutNearestSegmentParam)
				{
					// Convert LerpParam (0 to 1 range) to SegParam (+/- Extent aka Half-Length range)
					*OutNearestSegmentParam = (LerpParam - (RealType).5) * FMath::Sqrt(SegLenSq);
				}
			}
		};
		for (int SegIdx = 0, N = Vertices.Num(); SegIdx + 1 < N; ++SegIdx)
		{
			ProjectToSegmentHelper(SegIdx, SegIdx + 1);
		}
		if (bLoop)
		{
			ProjectToSegmentHelper(Vertices.Num() - 1, 0);
		}
		return true;
	}

	/**
	 * @return true if the given query point is inside the closed curve (in the XY plane), based on the winding integral
	 * Note: Negative and positive windings are both considered 'inside'
	 */
	template<typename RealType, typename VectorType>
	bool Contains2(const TArrayView<const VectorType>& Vertices, const VectorType& QueryPoint)
	{
		int WindingNumber = 0;

		int N = Vertices.Num();
		if (N == 0)
		{
			return false;
		}
		VectorType A = Vertices[N - 1], B = VectorType::Zero();
		for (int Idx = 0; Idx < N; ++Idx)
		{
			B = Vertices[Idx];

			if (A.Y <= QueryPoint.Y)     // y <= P.Y (below)
			{
				if (B.Y > QueryPoint.Y)									// an upward crossing
				{
					if (Orient(A, B, QueryPoint) > 0)  // P left of edge
					{
						++WindingNumber;                       // have a valid up intersect
					}
				}
			}
			else     // y > P.Y  (above)
			{
				if (B.Y <= QueryPoint.Y)									// a downward crossing
				{
					if (Orient(A, B, QueryPoint) < 0)  // P right of edge
					{
						--WindingNumber;						// have a valid down intersect
					}
				}
			}
			A = B;
		}
		return WindingNumber != 0;
	}

	template<typename RealType, typename VectorType>
	static RealType ArcLength(const TArrayView<const VectorType>& Vertices, bool bLoop = false) {
		RealType Sum = 0;
		int32 NV = Vertices.Num();
		for (int i = 1; i < NV; ++i)
		{
			Sum += Distance(Vertices[i], Vertices[i-1]);
		}
		if (bLoop && NV > 1)
		{
			Sum += Distance(Vertices[NV-1], Vertices[0]);
		}
		return Sum;
	}

	template<typename RealType, typename VectorType>
	int FindNearestIndex(const TArrayView<const VectorType>& Vertices, const VectorType& V)
	{
		int iNearest = -1;
		RealType dNear = TMathUtil<RealType>::MaxReal;
		int N = Vertices.Num();
		for ( int i = 0; i < N; ++i )
		{
			RealType dSqr = DistanceSquared(Vertices[i], V);
			if (dSqr < dNear)
			{
				dNear = dSqr;
				iNearest = i;
			}
		}
		return iNearest;
	}

	/**
	 * Use the Sutherland–Hodgman algorithm to clip the vertices to the given bounds
	 * Note if the path/polygon is concave, this may leave overlapping edges at the boundary.
	 * Note this modifies and re-sizes the input array, so is written for TArray instead of TArrayView
	 * @tparam bLoop	Whether the path is a loop
	 * @tparam ClipDims	How many dimensions of Min and Max to use for clipping
	 *					(e.g. if ClipDims is 2 and the vectors are 3D, clips only vs a rectangle in X and Y)
	 */
	template<typename RealType, typename VectorType, bool bLoop = true, int ClipDims = 2>
	static void ClipConvexToBounds(TArray<VectorType>& Vertices, VectorType Min, VectorType Max)
	{
		TArray<VectorType> Clipped;
		Clipped.Reserve(Vertices.Num());
		VectorType SidePt = Min;
		for (int32 SideIdx = 0; SideIdx < 2; ++SideIdx, SidePt = Max)
		{
			int32 SideSign = -SideIdx * 2 + 1;
			for (int32 ClipDim = 0; ClipDim < ClipDims; ++ClipDim)
			{
				RealType ClipCoord = SidePt[ClipDim];
				int32 VertNum = Vertices.Num();
				int32 StartCur, StartPrev;
				if constexpr (bLoop)
				{
					StartCur = 0;
					StartPrev = VertNum - 1;
				}
				else
				{
					StartCur = 1;
					StartPrev = 0;
				}
				RealType PrevDist = (Vertices[StartPrev][ClipDim] - ClipCoord) * RealType(SideSign);
				if constexpr (!bLoop)
				{
					if (PrevDist >= 0)
					{
						Clipped.Add(Vertices[0]);
					}
				}
				for (int32 CurIdx = StartCur, PrevIdx = StartPrev; CurIdx < VertNum; PrevIdx = CurIdx++)
				{
					RealType CurDist = (Vertices[CurIdx][ClipDim] - ClipCoord) * RealType(SideSign);
					if (CurDist >= 0)
					{
						if (PrevDist < 0 && CurDist > 0)
						{
							RealType T = CurDist / (CurDist - PrevDist);
							VectorType LerpVec = Lerp(Vertices[CurIdx], Vertices[PrevIdx], T);
							LerpVec[ClipDim] = ClipCoord; // snap to exact bounds
							Clipped.Add(LerpVec);
						}
						Clipped.Add(Vertices[CurIdx]);
					}
					else if (PrevDist > 0)
					{
						RealType T = CurDist / (CurDist - PrevDist);
						VectorType LerpVec = Lerp(Vertices[CurIdx], Vertices[PrevIdx], T);
						LerpVec[ClipDim] = ClipCoord; // snap to exact bounds
						Clipped.Add(LerpVec);
					}
					PrevDist = CurDist;
				}
				Swap(Vertices, Clipped);
				if (Vertices.IsEmpty())
				{
					return;
				}
				Clipped.Reset();
			}
		}
	}

	/**
	 * Use the Sutherland–Hodgman algorithm to clip the vertices to the given plane
	 * Note if the path/polygon is concave, this may leave overlapping edges at the boundary.
	 * @tparam bLoop				Whether the path is a loop
	 * @param Vertices				Vertices to clip vs the plane
	 * @param Plane					Plane to use for clipping
	 * @param OutClipped			Vertices clipped to the plane
	 * @param bKeepPositiveSide		Whether to keep the portion of the polygon that is on the positive side of the plane. Otherwise, keeps the negative side.
	 * @return Whether vertices were clipped
	 */
	template<typename RealType, typename VectorType, bool bLoop = true>
	static bool ClipConvexToPlane(TArray<VectorType>& Vertices, const TPlane<RealType>& Plane, TArray<VectorType>& OutClipped, bool bKeepPositiveSide = false)
	{
		OutClipped.Reset(Vertices.Num());
		bool bWasClipped = false;
		int32 VertNum = Vertices.Num();
		int32 StartCur, StartPrev;
		if constexpr (bLoop)
		{
			StartCur = 0;
			StartPrev = VertNum - 1;
		}
		else
		{
			StartCur = 1;
			StartPrev = 0;
		}
		RealType SideSign = bKeepPositiveSide ? (RealType)1 : (RealType)-1;
		RealType PrevDist = Plane.PlaneDot(Vertices[StartPrev]) * RealType(SideSign);
		if constexpr (!bLoop)
		{
			if (PrevDist >= 0)
			{
				OutClipped.Add(Vertices[0]);
			}
			else
			{
				bWasClipped = true;
			}
		}
		for (int32 CurIdx = StartCur, PrevIdx = StartPrev; CurIdx < VertNum; PrevIdx = CurIdx++)
		{
			RealType CurDist = Plane.PlaneDot(Vertices[CurIdx]) * RealType(SideSign);
			if (CurDist >= 0)
			{
				if (PrevDist < 0 && CurDist > 0)
				{
					RealType T = CurDist / (CurDist - PrevDist);
					VectorType LerpVec = Lerp(Vertices[CurIdx], Vertices[PrevIdx], T);
					OutClipped.Add(LerpVec);
					bWasClipped = true;
				}
				OutClipped.Add(Vertices[CurIdx]);
			}
			else
			{
				bWasClipped = true;
				if (PrevDist > 0)
				{
					RealType T = CurDist / (CurDist - PrevDist);
					VectorType LerpVec = Lerp(Vertices[CurIdx], Vertices[PrevIdx], T);
					OutClipped.Add(LerpVec);
				}
			}
			PrevDist = CurDist;
		}
		return bWasClipped;
	}

	/**
	 * Tests closed, 2D curve for convexity, with an optional tolerance allowing for approximately-collinear points
	 * Note that tolerance is per vertex angle, so a very-finely-sampled smooth concavity could be reported as convex
	 * 
	 * @param RadiansTolerance		Maximum turn in the 'wrong' direction that will still be considered convex (in radians)
	 * @param bDegenerateIsConvex	What to return for degenerate input (less than 3 points or equivalent due to repeated points)
	 * @return						true if polygon is convex
	 */
	template<typename RealType, typename VectorType>
	bool IsConvex2(const TArrayView<const VectorType> Vertices, RealType Tolerance = TMathUtil<RealType>::ZeroTolerance,
		bool bDegenerateIsConvex = true)
	{
		int32 N = Vertices.Num();
		if (N < 3)
		{
			// degenerate case, less than 3 points
			return bDegenerateIsConvex;
		}
		VectorType FromPrev = Vertices[N - 1] - Vertices[N - 2];
		int32 UsedPrevIdx = N - 2;
		while (!FromPrev.Normalize(0))
		{
			UsedPrevIdx--;
			if (UsedPrevIdx < 0)
			{
				return bDegenerateIsConvex; // degenerate case: all points coincident
			}
			FromPrev = Vertices[N - 1] - Vertices[UsedPrevIdx];
		}
		bool bSeenNeg = false, bSeenPos = false;
		RealType AngleSum = 0;
		int32 NonZeroEdges = 0;
		for (int32 Cur = N-1, Next = 0; Next < N; Next++)
		{
			VectorType ToNext = Vertices[Next] - Vertices[Cur];
			if (!ToNext.Normalize(0)) // skip duplicates
			{
				continue;
			}
			RealType Angle = SignedAngleR(FromPrev, ToNext);
			bSeenNeg |= (Angle < -Tolerance);
			bSeenPos |= (Angle > Tolerance);
			AngleSum += Angle;
			FromPrev = ToNext;
			Cur = Next;
			NonZeroEdges++;
		}
		if (NonZeroEdges < 3)
		{
			return bDegenerateIsConvex;
		}
		return !(bSeenNeg && bSeenPos) // curve should only turn one way, and
			// curve should loop only once, turning 2*Pi radians in that one direction
			&& int(TMathUtil<RealType>::Round(AngleSum / TMathUtil<RealType>::TwoPi)) == (int)bSeenPos - (int)bSeenNeg;
	}

	/**
	 * Project point inside a convex polygon with known orientation
	 *
	 * @param ProjPt				Point to project
	 * @param bReverseOrientation	Whether convex polygon orientation is reversed (i.e., has negative signed area)
	 * @return						true if the point was projected
	 */
	template<typename RealType>
	bool ProjectPointInsideConvexPolygon(const TArrayView<const TVector2<RealType>> Vertices, TVector2<RealType>& ProjPt, bool bReverseOrientation = false)
	{
		bool bProjected = false;
		const int32 N = Vertices.Num();
		for (int32 Idx = 0, PrevIdx = N - 1; Idx < N; PrevIdx = Idx++)
		{
			const TVector2<RealType>& V1 = Vertices[PrevIdx];
			const TVector2<RealType>& V2 = Vertices[Idx];
			TVector2<RealType> ToPt = ProjPt - V1;
			TVector2<RealType> Normal = PerpCW<RealType>(V2 - V1);
			RealType Orient = ToPt.Dot(Normal);
			bool bOutside = bReverseOrientation ? Orient < 0 : Orient > 0;
			if (bOutside)
			{
				RealType SqLen = Normal.SquaredLength();
				if (SqLen > (RealType)FLT_MIN)
				{
					bProjected = true;
					ProjPt -= Normal * Orient / SqLen;
				}
			}
		}
		return bProjected;
	}


	/**
	 * smooth vertices in-place (will not produce a symmetric result, but does not require extra buffer)
	 */
	template<typename RealType, typename VectorType>
	void InPlaceSmooth(TArrayView<VectorType> Vertices, int StartIdx, int EndIdx, double Alpha, int NumIterations, bool bClosed)
	{
		int N = Vertices.Num();
		if ( bClosed )
		{
			for (int Iter = 0; Iter < NumIterations; ++Iter)
			{
				for (int ii = StartIdx; ii < EndIdx; ++ii)
				{
					int i = (ii % N);
					int iPrev = (ii == 0) ? N - 1 : ii - 1;
					int iNext = (ii + 1) % N;
					TVector<RealType> prev = Vertices[iPrev], next = Vertices[iNext];
					TVector<RealType> c = (prev + next) * 0.5f;
					Vertices[i] = (1 - Alpha) * Vertices[i] + (Alpha) * c;
				}
			}
		}
		else
		{
			for (int Iter = 0; Iter < NumIterations; ++Iter)
			{
				for (int i = StartIdx; i < EndIdx; ++i)
				{
					if (i == 0 || i >= N - 1)
					{
						continue;
					}
					TVector<RealType> prev = Vertices[i - 1], next = Vertices[i + 1];
					TVector<RealType> c = (prev + next) * 0.5f;
					Vertices[i] = (1 - Alpha) * Vertices[i] + (Alpha) * c;
				}
			}
		}
	}



	/**
	 * smooth set of vertices using extra buffer
	 */
	template<typename RealType, typename VectorType>
	void IterativeSmooth(TArrayView<VectorType> Vertices, int StartIdx, int EndIdx, double Alpha, int NumIterations, bool bClosed)
	{
		int N = Vertices.Num();
		TArray<TVector<RealType>> Buffer;
		Buffer.SetNumZeroed(N);

		if (bClosed)
		{
			for (int Iter = 0; Iter < NumIterations; ++Iter)
			{
				for (int ii = StartIdx; ii < EndIdx; ++ii)
				{
					int i = (ii % N);
					int iPrev = (ii == 0) ? N - 1 : ii - 1;
					int iNext = (ii + 1) % N;
					TVector<RealType> prev = Vertices[iPrev], next = Vertices[iNext];
					TVector<RealType> c = (prev + next) * 0.5f;
					Buffer[i] = (1 - Alpha) * Vertices[i] + (Alpha) * c;
				}
				for (int ii = StartIdx; ii < EndIdx; ++ii)
				{
					int i = (ii % N);
					Vertices[i] = Buffer[i];
				}
			}
		}
		else
		{
			for (int Iter = 0; Iter < NumIterations; ++Iter)
			{
				for (int i = StartIdx; i < EndIdx && i < N; ++i)
				{
					if (i == 0 || i == N - 1)
					{
						continue;
					}
					TVector<RealType> prev = Vertices[i - 1], next = Vertices[i + 1];
					TVector<RealType> c = (prev + next) * 0.5f;
					Buffer[i] = (1 - Alpha) * Vertices[i] + (Alpha) * c;
				}
				for (int i = StartIdx; i < EndIdx && i < N; ++i)
				{
					if (i == 0 || i == N - 1)
					{
						continue;
					}
					Vertices[i] = Buffer[i];
				}
			}
		}
	}


} // end namespace UE::Geometry::CurveUtil
} // end namespace UE::Geometry
} // end namespace UE