// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreTypes.h"

#include "VectorTypes.h"
#include "IndexTypes.h"
#include "BoxTypes.h"
#include "GeometryTypes.h"
#include "Util/DynamicVector.h"
#include "Util/IndexUtil.h"
#include "Util/IteratorUtil.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{


class FDynamicGraph
{
public:
	static constexpr int InvalidID = IndexConstants::InvalidID; // -1;
	static constexpr int DuplicateEdgeID = -2;

	struct FEdge
	{
		int A, B, Group;
	};

	static FIndex2i InvalidEdgeV()
	{
		return FIndex2i(InvalidID, InvalidID);
	}
	static FEdge InvalidEdge3()
	{
		return FEdge{InvalidID, InvalidID, InvalidID};
	}

protected:
	FRefCountVector vertices_refcount;

	FSmallListSet vertex_edges;

	FRefCountVector edges_refcount;
	TDynamicVector<FEdge> edges; // each edge is a tuple (v0,v0,GroupID)

	int timestamp = 0;
	int shape_timestamp = 0;

	int max_group_id = 0;

public:
	FDynamicGraph()
		: max_group_id(0)
	{
	}

	virtual ~FDynamicGraph() = default;

protected:
	void updateTimeStamp(bool bShapeChange)
	{
		timestamp++;
		if (bShapeChange)
		{
			shape_timestamp++;
		}
	}

public:
	int Timestamp() const
	{
		return timestamp;
	}
	int ShapeTimestamp() const
	{
		return shape_timestamp;
	}

	int VertexCount() const
	{
		return vertices_refcount.GetCount();
	}
	int EdgeCount() const
	{
		return edges_refcount.GetCount();
	}

	// these values are (max_used+1), ie so an iteration should be < MaxVertexID, not <=
	int MaxVertexID() const
	{
		return vertices_refcount.GetMaxIndex();
	}
	int MaxEdgeID() const
	{
		return edges_refcount.GetMaxIndex();
	}
	int MaxGroupID() const
	{
		return max_group_id;
	}

	bool IsVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID);
	}
	bool IsEdge(int EID) const
	{
		return edges_refcount.IsValid(EID);
	}

	/**
	 * Enumerate "other" vertices of edges connected to vertex (i.e. vertex one-ring)
	 */
	FSmallListSet::MappedValueEnumerable VtxVerticesItr(int VID) const
	{
		check(vertices_refcount.IsValid(VID));
		return vertex_edges.MappedValues(VID, [VID, this](int EID) { return edge_other_v(EID, VID); });
	}

	/**
	 * Enumerate edge ids connected to vertex (i.e. edge one-ring)
	 */
	FSmallListSet::ValueEnumerable VtxEdgesItr(int VID) const
	{
		check(vertices_refcount.IsValid(VID));
		return vertex_edges.Values(VID);
	}

	int GetVtxEdgeCount(int VID) const
	{
		return vertices_refcount.IsValid(VID) ? vertex_edges.GetCount(VID) : -1;
	}

	int GetMaxVtxEdgeCount() const
	{
		int max = 0;
		for (int VID : vertices_refcount.Indices())
		{
			max = FMath::Max(max, vertex_edges.GetCount(VID));
		}
		return max;
	}

	FIndex2i GetEdgeV(int EID) const
	{
		return edges_refcount.IsValid(EID) ? FIndex2i(edges[EID].A, edges[EID].B) : InvalidEdgeV();
	}

	int GetEdgeGroup(int EID) const
	{
		return edges_refcount.IsValid(EID) ? edges[EID].Group : -1;
	}

	void SetEdgeGroup(int EID, int GroupID)
	{
		check(edges_refcount.IsValid(EID));
		if (edges_refcount.IsValid(EID))
		{
			edges[EID].Group = GroupID;
			max_group_id = FMath::Max(max_group_id, GroupID + 1);
			updateTimeStamp(false);
		}
	}

	int AllocateEdgeGroup()
	{
		return max_group_id++;
	}

	FEdge GetEdge(int EID) const
	{
		return edges_refcount.IsValid(EID) ? edges[EID] : InvalidEdge3();
	}

protected:
	int append_vertex_internal()
	{
		int VID = vertices_refcount.Allocate();
		allocate_edges_list(VID);
		updateTimeStamp(true);
		return VID;
	}

	bool insert_vertex_internal(int32 Vid)
	{
		if (!vertices_refcount.AllocateAt(Vid))
		{
			return false;
		}
		allocate_edges_list(Vid);
		updateTimeStamp(true);
		return true;
	}

private:
	void allocate_edges_list(int VID)
	{
		if (VID < (int)vertex_edges.Size())
		{
			vertex_edges.Clear(VID);
		}
		vertex_edges.AllocateAt(VID);
	}

public:
	int AppendEdge(const FEdge& E)
	{
		return AppendEdge(E.A, E.B, E.Group);
	}
	int AppendEdge(const FIndex2i& ev, int GID = -1)
	{
		return AppendEdge(ev[0], ev[1], GID);
	}
	int AppendEdge(int v0, int v1, int GID = -1)
	{
		if (IsVertex(v0) == false || IsVertex(v1) == false)
		{
			check(false);
			return InvalidID;
		}
		if (v0 == v1)
		{
			check(false);
			return InvalidID;
		}
		int e0 = FindEdge(v0, v1);
		if (e0 != InvalidID)
			return DuplicateEdgeID;

		// Increment ref counts and update/create edges
		vertices_refcount.Increment(v0);
		vertices_refcount.Increment(v1);
		max_group_id = FMath::Max(max_group_id, GID + 1);

		// now safe to insert edge
		int EID = add_edge(v0, v1, GID);

		updateTimeStamp(true);
		return EID;
	}

protected:
	int add_edge(int A, int B, int GID)
	{
		if (B < A)
		{
			int t = B;
			B = A;
			A = t;
		}
		int EID = edges_refcount.Allocate();
		edges.InsertAt(FEdge{A, B, GID}, EID);

		vertex_edges.Insert(A, EID);
		vertex_edges.Insert(B, EID);
		return EID;
	}

public:
	//
	// iterators
	//   The functions vertices() / triangles() / edges() are provided so you can do:
	//      for ( int EID : edges() ) { ... }
	//   and other related begin() / end() idioms
	//
	typedef typename FRefCountVector::IndexEnumerable vertex_iterator;
	vertex_iterator VertexIndices() const
	{
		return vertices_refcount.Indices();
	}

	typedef typename FRefCountVector::IndexEnumerable edge_iterator;
	edge_iterator EdgeIndices() const
	{
		return edges_refcount.Indices();
	}

	template <typename T>
	using value_iteration = FRefCountVector::MappedEnumerable<T>;

	/**
	 * Enumerate edges.
	 */
	value_iteration<FEdge> Edges() const
	{
		return edges_refcount.MappedIndices<FEdge>([this](int EID) {
			return edges[EID];
		});
	}

	int FindEdge(int VA, int VB) const
	{
		int vMax = FMath::Max(VA, VB);
		for (int EID : vertex_edges.Values(FMath::Min(VA, VB)))
		{
			//if (edge_has_v(EID, vMax)) // (slower option; does not use the fact that edges always have larger vertex as B)
			if (edges[EID].B == vMax)
			{
				return EID;
			}
		}
		return InvalidID;
	}

	EMeshResult RemoveEdge(int EID, bool bRemoveIsolatedVertices)
	{
		if (!edges_refcount.IsValid(EID))
		{
			check(false);
			return EMeshResult::Failed_NotAnEdge;
		}

		FIndex2i ev(edges[EID].A, edges[EID].B);
		vertex_edges.Remove(ev.A, EID);
		vertex_edges.Remove(ev.B, EID);

		edges_refcount.Decrement(EID);

		// Decrement vertex refcounts. If any hit 1 and we got remove-isolated flag,
		// we need to remove that vertex
		for (int j = 0; j < 2; ++j)
		{
			int VID = ev[j];
			vertices_refcount.Decrement(VID);
			if (bRemoveIsolatedVertices && vertices_refcount.GetRefCount(VID) == 1)
			{
				vertices_refcount.Decrement(VID);
				check(vertices_refcount.IsValid(VID) == false);
				vertex_edges.Clear(VID);
			}
		}

		updateTimeStamp(true);
		return EMeshResult::Ok;
	}

	EMeshResult RemoveVertex(int VID, bool bRemoveIsolatedVertices)
	{
		for (int EID : VtxEdgesItr(VID))
		{
			EMeshResult result = RemoveEdge(EID, bRemoveIsolatedVertices);
			if (result != EMeshResult::Ok)
				return result;
		}
		return EMeshResult::Ok;
	}

	struct FEdgeSplitInfo
	{
		int VNew;
		int ENewBN; // new edge [VNew,VB] (original was AB)
	};
	EMeshResult SplitEdge(int VA, int VB, FEdgeSplitInfo& Split)
	{
		int EID = FindEdge(VA, VB);
		if (EID == InvalidID)
		{
			return EMeshResult::Failed_NotAnEdge;
		}
		return SplitEdge(EID, Split);
	}
	EMeshResult SplitEdge(int EAB, FEdgeSplitInfo& Split)
	{
		if (!IsEdge(EAB))
		{
			return EMeshResult::Failed_NotAnEdge;
		}

		// look up primary edge
		int eab_i = 3 * EAB;
		int A = edges[EAB].A, B = edges[EAB].B;
		int GID = edges[EAB].Group;
		int f = append_new_split_vertex(A, B);

		// rewrite edge bc, create edge af
		int eaf = EAB;
		replace_edge_vertex(eaf, B, f);
		vertex_edges.Remove(B, EAB);
		vertex_edges.Insert(f, eaf);

		// create new edge fb
		int efb = add_edge(f, B, GID);

		// update vertex refcounts
		vertices_refcount.Increment(f, 2);

		Split.VNew = f;
		Split.ENewBN = efb;

		updateTimeStamp(true);
		return EMeshResult::Ok;
	}

	EMeshResult SplitEdgeWithExistingVertex(int EAB, int ExistingMidVert, FEdgeSplitInfo& Split)
	{
		if (!IsEdge(EAB))
		{
			return EMeshResult::Failed_NotAnEdge;
		}

		int f = ExistingMidVert;
		if (!IsVertex(f))
		{
			return EMeshResult::Failed_NotAVertex;
		}

		// look up primary edge
		int eab_i = 3 * EAB;
		int A = edges[EAB].A, B = edges[EAB].B;
		int GID = edges[EAB].Group;

		int EAf = FindEdge(A, f);
		int FIncr = 0; // track how much to increment the reference count of middle vertex, f
		int BIncr = 0;
		if (EAf != InvalidID)
		{
			RemoveEdge(EAB, false); // this will handle changes to refcounts, so no need to touch bIncr
			edges[EAf].Group = GID;
		}
		else
		{
			// rewrite edge ab to create edge af
			EAf = EAB;
			replace_edge_vertex(EAf, B, f);
			vertex_edges.Remove(B, EAB);
			vertex_edges.Insert(f, EAf);
			FIncr++;
			BIncr--;
		}

		int EfB = FindEdge(f, B);
		if (EfB == InvalidID)
		{
			EfB = add_edge(f, B, GID);
			FIncr++;
			BIncr++;
		}
		else
		{
			edges[EfB].Group = GID;
		}
		
		// update middle vertex refcounts
		vertices_refcount.Increment(f, (unsigned short) FIncr);
		if (BIncr > 0)
		{
			vertices_refcount.Increment(B, (unsigned short) BIncr);
		}
		if (BIncr < 0)
		{
			unsigned short BDecr = (unsigned short) (-BIncr);
			vertices_refcount.Decrement(B, BDecr);
		}

		Split.VNew = f;
		Split.ENewBN = EfB;

		updateTimeStamp(true);
		return EMeshResult::Ok;
	}

protected:
	virtual int append_new_split_vertex(int A, int B)
	{
		// Not Implemented: DGraph2.append_new_split_vertex
		unimplemented();
		return InvalidID;
	}

public:
	struct FEdgeCollapseInfo
	{
		int VKept;
		int VRemoved;

		int ECollapsed; // edge we collapsed
	};
	EMeshResult CollapseEdge(int VKeep, int VRemove, FEdgeCollapseInfo& Collapse)
	{
		bool DiscardIsolatedVertices = true;

		if (IsVertex(VKeep) == false || IsVertex(VRemove) == false)
		{
			return EMeshResult::Failed_NotAnEdge;
		}

		int B = VKeep; // renaming for sanity. We remove A and keep B
		int A = VRemove;

		int EAB = FindEdge(A, B);
		if (EAB == InvalidID)
		{
			return EMeshResult::Failed_NotAnEdge;
		}

		// get rid of any edges that will be duplicates
		bool done = false;
		while (!done)
		{
			done = true;
			for (int eax : vertex_edges.Values(A))
			{
				int o = edge_other_v(eax, A);
				if (o != B && FindEdge(B, o) != InvalidID)
				{
					RemoveEdge(eax, DiscardIsolatedVertices);
					done = false;
					break;
				}
			}
		}

		vertex_edges.Remove(B, EAB);
		for (int eax : vertex_edges.Values(A))
		{
			int o = edge_other_v(eax, A);
			if (o == B)
			{
				continue; // discard this edge
			}
			replace_edge_vertex(eax, A, B);
			vertices_refcount.Decrement(A);
			vertex_edges.Insert(B, eax);
			vertices_refcount.Increment(B);
		}

		edges_refcount.Decrement(EAB);
		vertices_refcount.Decrement(B);
		vertices_refcount.Decrement(A);
		if (DiscardIsolatedVertices)
		{
			vertices_refcount.Decrement(A); // second Decrement discards isolated vtx
			check(!IsVertex(A));
		}

		vertex_edges.Clear(A);

		Collapse.VKept = VKeep;
		Collapse.VRemoved = VRemove;
		Collapse.ECollapsed = EAB;

		updateTimeStamp(true);
		return EMeshResult::Ok;
	}

protected:
	bool edge_has_v(int EID, int VID) const
	{
		return (edges[EID].A == VID) || (edges[EID].B == VID);
	}
	int edge_other_v(int EID, int VID) const
	{
		int ev0 = edges[EID].A, ev1 = edges[EID].B;
		return (ev0 == VID) ? ev1 : ((ev1 == VID) ? ev0 : InvalidID);
	}
	int replace_edge_vertex(int EID, int VOld, int VNew)
	{
		int A = edges[EID].A, B = edges[EID].B;
		if (A == VOld)
		{
			edges[EID].A = FMath::Min(B, VNew);
			edges[EID].B = FMath::Max(B, VNew);
			return 0;
		}
		else if (B == VOld)
		{
			edges[EID].A = FMath::Min(A, VNew);
			edges[EID].B = FMath::Max(A, VNew);
			return 1;
		}
		else
			return -1;
	}

public:
	bool IsCompact() const
	{
		return vertices_refcount.IsDense() && edges_refcount.IsDense();
	}
	bool IsCompactV() const
	{
		return vertices_refcount.IsDense();
	}

	bool IsBoundaryVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID) && vertex_edges.GetCount(VID) == 1;
	}

	bool IsJunctionVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID) && vertex_edges.GetCount(VID) > 2;
	}

	bool IsRegularVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID) && vertex_edges.GetCount(VID) == 2;
	}


	/**
	 * This function checks that the graph is well-formed, ie all internal data
	 * structures are consistent
	 */
	virtual bool CheckValidity(EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check) const
	{
		bool is_ok = true;
		TFunction<void(bool)> CheckOrFailF = [&](bool b)
		{
			is_ok = is_ok && b;
		};
		if (FailMode == EValidityCheckFailMode::Check)
		{
			CheckOrFailF = [&](bool b)
			{
				checkf(b, TEXT("FEdgeLoop::CheckValidity failed!"));
				is_ok = is_ok && b;
			};
		}
		else if (FailMode == EValidityCheckFailMode::Ensure)
		{
			CheckOrFailF = [&](bool b)
			{
				ensureMsgf(b, TEXT("FEdgeLoop::CheckValidity failed!"));
				is_ok = is_ok && b;
			};
		}

		// edge verts/tris must exist
		for (int EID : EdgeIndices())
		{
			CheckOrFailF(IsEdge(EID));
			CheckOrFailF(edges_refcount.GetRefCount(EID) == 1);
			FIndex2i ev = GetEdgeV(EID);
			CheckOrFailF(IsVertex(ev[0]));
			CheckOrFailF(IsVertex(ev[1]));
			CheckOrFailF(ev[0] < ev[1]);
		}

		// verify compact check
		bool is_compact = vertices_refcount.IsDense();
		if (is_compact)
		{
			for (int VID = 0; VID < VertexCount(); ++VID)
			{
				CheckOrFailF(vertices_refcount.IsValid(VID));
			}
		}

		// vertex edges must exist and reference this vert
		for (int VID : VertexIndices())
		{
			CheckOrFailF(IsVertex(VID));

			//FVector3d V = GetVertex(VID);
			//CheckOrFailF(double.IsNaN(V.SquaredLength()) == false);
			//CheckOrFailF(double.IsInfinity(V.SquaredLength()) == false);

			for (int edgeid : vertex_edges.Values(VID))
			{
				CheckOrFailF(IsEdge(edgeid));
				CheckOrFailF(edge_has_v(edgeid, VID));

				int otherV = edge_other_v(edgeid, VID);
				int e2 = FindEdge(VID, otherV);
				CheckOrFailF(e2 != InvalidID);
				CheckOrFailF(e2 == edgeid);
				e2 = FindEdge(otherV, VID);
				CheckOrFailF(e2 != InvalidID);
				CheckOrFailF(e2 == edgeid);
			}

			CheckOrFailF(vertices_refcount.GetRefCount(VID) == vertex_edges.GetCount(VID) + 1);
		}

		subclass_validity_checks(CheckOrFailF);

		return is_ok;
	}

protected:
	virtual void subclass_validity_checks(TFunction<void(bool)> CheckOrFailF) const
	{
	}

	inline void debug_check_is_vertex(int V) const
	{
		check(IsVertex(V));
	}

	inline void debug_check_is_edge(int E) const
	{
		check(IsEdge(E));
	}
};

/**
 * Implementation of DGraph that has no dimensionality, ie no data
 * stored for vertices besides indices.
 */
class FDynamicGraphN : public FDynamicGraph
{
public:
	int AppendVertex()
	{
		return append_vertex_internal();
	}

protected:
	// internal used in SplitEdge
	virtual int append_new_split_vertex(int A, int B) override
	{
		return AppendVertex();
	}
};


} // end namespace UE::Geometry
} // end namespace UE
