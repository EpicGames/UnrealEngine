// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "BoxTypes.h"
#include "DynamicGraph.h"
#include "SegmentTypes.h"
#include "Util/DynamicVector.h"
#include "Util/IndexUtil.h"
#include "Util/IteratorUtil.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"
#include "VectorTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template <typename T>
class FDynamicGraph2 : public FDynamicGraph
{
	TAxisAlignedBox2<T> cached_bounds;
	int cached_bounds_timestamp = -1;

	TDynamicVectorN<T, 2> vertices;

public:
	static TVector2<T> InvalidVertex()
	{
		return TVector2<T>(TNumericLimits<T>::Max(), 0);
	}

	TVector2<T> GetVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID) ? vertices.AsVector2(VID) : InvalidVertex();
	}

	void SetVertex(int VID, TVector2<T> VNewPos)
	{
		check(VectorUtil::IsFinite(VNewPos)); // this will really catch a lot of bugs...
		if (vertices_refcount.IsValid(VID))
		{
			vertices.SetVector2(VID, VNewPos);
			updateTimeStamp(true);
		}
	}

	using FDynamicGraph::GetEdgeV;
	bool GetEdgeV(int EID, UE::Math::TVector2<T>& A, UE::Math::TVector2<T>& B) const
	{
		if (edges_refcount.IsValid(EID))
		{
			A = vertices.AsVector2(edges[EID].A);
			B = vertices.AsVector2(edges[EID].B);
			return true;
		}
		return false;
	}

	TSegment2<T> GetEdgeSegment(int EID) const
	{
		checkf(edges_refcount.IsValid(EID), TEXT("FDynamicGraph2.GetEdgeSegment: invalid segment with id %d"), EID);
		const FEdge& e = edges[EID];
		return TSegment2<T>(
			vertices.AsVector2(e.A),
			vertices.AsVector2(e.B));
	}

	TVector2<T> GetEdgeCenter(int EID) const
	{
		checkf(edges_refcount.IsValid(EID), TEXT("FDynamicGraph2.GetEdgeCenter: invalid segment with id %d"), EID);
		const FEdge& e = edges[EID];
		return 0.5 * (vertices.AsVector2(e.A) + vertices.AsVector2(e.B));
	}

	int AppendVertex(TVector2<T> V)
	{
		int vid = append_vertex_internal();
		vertices.InsertAt({{V.X, V.Y}}, vid);
		return vid;
	}

	//void AppendPolygon(Polygon2d poly, int gid = -1) {
	//	int first = -1;
	//	int prev = -1;
	//	int N = poly.VertexCount;
	//	for (int i = 0; i < N; ++i) {
	//		int cur = AppendVertex(poly[i]);
	//		if (prev == -1)
	//			first = cur;
	//		else
	//			AppendEdge(prev, cur, gid);
	//		prev = cur;
	//	}
	//	AppendEdge(prev, first, gid);
	//}
	//void AppendPolygon(GeneralPolygon2d poly, int gid = -1)
	//{
	//	AppendPolygon(poly.Outer, gid);
	//	foreach(var hole in poly.Holes)
	//		AppendPolygon(hole, gid);
	//}

	//void AppendPolyline(PolyLine2d poly, int gid = -1)
	//{
	//	int prev = -1;
	//	int N = poly.VertexCount;
	//	for (int i = 0; i < N; ++i) {
	//		int cur = AppendVertex(poly[i]);
	//		if (i > 0)
	//			AppendEdge(prev, cur, gid);
	//		prev = cur;
	//	}
	//}

	/*
	void AppendGraph(FDynamicGraph2 graph, int gid = -1)
	{
		int[] mapV = new int[graph.MaxVertexID];
		foreach(int vid in graph.VertexIndices()) {
			mapV[vid] = this.AppendVertex(graph.GetVertex(vid));
		}
		foreach(int eid in graph.EdgeIndices()) {
			FIndex2i ev = graph.GetEdgeV(eid);
			int use_gid = (gid == -1) ? graph.GetEdgeGroup(eid) : gid;
			this.AppendEdge(mapV[ev.A], mapV[ev.B], use_gid);
		}
	}

*/

	/** Enumerate positions of all vertices in graph */
	value_iteration<TVector2<T>> Vertices() const
	{
		return vertices_refcount.MappedIndices<TVector2<T>>(
			[&, this](int vid) {
				return vertices.template AsVector2<T>(vid);
			});
	}

	/**
	 * return edges around VID sorted by angle, in counter-clockwise order
	 */
	bool SortedVtxEdges(int VID, TArray<int>& Sorted) const
	{
		if (vertices_refcount.IsValid(VID) == false)
			return false;
		Sorted.Reserve(vertex_edges.GetCount(VID));
		for (int EID : vertex_edges.Values(VID))
		{
			Sorted.Add(EID);
		}
		TVector2<T> V = vertices.AsVector2(VID);
		Algo::SortBy(Sorted, [&](int EID) {
			int NbrVID = edge_other_v(EID, VID);
			TVector2<T> D = vertices.AsVector2(NbrVID) - V;
			return TMathUtil<T>::Atan2Positive(D.Y, D.X);
		});

		return true;
	}

	// compute vertex bounding box
	FAxisAlignedBox2d GetBounds() const
	{
		TAxisAlignedBox2<T> AABB;
		for (const TVector2<T>& V : Vertices())
		{
			AABB.Contain(V);
		}
		return AABB;
	}

	//! cached bounding box, lazily re-computed on access if mesh has changed
	TAxisAlignedBox2<T> CachedBounds()
	{
		if (cached_bounds_timestamp != Timestamp())
		{
			cached_bounds = GetBounds();
			cached_bounds_timestamp = Timestamp();
		}
		return cached_bounds;
	}

	/**
	 * Compute opening angle at vertex VID.
	 * If not a vertex, or valence != 2, returns InvalidValue argument.
	 * If either edge is degenerate, returns InvalidValue argument.
	 */
	double OpeningAngle(int VID, double InvalidValue = TNumericLimits<T>::Max()) const
	{
		if (vertices_refcount.IsValid(VID) == false)
		{
			return InvalidValue;
		}
		if (vertex_edges.GetCount(VID) != 2)
		{
			return InvalidValue;
		}
		FSmallListSet::ValueIterator ValueIterate = vertex_edges.Values(VID).begin();
		
		int nbra = edge_other_v(*ValueIterate, VID);
		int nbrb = edge_other_v(*++ValueIterate, VID);

		TVector2<T> V = vertices.AsVector2(VID);
		TVector2<T> A = vertices.AsVector2(nbra);
		TVector2<T> B = vertices.AsVector2(nbrb);
		A -= V;
		if (Normalize(A) == 0)
		{
			return InvalidValue;
		}
		B -= V;
		if (Normalize(B) == 0)
		{
			return InvalidValue;
		}
		return AngleD(A, B);
	}

protected:
	// internal used in SplitEdge
	virtual int append_new_split_vertex(int A, int B) override
	{
		TVector2<T> vNew = 0.5 * (GetVertex(A) + GetVertex(B));
		int f = AppendVertex(vNew);
		return f;
	}

	virtual void subclass_validity_checks(TFunction<void(bool)> CheckOrFailF) const override
	{
		for (int VID : VertexIndices())
		{
			TVector2<T> V = GetVertex(VID);
			CheckOrFailF(VectorUtil::IsFinite(V));
		}
	}
};

typedef FDynamicGraph2<double> FDynamicGraph2d;


} // end namespace UE::Geometry
} // end namespace UE