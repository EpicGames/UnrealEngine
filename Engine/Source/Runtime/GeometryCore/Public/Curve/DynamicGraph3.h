// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "BoxTypes.h"
#include "DynamicGraph.h"
#include "SegmentTypes.h"
#include "Util/DynamicVector.h"
#include "Util/IndexUtil.h"
#include "Util/IteratorUtil.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"
#include "VectorTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template <typename T>
class FDynamicGraph3 : public FDynamicGraph
{
	TDynamicVectorN<T, 3> Vertices;

public:
	static TVector<T> InvalidVertex()
	{
		return TVector<T>(TNumericLimits<T>::Max(), 0, 0);
	}

	TVector<T> GetVertex(int VID) const
	{
		return vertices_refcount.IsValid(VID) ? Vertices.AsVector3(VID) : InvalidVertex();
	}

	void SetVertex(int VID, TVector<T> VNewPos)
	{
		check(VectorUtil::IsFinite(VNewPos)); // this will really catch a lot of bugs...
		if (vertices_refcount.IsValid(VID))
		{
			Vertices.SetVector3(VID, VNewPos);
			updateTimeStamp(true);
		}
	}

	using FDynamicGraph::GetEdgeV;
	bool GetEdgeV(int EID, TVector<T>& A, TVector<T>& B) const
	{
		if (edges_refcount.IsValid(EID))
		{
			A = Vertices.AsVector3(edges[EID].A);
			B = Vertices.AsVector3(edges[EID].B);
			return true;
		}
		return false;
	}

	TSegment3<T> GetEdgeSegment(int EID) const
	{
		checkfSlow(edges_refcount.IsValid(EID), TEXT("FDynamicGraph2.GetEdgeSegment: invalid segment with id %d"), EID);
		const FEdge& e = edges[EID];
		return TSegment3<T>(
			Vertices.AsVector3(e.A),
			Vertices.AsVector3(e.B));
	}

	TVector<T> GetEdgeCenter(int EID) const
	{
		checkfSlow(edges_refcount.IsValid(EID), TEXT("FDynamicGraph3.GetEdgeCenter: invalid segment with id %d"), EID);
		const FEdge& e = edges[EID];
		return 0.5 * (Vertices.AsVector3(e.A) + Vertices.AsVector3(e.B));
	}

	int AppendVertex(TVector<T> V)
	{
		int vid = append_vertex_internal();
		Vertices.InsertAt({{V.X, V.Y, V.Z}}, vid);
		return vid;
	}

	/**
	 * @return false If insertion failed (because a vertex with that ID already exists).
	 */
	bool InsertVertex(int32 Vid, TVector<T> V)
	{
		// For now we desided not to add the "bUnsafe" optimization machinery like the similar method in
		// FDynamicMesh3. It is not needed if InsertVertex is used just for creating a graph with non compact 
		// Vids, where we would presumably still iterate through the source indices in increasing order, and 
		// so never have to worry about the performance cost of removing from the free list.

		if (!insert_vertex_internal(Vid))
		{
			return false;
		}
		Vertices.InsertAt({ {V.X, V.Y, V.Z} }, Vid);

		return true;
	}

	FRefCountVector::IndexEnumerable VertexIndicesItr() const
	{
		return vertices_refcount.Indices();
	}

	/** Enumerate positions of all Vertices in graph */
	value_iteration<TVector<T>> VerticesItr() const
	{
		return vertices_refcount.MappedIndices<TVector<T>>(
			[&, this](int vid) {
				return Vertices.template AsVector3<T>(vid);
			});
	}


	// compute vertex bounding box
	FAxisAlignedBox2d GetBounds() const
	{
		TAxisAlignedBox2<T> AABB;
		for (const TVector<T>& V : Vertices())
		{
			AABB.Contain(V);
		}
		return AABB;
	}


protected:
	// internal used in SplitEdge
	virtual int append_new_split_vertex(int A, int B) override
	{
		TVector<T> vNew = 0.5 * (GetVertex(A) + GetVertex(B));
		int f = AppendVertex(vNew);
		return f;
	}

	virtual void subclass_validity_checks(TFunction<void(bool)> CheckOrFailF) const override
	{
		for (int VID : VertexIndices())
		{
			TVector<T> V = GetVertex(VID);
			CheckOrFailF(VectorUtil::IsFinite(V));
		}
	}
};

typedef FDynamicGraph3<double> FDynamicGraph3d;


} // end namespace UE::Geometry
} // end namespace UE