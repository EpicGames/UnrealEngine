// Copyright Epic Games, Inc. All Rights Reserved.

// port of geometry3Sharp Polygon

#pragma once

#include "Templates/UnrealTemplate.h"
#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "Polygon2.h"
#include "BoxTypes.h"
#include "MatrixTypes.h"
#include "MathUtil.h"


namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TGeneralPolygon2 is a 2D polygon with holes
 */
template<typename T>
class TGeneralPolygon2
{
protected:
	/* The Outer boundary of the polygon */
	TPolygon2<T> Outer;

	/* if true, Outer polygon winding is clockwise */
	bool bOuterIsCW;

	/** The list of Holes in the polygon */
	TArray<TPolygon2<T>> Holes;

public:

	TGeneralPolygon2()
	{
	}

	/**
	* Construct a general polygon with the given polygon as boundary
	*/
	TGeneralPolygon2(const TPolygon2<T>& ToSetOuter) : Outer(ToSetOuter)
	{
		bOuterIsCW = ToSetOuter.IsClockwise();
	}

	void SetOuter(const TPolygon2<T>& ToSetOuter)
	{
		this->Outer = ToSetOuter;
		bOuterIsCW = ToSetOuter.IsClockwise();
	}

	void SetOuterWithOrientation(const TPolygon2<T>& ToSetOuter, bool bToSetOuterIsCW)
	{
		checkSlow(ToSetOuter.IsClockwise() == bToSetOuterIsCW);
		this->Outer = ToSetOuter;
		this->bOuterIsCW = bToSetOuterIsCW;
	}

	const TPolygon2<T>& GetOuter() const
	{
		return this->Outer;
	}

	const TArray<TPolygon2<T>>& GetHoles() const
	{
		return Holes;
	}

	/**
	 * Utility function to be able to modify a vertex in place. Will assert if out of bounds.
	 * @param Vertex       Vertex value to update
	 * @param HoleIndex    Hole Index, -1 if targeting Outer
	 * @param VertexIndex Index of the vertex to update in the Hole/Outer polygon.
	 */
	void UpdateVertexInPlace(const TVector2<T>& Vertex, int32 HoleIndex, int32 VertexIndex)
	{
		check(HoleIndex == -1 || Holes.IsValidIndex(HoleIndex));
		TPolygon2<T>& Polygon = HoleIndex == -1 ? Outer : Holes[HoleIndex];
		check(Polygon.GetVertices().IsValidIndex(VertexIndex));
		Polygon[VertexIndex] = Vertex;
	}

	bool AddHole(TPolygon2<T> Hole, bool bCheckContainment = true, bool bCheckOrientation = true)
	{
		if (bCheckContainment)
		{
			if (!Outer.Contains(Hole))
			{
				return false;
			}

			for (const TPolygon2<T>& ExistingHole : Holes)
			{
				if (Hole.Overlaps(ExistingHole))
				{
					return false;
				}
			}
		}


        if (bCheckOrientation)
		{
			bool bHoleIsClockwise = Hole.IsClockwise();
			if (bOuterIsCW == bHoleIsClockwise)
			{
				return false;
			}
        }

		Holes.Add(Hole);
		return true;
	}

    void ClearHoles()
	{
        Holes.Empty();
    }


	bool HasHoles() const
	{
		return Holes.Num() > 0;
	}


	/**
	 * Remove any Hole polygons for which RemoveHolePredicateFunc(Hole) returns true
	 */
	void FilterHoles(TFunctionRef<bool(const TPolygon2<T>&)> RemoveHolePredicateFunc)
	{
		int32 NumHoles = Holes.Num();
		for (int32 k = 0; k < NumHoles; ++k)
		{
			if ( RemoveHolePredicateFunc(Holes[k]) )
			{
				Holes.RemoveAtSwap(k);
				k--;		// need to reconsider index k
				NumHoles--;
			}
		}
	}


    double SignedArea() const
    {
        double Sign = (bOuterIsCW) ? -1.0 : 1.0;
        double AreaSum = Sign * Outer.SignedArea();
		for (const TPolygon2<T>& Hole : Holes)
		{
			AreaSum += Sign * Hole.SignedArea();
		}
        return AreaSum;
    }


    double HoleUnsignedArea() const
    {
		double AreaSum = 0;
		for (const TPolygon2<T>& Hole : Holes)
		{
			AreaSum += FMath::Abs(Hole.SignedArea());
		}
		return AreaSum;
    }


    double Perimeter() const
    {
		double PerimSum = Outer.Perimeter();
		for (const TPolygon2<T> &Hole : Holes)
		{
			PerimSum += Hole.Perimeter();
		}
		return PerimSum;
    }


    TAxisAlignedBox2<T> Bounds() const
    {
		TAxisAlignedBox2<T> Box = Outer.Bounds();
		for (const TPolygon2<T> Hole : Holes)
		{
			Box.Contain(Hole.Bounds());
		}
		return Box;
    }


	void Translate(TVector2<T> translate) 
	{
		Outer.Translate(translate);
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.Translate(translate);
		}
	}


    void Scale(TVector2<T> scale, TVector2<T> origin) {
		Outer.Scale(scale, origin);
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.Scale(scale, origin);
		}
	}

    void Transform(const TFunction<TVector2<T> (const TVector2<T>&)>& TransformFunc)
    {
        Outer.Transform(TransformFunc);
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.Transform(TransformFunc);
		}
    }

    void Reverse()
    {
        Outer.Reverse();
		bOuterIsCW = !bOuterIsCW;
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.Reverse();
		}
    }

	bool OuterIsClockwise() const
	{
		return bOuterIsCW;
	}

	bool Contains(TVector2<T> vTest) const
	{
		if (Outer.Contains(vTest) == false)
		{
			return false;
		}
		for (const TPolygon2<T>& Hole : Holes)
		{
			if (Hole.Contains(vTest))
			{
				return false;
			}
		}
		return true;
	}

    bool Contains(TPolygon2<T> Poly) const {
		if (Outer.Contains(Poly) == false)
		{
			return false;
		}
        for (const TPolygon2<T>& Hole : Holes)
		{
			if (Hole.Overlaps(Poly))
			{
				return false;
			}
        }
        return true;
    }


    bool Intersects(TPolygon2<T> Poly) const
    {
		if (Outer.Intersects(Poly))
		{
			return true;
		}
        for (const TPolygon2<T>& Hole : Holes)
		{
			if (Hole.Intersects(Poly))
			{
				return true;
			}
        }
        return false;
    }


    TVector2<T> GetSegmentPoint(int iSegment, double fSegT, int iHoleIndex = -1) const
	{
		if (iHoleIndex == -1)
		{
			return Outer.GetSegmentPoint(iSegment, fSegT);
		}
		return Holes[iHoleIndex].GetSegmentPoint(iSegment, fSegT);
	}

	TSegment2<T> Segment(int iSegment, int iHoleIndex = -1) const
	{
		if (iHoleIndex == -1)
		{
			return Outer.Segment(iSegment);
		}
		return Holes[iHoleIndex].Segment(iSegment);			
	}

	TVector2<T> GetNormal(int iSegment, double segT, int iHoleIndex = -1) const
	{
		if (iHoleIndex == -1)
		{
			return Outer.GetNormal(iSegment, segT);
		}
		return Holes[iHoleIndex].GetNormal(iSegment, segT);
	}

	// this should be more efficient when there are Holes...
	double DistanceSquared(TVector2<T> p, int &iHoleIndex, int &iNearSeg, double &fNearSegT) const
	{
		iNearSeg = iHoleIndex = -1;
		fNearSegT = TMathUtil<T>::MaxReal;
		double dist = Outer.DistanceSquared(p, iNearSeg, fNearSegT);
		for (int i = 0; i < Holes.Num(); ++i )
		{
			int seg; double segt;
			double holedist = Holes[i].DistanceSquared(p, seg, segt);
			if (holedist < dist)
			{
				dist = holedist;
				iHoleIndex = i;
				iNearSeg = seg;
				fNearSegT = segt;
			}
		}
		return dist;
	}


    void Simplify(double ClusterTol = 0.0001, double LineDeviationTol = 0.01)
    {
        // [TODO] should make sure that Holes stay inside Outer!!
        Outer.Simplify(ClusterTol, LineDeviationTol);
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.Simplify(ClusterTol, LineDeviationTol);
		}
    }


	/**
	 * Offset each polygon by the given Distance along vertex "normal" direction (ie for positive offset, outer polygon grows and holes shrink)
	 * @param OffsetDistance the distance to offset
	 * @param bUseFaceAvg if true, we offset by the average-face normal instead of the perpendicular-tangent normal
	 */
	void VtxNormalOffset(T OffsetDistance, bool bUseFaceAvg = false)
	{
		Outer.VtxNormalOffset(OffsetDistance, bUseFaceAvg);
		for (TPolygon2<T>& Hole : Holes)
		{
			Hole.VtxNormalOffset(OffsetDistance, bUseFaceAvg);
		}
	}

	[[nodiscard]] SIZE_T GetAllocatedSize(void) const
	{
		SIZE_T AllocatedSize = Outer.GetVertices().GetAllocatedSize();

		AllocatedSize += Holes.GetAllocatedSize();
		for (const TPolygon2<T>& Hole : Holes)
		{
			AllocatedSize += Hole.GetVertices().GetAllocatedSize();
		}

		return AllocatedSize;
	}
};

typedef TGeneralPolygon2<double> FGeneralPolygon2d;
typedef TGeneralPolygon2<float> FGeneralPolygon2f;


} // end namespace UE::Geometry
} // end namespace UE