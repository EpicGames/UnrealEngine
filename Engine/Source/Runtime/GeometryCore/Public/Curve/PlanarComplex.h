// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Very approximately ported from geometry3sharp's PlanarComplex
// Convert a set of polygons into general polygon form, tracking the outer-polygon and hole-polygon indices

#include "SegmentTypes.h"
#include "VectorTypes.h"
#include "BoxTypes.h"
#include "Polygon2.h"
#include "Curve/GeneralPolygon2.h"

namespace UE
{
namespace Geometry
{

template <typename RealType>
struct TPlanarComplex
{
	//
	// Inputs
	//
	TArray<TPolygon2<RealType>> Polygons;

	bool bTrustOrientations = true;
	bool bAllowOverlappingHoles = false;

	//
	// Outputs
	//
	struct FPolygonNesting
	{
		int OuterIndex;
		TArray<int> HoleIndices;
	};
	TArray<FPolygonNesting> Solids;

	// Finds set of "solid" regions -- e.g. boundary loops with interior holes.
	// Result has outer loops being clockwise, and holes counter-clockwise
	void FindSolidRegions()
	{
		Solids.Empty();

		int32 N = Polygons.Num();
		TArray<int> PolygonIndices;
		PolygonIndices.SetNum(N);
		for (int i = 0; i < N; i++)
		{
			PolygonIndices[i] = i;
		}

		// precomputed useful stats on input polygons
		struct FPolygonInfo
		{
			RealType Area;
			bool Orientation;
			TAxisAlignedBox2<RealType> Bounds;
		};
		TArray<FPolygonInfo> Info;
		Info.SetNum(N);
		for (int i = 0; i < N; i++)
		{
			RealType SignedArea = Polygons[i].SignedArea();
			Info[i].Orientation = SignedArea > 0;
			Info[i].Area = Info[i].Orientation ? SignedArea : -SignedArea;
			Info[i].Bounds = Polygons[i].Bounds();
		}

		PolygonIndices.Sort([&Info](int Ind1, int Ind2)
		{
			return Info[Ind1].Area > Info[Ind2].Area;
		}
		);

		TArray<int> Parent;
		Parent.SetNumUninitialized(N);
		for (int i = 0; i < N; i++)
		{
			Parent[i] = -1;
		}

		// From largest polygon to smallest, greedily assign parents where possible
		// If a polygon already has a parent, but a smaller containing parent is found, switch to the new parent (unless doing so would create an intersection)
		for (int ParentCand = 0; ParentCand+1 < N; ParentCand++)
		{
			int PIdx = PolygonIndices[ParentCand];
			const TPolygon2<RealType>& PPoly = Polygons[PIdx];
			const FPolygonInfo& PInfo = Info[PIdx];
			if (bTrustOrientations && !PInfo.Orientation)
			{
				// orientation tells us this is a hole; skip checking for holes in it
				continue;
			}
			for (int ChildCand = ParentCand + 1; ChildCand < N; ChildCand++)
			{
				int CIdx = PolygonIndices[ChildCand];
				const TPolygon2<RealType>& CPoly = Polygons[CIdx];
				const FPolygonInfo& CInfo = Info[CIdx];
				if (bTrustOrientations && CInfo.Orientation)
				{
					// orientation tells us this is an outer; skip checking if it could be a hole
					continue;
				}
				if (PInfo.Bounds.Contains(CInfo.Bounds) && PPoly.Contains(CPoly))
				{
					bool bOverlapFound = false;
					if (!bAllowOverlappingHoles)
					{
						for (int ParentRecheckIdx = ParentCand + 1; ParentRecheckIdx < ChildCand; ParentRecheckIdx++)
						{
							if (Parent[ParentRecheckIdx] == ParentCand)
							{
								if (Polygons[PolygonIndices[ParentRecheckIdx]].Overlaps(CPoly))
								{
									bOverlapFound = true;
									break;
								}
							}
						}
					}
					if (!bOverlapFound)
					{
						Parent[ChildCand] = ParentCand;
					}
				}
			}
		}

		// Build nests from parent relationship
		TArray<bool> Taken;
		Taken.SetNumZeroed(N);
		for (int NestCand = 0; NestCand < N; NestCand++)
		{
			int NIdx = PolygonIndices[NestCand];
			if (Taken[NestCand] || (bTrustOrientations && !Info[NIdx].Orientation))
			{
				continue;
			}
			FPolygonNesting &Nest = Solids.Emplace_GetRef();
			Nest.OuterIndex = NIdx;
			for (int HoleCand = NestCand + 1; HoleCand < N; HoleCand++)
			{
				int HIdx = PolygonIndices[HoleCand];
				if (Parent[HoleCand] == NestCand)
				{
					Nest.HoleIndices.Add(HIdx);
					Taken[HoleCand] = true;
				}
			}
		}
	}

	TGeneralPolygon2<RealType> ConvertNestToGeneralPolygon(const FPolygonNesting& Nest) const
	{
		TGeneralPolygon2<RealType> GPoly(Polygons[Nest.OuterIndex]);
		for (int HoleIdx : Nest.HoleIndices)
		{
			GPoly.AddHole(Polygons[HoleIdx], false, false);
		}
		return GPoly;
	}
	TArray<TGeneralPolygon2<RealType>> ConvertOutputToGeneralPolygons() const
	{
		TArray<TGeneralPolygon2<RealType>> Output;
		for (const FPolygonNesting& Nest : Solids)
		{
			TGeneralPolygon2<RealType>& GPoly = Output.Emplace_GetRef(TGeneralPolygon2<RealType>(Polygons[Nest.OuterIndex]));
			for (int HoleIdx : Nest.HoleIndices)
			{
				GPoly.AddHole(Polygons[HoleIdx], false, false);
			}
		}
		return Output;
	}
};

typedef TPlanarComplex<double> FPlanarComplexd;
typedef TPlanarComplex<float> FPlanarComplexf;


} // end namespace UE::Geometry
} // end namespace UE
