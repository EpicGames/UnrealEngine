// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "MatrixTypes.h"

namespace UE
{
namespace Geometry
{


template <typename KelvinletModelType>
class TKelvinletIntegrator
{
public:

	FVector3d Evaluate(const FVector3d& Pos) const;

	// a single step of RK3 
	FVector3d IntegrateRK3(const FVector3d& Pos, double dt) const
	{

		FVector3d k1 = static_cast<const KelvinletModelType*>(this)->Evaluate(Pos);
		FVector3d k2 = static_cast<const KelvinletModelType*>(this)->Evaluate(Pos + dt * 0.5 * k1);
		FVector3d k3 = static_cast<const KelvinletModelType*>(this)->Evaluate(Pos + dt * (2.0 * k2 - k1));

		// give result relative to pos
		return Pos + dt * ((1.0 / 6.0) * (k1 + k3) + (2.0 / 3.0) * k2);
	}
};

template <typename KelvinletModelType>
class TBaseKelvinlet : public TKelvinletIntegrator<KelvinletModelType>
{
public:

	TBaseKelvinlet(const double Size, const double ShearModulus, const double PoissonRatio)
	{
		SetMaterialParameters(ShearModulus, PoissonRatio);

		UpdateRegularization(Size);
	}


	// ShearModuls and Poisson Ratio - both => 0.  
	// Poisson Ratio should be in the interval [0, 1/2) 
	void SetMaterialParameters(const double ShearModulus, const double PoissonRatio)
	{
		checkSlow(PoissonRatio <= 0.5);
		a = 1.0 / (4. * PI * ShearModulus);
		bhat = 1.0 / (4. * (1.0 - PoissonRatio));
	}


	void UpdateRegularization(const double R)
	{
		E = R;
	}

protected:

	// Epsilon and powers of epsilon
	double E;
	// using lowercase to be consistent with standard notation (cf "Sharp Kelvinlets: Elastic Deformations with Cusps and Localized Falloffs" 2019 de Goes)
	double a;
	// Note: to translate to the notation of F.de Goes, b = a * bhat.
	double bhat;
};


// Note: this is independent of "a" the shear moduls term
class FPullKelvinlet : public TBaseKelvinlet < FPullKelvinlet>
{
public:
	typedef TBaseKelvinlet < FPullKelvinlet>  MyBase;



	FPullKelvinlet(const FVector3d& Force, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(Force)
	{
		// Note: b <= a /2  so this can't be 2/0
		// double NormConst = 2. / (3. * a - 2. * b);

		// we can factor a out of the solution..
		double NormConst = 2. / (3. - 2. * bhat);
		c = NormConst;
	}


	// The canonical Kelvinlet for a regularized force.
	// u(r) = [A_e(r) I + B_e(r) r r^t] f;   
	// where A_e(r) = (a-b)/r_e + a e^2 / (2 r_e^3)
	//       B_e(r) = b / (r_e^3)
	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;

		const double D2 = 1. + NPos.SquaredLength();
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;

		const double RadialTerm = (1. - bhat) / D + 0.5 / D3;
		const double NonIsoTerm = (bhat / D3) * F.Dot(NPos);

		//return c * (a * RadialTerm * F + a * NonIsoTerm * NPos);
		return c * (RadialTerm * F + NonIsoTerm * NPos);
	}

	double Divergence(const FVector3d& Pos) const
	{
		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double D2 = 1. + NPos.SquaredLength();
		const double D5 = D2 * D2 * FMath::Sqrt(D2);

		// note the divergence is zero in plane perpendicular to the brush direction.
		double Div = (2. * bhat - 1.) * F.Dot(NPos) * (3. + D2) / (2 * D5);

		return c * Div;
	}

private:

	FVector3d F;

	double c; // calibration
};

// Note: this is independent of "a" the shear moduls term
class FLaplacianPullKelvinlet : public TBaseKelvinlet < FLaplacianPullKelvinlet >
{
public:
	typedef TBaseKelvinlet < FLaplacianPullKelvinlet > MyBase;


	FLaplacianPullKelvinlet(const FVector3d& Force, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(Force)
	{
		// Normalize the radial term
		// Note: b <= a /2  so this can't be 2/0
		//double NormConst = 2. /(3. * a - 2. * b);
		double NormConst = 2. / (3. - 2. * bhat);
		c = NormConst / 5.;
	}

	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D7 = D * D3 * D3;


		// Radial (Isotropic) term

		double RadialTerm = (15. - bhat * D2 * (10. + 4. * R2)) / (2. * D7);


		// Non Isotropic term
		double NonIsoTerm = (3. * bhat * (7. + 2. * R2) / D7) * F.Dot(NPos);

		// we can factor the 'a' out
		//return c * (a * RadialTerm * F + a * NonIsoTerm * NPos);
		return c * (RadialTerm * F + NonIsoTerm * NPos);
	}

	double Divergence(const FVector3d& Pos) const
	{
		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double D2 = 1. + NPos.SquaredLength();
		const double D = FMath::Sqrt(D2);
		const double D9 = D2 * D2 * D2 * D2 * D;

		// note the divergence is zero in plane perpendicular to the brush direction.
		double Div = (2. * bhat - 1.) * 105 * F.Dot(NPos) / D9;
		return c * Div;
	}

	FVector3d F;

private:
	double c; // calibration
};

// Note: this is independent of "a" the shear modulus term
class FBiLaplacianPullKelvinlet : public TBaseKelvinlet < FBiLaplacianPullKelvinlet >
{
public:
	typedef TBaseKelvinlet < FBiLaplacianPullKelvinlet > MyBase;

	FBiLaplacianPullKelvinlet(const FVector3d& Force, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(Force)
	{
		// Normalize the radial term
		// Note: b <= a /2  so this can't be 2/0
		//double NormConst = 2. / (3. * a - 2. * b);
		double NormConst = 2. / (3. - 2. * bhat);
		c = NormConst / 105.;

	}


	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D11 = D3 * D3 * D3 * D2;

		// Radial (Isotropic) term

		const double RadialTerm = 105. * (3. - 6. * R2 - 2. * bhat * D2) / (2. * D11);


		// Non Isotropic term	
		const double NonIsoTerm = (bhat * 945. / D11) * F.Dot(NPos);

		//return  c * a * (RadialTerm * F + NonIsoTerm * NPos);
		// we factor the a out.
		return  c * (RadialTerm * F + NonIsoTerm * NPos);
	}


	double Divergence(const FVector3d& Pos) const
	{
		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double D2 = 1. + NPos.SquaredLength();
		const double D = FMath::Sqrt(D2);
		const double D8 = D2 * D2 * D2 * D2;
		const double D13 = D8 * D2 * D2 * D;

		// note the divergence is zero in plane perpendicular to the brush direction.
		double Div = 945. * (2. * bhat - 1.) * F.Dot(NPos) * (11 - 6. * D2) / (2. * D13);
		return c * Div;
	}

	FVector3d F;

private:

	double c; // calibration 

};



// Note: this is independent of "a" the shear modulus term
class FSharpLaplacianPullKelvinlet : public TBaseKelvinlet < FSharpLaplacianPullKelvinlet >
{
public:
	typedef TBaseKelvinlet < FSharpLaplacianPullKelvinlet > MyBase;

	FSharpLaplacianPullKelvinlet(const FVector3d& Force, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(Force)
	{
		// Normalize the radial term
		// Note: b <= a /2  so this can't be 1/0

		double NormConst = 1. / (10. * (3. - 2. * bhat));
		c = NormConst;

	}


	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double R = FMath::Sqrt(R2);
		const double R4 = R2 * R2;
		const double R5 = R4 * R;
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;

		const double D11 = D3 * D3 * D3 * D2;

		// Radial (Isotropic) term

		const double RadialTerm = 2. * ((15. - 10. * bhat) + (90. - 88. * bhat) * R2 + 120. * (1. - bhat) * R4 + 48. * (1. - bhat) * R * (R5 - D5)) / D5;


		// Non Isotropic term	

		const double NonIsoTerm = -12. * (bhat / (R * D5)) * (2. * R2 * (4. * D - 5. * R) + 4. * R4 * (D - R) + (4. * D - 7. * R)) * F.Dot(NPos);

		//return  c * a * (RadialTerm * F + NonIsoTerm * NPos);
		// we factor the a out.
		return  c * (RadialTerm * F + NonIsoTerm * NPos);
	}


	double Divergence(const FVector3d& Pos) const
	{
		return 1;
	}

	FVector3d F;

private:

	double c; // calibration 

};

class FSharpBiLaplacianPullKelvinlet : public TBaseKelvinlet <FSharpBiLaplacianPullKelvinlet >
{
public:
	typedef TBaseKelvinlet < FSharpBiLaplacianPullKelvinlet > MyBase;

	FSharpBiLaplacianPullKelvinlet(const FVector3d& Force, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(Force)
	{
		// Normalize the radial term
		// Note: b <= a /2  so this can't be 1/0

		double NormConst = 1. / (315. * (3. - 2. * bhat));
		c = NormConst;

	}


	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double R = FMath::Sqrt(R2);
		const double R4 = R2 * R2;
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;
		const double D9 = D5 * D2 * D2;
		const double D10 = D5 * D5;
		const double D11 = D3 * D3 * D3 * D2;

		FVector3d Result(0., 0., 0.);

		if (R < 5)
		{

			// Radial (Isotropic) term

			const double RadialTerm = (9. / D10) * (-512. * R * D10 + (((((512. * R2 + 2304.) * R2 + 4032.) * R2 + 3360.) * R2 + 1260.) * R2 + 105.) * D + 2. * bhat * (128 * R * D10 - D3 * (35. + R2 * (280. + R2 * (560. + R2 * (448. + 128 * R2))))));


			// Non Isotropic term	

			const double NonIsoTerm = 18. * (bhat / (R * D9)) * (128. * D9 - R * (R2 * (R2 * (R2 * (128. * R2 + 576.) + 1008.) + 840.) + 315.)) * F.Dot(NPos);

			Result = c * (RadialTerm * F + NonIsoTerm * NPos);
		}
		return  Result;
	}


	double Divergence(const FVector3d& Pos) const
	{
		return 1;
	}

	FVector3d F;

private:

	double c; // calibration 

};


// Implement the "Locally affine regularized Kelvinlet" from 
// "Regularized Kelvinlets: Sculpting Brushes based on Fundamental Solutions of Elasticity" - de Goes and James 2017
class FAffineKelvinlet : public TBaseKelvinlet < FAffineKelvinlet>
{
public:
	typedef TBaseKelvinlet < FAffineKelvinlet> MyBase;

	FAffineKelvinlet(const FMatrix3d& ForceMatrix, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(ForceMatrix)
	{
		SymF = F + F.Transpose() + F.Trace() * FMatrix3d::Identity();
		E2 = E * E;
	}

	void UpdateForce(const FMatrix3d& NF)
	{
		F = NF;
		SymF = F + F.Transpose() + F.Trace() * FMatrix3d::Identity();
	}

	FVector3d Evaluate(const FVector3d& Pos) const
	{
		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;



		// the displacement field.
		const FVector3d Fr = F * NPos;
		const double rDotFr = NPos.Dot(Fr);

		FVector3d Term1 = (bhat * SymF * NPos - Fr) / D3;
		FVector3d Term2 = -3. * (Fr + 2. * bhat * rDotFr * NPos) / (2. * D5);

		return a * (Term1 + Term2) / E2;

	}

protected:


	// Force Matrix
	FMatrix3d F;

	// 2 * symmetric part of F + trace.. : F  + F^T + I Trace(F)
	FMatrix3d SymF;

	double E2;
};

// Independent of shear modulus and is volume preserving. 
class FTwistKelvinlet : public TBaseKelvinlet <FTwistKelvinlet>
{
public:
	typedef TBaseKelvinlet < FTwistKelvinlet> MyBase;

	FTwistKelvinlet(const FVector3d& Twist, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(CrossProductMatrix(Twist))
	{
	}

	FTwistKelvinlet(const FTwistKelvinlet& Other) :
		MyBase(Other)
	{
		F = Other.F;
	}

	// Specialized override that should give the same result, but be faster.
	// note: <r | F r> = 0 and  F.Trace() = 0 for the skewsymmetric cross product matrix.  
	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;



		// the displacement field.
		const FVector3d Fr = F * NPos;



		FVector3d Term1 = (1. / D3 + 1.5 / D5) * Fr;
		// Correct value is (-a/E2) * Term1;
		// with Normalization =  1/ Angular velocity = 1 / ( -2.5 * a / E2 )

		// set the angular velocity at the tip to be Twist.Length().  This allows us to set the speed and chirality (left / right -handed) using the twist vector
		return  Term1 * (2. / 5.);

	}

	double Divergence(const FVector3d& Pos) const
	{
		return 0.;
	}

protected:

	// Force Matrix
	FMatrix3d F;
};


class FPinchKelvinlet : public TBaseKelvinlet <FPinchKelvinlet>
{
public:
	typedef TBaseKelvinlet <FPinchKelvinlet> MyBase;

	FPinchKelvinlet(const FMatrix3d& ForceMatrix, const double Size, const double ShearModulus, const double PoissonRatio) :
		MyBase(Size, ShearModulus, PoissonRatio),
		F(PinchMatrix(ForceMatrix))
	{
		// set the angular velocity at the tip to be 1.
		Normalize();
	}


	// Specialized override that should give the same result, but be faster.
	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// compute the regularized distance

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;

		const FVector3d Fr = F * NPos;
		const double rtFr = NPos.Dot(Fr);

		const FVector3d Term1 = ((2. * bhat - 1.) / D3 - (3. / 2.) / D5) * Fr;
		const FVector3d Term2 = -(3. * bhat * rtFr / D5) * NPos;

		// result = (Term1 + Term2 ) / E2
		// divide the a out with the normalization
		// return E * c * a* (Term1 + Term2);
		return E * c * (Term1 + Term2);
	}


private:

	// Convert a source matrix to a symmetric matrix with zero trace.wh
	FMatrix3d PinchMatrix(const FMatrix3d& Mat)
	{
		double Trace = Mat.Trace();

		// Symmetric matrix with zero trace
		return 0.5 * (Mat + Mat.Transpose()) - (Trace / 3.0) * FMatrix3d::Identity();
	}

	void Normalize()
	{
		// the jacobian of 'Evaluate()' is
		// du_i/dx_j = 1/(E3)(4b -5a) F_{i,j}
		// we use the same normalization as Fernado - just set this to  F
		//c = 1. / ( a (4. * bhat - 5. )) ;
		c = 1. / (4. * bhat - 5.);
	}

	double c; // calibration
	// Force Matrix
	FMatrix3d F;
};

// Note: this one doesn't care about a, b since we have scaled the divergence.
class FScaleKelvinlet : public TBaseKelvinlet < FScaleKelvinlet>
{
public:
	typedef TBaseKelvinlet < FScaleKelvinlet>   MyBase;

	FScaleKelvinlet(const double Scale, const double Size, const double ShearModulus, const double PoissonRatio)
		: MyBase(Size, ShearModulus, PoissonRatio),
		S(Scale)
	{
	}


	// Specialized override that should give the same result, but be faster.
	// note: <r | F r> = 0 and  F.Trace() = 0 for the skewsymmetric cross product matrix.  
	FVector3d Evaluate(const FVector3d& Pos) const
	{

		// compute the regularized distance

		// normalized on the regularization size.
		const FVector3d NPos = Pos / E;
		const double R2 = NPos.SquaredLength();
		const double D2 = 1. + R2;
		const double D = FMath::Sqrt(D2);
		const double D3 = D2 * D;
		const double D5 = D3 * D2;

		// Note, if normalized this could just be written as
		// ( 2 * E3 / 15 ) * ( 1.0d / RE3 + 1.5 * E2 / RE5) * Pos;

		// double Divergence = 3 * (5 / 2) * (2. * b -a ) / E3

		// return (1./Divergence) * (2. * b - a) * ( S / E2 ) * (1.0 / D3 + 1.5 / D5) * NPos;

		return S * E * (2. / 15.) * (1.0 / D3 + 1.5 / D5) * NPos;

	}

	double Divergence(const FVector3d& Pos) const
	{
		const FVector3d NPos = Pos / E;
		const double D2 = 1. + NPos.SquaredLength();
		const double D = FMath::Sqrt(D2);

		return S * E / (D2 * D2 * D2 * D);

	}

	// Scale
	double S;


};

template <typename BrushTypeA, typename BrushTypeB>
class TBlendPairedKelvinlet : public TKelvinletIntegrator < TBlendPairedKelvinlet <BrushTypeA, BrushTypeB> >
{
public:
	TBlendPairedKelvinlet(const BrushTypeA& BrushA, const BrushTypeB& BrushB, double StrengthA) :
		ABrush(BrushA),
		BBrush(BrushB)
	{
		Alpha = FMath::Clamp(StrengthA, 0., 1.);
	}

	FVector3d Evaluate(const FVector3d& Pos) const
	{
		return Alpha * ABrush.Evaluate(Pos) + (1. - Alpha) * BBrush.Evaluate(Pos);
	}

	double Divergence(const FVector3d& Pos) const
	{
		return Alpha * ABrush.Divergence(Pos) + (1. - Alpha) * BBrush.Divergence(Pos);
	}

protected:
	BrushTypeA ABrush;
	BrushTypeB BBrush;
	double Alpha;
};

typedef TBlendPairedKelvinlet< FTwistKelvinlet, FLaplacianPullKelvinlet >                      FLaplacianTwistPullKelvinlet;
typedef TBlendPairedKelvinlet< FTwistKelvinlet, FBiLaplacianPullKelvinlet >                    FBiLaplacianTwistPullKelvinlet;
typedef TBlendPairedKelvinlet< FBiLaplacianPullKelvinlet, FLaplacianPullKelvinlet >            FBlendPullKelvinlet;
typedef TBlendPairedKelvinlet< FSharpBiLaplacianPullKelvinlet, FSharpLaplacianPullKelvinlet >  FBlendPullSharpKelvinlet;


} // end namespace UE::Geometry
} // end namespace UE