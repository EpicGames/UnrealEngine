// Copyright Epic Games, Inc. All Rights Reserved.

// Port of gte's DistLine2AlignedBox2 to use GeometryCore data types

#pragma once

#include "Containers/StaticArray.h"
#include "LineTypes.h"
#include "Math/Box2D.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute unsigned distance between 2D line and 2D axis aligned box
 */
template <typename Real>
class TDistLine2AxisAlignedBox2
{
public:
	// Input
	TLine2<Real> Line;
	TBox2<Real> AxisAlignedBox;

	// Results
	Real ResultDistanceSquared = -1.0;
	TVector2<Real> BoxClosest, LineClosest;
	Real LineParameter;

	TDistLine2AxisAlignedBox2(const TLine2<Real> LineIn, const TBox2<Real>& AxisAlignedBoxIn) : Line(LineIn), AxisAlignedBox(AxisAlignedBoxIn)
	{
	}

	Real Get() 
	{
		return (Real)sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (ResultDistanceSquared >= 0)
		{
			return ResultDistanceSquared;
		}

		// Translate the line and box so that the box has center at the origin.
		TVector2<Real> boxCenter = AxisAlignedBox.GetCenter();
		TVector2<Real> boxExtent = AxisAlignedBox.GetExtent();
		TVector2<Real> origin = Line.Origin - boxCenter;
		TVector2<Real> direction = Line.Direction;

		// The query computes 'result' relative to the box with center at the origin.
		DoQuery(origin, direction, boxExtent, BoxClosest, LineClosest, LineParameter);

		// Translate the closest points to the original coordinates.
		BoxClosest += boxCenter;
		LineClosest += boxCenter;

		ResultDistanceSquared = DistanceSquared(BoxClosest, LineClosest);
		return ResultDistanceSquared;
	}

private:

	// Compute the distance and closest point between a line and an aligned box whose center is the origin. The origin and direction
	// are not const to allow for reflections that eliminate complicated sign logic in the queries themselves.
	static void DoQuery(TVector2<Real> origin, TVector2<Real> direction, const TVector2<Real>& boxExtent, TVector2<Real>& BoxClosest, TVector2<Real>& LineClosest, Real& LineParameter)
	{
		// Apply reflections so that the direction has nonnegative
        // components.
        Real const zero = static_cast<Real>(0);
		TStaticArray<bool, 2> reflect( InPlace, false );
        for (int32_t i = 0; i < 2; ++i)
        {
            if (direction[i] < zero)
            {
                origin[i] = -origin[i];
                direction[i] = -direction[i];
                reflect[i] = true;
            }
        }

        // Compute the line-box distance and closest points. The DoQueryND
        // calls compute LineParameter and BoxClosest. The
        // LineClosest can be computed after these calls.
        if (direction[0] > zero)
        {
            if (direction[1] > zero)
            {
                // The direction signs are (+,+). If the line does not
                // intersect the box, the only possible closest box points
                // are K[0] = (-e0,e1) or K[1] = (e0,-e1). If the line
                // intersects the box, the closest points are the same and
                // chosen to be the intersection with box edge x0 = e0 or
                // x1 = e1. For the remaining discussion, define K[2] =
                // (e0,e1).
                //
                // Test where the candidate corners are relative to the
                // line. If D = (d0,d1), then Perp(D) = (d1,-d0). The
                // corner K[i] = P + t[i] * D + s[i] * Perp(D), where
                // s[i] = Dot(K[i]-P,Perp(D))/|D|^2. K[0] is closest when
                // s[0] >= 0 or K[1] is closest when s[1] <= 0. Otherwise,
                // the line intersects the box. If s[2] >= 0, the common
                // closest point is chosen to be (p0+(e1-p1)*d0/d1,e1). If
                // s[2] < 0, the common closest point is chosen to be
                // (e0,p1+(e0-p0)*d1/d0).
                // 
                // It is sufficient to test the signs of Dot(K[i],Perp(D))
                // and defer the division by |D|^2 until needed for
                // computing the closest point.
                DoQuery2D(origin, direction, boxExtent, BoxClosest, LineClosest, LineParameter);
            }
            else
            {
                // The direction signs are (+,0). The parameter is the
                // value of t for which P + t * D = (e0, p1).
                DoQuery1D(0, 1, origin, direction, boxExtent, BoxClosest, LineClosest, LineParameter);
            }
        }
        else
        {
            if (direction[1] > zero)
            {
                // The direction signs are (0,+). The parameter is the
                // value of t for which P + t * D = (p0, e1).
                DoQuery1D(1, 0, origin, direction, boxExtent, BoxClosest, LineClosest, LineParameter);
            }
            else
            {
                // The direction signs are (0,0). The line is degenerate
                // to a point (its origin). Clamp the origin to the box
                // to obtain the closest point.
                DoQuery0D(origin, boxExtent, BoxClosest, LineClosest, LineParameter);
            }
        }

        LineClosest = origin + LineParameter * direction;

        // Undo the reflections. The origin and direction are not consumed
        // by the caller, so these do not need to be reflected. However,
        // the closest points are consumed.
        for (int32_t i = 0; i < 2; ++i)
        {
            if (reflect[i])
            {
                BoxClosest[i] = -BoxClosest[i];
                LineClosest[i] = -LineClosest[i];
            }
        }
	}

	static void DoQuery2D(TVector2<Real> const& origin, TVector2<Real> const& direction, TVector2<Real> const& extent, TVector2<Real>& BoxClosest, TVector2<Real>& LineClosest, Real& LineParameter)
    {
        Real const zero = static_cast<Real>(0);
        TVector2<Real> K0{ -extent[0], extent[1] };
        TVector2<Real> delta = K0 - origin;
        Real K0dotPerpD = DotPerp(delta, direction);
        if (K0dotPerpD >= zero)
        {
            LineParameter = delta.Dot(direction) / direction.Dot(direction);
            LineClosest = origin + LineParameter * direction;
            BoxClosest = K0;
        }
        else
        {
            TVector2<Real> K1{ extent[0], -extent[1] };
            delta = K1 - origin;
            Real K1dotPerpD = DotPerp(delta, direction);
            if (K1dotPerpD <= zero)
            {
                LineParameter = delta.Dot(direction) / direction.Dot(direction);
                LineClosest = origin + LineParameter * direction;
                BoxClosest = K1;
            }
            else
            {
                TVector2<Real> K2{ extent[0], extent[1] };
                delta = K2 - origin;
                Real K2dotPerpD = DotPerp(delta, direction);
                if (K2dotPerpD >= zero)
                {
                    LineParameter = (extent[1] - origin[1]) / direction[1];
                    LineClosest = origin + LineParameter * direction;
                    BoxClosest[0] = origin[0] + LineParameter * direction[0];
                    BoxClosest[1] = extent[1];
                }
                else
                {
                    LineParameter = (extent[0] - origin[0]) / direction[0];
                    LineClosest = origin + LineParameter * direction;
                    BoxClosest[0] = extent[0];
                    BoxClosest[1] = origin[1] + LineParameter * direction[1];
                }
            }
        }
    }

    static void DoQuery1D(int32_t i0, int32_t i1, TVector2<Real> const& origin, TVector2<Real> const& direction, TVector2<Real> const& extent, TVector2<Real>& BoxClosest, TVector2<Real>& LineClosest, Real& LineParameter)
    {
        LineParameter = (extent[i0] - origin[i0]) / direction[i0];
        LineClosest = origin + LineParameter * direction;
        BoxClosest[i0] = extent[i0];
        BoxClosest[i1] = FMath::Clamp(origin[i1], -extent[i1], extent[i1]);
    }

    static void DoQuery0D(TVector2<Real> const& origin, TVector2<Real> const& extent, TVector2<Real>& BoxClosest, TVector2<Real>& LineClosest, Real& LineParameter)
    {
        LineParameter = static_cast<Real>(0);
        LineClosest = origin;
        BoxClosest[0] = FMath::Clamp(origin[0], -extent[0], extent[0]);
        BoxClosest[1] = FMath::Clamp(origin[1], -extent[1], extent[1]);
    }
};

typedef TDistLine2AxisAlignedBox2<float> FDistLine2AxisAlignedBox2f;
typedef TDistLine2AxisAlignedBox2<double> FDistLine2AxisAlignedBox2d;



} // end namespace UE::Geometry
} // end namespace UE
