// Copyright Epic Games, Inc. All Rights Reserved.

// derived from geometry3Sharp DistLine3Ray3

#pragma once

#include "VectorTypes.h"
#include "LineTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute distance between two 3D lines
 */
template <typename Real>
class TDistLine3Line3
{
public:
	// Input
	TLine3<Real> Line1;
	TLine3<Real> Line2;

	// Results
	Real DistanceSquared = -1.0;

	bool bIsParallel = false;
	TVector<Real> Line1ClosestPoint;
	Real Line1Parameter;
	TVector<Real> Line2ClosestPoint;
	Real Line2Parameter;

	TDistLine3Line3(const TLine3<Real>& Line1In, const TLine3<Real>& Line2In)
	{
		Line1 = Line1In;
		Line2 = Line2In;
	}

	Real Get()
	{
		return (Real)sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		TVector<Real> kDiff = Line1.Origin - Line2.Origin;
		Real a01 = -Line1.Direction.Dot(Line2.Direction);
		Real b0 = kDiff.Dot(Line1.Direction);
		Real c = kDiff.SquaredLength();
		Real det = FMath::Abs((Real)1 - a01 * a01);
		Real b1, s0, s1, sqrDist;

		if (det >= TMathUtil<Real>::ZeroTolerance)
		{
			b1 = -kDiff.Dot(Line2.Direction);
			s1 = a01 * b0 - b1;

			// Two interior points are closest.
			Real invDet = ((Real)1) / det;
			s0 = (a01 * b1 - b0) * invDet;
			s1 *= invDet;
			sqrDist = s0 * (s0 + a01 * s1 + ((Real)2) * b0) +
				s1 * (a01 * s0 + s1 + ((Real)2) * b1) + c;

			Line1ClosestPoint = Line1.Origin + s0 * Line1.Direction;
			Line2ClosestPoint = Line2.Origin + s1 * Line2.Direction;
			Line1Parameter = s0;
			Line2Parameter = s1;

			bIsParallel = false;
		}
		else
		{
			// Lines are parallel, closest pair at line1 origin
			Line1Parameter = (Real)0;
			Line1ClosestPoint = Line1.Origin;
			Line2Parameter = Line2.Project(Line1.Origin);
			Line2ClosestPoint = Line2.PointAt(Line2Parameter);
			sqrDist = Line1.DistanceSquared(Line2ClosestPoint);

			bIsParallel = true;
		}

		// Account for numerical round-off errors.
		DistanceSquared = (sqrDist < (Real)0) ? (Real)0 : sqrDist;
		return DistanceSquared;
	}
};

typedef TDistLine3Line3<float> FDistLine3Line3f;
typedef TDistLine3Line3<double> FDistLine3Line3d;

} // end namespace UE::Geometry
} // end namespace UE
