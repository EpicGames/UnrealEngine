// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistLine3Ray3

#pragma once

#include "VectorTypes.h"
#include "LineTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute distance between 3D line and 3D ray
 */
template <typename Real>
class TDistLine3Ray3
{
public:
	// Input
	TLine3<Real> Line;
	TRay<Real> Ray;

	// Results
	Real DistanceSquared = -1.0;

	TVector<Real> LineClosestPoint;
	Real LineParameter;
	TVector<Real> RayClosestPoint;
	Real RayParameter;


	TDistLine3Ray3(const TLine3<Real>& LineIn, const TRay<Real>& RayIn)
	{
		Line = LineIn;
		Ray = RayIn;
	}

	Real Get()
	{
		return (Real)sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		TVector<Real> kDiff = Line.Origin - Ray.Origin;
		Real a01 = -Line.Direction.Dot(Ray.Direction);
		Real b0 = kDiff.Dot(Line.Direction);
		Real c = kDiff.SquaredLength();
		Real det = FMath::Abs((Real)1 - a01 * a01);
		Real b1, s0, s1, sqrDist;

		if (det >= TMathUtil<Real>::ZeroTolerance) 
		{
			b1 = -kDiff.Dot(Ray.Direction);
			s1 = a01 * b0 - b1;

			if (s1 >= (Real)0) 
			{
				// Two interior points are closest, one on Line and one on Ray.
				Real invDet = ((Real)1) / det;
				s0 = (a01 * b1 - b0) * invDet;
				s1 *= invDet;
				sqrDist = s0 * (s0 + a01 * s1 + ((Real)2) * b0) +
					s1 * (a01 * s0 + s1 + ((Real)2) * b1) + c;
			}
			else 
			{
				// Origin of Ray and interior point of Line are closest.
				s0 = -b0;
				s1 = (Real)0;
				sqrDist = b0 * s0 + c;
			}
		}
		else 
		{
			// Lines are parallel, closest pair with one point at Ray origin.
			s0 = -b0;
			s1 = (Real)0;
			sqrDist = b0 * s0 + c;
		}

		LineClosestPoint = Line.Origin + s0 * Line.Direction;
		RayClosestPoint = Ray.Origin + s1 * Ray.Direction;
		LineParameter = s0;
		RayParameter = s1;

		// Account for numerical round-off errors.
		if (sqrDist < (Real)0) 
		{
			sqrDist = (Real)0;
		}
		DistanceSquared = sqrDist;

		return sqrDist;
	}
};

typedef TDistLine3Ray3<float> FDistLine3Ray3f;
typedef TDistLine3Ray3<double> FDistLine3Ray3d;

} // end namespace UE::Geometry
} // end namespace UE
