// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistLine3Segment3
// which was ported from WildMagic 5 

#pragma once

#include "VectorTypes.h"
#include "SegmentTypes.h"
#include "LineTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* Compute unsigned distance between 3D line and 3D segment
*/
template <typename Real>
class TDistLine3Segment3
{
public:
	// Input
	TLine3<Real> Line;
	TSegment3<Real> Segment;

	// Output
	Real DistanceSquared = -1.0;
	Real LineParameter, SegmentParameter;
	TVector<Real> LineClosest, SegmentClosest;


	TDistLine3Segment3(const TLine3<Real>& LineIn, const TSegment3<Real>& SegmentIn) : Line(LineIn), Segment(SegmentIn)
	{
	}

	Real Get()
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		TVector<Real> diff = Line.Origin - Segment.Center;
		Real a01 = -Line.Direction.Dot(Segment.Direction);
		Real b0 = diff.Dot(Line.Direction);
		Real c = diff.SquaredLength();
		Real det = TMathUtil<Real>::Abs(1 - a01 * a01);
		Real b1, s0, s1, sqrDist, extDet;

		if (det >= TMathUtil<Real>::ZeroTolerance)
		{
			// The Line and Segment are not parallel.
			b1 = -diff.Dot(Segment.Direction);
			s1 = a01 * b0 - b1;
			extDet = Segment.Extent * det;

			if (s1 >= -extDet)
			{
				if (s1 <= extDet)
				{
					// Two interior points are closest, one on the Line and one
					// on the Segment.
					Real invDet = (1) / det;
					s0 = (a01 * b1 - b0) * invDet;
					s1 *= invDet;
					sqrDist = s0 * (s0 + a01 * s1 + (2) * b0) +
						s1 * (a01 * s0 + s1 + (2) * b1) + c;
				}
				else
				{
					// The endpoint e1 of the Segment and an interior point of
					// the Line are closest.
					s1 = Segment.Extent;
					s0 = -(a01 * s1 + b0);
					sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
				}
			}
			else
			{
				// The end point e0 of the Segment and an interior point of the
				// Line are closest.
				s1 = -Segment.Extent;
				s0 = -(a01 * s1 + b0);
				sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
			}
		}
		else
		{
			// The Line and Segment are parallel.  Choose the closest pair so that
			// one point is at Segment center.
			s1 = 0;
			s0 = -b0;
			sqrDist = b0 * s0 + c;
		}

		LineClosest = Line.Origin + s0 * Line.Direction;
		SegmentClosest = Segment.Center + s1 * Segment.Direction;
		LineParameter = s0;
		SegmentParameter = s1;

		// Account for numerical round-off errors.
		if (sqrDist < 0)
		{
			sqrDist = 0;
		}

		DistanceSquared = sqrDist;

		return DistanceSquared;
	}
};

typedef TDistLine3Segment3<float> FDistLine3Segment3f;
typedef TDistLine3Segment3<double> FDistLine3Segment3d;

/**
 * Integrates the squared distance to a line along a given segment.
 */
template <typename Real>
double SquaredDistanceFromLineIntegratedAlongSegment(const TLine3<Real>& Line, const TSegment3<Real>& Segment)
{
	double SegmentLength = Segment.Length();

	TVector<Real> SegmentStart = Segment.StartPoint();
	// Point at length t along segment is SegmentStart + Segment.Direction * t

	TVector<Real> ProjectedSegmentStart = Line.Origin + (SegmentStart - Line.Origin).Dot(Line.Direction) * Line.Direction;
	TVector<Real> ProjectedSegmentDirection = Segment.Direction.Dot(Line.Direction) * Line.Direction;
	// Projected point onto line at t is ProjectedSegmentStart + ProjectedSegmentDirection * t

	// Difference of point on segment and projected point is the vector that we actually want to integrate over t.
	TVector<Real> P_0 = SegmentStart - ProjectedSegmentStart;
	TVector<Real> P_v = Segment.Direction - ProjectedSegmentDirection;

	// Squared length of this vector is sum of component-wise squares, and after doing some math, you get that
	//  the integral from 0 to SegmentLength is: 
	//  p_0.p_0(SegmentLength) + p_0.p_v(SegmentLength^2) + p_v.p_v (SegmentLength^3)/3
	// Rearranging a bit:
	return ((P_v.Dot(P_v) * SegmentLength / 3.0 + P_0.Dot(P_v)) * SegmentLength + P_0.Dot(P_0)) * SegmentLength;
}

} // end namespace UE::Geometry
} // end namespace UE
