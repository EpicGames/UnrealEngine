// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistLine3Triangle3
// which was ported from WildMagic 5 

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "DistLine3Segment3.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* Compute unsigned distance between 3D line and 3D triangle
*/
template <typename Real>
class TDistLine3Triangle3
{
public:
	// Input
	TLine3<Real> Line;
	TTriangle3<Real> Triangle;

	// Output
	Real DistanceSquared = -1.0;
	Real LineParam;
	TVector<Real> LineClosest, TriangleClosest, TriangleBaryCoords;


	TDistLine3Triangle3(const TLine3<Real>& LineIn, const TTriangle3<Real>& TriangleIn) : Line(LineIn), Triangle(TriangleIn)
	{
	}

	Real Get()
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		// Test if Line intersects Triangle.  If so, the squared distance is zero.
		TVector<Real> edge0 = Triangle.V[1] - Triangle.V[0];
		TVector<Real> edge1 = Triangle.V[2] - Triangle.V[0];
		TVector<Real> normal = edge0.Cross(edge1);
		Normalize(normal);
		Real NdD = normal.Dot(Line.Direction);
		if (TMathUtil<Real>::Abs(NdD) > TMathUtil<Real>::ZeroTolerance)
		{
			// The line and triangle are not parallel, so the line intersects
			// the plane of the triangle.
			TVector<Real> diff = Line.Origin - Triangle.V[0];
			TVector<Real> U, V;
			VectorUtil::MakePerpVectors(Line.Direction, U, V);
			Real UdE0 = U.Dot(edge0);
			Real UdE1 = U.Dot(edge1);
			Real UdDiff = U.Dot(diff);
			Real VdE0 = V.Dot(edge0);
			Real VdE1 = V.Dot(edge1);
			Real VdDiff = V.Dot(diff);
			Real invDet = (1) / (UdE0 * VdE1 - UdE1 * VdE0);

			// Barycentric coordinates for the point of intersection.
			Real b1 = (VdE1 * UdDiff - UdE1 * VdDiff) * invDet;
			Real b2 = (UdE0 * VdDiff - VdE0 * UdDiff) * invDet;
			Real b0 = 1 - b1 - b2;

			if (b0 >= 0 && b1 >= 0 && b2 >= 0)
			{
				// Line parameter for the point of intersection.
				Real DdE0 = Line.Direction.Dot(edge0);
				Real DdE1 = Line.Direction.Dot(edge1);
				Real DdDiff = Line.Direction.Dot(diff);
				LineParam = b1 * DdE0 + b2 * DdE1 - DdDiff;

				// Barycentric coordinates for the point of intersection.
				TriangleBaryCoords = TVector<Real>(b0, b1, b2);

				// The intersection point is inside or on the Triangle.
				LineClosest = Line.Origin + LineParam * Line.Direction;
				TriangleClosest = Triangle.V[0] + b1 * edge0 + b2 * edge1;
				DistanceSquared = 0;
				return 0;
			}
		}

		// Either (1) the Line is not parallel to the Triangle and the point of
		// intersection of the Line and the plane of the Triangle is outside the
		// Triangle or (2) the Line and Triangle are parallel.  Regardless, the
		// closest point on the Triangle is on an edge of the Triangle.  Compare
		// the Line to all three edges of the Triangle.
		Real sqrDist = TMathUtil<Real>::MaxReal;
		for (int i0 = 2, i1 = 0; i1 < 3; i0 = i1++)
		{
			TSegment3<Real> segment(Triangle.V[i0], Triangle.V[i1]);
			TDistLine3Segment3<Real> queryLS(Line, segment);
			Real sqrDistTmp = queryLS.GetSquared();
			if (sqrDistTmp < sqrDist)
			{
				LineClosest = queryLS.LineClosest;
				TriangleClosest = queryLS.SegmentClosest;
				sqrDist = sqrDistTmp;
				LineParam = queryLS.LineParameter;
				Real ratio = queryLS.SegmentParameter / segment.Extent;
				TriangleBaryCoords[i0] = (0.5) * (1 - ratio);
				TriangleBaryCoords[i1] = 1 - TriangleBaryCoords[i0];
				TriangleBaryCoords[3 - i0 - i1] = 0;
			}
		}

		DistanceSquared = sqrDist;
		return DistanceSquared;
	}
};

typedef TDistLine3Triangle3<float> FDistLine3Triangle3f;
typedef TDistLine3Triangle3<double> FDistLine3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
