// Copyright Epic Games, Inc. All Rights Reserved.

// Port of gte's GteDistPointTriangle to use GeometryProcessing data types

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"


namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute unsigned distance between 3D Point and 3D Triangle
 */
template <typename Real>
class TDistPoint3Triangle3
{
public:
	// Input
	TVector<Real> Point;
	TTriangle3<Real> Triangle;

	// Results
	TVector<Real> TriangleBaryCoords;
	TVector<Real> ClosestTrianglePoint;  // do we need this just use Triangle.BarycentricPoint


	TDistPoint3Triangle3(const TVector<Real>& PointIn, const TTriangle3<Real>& TriangleIn)
	{
		Point = PointIn;
		Triangle = TriangleIn;
	}

	Real Get() 
	{
		return (Real)sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		TVector<Real> diff = Point - Triangle.V[0];
		TVector<Real> edge0 = Triangle.V[1] - Triangle.V[0];
		TVector<Real> edge1 = Triangle.V[2] - Triangle.V[0];
		Real a00 = edge0.SquaredLength();
		Real a01 = edge0.Dot(edge1);
		Real a11 = edge1.SquaredLength();
		Real b0 = -diff.Dot(edge0);
		Real b1 = -diff.Dot(edge1);

		Real f00 = b0;
		Real f10 = b0 + a00;
		Real f01 = b0 + a01;

		TVector2<Real> p0, p1, p;
		Real dt1, h0, h1;

		// Compute the endpoints p0 and p1 of the segment.  The segment is
		// parameterized by L(z) = (1-z)*p0 + z*p1 for z in [0,1] and the
		// directional derivative of half the quadratic on the segment is
		// H(z) = Dot(p1-p0,gradient[Q](L(z))/2), where gradient[Q]/2 = (F,G).
		// By design, F(L(z)) = 0 for cases (2), (4), (5), and (6).  Cases (1) and
		// (3) can correspond to no-intersection or intersection of F = 0 with the
		// Triangle.
		if (f00 >= (Real)0)
		{
			if (f01 >= (Real)0)
			{
				// (1) p0 = (0,0), p1 = (0,1), H(z) = G(L(z))
				GetMinEdge02(a11, b1, p);
			}
			else
			{
				// (2) p0 = (0,t10), p1 = (t01,1-t01), H(z) = (t11 - t10)*G(L(z))
				p0[0] = (Real)0;
				p0[1] = f00 / (f00 - f01);
				p1[0] = f01 / (f01 - f10);
				p1[1] = (Real)1 - p1[0];
				dt1 = p1[1] - p0[1];
				h0 = dt1 * (a11 * p0[1] + b1);
				if (h0 >= (Real)0)
				{
					GetMinEdge02(a11, b1, p);
				}
				else
				{
					h1 = dt1 * (a01 * p1[0] + a11 * p1[1] + b1);
					if (h1 <= (Real)0)
					{
						GetMinEdge12(a01, a11, b1, f10, f01, p);
					}
					else
					{
						GetMinInterior(p0, h0, p1, h1, p);
					}
				}
			}
		}
		else if (f01 <= (Real)0)
		{
			if (f10 <= (Real)0)
			{
				// (3) p0 = (1,0), p1 = (0,1), H(z) = G(L(z)) - F(L(z))
				GetMinEdge12(a01, a11, b1, f10, f01, p);
			}
			else
			{
				// (4) p0 = (t00,0), p1 = (t01,1-t01), H(z) = t11*G(L(z))
				p0[0] = f00 / (f00 - f10);
				p0[1] = (Real)0;
				p1[0] = f01 / (f01 - f10);
				p1[1] = (Real)1 - p1[0];
				h0 = p1[1] * (a01 * p0[0] + b1);
				if (h0 >= (Real)0)
				{
					p = p0;  // GetMinEdge01
				}
				else
				{
					h1 = p1[1] * (a01 * p1[0] + a11 * p1[1] + b1);
					if (h1 <= (Real)0)
					{
						GetMinEdge12(a01, a11, b1, f10, f01, p);
					}
					else
					{
						GetMinInterior(p0, h0, p1, h1, p);
					}
				}
			}
		}
		else if (f10 <= (Real)0)
		{
			// (5) p0 = (0,t10), p1 = (t01,1-t01), H(z) = (t11 - t10)*G(L(z))
			p0[0] = (Real)0;
			p0[1] = f00 / (f00 - f01);
			p1[0] = f01 / (f01 - f10);
			p1[1] = (Real)1 - p1[0];
			dt1 = p1[1] - p0[1];
			h0 = dt1 * (a11 * p0[1] + b1);
			if (h0 >= (Real)0)
			{
				GetMinEdge02(a11, b1, p);
			}
			else
			{
				h1 = dt1 * (a01 * p1[0] + a11 * p1[1] + b1);
				if (h1 <= (Real)0)
				{
					GetMinEdge12(a01, a11, b1, f10, f01, p);
				}
				else
				{
					GetMinInterior(p0, h0, p1, h1, p);
				}
			}
		}
		else
		{
			// (6) p0 = (t00,0), p1 = (0,t11), H(z) = t11*G(L(z))
			p0[0] = f00 / (f00 - f10);
			p0[1] = (Real)0;
			p1[0] = (Real)0;
			p1[1] = f00 / (f00 - f01);
			h0 = p1[1] * (a01 * p0[0] + b1);
			if (h0 >= (Real)0)
			{
				p = p0;  // GetMinEdge01
			}
			else
			{
				h1 = p1[1] * (a11 * p1[1] + b1);
				if (h1 <= (Real)0)
				{
					GetMinEdge02(a11, b1, p);
				}
				else
				{
					GetMinInterior(p0, h0, p1, h1, p);
				}
			}
		}

		TriangleBaryCoords = TVector<Real>((Real)1 - p[0] - p[1], p[0], p[1]);
		ClosestTrianglePoint = Triangle.V[0] + p[0] * edge0 + p[1] * edge1;
		return DistanceSquared(Point, ClosestTrianglePoint);
	}

private:
	void GetMinEdge02(Real const& a11, Real const& b1, TVector2<Real>& p) const
	{
		p[0] = (Real)0;
		if (b1 >= (Real)0)
		{
			p[1] = (Real)0;
		}
		else if (a11 + b1 <= (Real)0)
		{
			p[1] = (Real)1;
		}
		else
		{
			p[1] = -b1 / a11;
		}
	}

	void GetMinEdge12(
			Real const& a01, Real const& a11, Real const& b1, Real const& f10,
			Real const& f01, TVector2<Real>& p) const
	{
		Real h0 = a01 + b1 - f10;
		if (h0 >= (Real)0)
		{
			p[1] = (Real)0;
		}
		else
		{
			Real h1 = a11 + b1 - f01;
			if (h1 <= (Real)0)
			{
				p[1] = (Real)1;
			}
			else
			{
				p[1] = h0 / (h0 - h1);
			}
		}
		p[0] = (Real)1 - p[1];
	}

	void GetMinInterior(
			TVector2<Real> const& p0, Real const& h0, TVector2<Real> const& p1,
			Real const& h1, TVector2<Real>& p) const
	{
		Real z = h0 / (h0 - h1);
		p = ((Real)1 - z) * p0 + z * p1;
	}

};

typedef TDistPoint3Triangle3<float> FDistPoint3Triangle3f;
typedef TDistPoint3Triangle3<double> FDistPoint3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
