// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistRay3Segment3

#pragma once

#include "VectorTypes.h"
#include "SegmentTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute distance between 3D ray and 3D segment
 */
template <typename Real>
class TDistRay3Segment3
{
public:
	// Input
	TRay<Real> Ray;
	TSegment3<Real> Segment;

	// Results
	Real DistanceSquared = -1.0;

	TVector<Real> RayClosestPoint;
	Real RayParameter;
	TVector<Real> SegmentClosestPoint;
	Real SegmentParameter;


	TDistRay3Segment3(const TRay<Real>& RayIn, const TSegment3<Real>& SegmentIn)
	{
		Ray = RayIn;
		Segment = SegmentIn;
	}

	Real Get()
	{
		return (Real)sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}
		FVector3d diff = Ray.Origin - Segment.Center;
		double a01 = -Ray.Direction.Dot(Segment.Direction);
		double b0 = diff.Dot(Ray.Direction);
		double b1 = -diff.Dot(Segment.Direction);
		double c = diff.SquaredLength();
		double det = TMathUtil<Real>::Abs(1 - a01 * a01);
		double s0, s1, sqrDist, extDet;

		if (det >= TMathUtil<Real>::ZeroTolerance ) 
		{
			// The Ray and Segment are not parallel.
			s0 = a01 * b1 - b0;
			s1 = a01 * b0 - b1;
			extDet = Segment.Extent * det;

			if (s0 >= 0) 
			{
				if (s1 >= -extDet) 
				{
					if (s1 <= extDet)  // region 0
					{
						// Minimum at interior points of Ray and Segment.
						double invDet = (1) / det;
						s0 *= invDet;
						s1 *= invDet;
						sqrDist = s0 * (s0 + a01 * s1 + (2) * b0) +
							s1 * (a01 * s0 + s1 + (2) * b1) + c;
					}
					else  // region 1
					{
						s1 = Segment.Extent;
						s0 = -(a01 * s1 + b0);
						if (s0 > 0) 
						{
							sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
						}
						else 
						{
							s0 = 0;
							sqrDist = s1 * (s1 + (2) * b1) + c;
						}
					}
				}
				else  // region 5
				{
					s1 = -Segment.Extent;
					s0 = -(a01 * s1 + b0);
					if (s0 > 0) 
					{
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
			}
			else 
			{
				if (s1 <= -extDet)  // region 4
				{
					s0 = -(-a01 * Segment.Extent + b0);
					if (s0 > 0) 
					{
						s1 = -Segment.Extent;
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						s1 = -b1;
						if (s1 < -Segment.Extent) 
						{
							s1 = -Segment.Extent;
						}
						else if (s1 > Segment.Extent) 
						{
							s1 = Segment.Extent;
						}
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
				else if (s1 <= extDet)  // region 3
				{
					s0 = 0;
					s1 = -b1;
					if (s1 < -Segment.Extent) 
					{
						s1 = -Segment.Extent;
					}
					else if (s1 > Segment.Extent) 
					{
						s1 = Segment.Extent;
					}
					sqrDist = s1 * (s1 + (2) * b1) + c;
				}
				else  // region 2
				{
					s0 = -(a01 * Segment.Extent + b0);
					if (s0 > 0) 
					{
						s1 = Segment.Extent;
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						s1 = -b1;
						if (s1 < -Segment.Extent) 
						{
							s1 = -Segment.Extent;
						}
						else if (s1 > Segment.Extent) 
						{
							s1 = Segment.Extent;
						}
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
			}
		}
		else 
		{
			// Ray and Segment are parallel.
			if (a01 > 0) 
			{
				// Opposite direction vectors.
				s1 = -Segment.Extent;
			}
			else 
			{
				// Same direction vectors.
				s1 = Segment.Extent;
			}

			s0 = -(a01 * s1 + b0);
			if (s0 > 0) 
			{
				sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
			}
			else 
			{
				s0 = 0;
				sqrDist = s1 * (s1 + (2) * b1) + c;
			}
		}

		RayClosestPoint = Ray.Origin + s0 * Ray.Direction;
		SegmentClosestPoint = Segment.Center + s1 * Segment.Direction;
		RayParameter = s0;
		SegmentParameter = s1;

		// Account for numerical round-off errors.
		if (sqrDist < (Real)0) 
		{
			sqrDist = (Real)0;
		}
		DistanceSquared = sqrDist;

		return sqrDist;
	}





	static double SquaredDistance(const TRay<Real>& Ray, const TSegment3<Real>& Segment,
		Real& RayParam, Real& SegParam)
	{
		TVector<Real> diff = Ray.Origin - Segment.Center;
		double a01 = -Ray.Direction.Dot(Segment.Direction);
		double b0 = diff.Dot(Ray.Direction);
		double b1 = -diff.Dot(Segment.Direction);
		double c = diff.SquaredLength();
		double det = TMathUtil<Real>::Abs(1 - a01 * a01);
		double s0, s1, sqrDist, extDet;

		if (det >= TMathUtil<Real>::ZeroTolerance) 
		{
			// The Ray and Segment are not parallel.
			s0 = a01 * b1 - b0;
			s1 = a01 * b0 - b1;
			extDet = Segment.Extent * det;

			if (s0 >= 0) 
			{
				if (s1 >= -extDet) 
				{
					if (s1 <= extDet)  // region 0
					{
						// Minimum at interior points of Ray and Segment.
						double invDet = (1) / det;
						s0 *= invDet;
						s1 *= invDet;
						sqrDist = s0 * (s0 + a01 * s1 + (2) * b0) +
							s1 * (a01 * s0 + s1 + (2) * b1) + c;
					}
					else  // region 1
					{
						s1 = Segment.Extent;
						s0 = -(a01 * s1 + b0);
						if (s0 > 0) 
						{
							sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
						}
						else 
						{
							s0 = 0;
							sqrDist = s1 * (s1 + (2) * b1) + c;
						}
					}
				}
				else  // region 5
				{
					s1 = -Segment.Extent;
					s0 = -(a01 * s1 + b0);
					if (s0 > 0) 
					{
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
			}
			else 
			{
				if (s1 <= -extDet)  // region 4
				{
					s0 = -(-a01 * Segment.Extent + b0);
					if (s0 > 0) 
					{
						s1 = -Segment.Extent;
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						s1 = -b1;
						if (s1 < -Segment.Extent) 
						{
							s1 = -Segment.Extent;
						}
						else if (s1 > Segment.Extent) 
						{
							s1 = Segment.Extent;
						}
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
				else if (s1 <= extDet)  // region 3
				{
					s0 = 0;
					s1 = -b1;
					if (s1 < -Segment.Extent) 
					{
						s1 = -Segment.Extent;
					}
					else if (s1 > Segment.Extent) 
					{
						s1 = Segment.Extent;
					}
					sqrDist = s1 * (s1 + (2) * b1) + c;
				}
				else  // region 2
				{
					s0 = -(a01 * Segment.Extent + b0);
					if (s0 > 0) 
					{
						s1 = Segment.Extent;
						sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
					}
					else 
					{
						s0 = 0;
						s1 = -b1;
						if (s1 < -Segment.Extent) 
						{
							s1 = -Segment.Extent;
						}
						else if (s1 > Segment.Extent) 
						{
							s1 = Segment.Extent;
						}
						sqrDist = s1 * (s1 + (2) * b1) + c;
					}
				}
			}
		}
		else {
			// Ray and Segment are parallel.
			if (a01 > 0) 
			{
				// Opposite direction vectors.
				s1 = -Segment.Extent;
			}
			else 
			{
				// Same direction vectors.
				s1 = Segment.Extent;
			}

			s0 = -(a01 * s1 + b0);
			if (s0 > 0) 
			{
				sqrDist = -s0 * s0 + s1 * (s1 + (2) * b1) + c;
			}
			else 
			{
				s0 = 0;
				sqrDist = s1 * (s1 + (2) * b1) + c;
			}
		}

		RayParam = s0;
		SegParam = s1;

		// Account for numerical round-off errors.
		if (sqrDist < 0)
		{
			sqrDist = 0;
		}
		return sqrDist;
	}







};

typedef TDistRay3Segment3<float> FDistRay3Segment3f;
typedef TDistRay3Segment3<double> FDistRay3Segment3d;

} // end namespace UE::Geometry
} // end namespace UE
