// Copyright Epic Games, Inc. All Rights Reserved.

// Port of gte's DistSegment2AlignedBox2 to use GeometryCore data types

#pragma once

#include "DistLine2AxisAlignedBox2.h"
#include "Math/Box2D.h"
#include "SegmentTypes.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* Compute unsigned distance between 2D segment and 2D axis aligned box
*/
template <typename Real>
class TDistSegment2AxisAlignedBox2
{
public:
	// Input
	TSegment2<Real> Segment;
	TBox2<Real> AxisAlignedBox;

	// Output
	Real DistanceSquared = -1.0;
	Real SegmentParameter;
	TVector2<Real> BoxClosest, SegmentClosest;

	TDistSegment2AxisAlignedBox2(const TSegment2<Real>& SegmentIn, const TBox2<Real>& AxisAlignedBoxIn) : Segment(SegmentIn), AxisAlignedBox(AxisAlignedBoxIn)
	{
	}

	Real Get() 
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		TLine2<Real> line(Segment.Center, Segment.Direction);
		TDistLine2AxisAlignedBox2<Real> queryLB(line, AxisAlignedBox);
		Real sqrDist = queryLB.GetSquared();
		SegmentParameter = queryLB.LineParameter;

		if (SegmentParameter >= -Segment.Extent) {
			if (SegmentParameter <= Segment.Extent) {
				SegmentClosest = queryLB.LineClosest;
				BoxClosest = queryLB.BoxClosest;
			}
			else {
				SegmentClosest = Segment.EndPoint();
				BoxClosest = AxisAlignedBox.GetClosestPointTo(SegmentClosest);
				sqrDist = (SegmentClosest - BoxClosest).SquaredLength();
				SegmentParameter = Segment.Extent;
			}
		}
		else {
			SegmentClosest = Segment.StartPoint();
			BoxClosest = AxisAlignedBox.GetClosestPointTo(SegmentClosest);
			sqrDist = (SegmentClosest - BoxClosest).SquaredLength();
			SegmentParameter = -Segment.Extent;
		}

		DistanceSquared = sqrDist;
		return DistanceSquared;
	}
};

typedef TDistSegment2AxisAlignedBox2<float> FDistSegment2AxisAlignedBox2f;
typedef TDistSegment2AxisAlignedBox2<double> FDistSegment2AxisAlignedBox2d;

} // end namespace UE::Geometry
} // end namespace UE
