// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistSegment3Triangle3
// which was ported from WildMagic 5 

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "DistLine3Triangle3.h"
#include "DistPoint3Triangle3.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* Compute unsigned distance between 3D segment and 3D triangle
*/
template <typename Real>
class TDistSegment3Triangle3
{
public:
	// Input
	TSegment3<Real> Segment;
	TTriangle3<Real> Triangle;

	// Output
	Real DistanceSquared = -1.0;
	Real SegmentParameter;
	TVector<Real> TriangleClosest, TriangleBaryCoords, SegmentClosest;
	
	TDistSegment3Triangle3(const TSegment3<Real>& SegmentIn, const TTriangle3<Real>& TriangleIn) : Segment(SegmentIn), Triangle(TriangleIn)
	{
	}

	Real Get() 
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}

		TLine3<Real> line(Segment.Center, Segment.Direction);
		TDistLine3Triangle3<Real> queryLT(line, Triangle);
		double sqrDist = queryLT.GetSquared();
		SegmentParameter = queryLT.LineParam;

		if (SegmentParameter >= -Segment.Extent) {
			if (SegmentParameter <= Segment.Extent) {
				SegmentClosest = queryLT.LineClosest;
				TriangleClosest = queryLT.TriangleClosest;
				TriangleBaryCoords = queryLT.TriangleBaryCoords;
			}
			else {
				SegmentClosest = Segment.EndPoint();
				TDistPoint3Triangle3<Real> queryPT(SegmentClosest, Triangle);
				sqrDist = queryPT.GetSquared();
				TriangleClosest = queryPT.ClosestTrianglePoint;
				SegmentParameter = Segment.Extent;
				TriangleBaryCoords = queryPT.TriangleBaryCoords;
			}
		}
		else {
			SegmentClosest = Segment.StartPoint();
			TDistPoint3Triangle3<Real> queryPT(SegmentClosest, Triangle);
			sqrDist = queryPT.GetSquared();
			TriangleClosest = queryPT.ClosestTrianglePoint;
			SegmentParameter = -Segment.Extent;
			TriangleBaryCoords = queryPT.TriangleBaryCoords;
		}

		DistanceSquared = sqrDist;
		return DistanceSquared;
	}
};

typedef TDistSegment3Triangle3<float> FDistSegment3Triangle3f;
typedef TDistSegment3Triangle3<double> FDistSegment3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
