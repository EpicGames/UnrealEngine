// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistTriangle3Triangle3
// which was ported from WildMagic 5 


#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "DistSegment3Triangle3.h"
#include "DistPoint3Triangle3.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* Compute unsigned distance between two 3D triangles
*/
template <typename Real>
class TDistTriangle3Triangle3
{
public:
	// Output
	Real DistanceSquared = -1.0;
	TVector<Real> TriangleClosest[2];
	TVector<Real> TriangleBaryCoords[2];


	TDistTriangle3Triangle3()
	{
	}
	TDistTriangle3Triangle3(const TTriangle3<Real>& TriangleA, const TTriangle3<Real>& TriangleB)
	{
		Triangle[0] = TriangleA;
		Triangle[1] = TriangleB;
	}

	void SetTriangle(int WhichTriangle, const TTriangle3<Real>& TriangleIn)
	{
		check(WhichTriangle >= 0 && WhichTriangle < 2);
		Triangle[WhichTriangle] = TriangleIn;
		DistanceSquared = -1.0; // clear cached result
	}

	Real Get()
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult()
	{
		if (DistanceSquared >= 0)
		{
			return DistanceSquared;
		}


		// Compare edges of Triangle[0] to the interior of Triangle[1].
		Real sqrDist = TMathUtil<Real>::MaxReal, sqrDistTmp;
		Real ratio;
		int i0, i1;
		for (i0 = 2, i1 = 0; i1 < 3; i0 = i1++)
		{
			TSegment3<Real> edge(Triangle[0].V[i0], Triangle[0].V[i1]);

			TDistSegment3Triangle3<Real> queryST(edge, Triangle[1]);
			sqrDistTmp = queryST.GetSquared();
			if (sqrDistTmp < sqrDist)
			{
				TriangleClosest[0] = queryST.SegmentClosest;
				TriangleClosest[1] = queryST.TriangleClosest;
				sqrDist = sqrDistTmp;

				ratio = queryST.SegmentParameter / edge.Extent;
				TriangleBaryCoords[0][i0] = (0.5) * (1 - ratio);
				TriangleBaryCoords[0][i1] = 1 - TriangleBaryCoords[0][i0];
				TriangleBaryCoords[0][3 - i0 - i1] = 0;
				TriangleBaryCoords[1] = queryST.TriangleBaryCoords;

				if (sqrDist <= TMathUtil<Real>::ZeroTolerance)
				{
					DistanceSquared = 0;
					return 0;
				}
			}
		}

		// Compare edges of Triangle[1] to the interior of Triangle[0].
		for (i0 = 2, i1 = 0; i1 < 3; i0 = i1++)
		{
			TSegment3<Real> edge(Triangle[1].V[i0], Triangle[1].V[i1]);

			TDistSegment3Triangle3<Real> queryST(edge, Triangle[0]);
			sqrDistTmp = queryST.GetSquared();
			if (sqrDistTmp < sqrDist)
			{
				TriangleClosest[0] = queryST.SegmentClosest;
				TriangleClosest[1] = queryST.TriangleClosest;
				sqrDist = sqrDistTmp;

				ratio = queryST.SegmentParameter / edge.Extent;
				TriangleBaryCoords[1][i0] = (0.5) * (1 - ratio);
				TriangleBaryCoords[1][i1] = 1 - TriangleBaryCoords[1][i0];
				TriangleBaryCoords[1][3 - i0 - i1] = 0;
				TriangleBaryCoords[0] = queryST.TriangleBaryCoords;

				if (sqrDist <= TMathUtil<Real>::ZeroTolerance)
				{
					DistanceSquared = 0;
					return 0;
				}
			}
		}


		DistanceSquared = sqrDist;
		return DistanceSquared;
	}
	
private:

	// Input triangles; must be set via SetTriangle to clear cached result
	TTriangle3<Real> Triangle[2];
};

typedef TDistTriangle3Triangle3<float> FDistTriangle3Triangle3f;
typedef TDistTriangle3Triangle3<double> FDistTriangle3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
