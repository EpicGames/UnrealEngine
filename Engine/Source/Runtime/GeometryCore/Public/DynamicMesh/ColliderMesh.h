// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IndexTypes.h"
#include "Math/Ray.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/SpatialInterfaces.h"
#include "Templates/Function.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * FColliderMesh is a minimal representation of an Indexed Triangle Mesh suitable to use
 * with a TMeshAABBTree3. This class is intended to be used in situations where a copy
 * of a FDynamicMesh3 would be created/kept only to use with an AABBTree or FWNTree. In such 
 * situations, the FDynamicMesh3 copy is (relatively) heavy memory-wise, particularly if a large 
 * number of small mesh/AABBTree pairs are stored. In those cases a FColliderMesh can be used instead.
 * 
 * If FBuildOptions.bBuildAABBTree is true (default), then an AABBTree will automatically be
 * built and available via GetAABBTree()
 * 
 * If the source FDynamicMesh3 is not compact, it will be compacted. ID mappings can optionally
 * be stored if the FBuildOptions are configured to request it (default false), allowing 
 * hit-triangle/vertex IDs to be mapped back to IDs on the source mesh.
 * 
 * Currently there is no support for UVs or Vertex Normals.
 * 
 * FColliderMesh (effectively) implements the TTriangleMeshAdapter interface, and so can be used 
 * anywhere a TTriangleMeshAdapter could be used (eg with TMeshQueries functions). 
 */
class FColliderMesh
{
public:
	struct FBuildOptions
	{
		FBuildOptions()
		{
			bBuildAABBTree = true;
			bBuildVertexMap = false;
			bBuildTriangleMap = false;
		}

		/** If true, AABBTree is automatically built for the mesh on construction */
		bool bBuildAABBTree;
		/** If true and SourceWasCompactV() == false, mapping from FColliderMesh to source mesh vertex IDs is constructed */
		bool bBuildVertexMap;
		/** If true and SourceWasCompactT() == false, mapping from FColliderMesh to source mesh triangle IDs is constructed */
		bool bBuildTriangleMap;
	};

public:
	GEOMETRYCORE_API FColliderMesh();
	GEOMETRYCORE_API FColliderMesh(const FDynamicMesh3& SourceMesh, const FBuildOptions& BuildOptions = FBuildOptions());

	GEOMETRYCORE_API void Initialize(const FDynamicMesh3& SourceMesh, const FBuildOptions& BuildOptions = FBuildOptions());

	GEOMETRYCORE_API void Reset(EAllowShrinking AllowShrinking);

	// mesh API required for TMeshAABBTree3
	bool IsTriangle(int32 TriangleID) const { return TriangleID >= 0 && TriangleID < Triangles.Num(); }
	bool IsVertex(int32 VertexID) const { return VertexID >= 0 && VertexID < Vertices.Num(); }
	int32 MaxTriangleID() const { return Triangles.Num(); }
	int32 MaxVertexID() const { return Vertices.Num(); }
	int32 TriangleCount() const { return Triangles.Num(); }
	int32 VertexCount() const { return Vertices.Num(); }
	uint64 GetChangeStamp() const { return 0; }
	FIndex3i GetTriangle(int32 TriangleID) const { return Triangles[TriangleID]; }
	FVector3d GetVertex(int32 VertexID) const { return Vertices[VertexID]; }

	void GetTriVertices(int32 TriangleID, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		FIndex3i Tri = Triangles[TriangleID];
		V0 = Vertices[Tri.A];
		V1 = Vertices[Tri.B];
		V2 = Vertices[Tri.C];
	}

	FVector3d GetTriNormal(int32 TriangleID) const
	{
		FIndex3i Tri = Triangles[TriangleID];
		return VectorUtil::Normal(Vertices[Tri.A], Vertices[Tri.B], Vertices[Tri.C]);
	}

	bool SourceWasCompactV() const { return bSourceWasCompactV; }
	bool HasVertexIDMap() const { return SourceVertexIDs.Num() > 0 && SourceVertexIDs.Num() == Vertices.Num(); }
	GEOMETRYCORE_API int32 GetSourceVertexID(int32 VertexID) const;
	bool SourceWasCompactT() const { return bSourceWasCompactT; }
	bool HasTriangleIDMap() const { return SourceTriangleIDs.Num() > 0 && SourceTriangleIDs.Num() == Triangles.Num(); }
	GEOMETRYCORE_API int32 GetSourceTriangleID(int32 TriangleID) const;

	// spatial APIs

	/**
	 * @param HitTriangleIDOut TriangleID on collider mesh - use GetSourceTriangleID() to map back to source mesh
	 * @return true if a triangle was hit and Out parameters are initialized
	 */
	GEOMETRYCORE_API bool FindNearestHitTriangle( const FRay3d& Ray, double& RayParameterOut, int& HitTriangleIDOut, FVector3d& BaryCoordsOut, const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions() ) const;


	/**
	 * Find the TriangleID closest to P on the collider mesh, and distance to it, within distance MaxDist, or return InvalidID.
	 * Use GetSourceTriangleID() to map the TriangleID back to the source mesh.
	 * Use MeshQueries.TriangleDistance() to get more information.
	 */
	GEOMETRYCORE_API int FindNearestTriangle( const FVector3d& Point, double& NearestDistSqrOut, const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions() ) const;


	/**
	 * Direct access to the AABBTree pointer inside the ColliderMesh. 
	 * @warning this function should be avoided if possible, and may be removed in the future. 
	 */
	GEOMETRYCORE_API TMeshAABBTree3<FColliderMesh>* GetRawAABBTreeUnsafe();




protected:
	// For large meshes it might be better to optionally use TDynamicVector here (but keeping the TArray for
	// small meshes is still also desirable due to minimum-size). Adds a branch to every access, though.
	TArray<FVector3d> Vertices;
	TArray<FIndex3i> Triangles;

	TArray<int32> SourceVertexIDs;
	bool bSourceWasCompactV;
	TArray<int32> SourceTriangleIDs;
	bool bSourceWasCompactT;

	TMeshAABBTree3<FColliderMesh> AABBTree;

};


/**
 * Projection target API wrapper for an FColliderMesh
 */
class FColliderMeshProjectionTarget : public IOrientedProjectionTarget
{
public:
	FColliderMesh* ColliderMesh = nullptr;

	FColliderMeshProjectionTarget()
	{
		ColliderMesh = nullptr;
	}

	FColliderMeshProjectionTarget(FColliderMesh* ColliderMeshIn)
	{
		check(ColliderMeshIn);
		ColliderMesh = ColliderMeshIn;
	}

	virtual ~FColliderMeshProjectionTarget() = default;

	GEOMETRYCORE_API virtual FVector3d Project(const FVector3d& Point, int Identifier = -1) override;
	GEOMETRYCORE_API virtual FVector3d Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier = -1) override;
};



} // end namespace UE::Geometry
} // end namespace UE





