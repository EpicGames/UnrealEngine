// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshIndexMappings.h"
#include "Serialization/NameAsStringProxyArchive.h"

namespace UE
{
namespace Geometry
{

struct FMeshIndexMappings;

template<typename ParentType>
class TDynamicAttributeBase;

/**
* Generic base class for change tracking of an attribute layer
*/
template<typename ParentType>
class TDynamicAttributeChangeBase
{
public:
	virtual ~TDynamicAttributeChangeBase() = default;

	// default do-nothing implementations are provided because many attribute layers will only care about some kinds of elements and won't implement all of these

	virtual void SaveInitialTriangle(const TDynamicAttributeBase<ParentType>* Attribute, int TriangleID)
	{
	}
	virtual void SaveInitialVertex(const TDynamicAttributeBase<ParentType>* Attribute, int VertexID)
	{
	}

	virtual void StoreAllFinalTriangles(const TDynamicAttributeBase<ParentType>* Attribute, const TArray<int>& TriangleIDs)
	{
	}
	UE_DEPRECATED(5.6, "Use the version of this method that takes a TSet instead")
	virtual void StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TArray<int>& TriangleIDs)
	{
		ensure(false);
	}
	virtual void StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TSet<int>& TriangleIDs)
	{
	}

	virtual bool Apply(TDynamicAttributeBase<ParentType>* Attribute, bool bRevert) const
	{
		return false;
	}
};

typedef TDynamicAttributeChangeBase<FDynamicMesh3> FDynamicMeshAttributeChangeBase;


/**
 * Base class for attributes that live on a dynamic mesh (or similar dynamic object)
 *
 * Subclasses can override the On* functions to ensure the attribute remains up to date through changes to the dynamic object
 */
template<typename ParentType>
class TDynamicAttributeBase
{

public:
	virtual ~TDynamicAttributeBase() = default;


public:
	/** Get optional identifier for this attribute set. */
	FName GetName() const { return Name; }

	/** Set optional identifier for this attribute set. */
	void SetName(FName NameIn) { Name = NameIn; }

protected:
	/** Optional FName identifier for this attribute set. Not guaranteed to be unique. */
	FName Name = FName();




public:


	/** Allocate a new copy of the attribute layer, optionally with a different parent */
	virtual TDynamicAttributeBase* MakeCopy(ParentType* ParentIn) const = 0;
	/** Allocate a new empty instance of the same type of attribute layer */
	virtual TDynamicAttributeBase* MakeNew(ParentType* ParentIn) const = 0;
	/**
	 * Allocate a new compact copy of the attribute layer, optionally with a different parent.
	 * Default implementation does a full copy and then compacts it, usually derived class will want to override this with a more efficient direct compact copy implementation
	 */
	virtual TDynamicAttributeBase* MakeCompactCopy(const FCompactMaps& CompactMaps, ParentType* ParentIn) const
	{
		TDynamicAttributeBase* Copy = MakeCopy(ParentIn);
		Copy->CompactInPlace(CompactMaps);
		return Copy;
	}

	/** Compact the attribute in place */
	virtual void CompactInPlace(const FCompactMaps& CompactMaps) = 0;
	
	/** Update any held pointer to the parent */
	virtual void Reparent(ParentType* NewParent) = 0;

	/**
	  * Copy data from a different attribute to this one, using the mesh index mapping to determine the correspondence 
	  * @param Source copy attribute data from this source
	  * @param Mapping the correspondence from Source's parent to this attribute's parent 
	  * @return true if the copy succeeded, false otherwise (e.g. false if the data from the source attribute was not compatible and the CopyOut failed to copy across)
	  */
	virtual bool CopyThroughMapping(const TDynamicAttributeBase* Source, const FMeshIndexMappings& Mapping) = 0;

	/**
	 * Add elements from the Source attribute, using the AppendInfo to determine the range to copy.
	 * @return true if the append succeeded, false otherwise (e.g. false if the data from the source attribute was not compatible and the CopyOut failed to copy across)
	 */
	virtual bool Append(const TDynamicAttributeBase& Source, const FDynamicMesh3::FAppendInfo& AppendInfo) = 0;

	/**
	 * Add default-valued elements, using the AppendInfo to determine the range to add. Called when appending from a mesh that does not have this attribute.
	 */
	virtual void AppendDefaulted(const FDynamicMesh3::FAppendInfo& AppendInfo) = 0;

	/** Generic function to copy data out of an attribute; it's up to the derived class to map RawID to chunks of attribute data */
	virtual bool CopyOut(int RawID, void* Buffer, int BufferSize) const = 0;

	/** Generic function to copy data in to an attribute; it's up to the derived class to map RawID to chunks of attribute data */
	virtual bool CopyIn(int RawID, void* Buffer, int BufferSize) = 0;

	virtual void OnNewVertex(int VertexID, bool bInserted)
	{
	}

	virtual void OnRemoveVertex(int VertexID)
	{
	}

	virtual void OnNewTriangle(int TriangleID, bool bInserted)
	{
	}

	virtual void OnRemoveTriangle(int TriangleID)
	{
	}

	virtual void OnReverseTriOrientation(int TriangleID)
	{
	}

	/**
	* Check validity of attribute
	* 
	* @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	* @param FailMode Desired behavior if mesh is found invalid
	*/
	virtual bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const
	{
		// default impl just doesn't check anything; override with any useful sanity checks
		return true;
	}


	virtual TUniquePtr<TDynamicAttributeChangeBase<ParentType>> NewBlankChange() const = 0;



	/** Update to reflect an edge split in the parent mesh */
	virtual void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo& SplitInfo)
	{
	}

	/** Update to reflect an edge flip in the parent mesh */
	virtual void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo)
	{
	}

	/** Update to reflect an edge collapse in the parent mesh */
	virtual void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo& CollapseInfo)
	{
	}

	/** Update to reflect a face poke in the parent mesh */
	virtual void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo& PokeInfo)
	{
	}

	/** Update to reflect an edge merge in the parent mesh */
	virtual void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo& MergeInfo)
	{
	}

	/** 
	 * Update to reflect a vertex merge that does not resolve as a collapse or edge merge (i.e. a
	 *  vertex merge that resolves as bowtie creation).
	 */
	virtual void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& MergeInfo)
	{
	}

	/** Update to reflect an edge merge in the parent mesh */
	virtual void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate)
	{
	}

	/**
	 * Serialization operator for TDynamicAttributeBase.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Attr Mesh attribute to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, TDynamicAttributeBase<ParentType>& Attr)
	{
		Attr.Serialize(Ar);
		return Ar;
	}

	/**
	* Serialize to and from an archive.
	*
	* @param Ar Archive to serialize with.
	*/
	void Serialize(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		if (!Ar.IsLoading() || Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::DynamicMeshAttributesWeightMapsAndNames)
		{
			FNameAsStringProxyArchive ProxyArchive(Ar);
			ProxyArchive << Name;
		}
	}

	virtual SIZE_T GetByteCount() const { return 0; }

protected:

	/** 
	 * Implementation of parent-class copy. MakeCopy() and MakeCompactCopy() implementations should call
	 * this to transfer any custom data added by parent attribute set class.
	 */
	virtual void CopyParentClassData(const TDynamicAttributeBase<ParentType>& Other)
	{
		Name = Other.Name;
	}
};


typedef TDynamicAttributeBase<FDynamicMesh3> FDynamicMeshAttributeBase;


/**
* Generic base class for managing a set of registered attributes that must all be kept up to date
*/
template<typename ParentType>
class TDynamicAttributeSetBase
{
protected:
	// not managed by the base class; we should be able to register any attributes here that we want to be automatically updated
	TArray<TDynamicAttributeBase<ParentType>*> RegisteredAttributes;

	/**
	 * Stores the given attribute pointer in the attribute register, so that it will be updated with mesh changes, but does not take ownership of the attribute memory.
	 */
	void RegisterExternalAttribute(TDynamicAttributeBase<ParentType>* Attribute)
	{
		RegisteredAttributes.Add(Attribute);
	}

	void UnregisterExternalAttribute(TDynamicAttributeBase<ParentType>* Attribute)
	{
		RegisteredAttributes.Remove(Attribute);
	}

	void ResetRegisteredAttributes()
	{
		RegisteredAttributes.Reset();
	}

public:
	virtual ~TDynamicAttributeSetBase() = default;

	int NumRegisteredAttributes() const
	{
		return RegisteredAttributes.Num();
	}

	TDynamicAttributeBase<ParentType>* GetRegisteredAttribute(int Idx) const
	{
		return RegisteredAttributes[Idx];
	}

	// These functions are called by the FDynamicMesh3 to update the various
	// attributes when the parent mesh topology has been modified.
	virtual void OnNewTriangle(int TriangleID, bool bInserted)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnNewTriangle(TriangleID, bInserted);
		}
	}
	virtual void OnNewVertex(int VertexID, bool bInserted)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnNewVertex(VertexID, bInserted);
		}
	}
	virtual void OnRemoveTriangle(int TriangleID)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnRemoveTriangle(TriangleID);
		}
	}
	virtual void OnRemoveVertex(int VertexID)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnRemoveVertex(VertexID);
		}
	}
	virtual void OnReverseTriOrientation(int TriangleID)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnReverseTriOrientation(TriangleID);
		}
	}

	/**
	* Check validity of attributes
	* 
	* @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	* @param FailMode Desired behavior if mesh is found invalid
	*/
	virtual bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const
	{
		bool bValid = true;
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			bValid = A->CheckValidity(bAllowNonmanifold, FailMode) && bValid;
		}
		return bValid;
	}


	// mesh-specific on* functions; may be split out
public:

	virtual void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo& SplitInfo)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnSplitEdge(SplitInfo);
		}
	}
	virtual void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnFlipEdge(FlipInfo);
		}
	}
	virtual void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo& CollapseInfo)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnCollapseEdge(CollapseInfo);
		}
	}
	virtual void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo& PokeInfo)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnPokeTriangle(PokeInfo);
		}
	}
	virtual void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo& MergeInfo)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnMergeEdges(MergeInfo);
		}
	}
	virtual void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& MergeInfo)
	{
		if (!ensureMsgf(!MergeInfo.EdgeCollapseInfo.IsSet(), TEXT("Vertex merge that resolves as edge collapse "
			"is expected to have called OnCollapseEdge, not OnMergeVertices.")))
		{
			OnCollapseEdge(MergeInfo.EdgeCollapseInfo.GetValue());
			return;
		}
		if (!ensureMsgf(!MergeInfo.MergeEdgesInfo.IsSet(), TEXT("Vertex merge that resolves as edge merge "
			"is expected to have called OnMergeEdges, not OnMergeVertices.")))
		{
			OnMergeEdges(MergeInfo.MergeEdgesInfo.GetValue());
			return;
		}

		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnMergeVertices(MergeInfo);
		}
	}
	virtual void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate)
	{
		for (TDynamicAttributeBase<ParentType>* A : RegisteredAttributes)
		{
			A->OnSplitVertex(SplitInfo, TrianglesToUpdate);
		}
	}
};

typedef TDynamicAttributeSetBase<FDynamicMesh3> FDynamicMeshAttributeSetBase;

} // end namespace UE::Geometry
} // end namespace UE
