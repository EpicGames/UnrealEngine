// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicAttribute.h"
#include "Containers/Array.h"

//
// Forward declarations
//
namespace UE::Geometry { class FDynamicMesh3; }

namespace UE
{
namespace Geometry
{

/**
 * TDynamicAttributeBase is a base class for storing per-bone data. 
 */
template<typename ParentType, typename AttribValueType>
class TDynamicBoneAttributeBase
: 
public TDynamicAttributeBase<ParentType>
{
protected:
	friend class FDynamicMeshAttributeSet;
	
	/** The parent object (e.g. mesh, point set) this attribute belongs to. */
	ParentType* Parent = nullptr;

	/** List of per-bone values. */
	TArray<AttribValueType> AttribValues;
public:
	
	/** Create an empty attribute. */
	TDynamicBoneAttributeBase() = default;

	/** Create an attribute for the given parent and set the number of unintialized values. */
	TDynamicBoneAttributeBase(ParentType* ParentIn, const int InNumBones = 0);

	/** Create an attribute for the given parent and number of bones and set all values to the InitialValue. */
	TDynamicBoneAttributeBase(ParentType* ParentIn, const int InNumBones, const AttribValueType& InitialValue);

	virtual ~TDynamicBoneAttributeBase() = default;

	/** Resize the array of values to the given max bone number and set all the values to the InitialValue. */
	void Initialize(const int32 InNumBones, const AttribValueType& InitialValue);

	/** Resize the array of values to the given max bone number. */
	void Resize(const int32 InNumBones);

	 /** Set this attribute to contain the same arrays as the copy attribute. */
	void Copy(const TDynamicBoneAttributeBase<ParentType, AttribValueType>& Copy);

 	/** @return true if the attribute has no bone values stored */
	inline bool IsEmpty() const
	{
		return AttribValues.IsEmpty();
	}
	
	/** @return the number of the values stored. */
	inline int32 Num() const 
	{
		return AttribValues.Num();
	}

	/** 
	 * @return true, if this attribute is the same as the Other.
	 * @note assumes that the AttribValueType support equality operator. If not, create a template specialization.
	 */
	virtual bool IsSameAs(const TDynamicBoneAttributeBase<ParentType, AttribValueType>& Other) const;

	inline const TArray<AttribValueType>& GetAttribValues() const
	{ 
		return AttribValues; 
	}

	inline void SetValue(const int32 InBoneID, const AttribValueType& InValue)
	{
		checkSlow(InBoneID >= 0 && InBoneID < AttribValues.Num());
		AttribValues[InBoneID] = InValue;
	}

	inline const AttribValueType& GetValue(const int32 InBoneID) const
	{
		checkSlow(InBoneID >= 0 && InBoneID < AttribValues.Num())
		return AttribValues[InBoneID];
	}

	/** Add value at the end of the value array. */
	void Append(const AttribValueType& InValue);

	SIZE_T GetByteCount() const
	{
		return AttribValues.GetAllocatedSize();
	}
public:

    //
    // TDynamicAttributeBase interface
    //

    virtual TDynamicAttributeBase<ParentType>* MakeCopy(ParentType* ParentIn) const override;

	virtual TDynamicAttributeBase<ParentType>* MakeNew(ParentType* ParentIn) const override;

    virtual void CompactInPlace(const FCompactMaps& CompactMaps) override
	{
		// Compact maps don't affect the bone attribute.
	}

    virtual void Reparent(ParentType* NewParent) override 
    { 
        Parent = NewParent;  
    }

	virtual bool CopyThroughMapping(const TDynamicAttributeBase<ParentType>* Source, const FMeshIndexMappings& Mapping) override
	{
		// Index remapping is not supported by the bone attribute. We need to be careful with the order of the 
		// bones since it affects the indexing of the skin weight attributes.
		return true;
	}

	virtual bool Append(const TDynamicAttributeBase<ParentType>& Source, const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		// Appending bone attributes is a special case that must be handled AttributeSet level,
		// because it may require also consistently remapping bone weights
		ensureMsgf(false, TEXT("Directly appending bone attributes is not supported"));
		return false;
	}

	virtual void AppendDefaulted(const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		// bone attributes are not attached to mesh vertices/triangles/edges, so there is no work to do in the append-defaulted case
	}

	virtual bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		return false; // Not supported for bones
	}

	virtual bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		return false; // Not supported for bones
	}

	virtual TUniquePtr<TDynamicAttributeChangeBase<ParentType>> NewBlankChange() const override
	{
		return MakeUnique<TDynamicAttributeChangeBase<ParentType>>();
	}

public:
    
    //
    // Serialization
    //

    void Serialize(FArchive& Ar);
};

/** FTransform doesn't support equality operator. */
template<>
bool TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>::IsSameAs(const TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>& Other) const;

using FDynamicMeshBoneNameAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FName>;
using FDynamicMeshBoneParentIndexAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, int32>;
using FDynamicMeshBoneColorAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FVector4f>;
using FDynamicMeshBonePoseAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>;

} // namespace Geometry
} // namespace UE