// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp DMesh3

#pragma once

#include "Async/TransactionallySafeMutex.h"
#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "FrameTypes.h"
#include "GeometryTypes.h"
#include "HAL/Platform.h"
#include "IndexTypes.h"
#include "InfoTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Quaternion.h"
#include "Serialization/Archive.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Util/CompactMaps.h"
#include "Util/DynamicVector.h"
#include "Util/IndexUtil.h"
#include "Util/IteratorUtil.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"
#include "VectorTypes.h"
#include "VectorUtil.h"

#include <atomic>
#include <initializer_list>

namespace UE
{
namespace Geometry
{

class FCompactMaps;
class FDynamicMeshAttributeSet;
class FMeshShapeGenerator;

enum class EMeshComponents : uint8
{
	None          = 0,
	VertexNormals = 1,
	VertexColors  = 2,
	VertexUVs     = 4,
	FaceGroups    = 8,
	All           = 15
};

/**
* FDynamicMesh3 is a dynamic triangle mesh class. The mesh has has connectivity,
* is an indexed mesh, and allows for gaps in the index space.
*
* internally, all data is stored in POD-type buffers, except for the vertex->edge
* links, which are stored as List<int>'s. The arrays of POD data are stored in
* TDynamicVector's, so they grow in chunks, which is relatively efficient. The actual
* blocks are arrays, so they can be efficiently mem-copied into larger buffers
* if necessary.
*
* Reference counts for verts/tris/edges are stored as separate FRefCountVector
* instances.
*
* Vertices are stored as doubles, although this should be easily changed
* if necessary, as the internal data structure is not exposed
*
* Per-vertex Vertex Normals, Colors, and UVs are optional and stored as floats.
* Note that in practice, these are generally only used as scratch space, in limited 
* circumstances, usually when needed for performance reasons. Most of our geometry
* code instead prefers to read attributes from the per-triangle AttributeSet accessed 
* via Attributes() (see TDynamicMeshOverlay for a description of the structure). For
* instance, an empty (but existing) attribute set will take precedence over non-empty
* vertex normals in much of our processing code.
*
* For each vertex, VertexEdgeLists[i] is the unordered list of connected edges. The
* elements of the list are indices into the edges list.
* This list is unsorted but can be traversed in-order (ie cw/ccw) at some additional cost.
*
* Triangles are stored as 3 ints, with optionally a per-triangle integer group id. 
* The group IDs stored here DO get widely used and preserved in our geometry code 
* (unlike the per-vertex attributes described earlier), even though the AttributeSet
* can store group IDs as well (potentially in multiple layers).
*
* The edges of a triangle are similarly stored as 3 ints, in triangle_edes. If the
* triangle is [v1,v2,v3], then the triangle edges [e1,e2,e3] are
* e1=edge(v1,v2), e2=edge(v2,v3), e3=edge(v3,v1), where the e# are indexes into edges.
*
* Edges are stored as tuples of 4 ints. If the edge is between v1 and v2, with neighbour
* tris t1 and t2, then the edge is [min(v1,v2), max(v1,v2), t1, t2]. For a boundary
* edge, t2 is InvalidID. t1 is never InvalidID.
*
* Most of the class assumes that the mesh is manifold. Many functions will
* work if the topology is non-manifold, but behavior of operators like Split/Flip/Collapse
* edge is untested.
*
* The function CheckValidity() does extensive sanity checking on the mesh data structure.
* Use this to test your code, both for mesh construction and editing!!
*/
class FDynamicMesh3
{

// TODO:
//  - Many of the iterators depend on lambda functions, can we replace these with calls to
//    internal/static functions that do the same thing?
//  - CompactInPlace() does not compact VertexEdgeLists

public:
	using FEdgeFlipInfo     = DynamicMeshInfo::FEdgeFlipInfo;
	using FEdgeSplitInfo    = DynamicMeshInfo::FEdgeSplitInfo;
	using FEdgeCollapseInfo = DynamicMeshInfo::FEdgeCollapseInfo;
	using FMergeEdgesInfo   = DynamicMeshInfo::FMergeEdgesInfo;
	using FMergeVerticesInfo = DynamicMeshInfo::FMergeVerticesInfo;
	using FPokeTriangleInfo = DynamicMeshInfo::FPokeTriangleInfo;
	using FVertexSplitInfo  = DynamicMeshInfo::FVertexSplitInfo;

	// Inline-allocator array types optionally used for mesh queries, to reduce heap allocations
	using FLocalIntArray = TArray<int32, TInlineAllocator<8>>;
	using FLocalBoolArray = TArray<bool, TInlineAllocator<8>>;

	struct FEdge
	{
		FIndex2i Vert;
		FIndex2i Tri;
		friend bool operator!=(const FEdge& e0, const FEdge& e1)
		{
			return (e0.Vert != e1.Vert) || (e0.Tri != e1.Tri);
		}

		/**
		 * Serialization operator for FEdge.
		 *
		 * @param Ar Archive to serialize with.
		 * @param E Edge to serialize.
		 * @returns Passing down serializing archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FEdge& E)
		{
			E.Serialize(Ar);
			return Ar;
		}

		/** Serialize FEdge to an archive. */
		void Serialize(FArchive& Ar)
		{
			Ar << Vert;
			Ar << Tri;
		}
	};
	/** InvalidID indicates that a vertex/edge/triangle ID is invalid */
	constexpr static int InvalidID = IndexConstants::InvalidID;
	/** NonManifoldID is returned by AppendTriangle() to indicate that the added triangle would result in nonmanifold geometry and hence was ignored */
	constexpr static int NonManifoldID = -2;
	/** DuplicateTriangleID is returned by AppendTriangle() to indicate that the added triangle already exists in the mesh, and was ignored because we do not support duplicate triangles */
	constexpr static int DuplicateTriangleID = -3;

	GEOMETRYCORE_API const static FVector3d InvalidVertex;
	constexpr static FIndex3i InvalidTriangle{InvalidID, InvalidID, InvalidID};
	constexpr static FIndex2i InvalidEdge{InvalidID, InvalidID};

protected:
	/** List of vertex positions */
	TDynamicVector<FVector3d> Vertices{};
	/** Reference counts of vertex indices. For vertices that exist, the count is 1 + num_triangle_using_vertex. Iterate over this to find out which vertex indices are valid. */
	FRefCountVector VertexRefCounts{};
	/** (optional) List of per-vertex normals */
	TOptional<TDynamicVector<FVector3f>> VertexNormals{};
	/** (optional) List of per-vertex colors */
	TOptional<TDynamicVector<FVector3f>> VertexColors{};
	/** (optional) List of per-vertex uv's */
	TOptional<TDynamicVector<FVector2f>> VertexUVs{};
	/** List of per-vertex edge one-rings */
	FSmallListSet VertexEdgeLists;

	/** List of triangle vertex-index triplets [Vert0 Vert1 Vert2]*/
	TDynamicVector<FIndex3i> Triangles;
	/** Reference counts of triangle indices. Ref count is always 1 if the triangle exists. Iterate over this to find out which triangle indices are valid. */
	FRefCountVector TriangleRefCounts;
	/** List of triangle edge triplets [Edge0 Edge1 Edge2] */
	TDynamicVector<FIndex3i> TriangleEdges;
	/** (optional) List of per-triangle group identifiers */
	TOptional<TDynamicVector<int>> TriangleGroups{};
	/** Upper bound on the triangle group IDs used in the mesh (may be larger than the actual maximum if triangles have been deleted) */
	int GroupIDCounter = 0;

	/** List of edge elements. An edge is four elements [VertA, VertB, Tri0, Tri1], where VertA < VertB, and Tri1 may be InvalidID (if the edge is a boundary edge) */
	TDynamicVector<FEdge> Edges;
	/** Reference counts of edge indices. Ref count is always 1 if the edge exists. Iterate over this to find out which edge indices are valid. */
	FRefCountVector EdgeRefCounts;

	/** Extended Attributes for the Mesh (UV layers, Hard Normals, additional Polygroup Layers, etc) */
	TUniquePtr<FDynamicMeshAttributeSet> AttributeSet{};

protected:
	UE_DEPRECATED(5.7, "bEnableShapeChangeStamp is no longer used. Use SetShapeChangeStampEnabled and HasShapeChangeStampEnabled instead.") 
	bool bEnableShapeChangeStamp = false;
	
	UE_DEPRECATED(5.7, "ShapeChangeStamp is no longer used. Use GetShapeChangeStamp or UpdateChangeStamps instead.") 
	std::atomic<uint32> ShapeChangeStamp = 0;
	UE_DEPRECATED(5.7, "TopologyChangeStamp is no longer used. Use GetTopologyChangeStamp or UpdateChangeStamps instead.") 
	std::atomic<uint32> TopologyChangeStamp = 0;

private:
	struct FChangeStamp
	{
		explicit FChangeStamp(bool bInIsEnabled)
			: bIsEnabled(bInIsEnabled)
		{
		}

		UE_NONCOPYABLE(FChangeStamp);

		/** Enable/disable this change stamp. */
		bool bIsEnabled = false;

		/** Guards `Value`. */
		mutable FTransactionallySafeMutex Mutex;

		/** The change stamp is incremented when modifications occur, if `bIsEnabled` is set. It's guarded by `Mutex`. */
		uint32 Value = 1;

		/** Updates the change stamp in an thread-safe, transactionally-safe way. Does nothing if `bIsEnabled` is not set. */
		void IncrementIfEnabled()
		{
			if (bIsEnabled)
			{
				Mutex.Lock();
				++Value;
				Mutex.Unlock();
			}
		}

		/** Overwrites the change stamp in an thread-safe, transactionally-safe way. Does nothing if `bIsEnabled` is not set. */
		void Set(uint32 NewValue)
		{
			if (bIsEnabled)
			{
				Mutex.Lock();
				Value = NewValue;
				Mutex.Unlock();
			}
		}

		/** Returns the current change value in a thread-safe, transactionally-safe way. Returns 1 if `bIsEnabled` is not set. */
		uint32 GetValue() const
		{
			uint32 Result = 1; 

			if (bIsEnabled)
			{
				Mutex.Lock();
				Result = Value;
				Mutex.Unlock();
			}

			return Result;
		}
	};

	// Shape change tracking can be problematic in multi-threaded contexts so they're disabled by default. 
	// In fact, it is not suggested that these be used at all. (See comment for SetShapeChangeStampEnabled.)
	FChangeStamp ChangeStampShape{/*bIsEnabled=*/ false};

	// Topological change tracking is enabled by default.
	FChangeStamp ChangeStampTopology{/*bIsEnabled=*/ true};

public:
	/** Default constructor */
	GEOMETRYCORE_API FDynamicMesh3();

	/** Copy/Move construction */
	GEOMETRYCORE_API FDynamicMesh3(const FDynamicMesh3& CopyMesh);
	GEOMETRYCORE_API FDynamicMesh3(FDynamicMesh3&& MoveMesh);

	/** Copy and move assignment */
	GEOMETRYCORE_API const FDynamicMesh3& operator=(const FDynamicMesh3& CopyMesh);
	GEOMETRYCORE_API const FDynamicMesh3& operator=(FDynamicMesh3&& MoveMesh);

	/** Destructor */
	GEOMETRYCORE_API virtual ~FDynamicMesh3();


	/** Construct an empty mesh with specified attributes */
	GEOMETRYCORE_API explicit FDynamicMesh3(bool bWantNormals, bool bWantColors, bool bWantUVs, bool bWantTriGroups);
	GEOMETRYCORE_API explicit FDynamicMesh3(EMeshComponents flags);

	/** Construction from Mesh Generator */
	GEOMETRYCORE_API FDynamicMesh3(const FMeshShapeGenerator* Generator);

	/** Set internal data structures to be a copy of input mesh using the specified attributes*/
	GEOMETRYCORE_API void Copy(const FDynamicMesh3& CopyMesh, bool bNormals = true, bool bColors = true, bool bUVs = true,
	          bool bAttributes = true);

	/** 
	  * Initialize mesh from the output of a MeshShapeGenerator (assumes Generate() was already called) 
	  * @return false if any triangles defined in Generator were skipped due to invalid topology
	  */
	GEOMETRYCORE_API bool Copy(const FMeshShapeGenerator* Generator);

	// Tracks how IDs are offset and number of elements appended by a mesh append operation
	struct FAppendInfo
	{
		// Offsets for base mesh element IDs
		int32 VertexOffset = 0, TriangleOffset = 0, EdgeOffset = 0, GroupOffset = 0;

		// Offsets for normals/tangents overlay element; will remain zero if the corresponding 'normal layer' is not present
		int32 NormalOverlayOffsets[3]{ 0,0,0 };

		// The number appended of each element type -- including 'invalid' slots, i.e. the amount by which MaxID increased
		int32 NumVertex = 0, NumTriangle = 0, NumEdge = 0;
	};

	/**
	 * Append the ToAppend mesh geometry to this mesh, preserving element IDs aside from a constant offset.
	 *
	 * Attributes from the ToAppend mesh will only be included if they already exist on this mesh. 
	 * If more attributes are needed, consider calling EnableMatchingAttributes.
	 * 
	 * Note that OutAppendInfo optionally provides some offsets for convenience, but callers may also compute the offset for any element type
	 * by storing the corresponding MaxID before calling AppendWithOffsets.
	 * 
	 * @param ToAppend Mesh to append
	 * @param OutAppendInfo Optionally stores offsets and number of IDs appended for mesh elements.
	 */
	GEOMETRYCORE_API void AppendWithOffsets(const FDynamicMesh3& ToAppend, FAppendInfo* OutAppendInfo = nullptr);

	/** 
	 * Copy input mesh while compacting, i.e. removing unused vertices/triangles/edges. 
	 * Note that it is currently unsafe to hold on to the attribute set pointer (returned from Attributes())
	 * across the CompactCopy call, as the latter may rebuild an entirely new attribute set.
	 * 
	 * @param CopyMesh Mesh to copy
	 * @param bNormals if true, will copy normals
	 * @param bColors if true, will copy colors
	 * @param bUVs if true, will copy UVs
	 * @param bAttributes if true, will copy attributes
	 * @param CompactInfo if not nullptr, will be filled with mapping indicating how vertex and triangle IDs were changed during compaction
	 */
	GEOMETRYCORE_API void CompactCopy(const FDynamicMesh3& CopyMesh, bool bNormals = true, bool bColors = true, bool bUVs = true,
	                 bool bAttributes = true, FCompactMaps* CompactInfo = nullptr);

	/** Discard all data */
	GEOMETRYCORE_API void Clear();

	/**
	 * Ensure that all the same extended attributes available in ToMatch are also enabled.
	 * By default, clears existing attributes, so that there will be an exact match
	 * If bClearExisting is passed as false, existing attributes are not removed/cleared.
	 * If bDiscardExtraAttributes=true and bClearExisting=false, extra attributes not in ToMatch are discarded, but existing attributes are not cleared/reset
	 */
	GEOMETRYCORE_API void EnableMatchingAttributes(const FDynamicMesh3& ToMatch, bool bClearExisting = true, bool bDiscardExtraAttributes = false);

	/**
	 * Serialization operator for FDynamicMesh3.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Mesh Mesh to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FDynamicMesh3& Mesh)
	{
		Mesh.Serialize(Ar);
		return Ar;
	}

	/** Serialize the mesh to an archive. */
	GEOMETRYCORE_API void Serialize(FArchive& Ar);

public:
	/** @return number of vertices in the mesh */
	int VertexCount() const
	{
		return (int)VertexRefCounts.GetCount();
	}
	/** @return number of triangles in the mesh */
	int TriangleCount() const
	{
		return (int)TriangleRefCounts.GetCount();
	}
	/** @return number of edges in the mesh */
	int EdgeCount() const
	{
		return (int)EdgeRefCounts.GetCount();
	}

	/** @return upper bound on vertex IDs used in the mesh, i.e. all vertex IDs in use are < MaxVertexID */
	int MaxVertexID() const
	{
		return (int)VertexRefCounts.GetMaxIndex();
	}
	/** @return upper bound on triangle IDs used in the mesh, i.e. all triangle IDs in use are < MaxTriangleID */
	int MaxTriangleID() const
	{
		return (int)TriangleRefCounts.GetMaxIndex();
	}
	/** @return upper bound on edge IDs used in the mesh, i.e. all edge IDs in use are < MaxEdgeID */
	int MaxEdgeID() const
	{
		return (int)EdgeRefCounts.GetMaxIndex();
	}
	/** @return upper bound on group IDs used in the mesh, i.e. all group IDs in use are < MaxGroupID */
	int MaxGroupID() const
	{
		return GroupIDCounter;
	}


	/** @return true if this mesh has per-vertex normals */
	bool HasVertexNormals() const
	{
		return VertexNormals.IsSet();
	}
	/** @return true if this mesh has per-vertex colors */
	bool HasVertexColors() const
	{
		return VertexColors.IsSet();
	}
	/** @return true if this mesh has per-vertex UVs */
	bool HasVertexUVs() const
	{
		return VertexUVs.IsSet();
	}
	/** @return true if this mesh has per-triangle groups */
	bool HasTriangleGroups() const
	{
		return TriangleGroups.IsSet();
	}
	/** @return true if this mesh has attribute layers */
	bool HasAttributes() const
	{
		return AttributeSet.IsValid();
	}

	/** @return bitwise-or of EMeshComponents flags specifying which extra data this mesh has */
	GEOMETRYCORE_API int GetComponentsFlags() const;


	/** @return true if VertexID is a valid vertex in this mesh */
	inline bool IsVertex(int VertexID) const
	{
		return VertexRefCounts.IsValid(VertexID);
	}
	/** @return true if VertexID is a valid vertex in this mesh AND is used by at least one triangle */
	inline bool IsReferencedVertex(int VertexID) const
	{
		return VertexID >= 0 && VertexID < (int)VertexRefCounts.GetMaxIndex() &&
		    VertexRefCounts.GetRefCount(VertexID) > 1;
	}
	/** @return true if TriangleID is a valid triangle in this mesh */
	inline bool IsTriangle(int TriangleID) const
	{
		return TriangleRefCounts.IsValid(TriangleID);
	}
	/** @return true if EdgeID is a valid edge in this mesh */
	inline bool IsEdge(int EdgeID) const
	{
		return EdgeRefCounts.IsValid(EdgeID);
	}




	//
	// Change Tracking support
	// 
	// Note: May someday be removed (see comment for SetShapeChangeStampEnabled) 
	// 
public:

	/** 
	 * Enable/Disable incrementing of the ShapeChangeStamp.
	 * 
	 * NOTE: Both change stamps are unreliable in many contexts, and may someday get removed.
	 * Specifically, they get reset on Clear(), copied to the destination rather than incremented
	 * in copies and moves, and do not track any changes via access to the buffers directly (through
	 * GetVerticesBuffer(), etc).
	 */
	void SetShapeChangeStampEnabled(bool bEnabled)
	{
		ChangeStampShape.bIsEnabled = bEnabled;
	}

	/** Enable/disable incrementing of the TopologyChangeStamp. */
	void SetTopologyChangeStampEnabled(bool bEnabled)
	{
		ChangeStampTopology.bIsEnabled = bEnabled;
	}

	/** @return true if shape ChangeStamp is enabled (disabled by default) */
	bool HasShapeChangeStampEnabled() const
	{
		return ChangeStampShape.bIsEnabled;
	}

	/** @return true if topology ChangeStamp is enabled (disabled by default) */
	bool HasTopologyChangeStampEnabled() const
	{
		return ChangeStampTopology.bIsEnabled;
	}

	/** Increment the specified ChangeStamps, if they are enabled. Thread-safe. */
	void UpdateChangeStamps(bool bShapeChange, bool bTopologyChange)
	{
		if (bShapeChange || bTopologyChange)
		{
			ChangeStampShape.IncrementIfEnabled();
		}
		if (bTopologyChange)
		{
			ChangeStampTopology.IncrementIfEnabled();
		}
	}

	/** 
	 * Returns the current ShapeChangeStamp. This is incremented any time a mesh vertex position is changed _or_ the mesh topology is modified. 
	 * Change stamps are disabled by default.
	 */
	inline uint32 GetShapeChangeStamp() const
	{
		ensureMsgf(ChangeStampShape.bIsEnabled, TEXT("Shape change tracking is not enabled on this mesh. Use SetShapeChangeStampEnabled() to enable."));
		return ChangeStampShape.GetValue();
	}

	/** 
	 * Returns the current TopologyChangeStamp. This is incremented when the mesh topology is modified. 
	 * Change stamps are disabled by default. 
	 */
	inline uint32 GetTopologyChangeStamp() const
	{
		ensureMsgf(ChangeStampTopology.bIsEnabled, TEXT("Topology change tracking is not enabled on this mesh. Use SetTopologyChangeStampEnabled() to enable."));
		return ChangeStampTopology.GetValue();
	}

	/** ChangeStamp is a combination of the Shape and Topology ChangeStamps. If neither flag is enabled, this value will never change. */
	inline uint64 GetChangeStamp() const
	{
		return ChangeStampShape.GetValue() + ChangeStampTopology.GetValue();
	}

	//
	// Mesh Element Iterators
	//   The functions VertexIndicesItr() / TriangleIndicesItr() / EdgeIndicesItr() allow you to do:
	//      for ( int eid : EdgeIndicesItr() ) { ... }
	//   and other related begin() / end() idioms
public:
	// simplify names for iterations
	typedef typename FRefCountVector::IndexEnumerable vertex_iterator;
	typedef typename FRefCountVector::IndexEnumerable triangle_iterator;
	typedef typename FRefCountVector::IndexEnumerable edge_iterator;
	template<typename T>
	using value_iteration          = FRefCountVector::MappedEnumerable<T>;
	using vtx_triangles_enumerable = TPairExpandEnumerable<FSmallListSet::ValueIterator>;

	/** @return enumerable object for valid vertex indices suitable for use with range-based for, ie for ( int i : VertexIndicesItr() ) */
	vertex_iterator VertexIndicesItr() const
	{
		return VertexRefCounts.Indices();
	}

	/** @return enumerable object for valid triangle indices suitable for use with range-based for, ie for ( int i : TriangleIndicesItr() ) */
	triangle_iterator TriangleIndicesItr() const
	{
		return TriangleRefCounts.Indices();
	}

	/** @return enumerable object for valid edge indices suitable for use with range-based for, ie for ( int i : EdgeIndicesItr() ) */
	edge_iterator EdgeIndicesItr() const
	{
		return EdgeRefCounts.Indices();
	}

	// TODO: write helper functions that allow us to do these iterations w/o lambdas

	/** @return enumerable object for boundary edge indices suitable for use with range-based for, ie for ( int i : BoundaryEdgeIndicesItr() ) */
	FRefCountVector::FilteredEnumerable BoundaryEdgeIndicesItr() const
	{
		return EdgeRefCounts.FilteredIndices([this](int EdgeID) { return Edges[EdgeID].Tri[1] == InvalidID; });
	}

	/** Enumerate positions of all vertices in mesh */
	value_iteration<FVector3d> VerticesItr() const
	{
		return VertexRefCounts.MappedIndices<FVector3d>([this](int VertexID) { return Vertices[VertexID]; });
	}

	/** Enumerate all triangles in the mesh */
	value_iteration<FIndex3i> TrianglesItr() const
	{
		return TriangleRefCounts.MappedIndices<FIndex3i>([this](int TriangleID) { return Triangles[TriangleID]; });
	}

	/** Enumerate edges. Each returned element is [v0,v1,t0,t1], where t1 will be InvalidID if this is a boundary edge */
	value_iteration<FEdge> EdgesItr() const
	{
		return EdgeRefCounts.MappedIndices<FEdge>([this](int EdgeID) {
			return Edges[EdgeID];
		});
	}

	/** @return enumerable object for one-ring vertex neighbours of a vertex, suitable for use with range-based for, ie for ( int i : VtxVerticesItr(VertexID) ) */
	FSmallListSet::MappedValueEnumerable VtxVerticesItr(int VertexID) const
	{
		checkSlow(VertexRefCounts.IsValid(VertexID));
		return VertexEdgeLists.MappedValues(VertexID,
			[VertexID, this](int eid) { return GetOtherEdgeVertex(eid, VertexID); });
	}

	/** Call VertexFunc for each one-ring vertex neighbour of a vertex. Currently this is more efficient than VtxVerticesItr() due to overhead in the Values() enumerable */
	void EnumerateVertexVertices(int32 VertexID, TFunctionRef<void(int32)> VertexFunc) const
	{
		checkSlow(VertexRefCounts.IsValid(VertexID));
		VertexEdgeLists.Enumerate(VertexID, [this, &VertexFunc, VertexID](int32 eid)
		{
			VertexFunc(GetOtherEdgeVertex(eid, VertexID));
		});
	}


	/** @return enumerable object for one-ring edges of a vertex, suitable for use with range-based for, ie for ( int i : VtxEdgesItr(VertexID) ) */
	FSmallListSet::ValueEnumerable VtxEdgesItr(int VertexID) const
	{
		checkSlow(VertexRefCounts.IsValid(VertexID));
		return VertexEdgeLists.Values(VertexID);
	}

	/** Call EdgeFunc for each one-ring edge of a vertex. Currently this is more efficient than VtxEdgesItr() due to overhead in the Values() enumerable */
	void EnumerateVertexEdges(int32 VertexID, TFunctionRef<void(int32)> EdgeFunc) const
	{
		checkSlow(VertexRefCounts.IsValid(VertexID));
		VertexEdgeLists.Enumerate(VertexID, EdgeFunc);
	}


	/** @return enumerable object for one-ring triangles of a vertex, suitable for use with range-based for, ie for ( int i : VtxTrianglesItr(VertexID) ) */
	vtx_triangles_enumerable VtxTrianglesItr(int VertexID) const
	{
		checkSlow(VertexRefCounts.IsValid(VertexID));
		return vtx_triangles_enumerable(VertexEdgeLists.Values(VertexID), [this, VertexID](int EdgeID) {
			return GetOrderedOneRingEdgeTris(VertexID, EdgeID);
		});
	}

	/** Call ApplyFunc for each one-ring triangle of a vertex. Currently this is significantly more efficient than VtxTrianglesItr() in many use cases. */
	GEOMETRYCORE_API void EnumerateVertexTriangles(int32 VertexID, TFunctionRef<void(int32)> ApplyFunc) const;

	/** @return a single triangle connected to the given vertex, or INDEX_NONE if the vertex has no triangles */
	GEOMETRYCORE_API int32 GetSingleVertexTriangle(int32 VID) const;

	/** Call ApplyFunc for each triangle connected to an Edge (1 or 2 triangles) */
	GEOMETRYCORE_API void EnumerateEdgeTriangles(int32 EdgeID, TFunctionRef<void(int32)> ApplyFunc) const;


	//
	// Mesh Construction
	//
public:
	/** Append vertex at position and other fields, returns vid */
	GEOMETRYCORE_API int AppendVertex(const FVertexInfo& VertInfo);

	/** Append vertex at position, returns vid */
	int AppendVertex(const FVector3d& Position)
	{
		return AppendVertex(FVertexInfo(Position));
	}

	/** Copy vertex SourceVertexID from existing SourceMesh, returns new vertex id */
	GEOMETRYCORE_API int AppendVertex(const FDynamicMesh3& SourceMesh, int SourceVertexID);

	/** TriVertices must be distinct and refer to existing, valid vertices */
	GEOMETRYCORE_API int AppendTriangle(const FIndex3i& TriVertices, int GroupID = 0);

	/** Vertex0, Vertex1, and Vertex2 must be distinct and refer to existing, valid vertices */
	inline int AppendTriangle(int Vertex0, int Vertex1, int Vertex2, int GroupID = 0)
	{
		return AppendTriangle(FIndex3i(Vertex0, Vertex1, Vertex2), GroupID);
	}

	//
	// Support for inserting vertex and triangle at specific IDs. This is a bit tricky
	// because we likely will need to update the free lists in the RefCountVectors, which
	// can be expensive. If you are going to do many inserts (eg inside a loop), wrap in
	// BeginUnsafe / EndUnsafe calls, and pass bUnsafe = true to the InsertX() calls, to
	// the defer free list rebuild until you are done.
	//

	/** Call this before a set of unsafe InsertVertex() calls */
	virtual void BeginUnsafeVerticesInsert()
	{
		// do nothing...
	}

	/** Call after a set of unsafe InsertVertex() calls to rebuild free list */
	virtual void EndUnsafeVerticesInsert()
	{
		VertexRefCounts.RebuildFreeList();
	}

	/**
	 * Insert vertex at given index, assuming it is unused.
	 * If bUnsafe, we use fast id allocation that does not update free list.
	 * You should only be using this between BeginUnsafeVerticesInsert() / EndUnsafeVerticesInsert() calls
	 */
	GEOMETRYCORE_API EMeshResult InsertVertex(int VertexID, const FVertexInfo& VertInfo, bool bUnsafe = false);

	/** Call this before a set of unsafe InsertTriangle() calls */
	virtual void BeginUnsafeTrianglesInsert()
	{
		// do nothing...
	}

	/** Call after a set of unsafe InsertTriangle() calls to rebuild free list */
	virtual void EndUnsafeTrianglesInsert()
	{
		TriangleRefCounts.RebuildFreeList();
	}

	/**
	 * Insert triangle at given index, assuming it is unused.
	 * If bUnsafe, we use fast id allocation that does not update free list.
	 * You should only be using this between BeginUnsafeTrianglesInsert() / EndUnsafeTrianglesInsert() calls
	 */
	GEOMETRYCORE_API EMeshResult InsertTriangle(int TriangleID, const FIndex3i& TriVertices, int GroupID = 0, bool bUnsafe = false);

	//
	// Vertex/Tri/Edge accessors
	//
public:
	/** @return the vertex position */
	inline FVector3d GetVertex(int VertexID) const
	{
		checkSlow(IsVertex(VertexID));
		return Vertices[VertexID];
	}

	/** @return the vertex position */
	inline const FVector3d& GetVertexRef(int VertexID) const
	{
		checkSlow(IsVertex(VertexID));
		return Vertices[VertexID];
	}

	/** 
	 * Set vertex position 
	 * @param bTrackChange if true, ShapeChangeStamp will be incremented (if enabled)
	 */
	inline void SetVertex(int VertexID, const FVector3d& vNewPos, bool bTrackChange = true)
	{
		checkSlow(VectorUtil::IsFinite(vNewPos));
		checkSlow(IsVertex(VertexID));
		if (VectorUtil::IsFinite(vNewPos))
		{
			Vertices[VertexID] = vNewPos;
			if (bTrackChange)
			{
				UpdateChangeStamps(true, false);
			}
		}
	}


	/** Get extended vertex information */
	GEOMETRYCORE_API bool GetVertex(int VertexID, FVertexInfo& VertInfo, bool bWantNormals, bool bWantColors, bool bWantUVs) const;

	/** Get all vertex information available */
	GEOMETRYCORE_API FVertexInfo GetVertexInfo(int VertexID) const;

	/** @return the valence of a vertex (the number of connected edges) */
	int GetVtxEdgeCount(int VertexID) const
	{
		return VertexRefCounts.IsValid(VertexID) ? VertexEdgeLists.GetCount(VertexID) : -1;
	}

	/** @return the max valence of all vertices in the mesh */
	GEOMETRYCORE_API int GetMaxVtxEdgeCount() const;

	/** Get triangle vertices */
	inline FIndex3i GetTriangle(int TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return Triangles[TriangleID];
	}

	/** Get triangle vertices */
	inline const FIndex3i& GetTriangleRef(int TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return Triangles[TriangleID];
	}

	/** Get triangle edges */
	inline FIndex3i GetTriEdges(int TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return TriangleEdges[TriangleID];
	}

	/** Get triangle edges */
	inline const FIndex3i& GetTriEdgesRef(int TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return TriangleEdges[TriangleID];
	}

	/** Get one of the edges of a triangle */
	inline int GetTriEdge(int TriangleID, int j) const
	{
		checkSlow(IsTriangle(TriangleID));
		return TriangleEdges[TriangleID][j];
	}

	/**  Applies a given function to both TriEdgeIDs which each EdgeID in a given Triangle is associated with */
	void EnumerateTriEdgeIDsFromTriID(const int TriID, const TFunctionRef<void(FMeshTriEdgeID TriEdgeID)>& TriEdgeFunc) const
	{
		FIndex3i TriEdges = GetTriEdges(TriID);
		for (int TriEdgesIndex = 0 ; TriEdgesIndex <=2; TriEdgesIndex++)
		{
			EnumerateTriEdgeIDsFromEdgeID(TriEdges[TriEdgesIndex], TriEdgeFunc);
		}
	}

	/** Find the neighbour triangles of a triangle (any of them might be InvalidID) */
	GEOMETRYCORE_API FIndex3i GetTriNeighbourTris(int TriangleID) const;

	/** Get the three vertex positions of a triangle */
	template<typename VecType>
	inline void GetTriVertices(int TriangleID, VecType& v0, VecType& v1, VecType& v2) const
	{
		const FIndex3i& Triangle = Triangles[TriangleID];
		v0                       = Vertices[Triangle[0]];
		v1                       = Vertices[Triangle[1]];
		v2                       = Vertices[Triangle[2]];
	}

	/** Get the position of one of the vertices of a triangle */
	inline FVector3d GetTriVertex(int TriangleID, int j) const
	{
		return Vertices[Triangles[TriangleID][j]];
	}

	/** Get the vertices and triangles of an edge, returned as [v0,v1,t0,t1], where t1 may be InvalidID */
	inline FEdge GetEdge(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID];
	}

	/** Get the vertices and triangles of an edge, returned as [v0,v1,t0,t1], where t1 may be InvalidID */
	inline const FEdge& GetEdgeRef(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID];
	}

	/** Get the vertex pair for an edge */
	inline FIndex2i GetEdgeV(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Vert;
	}

	/** Get the vertex positions of an edge */
	inline bool GetEdgeV(int EdgeID, FVector3d& a, FVector3d& b) const
	{
		checkSlow(IsEdge(EdgeID));

		const FIndex2i Verts = Edges[EdgeID].Vert;

		a = Vertices[Verts[0]];
		b = Vertices[Verts[1]];

		return true;
	}

	/** Get the triangle pair for an edge. The second triangle may be InvalidID */
	inline FIndex2i GetEdgeT(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Tri;
	}

	/** Return edge vertex indices, but oriented based on attached triangle (rather than min-sorted) */
	GEOMETRYCORE_API FIndex2i GetOrientedBoundaryEdgeV(int EdgeID) const;

	/** Return (triangle, edge_index) representation for given Edge ID */
	inline FMeshTriEdgeID GetTriEdgeIDFromEdgeID(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		int32 TriIndex = Edges[EdgeID].Tri.A;
		FIndex3i TriEdges = TriangleEdges[TriIndex];
		if (TriEdges.A == EdgeID)
		{
			return FMeshTriEdgeID(TriIndex, 0);
		}
		{ 
			return FMeshTriEdgeID(TriIndex, ( TriEdges.B == EdgeID ) ? 1 : 2 );
		}
	}

	/** Applies a given function to both TriEdgeIDs which a given EdgeID is associated with*/
	void EnumerateTriEdgeIDsFromEdgeID(const int32 EdgeID, const TFunctionRef<void(FMeshTriEdgeID TriEdgeID)>& TriEdgeFunc) const
	{
		const FMeshTriEdgeID FirstTriEdgeID = GetTriEdgeIDFromEdgeID(EdgeID); // function gets MeshTriEdgeID for edge included in EdgeTri.A only
		TriEdgeFunc(FirstTriEdgeID);

		// have to get MeshTriEdgeID for edge included in EdgeTri.B
		const int OtherTriID = GetEdgeT(EdgeID).B;
		if (OtherTriID != IndexConstants::InvalidID)
		{
			FMeshTriEdgeID SecondTriEdgeID;
			const FIndex3i SecondTriEdges = GetTriEdges(OtherTriID);
			if (SecondTriEdges.A == EdgeID)
			{
				SecondTriEdgeID = FMeshTriEdgeID(OtherTriID, 0);
			}
			else
			{
				SecondTriEdgeID = FMeshTriEdgeID(OtherTriID, ( SecondTriEdges.B == EdgeID ) ? 1 : 2 );
			}
			TriEdgeFunc(SecondTriEdgeID);
		}
	}

	//
	// Vertex and Triangle attribute arrays
	//
public:
	/**
	 * Enable requested set of mesh components (triangle groups and vertex normals/colors/UVs)
	 * and discard any that are not requested
	 * @param MeshComponentsFlags A 'bitwise or' of requested EMeshComponents flags
	 */
	GEOMETRYCORE_API void EnableMeshComponents(int MeshComponentsFlags);

	GEOMETRYCORE_API void EnableVertexNormals(const FVector3f& InitialNormal);
	GEOMETRYCORE_API void DiscardVertexNormals();

	FVector3f GetVertexNormal(int vID) const
	{
		if (HasVertexNormals() == false)
		{
			return FVector3f::UnitY();
		}
		checkSlow(IsVertex(vID));
		const TDynamicVector<FVector3f>& Normals = VertexNormals.GetValue();
		return Normals[vID];
	}

	void SetVertexNormal(int vID, const FVector3f& vNewNormal)
	{
		if (HasVertexNormals())
		{
			checkSlow(IsVertex(vID));
			TDynamicVector<FVector3f>& Normals = VertexNormals.GetValue();
			Normals[vID]                       = vNewNormal;
		}
	}

	GEOMETRYCORE_API void EnableVertexColors(const FVector3f& InitialColor);
	GEOMETRYCORE_API void DiscardVertexColors();


	FVector3f GetVertexColor(int vID) const
	{
		if (HasVertexColors() == false)
		{
			return FVector3f::One();
		}
		checkSlow(IsVertex(vID));

		const TDynamicVector<FVector3f>& Colors = VertexColors.GetValue();
		return Colors[vID];
	}

	void SetVertexColor(int vID, const FVector3f& vNewColor)
	{
		if (HasVertexColors())
		{
			checkSlow(IsVertex(vID));
			TDynamicVector<FVector3f>& Colors = VertexColors.GetValue();
			Colors[vID]                       = vNewColor;
		}
	}

	GEOMETRYCORE_API void EnableVertexUVs(const FVector2f& InitialUV);
	GEOMETRYCORE_API void DiscardVertexUVs();

	FVector2f GetVertexUV(int vID) const
	{
		if (HasVertexUVs() == false)
		{
			return FVector2f::Zero();
		}
		checkSlow(IsVertex(vID));
		const TDynamicVector<FVector2f>& UVs = VertexUVs.GetValue();
		return UVs[vID];
	}

	void SetVertexUV(int vID, const FVector2f& vNewUV)
	{
		if (HasVertexUVs())
		{
			checkSlow(IsVertex(vID));
			TDynamicVector<FVector2f>& UVs = VertexUVs.GetValue();
			UVs[vID]                       = vNewUV;
		}
	}

	GEOMETRYCORE_API void EnableTriangleGroups(int InitialGroupID = 0);
	GEOMETRYCORE_API void DiscardTriangleGroups();

	int AllocateTriangleGroup()
	{
		return GroupIDCounter++;
	}

	int GetTriangleGroup(int tID) const
	{
		return (HasTriangleGroups() == false) ? -1 :
		                                        (TriangleRefCounts.IsValid(tID) ? TriangleGroups.GetValue()[tID] : 0);
	}

	void SetTriangleGroup(int tid, int group_id)
	{
		if (HasTriangleGroups())
		{
			checkSlow(IsTriangle(tid));
			TriangleGroups.GetValue()[tid] = group_id;
			GroupIDCounter                 = FMath::Max(GroupIDCounter, group_id + 1);
		}
	}

	FDynamicMeshAttributeSet* Attributes()
	{
		return HasAttributes() ? AttributeSet.Get() : nullptr;
	}
	const FDynamicMeshAttributeSet* Attributes() const
	{
		return HasAttributes() ? AttributeSet.Get() : nullptr;
	}

	GEOMETRYCORE_API void EnableAttributes();
	GEOMETRYCORE_API void DiscardAttributes();


	//
	// topological queries
	//
public:
	/** Returns true if edge is on the mesh boundary, ie only connected to one triangle */
	inline bool IsBoundaryEdge(int EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Tri[1] == InvalidID;
	}

	/** Returns true if the vertex is part of any boundary edges */
	GEOMETRYCORE_API bool IsBoundaryVertex(int VertexID) const;

	/** Returns true if any edge of triangle is a boundary edge */
	GEOMETRYCORE_API bool IsBoundaryTriangle(int TriangleID) const;

	/** Find id of edge connecting A and B */
	GEOMETRYCORE_API int FindEdge(int VertexA, int VertexB) const;

	/** Find edgeid for edge [a,b] from triangle that contains the edge. Faster than FindEdge() because it is constant-time. */
	GEOMETRYCORE_API int FindEdgeFromTri(int VertexA, int VertexB, int TriangleID) const;

	/** Find edgeid for edge connecting two triangles */
	GEOMETRYCORE_API int FindEdgeFromTriPair(int TriangleA, int TriangleB) const;

	/** Find triangle made up of any permutation of vertices [a,b,c] */
	GEOMETRYCORE_API int FindTriangle(int A, int B, int C) const;

	/**
	 * If edge has vertices [a,b], and is connected two triangles [a,b,c] and [a,b,d],
	 * this returns [c,d], or [c,InvalidID] for a boundary edge
	 */
	GEOMETRYCORE_API FIndex2i GetEdgeOpposingV(int EdgeID) const;

	/**
	 * Given an edge and vertex on that edge, returns other vertex of edge, the two opposing verts, and the two connected triangles (OppVert2Out and Tri2Out are be InvalidID for boundary edge)
	 */
	GEOMETRYCORE_API void GetVtxNbrhood(int EdgeID, int VertexID, int& OtherVertOut, int& OppVert1Out, int& OppVert2Out, int& Tri1Out,
	                   int& Tri2Out) const;

	/**
	 * Returns count of boundary edges at vertex, and the first two boundary
	 * edges if found. If return is > 2, call GetAllVtxBoundaryEdges
	 */
	GEOMETRYCORE_API int GetVtxBoundaryEdges(int VertexID, int& Edge0Out, int& Edge1Out) const;

	/**
	 * Find edge ids of boundary edges connected to vertex.
	 * @param vID Vertex ID
	 * @param EdgeListOut boundary edge IDs are appended to this list
	 * @return count of number of elements of e that were filled
	 * Note: ArrayType must by TArray<int> or FLocalIntArray
	 */
	template<typename ArrayType = FLocalIntArray>
	GEOMETRYCORE_API int GetAllVtxBoundaryEdges(int VertexID, ArrayType& EdgeListOut) const;

	/**
	 * return # of triangles attached to vID, or -1 if invalid vertex
	 */
	GEOMETRYCORE_API int GetVtxTriangleCount(int VertexID) const;

	/**
	 * Get triangle one-ring at vertex.
	 * Note: ArrayType must by TArray<int> or FLocalIntArray
	 */
	template<typename ArrayType = FLocalIntArray>
	GEOMETRYCORE_API EMeshResult GetVtxTriangles(int VertexID, ArrayType& TrianglesOut) const;
	

	/**
	 * @return Triangle ID for a single triangle connected to VertexID, or InvalidID if VertexID does not exist or has no attached triangles
	 */
	GEOMETRYCORE_API int GetVtxSingleTriangle(int VertexID) const;

	/**
	* Get triangles connected to vertex in contiguous order, with multiple groups if vertex is a bowtie.
	* @param VertexID Vertex ID to search around
	* @param TrianglesOut All triangles connected to the vertex, in contiguous order; if there are multiple contiguous groups they are packed one after another
	* @param ContiguousGroupLengths Lengths of contiguous groups packed into TrianglesOut (if not a bowtie, this will just be a length-one array w/ {TrianglesOut.Num()})
	* @param GroupIsLoop Indicates whether each contiguous group is a loop (first triangle connected to last) or not
	* Note: ArrayTypes must by TArray<int>/<bool> or FLocalIntArray/FLocalBoolArry
	*/
	template<typename IntArrayType = FLocalIntArray, typename BoolArrayType = FLocalBoolArray>
	GEOMETRYCORE_API EMeshResult GetVtxContiguousTriangles(int VertexID, IntArrayType& TrianglesOut, IntArrayType& ContiguousGroupLengths,
	                                      BoolArrayType& GroupIsLoop) const;

	/** Returns true if the two triangles connected to edge have different group IDs */
	GEOMETRYCORE_API bool IsGroupBoundaryEdge(int EdgeID) const;

	/** Returns true if vertex has more than one tri group in its tri nbrhood */
	GEOMETRYCORE_API bool IsGroupBoundaryVertex(int VertexID) const;

	/** Returns true if more than two group boundary edges meet at vertex (ie 3+ groups meet at this vertex) */
	GEOMETRYCORE_API bool IsGroupJunctionVertex(int VertexID) const;

	/** Returns up to 4 group IDs at vertex. Returns false if > 4 encountered */
	GEOMETRYCORE_API bool GetVertexGroups(int VertexID, FIndex4i& GroupsOut) const;

	/** Returns all group IDs at vertex. ArrayType must by TArray<int> or FLocalIntArray */
	template<typename ArrayType = FLocalIntArray>
	GEOMETRYCORE_API bool GetAllVertexGroups(int VertexID, ArrayType& GroupsOut) const;

	/** returns true if vID is a "bowtie" vertex, ie multiple disjoint triangle sets in one-ring */
	GEOMETRYCORE_API bool IsBowtieVertex(int VertexID) const;

	/** returns true if vertices, edges, and triangles are all dense (Count == MaxID) **/
	bool IsCompact() const
	{
		return VertexRefCounts.IsDense() && EdgeRefCounts.IsDense() && TriangleRefCounts.IsDense();
	}

	/** @return true if vertex count == max vertex id */
	bool IsCompactV() const
	{
		return VertexRefCounts.IsDense();
	}

	/** @return true if triangle count == max triangle id */
	bool IsCompactT() const
	{
		return TriangleRefCounts.IsDense();
	}

	/** returns measure of compactness in range [0,1], where 1 is fully compacted */
	double CompactMetric() const
	{
		return ((double)VertexCount() / (double)MaxVertexID() + (double)TriangleCount() / (double)MaxTriangleID()) * 0.5;
	}

	/** @return true if mesh has no boundary edges */
	GEOMETRYCORE_API bool IsClosed() const;


	//
	// Geometric queries
	//
public:
	/** Returns bounding box of all mesh vertices (including unreferenced vertices) */
	GEOMETRYCORE_API FAxisAlignedBox3d GetBounds(bool bParallel = false) const;

	/** Returns bounding box of all selected mesh vertices. Will use a chunked parallel implementation for larger selections. */
	GEOMETRYCORE_API FAxisAlignedBox3d GetBoundsForVertexSelection(TConstArrayView<int32> VertexIDs) const;

	/** Returns bounding box of all selected mesh triangles. Will use a chunked parallel implementation for larger selections. */
	GEOMETRYCORE_API FAxisAlignedBox3d GetBoundsForTriangleSelection(TConstArrayView<int32> TriangleIDs) const;

	/**
	 * Compute a normal/tangent frame at vertex that is "stable" as long as
	 * the mesh topology doesn't change, meaning that one axis of the frame
	 * will be computed from projection of outgoing edge. 
	 * By default, frame.Z is normal, and .X points along mesh edge.
	 * Requires that VertexNormals are available, or UseNormal != nullptr, otherwise returns default Frame3d
	 * @param bFrameNormalY if true, then frame.Y is normal (X still points along mesh edge)
	 * @param UseNormal if defined, this normal is used instead of VertexNormals normal
	 */
	GEOMETRYCORE_API FFrame3d GetVertexFrame(int VertexID, bool bFrameNormalY = false, FVector3d* UseNormal = nullptr) const;

	/** Calculate face normal of triangle */
	GEOMETRYCORE_API FVector3d GetTriNormal(int TriangleID) const;

	/** Calculate area triangle */
	GEOMETRYCORE_API double GetTriArea(int TriangleID) const;

	/**
	 * Compute triangle normal, area, and centroid all at once. Re-uses vertex
	 * lookups and computes normal & area simultaneously. *However* does not produce
	 * the same normal/area as separate calls, because of this.
	 */
	GEOMETRYCORE_API void GetTriInfo(int TriangleID, FVector3d& Normal, double& Area, FVector3d& Centroid) const;

	/** Compute centroid of triangle */
	GEOMETRYCORE_API FVector3d GetTriCentroid(int TriangleID) const;

	/** Interpolate vertex positions of triangle using barycentric coordinates */
	GEOMETRYCORE_API FVector3d GetTriBaryPoint(int TriangleID, double Bary0, double Bary1, double Bary2) const;

	/** Interpolate vertex normals of triangle using barycentric coordinates */
	GEOMETRYCORE_API FVector3d GetTriBaryNormal(int TriangleID, double Bary0, double Bary1, double Bary2) const;

	/** Compute interpolated vertex attributes at point of triangle */
	GEOMETRYCORE_API void GetTriBaryPoint(int TriangleID, double Bary0, double Bary1, double Bary2, FVertexInfo& VertInfo) const;

	/** Construct bounding box of triangle as efficiently as possible */
	GEOMETRYCORE_API FAxisAlignedBox3d GetTriBounds(int TriangleID) const;

	/** Construct stable frame at triangle centroid, where frame.Z is face normal, and frame.X is aligned with edge nEdge of triangle. */
	GEOMETRYCORE_API FFrame3d GetTriFrame(int TriangleID, int Edge = 0) const;

	/** Compute solid angle of oriented triangle tID relative to point p - see WindingNumber() */
	GEOMETRYCORE_API double GetTriSolidAngle(int TriangleID, const FVector3d& p) const;

	/** Compute internal angle at vertex i of triangle (where i is 0,1,2); */
	GEOMETRYCORE_API double GetTriInternalAngleR(int TriangleID, int i) const;

	/** Compute internal angles at all vertices of triangle */
	GEOMETRYCORE_API FVector3d GetTriInternalAnglesR(int TriangleID) const;

	/** Returns average normal of connected face normals */
	GEOMETRYCORE_API FVector3d GetEdgeNormal(int EdgeID) const;

	/** Get point along edge, t clamped to range [0,1] */
	GEOMETRYCORE_API FVector3d GetEdgePoint(int EdgeID, double ParameterT) const;

	/**
	 * Fastest possible one-ring centroid. This is used inside many other algorithms
	 * so it helps to have it be maximally efficient
	 */
	GEOMETRYCORE_API void GetVtxOneRingCentroid(int VertexID, FVector3d& CentroidOut) const;

	/**
	 * Compute mesh winding number, from Jacobson et. al., Robust Inside-Outside Segmentation using Generalized Winding Numbers
	 * http://igl.ethz.ch/projects/winding-number/
	 * returns ~0 for points outside a closed, consistently oriented mesh, and a positive or negative integer
	 * for points inside, with value > 1 depending on how many "times" the point inside the mesh (like in 2D polygon winding)
	 */
	GEOMETRYCORE_API double CalculateWindingNumber(const FVector3d& QueryPoint) const;

	//
	// direct buffer access
	//
public:
	const TDynamicVector<FVector3d>& GetVerticesBuffer() const 
	{
		return Vertices;
	}
	const FRefCountVector& GetVerticesRefCounts() const 
	{
		return VertexRefCounts;
	}
	const TDynamicVector<FVector3f>* GetNormalsBuffer() const 
	{
		return HasVertexNormals() ? &VertexNormals.GetValue() : nullptr;
	}
	const TDynamicVector<FVector3f>* GetColorsBuffer() const 
	{
		return HasVertexColors() ? &VertexColors.GetValue() : nullptr;
	}
	const TDynamicVector<FVector2f>* GetUVBuffer() const 
	{
		return HasVertexUVs() ? &VertexUVs.GetValue() : nullptr;
	}
	const TDynamicVector<FIndex3i>& GetTrianglesBuffer() const 
	{
		return Triangles;
	}
	const FRefCountVector& GetTrianglesRefCounts() const
	{
		return TriangleRefCounts;
	}
	const TDynamicVector<int>* GetTriangleGroupsBuffer() const
	{
		return HasTriangleGroups() ? &TriangleGroups.GetValue() : nullptr;
	}
	const TDynamicVector<FEdge>& GetEdgesBuffer() const 
	{
		return Edges;
	}
	const FRefCountVector& GetEdgesRefCounts() const 
	{
		return EdgeRefCounts;
	}
	const FSmallListSet& GetVertexEdges() const 
	{
		return VertexEdgeLists;
	}
	const TDynamicVector<FIndex3i>& GetTriangleEdges() const 
	{
		return TriangleEdges;
	}
	//
	// Mesh Edit operations
	//
public:
	/**
	 * Compact mesh in-place, by moving vertices around and rewriting indices.
	 * Should be faster if the amount of compacting is not too significant, and is useful in some places.
	 *
	 * @param CompactInfo if not nullptr, will be filled with mapping indicating how vertex and triangle IDs were changed during compaction
	 */
	GEOMETRYCORE_API void CompactInPlace(FCompactMaps* CompactInfo = nullptr);

	/**
	 * Remove unused vertices. Note: Does not compact the remaining vertices.
	 * @return number of removed vertices
	 */
	GEOMETRYCORE_API int32 RemoveUnusedVertices();

	/**
	 * @return true if any vertices are unused (not in any triangles)
	 * */
	GEOMETRYCORE_API bool HasUnusedVertices() const;

	/**
	 * Reverse the ccw/cw orientation of all triangles in the mesh, and
	 * optionally flip the vertex normals if they exist
	 */
	GEOMETRYCORE_API void ReverseOrientation(bool bFlipNormals = true);

	/**
	 * Reverse the ccw/cw orientation of a triangle
	 */
	GEOMETRYCORE_API EMeshResult ReverseTriOrientation(int TriangleID);

	/**
	 * Remove vertex VertexID and all connected triangles. 
	 * Returns Failed_VertexStillReferenced if VertexID is still referenced by any triangles.
	 * If bPreserveManifold is true, checks that we will not create a bowtie vertex first.
	 * In this case, returns Failed_WouldCreateBowtie if removing the triangles would create a bowtie.
	 */
	GEOMETRYCORE_API EMeshResult RemoveVertex(int VertexID, bool bPreserveManifold = false);

	/**
	* Remove a triangle from the mesh. Also removes any unreferenced edges after tri is removed.
	* If bRemoveIsolatedVertices is true, then if you remove all tris from a vert, that vert is also removed.
	* If bPreserveManifold, we check that you will not create a bow tie vertex (and return false).
	* If this check is not done, you have to make sure you don't create a bow tie, because other
	* code assumes we don't have bow ties, and will not handle it properly
	*/
	GEOMETRYCORE_API EMeshResult RemoveTriangle(int TriangleID, bool bRemoveIsolatedVertices = true, bool bPreserveManifold = false);

	/**
	 * Rewrite the triangle to reference the new tuple of vertices.
	 *
	 * @todo this function currently does not guarantee that the returned mesh is well-formed. Only call if you know it's OK.
	 */
	GEOMETRYCORE_API virtual EMeshResult SetTriangle(int TriangleID, const FIndex3i& NewVertices, bool bRemoveIsolatedVertices = true);

	/**
	 * Split an edge of the mesh by inserting a vertex. This creates a new triangle on either side of the edge (ie a 2-4 split).
	 * If the original edge had vertices [a,b], with triangles t0=[a,b,c] and t1=[b,a,d],  then the split inserts new vertex f.
	 * After the split t0=[a,f,c] and t1=[f,a,d], and we have t2=[f,b,c] and t3=[f,d,b]  (it's best to draw it out on paper...)
	 * 
	 * @param EdgeAB index of the edge to be split
	 * @param SplitInfo returned information about new and modified mesh elements
	 * @param SplitParameterT defines the position along the edge that we split at, must be between 0 and 1, and is assumed to be based on the order of vertices returned by GetEdgeV()
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult SplitEdge(int EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT = 0.5);

	/** 
	 * Splits the edge between two vertices at the midpoint, if this edge exists 
	 * @param EdgeVertA index of first vertex
	 * @param EdgeVertB index of second vertex
	 * @param SplitInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API EMeshResult SplitEdge(int EdgeVertA, int EdgeVertB, FEdgeSplitInfo& SplitInfo);

	/**
	 * Flip/Rotate an edge of the mesh. This does not change the number of edges, vertices, or triangles.
	 * Boundary edges of the mesh cannot be flipped.
	 * 
	 * On success, the edges of the new triangles are ordered such that the shared edge is on index 0. i.e.,
	 * GetTriEdge(FlipInfo.Triangles[0], 0) == GetTriEdge(FlipInfo.Triangles[1], 0)
	 * 
	 * @param EdgeAB index of edge to be flipped
	 * @param FlipInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult FlipEdge(int EdgeAB, FEdgeFlipInfo& FlipInfo);

	/** calls FlipEdge() on the edge between two vertices, if it exists
	 * @param EdgeVertA index of first vertex
	 * @param EdgeVertB index of second vertex
	 * @param FlipInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult FlipEdge(int EdgeVertA, int EdgeVertB, FEdgeFlipInfo& FlipInfo);


	/**
	 * Clones the given vertex and updates any provided triangles to use the new vertex if/where they used the old one.
	 * @param VertexID the vertex to split
	 * @param TrianglesToUpdate triangles that should be updated to use the new vertex anywhere they previously had the old one
	 * @param SplitInfo returned info about the new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult SplitVertex(int VertexID, const TArrayView<const int>& TrianglesToUpdate,
	                                FVertexSplitInfo& SplitInfo);

	/**
	 * Tests whether splitting the given vertex with the given triangles would leave no triangles attached to the original vertex (creating an isolated vertex)
	 * @param VertexID the vertex to split
	 * @param TrianglesToUpdate triangles that should be updated to use the new vertex anywhere they previously had the old one
	 * @return true if calling SplitVertex with these arguments would leave an isolated vertex at the original VertexID
	 */
	GEOMETRYCORE_API virtual bool SplitVertexWouldLeaveIsolated(int VertexID, const TArrayView<const int>& TrianglesToUpdate);

	struct FCollapseEdgeOptions
	{
		/**
		 * When false, collapse is disallowed if the edge is the boundary of a single triangle hole,
		 *  such that collapsing it would clse the hole. I.e. collapse is dissallowed if there
		 *  is some vertex that connects vKeep and vRemove that is not part of the triangle(s) being
		 *  collapsed (note that if such a vertex is connected by a non-boundary edge, collapse will
		 *  always be disallowed regardless of bAllowHoleCollapse, as it would create non-manifold geometry).
		 */ 
		bool bAllowHoleCollapse = false;
		/**
		 * When false, collapse is disallowed if the edge is an interior edge, yet both vertices are
		 *  connected to boundary edges. In some circumstances this could create a bowtie. In other
		 *  cases, it could disconnected parts of a mesh that were connected by a bowtie.
		 */
		bool bAllowCollapsingInternalEdgeWithBoundaryVertices = false;
		/**
		 * When false, collapse is disallowed if we are collapsing the side of a tetrahedron. Note
		 *  that a base edge of an open-base tetrahedron could be collapsed even if bAllowTetrahedronCollapse
		 *  is false if bAllowHoleCollapse is true.
		 */
		bool bAllowTetrahedronCollapse = false;
	};
	GEOMETRYCORE_API virtual EMeshResult CanCollapseEdge(int vKeep, int vRemove, const FCollapseEdgeOptions& Options) const;

	/**
	 * Tests whether collapsing the specified edge using the CollapseEdge function would succeed.
	 *  Equivalent to calling the options overload with default options.
	 * @param KeepVertID index of the vertex that should be kept
	 * @param RemoveVertID index of the vertex that should be removed
	 * @param EdgeParameterT vKeep is moved to Lerp(KeepPos, RemovePos, EdgeParameterT). Note: Does not currently affect whether the edge is collapsable.
	 * @return Ok if the edge can be collapsed, or enum value indicating why the operation cannot be applied
	 */
	GEOMETRYCORE_API virtual EMeshResult CanCollapseEdge(int vKeep, int vRemove, double EdgeParameterT = 0) const;

	/**
	 * Collapse the edge between the two vertices, if topologically possible.
	 * @param KeepVertID index of the vertex that should be kept
	 * @param RemoveVertID index of the vertex that should be removed
	 * @param EdgeParameterT vKeep is moved to Lerp(KeepPos, RemovePos, EdgeParameterT)
	 * @param Options Sets options for the collapse
	 * @param CollapseInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult CollapseEdge(int KeepVertID, int RemoveVertID, double EdgeParameterT, 
		const FCollapseEdgeOptions& Options, FEdgeCollapseInfo& CollapseInfo);

	/**
	 * Collapse the edge between the two vertices, if topologically possible. Equivalent to
	 *  using the other overload with 0 for EdgeParameterT.
	 */
	virtual EMeshResult CollapseEdge(int KeepVertID, int RemoveVertID, const FCollapseEdgeOptions& Options, FEdgeCollapseInfo& CollapseInfo)
	{
		return CollapseEdge(KeepVertID, RemoveVertID, 0, Options, CollapseInfo);
	}

	/**
	 * Collapse the edge between the two vertices, if topologically possible. Equivalent to calling
	 *  the options overload with default options.
	 * @param KeepVertID index of the vertex that should be kept
	 * @param RemoveVertID index of the vertex that should be removed
	 * @param EdgeParameterT vKeep is moved to Lerp(KeepPos, RemovePos, EdgeParameterT)
	 * @param CollapseInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult CollapseEdge(int KeepVertID, int RemoveVertID, double EdgeParameterT,
	                                 FEdgeCollapseInfo& CollapseInfo);
	/**
	 * Collapse the edge between the two vertices, if topologically possible. Equivalent to calling
	 *  the options overload with default options and using 0 for EdgeParameterT.
	 */
	virtual EMeshResult CollapseEdge(int KeepVertID, int RemoveVertID, FEdgeCollapseInfo& CollapseInfo)
	{
		return CollapseEdge(KeepVertID, RemoveVertID, 0, CollapseInfo);
	}


	/**
	 * Given two edges of the mesh, weld both their vertices, so that one edge is removed.
	 * This could result in one neighbour edge-pair attached to each vertex also collapsing,
	 * so those cases are detected and handled (eg middle edge-pair in abysmal ascii drawing below)
	 *
	 *   ._._._.    (dots are vertices)
	 *    \._./
	 *
	 * @param KeepEdgeID index of the edge that should be kept
	 * @param DiscardEdgeID index of the edge that should be removed
	 * @param InterpolationT each kept vertex is moved to Lerp(KeptPos, RemovePos, InterpolationT)
	 * @param MergeInfo returned information about new and modified mesh elements
	 * @param CheckValidOrientation perform edge consistency orientation checks before merging. Specifically,
	 *  check that each discarded vertex is closer to the vertex it is being collapsed to than the other
	 *  kept vertex (where the pairing is determined by the adjoining triangle winding). 
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult MergeEdges(int KeepEdgeID, int DiscardEdgeID, double InterpolationT, FMergeEdgesInfo& MergeInfo, bool bCheckValidOrientation = true);
	
	/**
	 * Weld one edge to the other. Equivalent to calling the other overload with 0 for InterpolationT
	 *  (i.e. the vertices stay at unmodified kept vertex positions).
	 */
	GEOMETRYCORE_API virtual EMeshResult MergeEdges(int KeepEdgeID, int DiscardEdgeID, FMergeEdgesInfo& MergeInfo, bool bCheckValidOrientation=true);

	struct FMergeVerticesOptions
	{
		// If false, we disallow vertex merges that attempt to merge one non-boundary vert to a
		//  a non-adjacent vert, even if this is possible through a bowtie. Note that merging
		//  boundary verts to create a bowtie on the boundary is still allowed, as this is a 
		//  common intermediate step when welding edges. 
		bool bAllowNonBoundaryBowtieCreation = false;
	};

	/**
	 * Weld DiscardVid to KeepVid, if topologically possible and options allow. If the vertices are connected 
	 *  by an existing edge, this resolves as a collapse of that edge, and therefore calls OnCollapseEdge in
	 *  overlays. If not, but the two vertices share a vertex neighbor, this resolves as a weld of the intervening 
	 *  edges (failing if the edges are not boundary edges, since that would create non-manifold edge), and therefore
	 *  calls OnMergeEdges in the overlays. Otherwise, the merge resolves as bowtie creation, and calls OnMergeVertices
	 *  in the overlays.
	 * @param KeepVid vertex ID of the kept vertex
	 * @param DiscardVid vertex ID of the vertex whose triangles are reattached to the kept vertex
	 * @param InterpolationT the kept vertex is moved to Lerp(KeptPos, RemovePos, InterpolationT)
	 * @param Options set the options for the merge
	 * @param MergeInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult MergeVertices(int KeepVid, int DiscardVid, double InterpolationT, 
		const FMergeVerticesOptions& Options, FMergeVerticesInfo& MergeInfo);

	/**
	 * Weld DiscardVid to KeepVid. Equivalent to calling the options overload with default options.
	 */
	virtual EMeshResult MergeVertices(int KeepVid, int DiscardVid, double InterpolationT, FMergeVerticesInfo& MergeInfo)
	{
		return MergeVertices(KeepVid, DiscardVid, InterpolationT, FMergeVerticesOptions(), MergeInfo);
	}

	/**
	 * Weld DiscardVid to KeepVid. Equivalent to calling the options overload with default options and 0 for InterpolationT.
	 */
	virtual EMeshResult MergeVertices(int KeepVid, int DiscardVid, FMergeVerticesInfo& MergeInfo)
	{
		return MergeVertices(KeepVid, DiscardVid, 0, FMergeVerticesOptions(), MergeInfo);
	}

	/**
	 * Insert a new vertex inside a triangle, ie do a 1 to 3 triangle split
	 * @param TriangleID index of triangle to poke
	 * @param BaryCoordinates barycentric coordinates of poke position
	 * @param PokeInfo returned information about new and modified mesh elements
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	GEOMETRYCORE_API virtual EMeshResult PokeTriangle(int TriangleID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo);

	/** Call PokeTriangle at the centroid of the triangle */
	virtual EMeshResult PokeTriangle(int TriangleID, FPokeTriangleInfo& PokeInfo)
	{
		return PokeTriangle(TriangleID, FVector3d::One() / 3.0, PokeInfo);
	}

	//
	// Debug utility functions
	//
public:
	/**
	 * Returns a debug string that contains mesh statistics and other information
	 */
	GEOMETRYCORE_API virtual FString MeshInfoString() const;
	
	GEOMETRYCORE_API SIZE_T GetByteCount() const;

	/**
	 * Options for the IsSameAs check
	 */
	struct FSameAsOptions
	{
		bool bCheckConnectivity = true;
		bool bCheckEdgeIDs = false;
		bool bCheckNormals = false;
		bool bCheckColors = false;
		bool bCheckUVs = false;
		bool bCheckGroups = false;
		bool bCheckAttributes = false;

		// Ignore gaps and padding in the data layout, i.e. the exact index is irrelevant as long as valid entries are equal in value and in the same order.
		bool bIgnoreDataLayout = false;

		// comparisons check for smaller or equal than this value. used on on 32-bit floating point checks
		float EpsilonFloat { TMathUtil<float>::Epsilon };

		// comparisons check for smaller or equal than this value. used for 64-bit comparisons, such as on points.
		double EpsilonDouble { static_cast<double>(TMathUtil<float>::Epsilon) };
	};

	struct FMeshDifferenceInfo
	{
		// Reasons for difference between meshes
		enum class EReason
		{
			Unknown,
			VertexCount,
			TriangleCount,
			EdgeCount,
			Vertex,
			Triangle,
			Edge,
			Connectivity,
			Normal,
			Color,
			UV,
			Group,
			Attribute
		};
		EReason Reason = EReason::Unknown;

		// May contain further detail on the difference
		FString Detail;

		// Types of element ID that could be set below
		enum class EIDType
		{
			None,
			Vertex,
			Triangle,
			Edge
		};

		// ID in this mesh of element where the difference was found, or InvalidID if not applicable
		int32 ID = InvalidID;
		// ID in the compared-against mesh where the difference was found, or InvalidID if not applicable
		int32 OtherID = InvalidID;
		// Type of element that the ID references
		EIDType IDType = EIDType::None;

		// Helpers to set different ID types
		void SetVID(int32 VID, int32 OtherVID = InvalidID)
		{
			ID = VID;
			OtherID = OtherVID;
			IDType = EIDType::Vertex;
		}
		void SetTID(int32 TID, int32 OtherTID = InvalidID)
		{
			ID = TID;
			OtherID = OtherTID;
			IDType = EIDType::Triangle;
		}
		void SetEID(int32 EID, int32 OtherEID = InvalidID)
		{
			ID = EID;
			OtherID = OtherEID;
			IDType = EIDType::Edge;
		}
	};

	/**
	 * Check if another mesh is the same as this mesh. By default only checks
	 * vertices and triangles, turn on other parameters w/ flags
	 */
	GEOMETRYCORE_API virtual bool IsSameAs(const FDynamicMesh3& OtherMesh, const FSameAsOptions& Options) const;
	/**
	 * Check if another mesh is the same as this mesh. By default only checks
	 * vertices and triangles, turn on other parameters w/ flags
	 * @param OutMeshDifferenceInfo If the meshes are not the same, this struct may provide additional info as to the source of difference. Note it will only indicate the first difference found, not a complete report of differences.
	 */
	GEOMETRYCORE_API virtual bool IsSameAs(const FDynamicMesh3& OtherMesh, const FSameAsOptions& Options, FMeshDifferenceInfo& OutMeshDifferenceInfo) const;
private:
	GEOMETRYCORE_API bool IsSameAs_Helper(const FDynamicMesh3& OtherMesh, const FSameAsOptions& Options, FMeshDifferenceInfo* OutMeshDifferenceInfo = nullptr) const;
public:

	/**
	 * Options for what the validity check will permit
	 */
	struct FValidityOptions
	{
		bool bAllowNonManifoldVertices             = false;
		bool bAllowAdjacentFacesReverseOrientation = false;

		/**
		 * Construct validity checking options
		 */
		FValidityOptions(bool bAllowNonManifoldVertices = false, bool bAllowAdjacentFacesReverseOrientation = false)
		    : bAllowNonManifoldVertices(bAllowNonManifoldVertices)
		    , bAllowAdjacentFacesReverseOrientation(bAllowAdjacentFacesReverseOrientation)
		{}

		/**
		 * Construct with most-permissive options that we still consider valid for processing
		 */
		static FValidityOptions Permissive()
		{
			FValidityOptions ToRet;
			ToRet.bAllowAdjacentFacesReverseOrientation = true;
			ToRet.bAllowNonManifoldVertices             = true;
			return ToRet;
		}
	};

	/**
	 * Checks that the mesh is well-formed, ie all internal data structures are consistent
	 */
	GEOMETRYCORE_API virtual bool CheckValidity(FValidityOptions ValidityOptions = FValidityOptions(),
	                           EValidityCheckFailMode FailMode  = EValidityCheckFailMode::Check) const;

	//
	// Internal functions
	//
protected:
	inline void SetTriangleInternal(int TriangleID, int v0, int v1, int v2)
	{
		Triangles[TriangleID] = FIndex3i(v0, v1, v2);
	}
	inline void SetTriangleEdgesInternal(int TriangleID, int e0, int e1, int e2)
	{
		TriangleEdges[TriangleID] = FIndex3i(e0, e1, e2);
	}

	inline int AddEdgeInternal(int vA, int vB, int tA, int tB = InvalidID)
	{
		if (vB < vA) {
			int t = vB; vB = vA; vA = t;
		}
		int eid = EdgeRefCounts.Allocate();
		Edges.InsertAt(FEdge{ {vA, vB},{tA, tB} }, eid);
		VertexEdgeLists.Insert(vA, eid);
		VertexEdgeLists.Insert(vB, eid);
		return eid;

	}
	GEOMETRYCORE_API int AddTriangleInternal(int a, int b, int c, int e0, int e1, int e2);

	inline int ReplaceTriangleVertex(int TriangleID, int vOld, int vNew)
	{
		FIndex3i& Triangle = Triangles[TriangleID];
		for (int i : {0, 1, 2})
		{
			if (Triangle[i] == vOld)
			{
				Triangle[i] = vNew;
				return 0;
			}
		}
		return -1;
	}

	inline void AllocateEdgesList(int VertexID)
	{
		if (VertexID < (int)VertexEdgeLists.Size())
		{
			VertexEdgeLists.Clear(VertexID);
		}
		VertexEdgeLists.AllocateAt(VertexID);
	}

	template<typename ArrayType = FLocalIntArray>
	void GetVertexEdgesList(int VertexID, ArrayType& EdgesOut) const
	{
		for (int eid : VertexEdgeLists.Values(VertexID))
		{
			EdgesOut.Add(eid);
		}
	}

	inline void SetEdgeVerticesInternal(int EdgeID, int a, int b)
	{
		if (a > b)
		{
			Swap(a,b);
		}
		Edges[EdgeID].Vert[0] = a;
		Edges[EdgeID].Vert[1] = b;
	}

	inline void SetEdgeTrianglesInternal(int EdgeID, int t0, int t1)
	{
		Edges[EdgeID].Tri[0] = t0;
		Edges[EdgeID].Tri[1] = t1;
	}

	GEOMETRYCORE_API int ReplaceEdgeVertex(int EdgeID, int vOld, int vNew);
	GEOMETRYCORE_API int ReplaceEdgeTriangle(int EdgeID, int tOld, int tNew);
	GEOMETRYCORE_API int ReplaceTriangleEdge(int EdgeID, int eOld, int eNew);

	inline bool TriangleHasVertex(int TriangleID, int VertexID) const
	{
		return Triangles[TriangleID][0] == VertexID || Triangles[TriangleID][1] == VertexID ||
		    Triangles[TriangleID][2] == VertexID;
	}

	inline bool TriHasNeighbourTri(int CheckTriID, int NbrTriID) const
	{
		return EdgeHasTriangle(TriangleEdges[CheckTriID][0], NbrTriID) ||
		    EdgeHasTriangle(TriangleEdges[CheckTriID][1], NbrTriID) ||
		    EdgeHasTriangle(TriangleEdges[CheckTriID][2], NbrTriID);
	}

	inline bool TriHasSequentialVertices(int TriangleID, int vA, int vB) const
	{
		const FIndex3i& Tri = Triangles[TriangleID];
		return ((Tri.A == vA && Tri.B == vB) || (Tri.B == vA && Tri.C == vB) || (Tri.C == vA && Tri.A == vB));
	}

	GEOMETRYCORE_API int FindTriangleEdge(int TriangleID, int vA, int vB) const;

	GEOMETRYCORE_API int32 FindEdgeInternal(int32 vA, int32 vB, bool& bIsBoundary) const;

	inline bool EdgeHasVertex(int EdgeID, int VertexID) const
	{
		const FIndex2i Verts = Edges[EdgeID].Vert;
		return (Verts[0] == VertexID) || (Verts[1] == VertexID);
	}
	inline bool EdgeHasTriangle(int EdgeID, int TriangleID) const
	{
		const FIndex2i Tris = Edges[EdgeID].Tri;
		return (Tris[0] == TriangleID) || (Tris[1] == TriangleID);
	}

	inline int GetOtherEdgeVertex(int EdgeID, int VertexID) const
	{
		const FIndex2i Verts = Edges[EdgeID].Vert;
		return (Verts[0] == VertexID) ? Verts[1] : ((Verts[1] == VertexID) ? Verts[0] : InvalidID);
	}
	inline int GetOtherEdgeTriangle(int EdgeID, int TriangleID) const
	{
		const FIndex2i Tris = Edges[EdgeID].Tri;
		return (Tris[0] == TriangleID) ? Tris[1] : ((Tris[1] == TriangleID) ? Tris[0] : InvalidID);
	}

	inline void AddTriangleEdge(int TriangleID, int v0, int v1, int j, int EdgeID)
	{
		FIndex3i& TriEdges = TriangleEdges.ElementAt(TriangleID, FIndex3i(InvalidID, InvalidID, InvalidID));
		if (EdgeID != InvalidID)
		{
			Edges[EdgeID].Tri[1] = TriangleID;
			TriEdges[j] = EdgeID;
		}
		else
		{
			TriEdges[j] = AddEdgeInternal(v0, v1, TriangleID);
		}
	}

	// utility function that returns one or two triangles of edge, used to enumerate vertex one-ring triangles
	// The logic is a bit tricky to follow without drawing it out on paper, but this will only return
	// each triangle once, for the 'outgoing' edge from the vertex, and each triangle only has one such edge
	// at any vertex (including boundary triangles)
	inline FIndex2i GetOrderedOneRingEdgeTris(int VertexID, int EdgeID) const
	{
		const FIndex2i Tris = Edges[EdgeID].Tri;

		int vOther = GetOtherEdgeVertex(EdgeID, VertexID);
		int et1    = Tris[1];
		et1        = (et1 != InvalidID && TriHasSequentialVertices(et1, VertexID, vOther)) ? et1 : InvalidID;
		int et0    = Tris[0];
		return TriHasSequentialVertices(et0, VertexID, vOther) ? FIndex2i(et0, et1) : FIndex2i(et1, InvalidID);
	}


	GEOMETRYCORE_API void ReverseTriOrientationInternal(int TriangleID);

	/**
	* Internal implementations for serialization to allow for better code separation between different versions and other implementation details.
	* @tparam Variant Identifier for a specific implementation, e.g. depending on the version and/or specific options.
	* @param Ptr Pointer to optional data, e.g. version specific options.
	*/
	template<int Variant>
	void SerializeInternal(FArchive& Ar, void* Ptr);

	/* We keep this version of CanCollapseEdge internal because the CollapseInfo struct may only be partially filled out by the function */
	GEOMETRYCORE_API virtual EMeshResult CanCollapseEdgeInternal(int vKeep, int vRemove, double collapse_t, FEdgeCollapseInfo* OutCollapseInfo) const;
private:
	GEOMETRYCORE_API virtual EMeshResult CanCollapseEdgeInternal(int vKeep, int vRemove, double collapse_t,
		const FCollapseEdgeOptions& Options, FEdgeCollapseInfo* OutCollapseInfo) const;

	friend class FDynamicMeshAttributeSet;
	friend class FDynamicMeshBulkEdit;
};




namespace Debug
{
	/*
	 * Store a copy of Mesh in a global Debug Meshes TMap<DebugMeshName, Mesh>.
	 * The mesh will exist until the Engine exits, unless FetchDebugMesh or ClearAllDebugMeshes removes it
	 */
	GEOMETRYCORE_API void StashDebugMesh(const FDynamicMesh3& Mesh, FString DebugMeshName);

	/**
	 * If a mesh was previously stored via StashDebugMesh with the given DebugMeshName, 
	 * copy it to MeshOut, and optionally remove it from the Debug Meshes map
	 */
	GEOMETRYCORE_API bool FetchDebugMesh(FString DebugMeshName, FDynamicMesh3& MeshOut, bool bClear);

	/**
	 * Discard all debug meshes stored in the global Debug Meshes map
	 */
	GEOMETRYCORE_API void ClearAllDebugMeshes();
}



} // end namespace UE::Geometry
} // end namespace UE

template<> struct TCanBulkSerialize<UE::Geometry::FDynamicMesh3::FEdge> { enum { Value = true }; };
