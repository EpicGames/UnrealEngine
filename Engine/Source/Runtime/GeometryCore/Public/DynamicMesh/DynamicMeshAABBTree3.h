// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/MeshAABBTree3.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
namespace Geometry
{

typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;

} // end namespace UE::Geometry
} // end namespace UE
