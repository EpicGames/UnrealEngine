// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "DynamicMesh/DynamicAttribute.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "DynamicMesh/DynamicVertexAttribute.h"
#include "DynamicMesh/DynamicMeshSculptLayers.h"
#include "GeometryTypes.h"
#include "HAL/PlatformCrt.h"
#include "InfoTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "Util/DynamicVector.h"

class FArchive;
namespace DynamicMeshInfo { struct FEdgeCollapseInfo; }
namespace DynamicMeshInfo { struct FEdgeFlipInfo; }
namespace DynamicMeshInfo { struct FEdgeSplitInfo; }
namespace DynamicMeshInfo { struct FMergeEdgesInfo; }
namespace DynamicMeshInfo { struct FPokeTriangleInfo; }
namespace DynamicMeshInfo { struct FVertexSplitInfo; }

namespace UE
{
namespace Geometry
{
class FCompactMaps;
class FDynamicMesh3;

/** Standard UV overlay type - 2-element float */
typedef TDynamicMeshVectorOverlay<float, 2, FVector2f> FDynamicMeshUVOverlay;
/** Standard Normal overlay type - 3-element float */
typedef TDynamicMeshVectorOverlay<float, 3, FVector3f> FDynamicMeshNormalOverlay;
/** Standard Color overlay type - 4-element float (rbga) */
typedef TDynamicMeshVectorOverlay<float, 4, FVector4f> FDynamicMeshColorOverlay;
/** Standard per-triangle integer material ID */
typedef TDynamicMeshScalarTriangleAttribute<int32> FDynamicMeshMaterialAttribute;

/** Per-triangle integer polygroup ID */
typedef TDynamicMeshScalarTriangleAttribute<int32> FDynamicMeshPolygroupAttribute;

/** Per-vertex scalar float weight */
typedef TDynamicMeshVertexAttribute<float, 1> FDynamicMeshWeightAttribute;

/** Per-vertex 3-element float morph target*/
typedef TDynamicMeshVertexAttribute<float, 3> FDynamicMeshMorphTargetAttribute;

/** Forward declarations */
template<typename ParentType>
class TDynamicVertexSkinWeightsAttribute;

using FDynamicMeshVertexSkinWeightsAttribute = TDynamicVertexSkinWeightsAttribute<FDynamicMesh3>;
	
/** Bone Attributes */
template<typename ParentType, typename AttribValueType>
class TDynamicBoneAttributeBase;

using FDynamicMeshBoneNameAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FName>;
using FDynamicMeshBoneParentIndexAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, int32>;
using FDynamicMeshBoneColorAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FVector4f>;
using FDynamicMeshBonePoseAttribute = TDynamicBoneAttributeBase<FDynamicMesh3, FTransform>;


/**
 * FDynamicMeshAttributeSet manages a set of extended attributes for a FDynamicMesh3.
 * This includes UV and Normal overlays, etc.
 * 
 * Currently the default is to always have one UV layer and one Normal layer, 
 * but the number of layers can be requested on construction.
 * 
 * @todo current internal structure is a work-in-progress
 */
class FDynamicMeshAttributeSet : public FDynamicMeshAttributeSetBase
{
public:
	GEOMETRYCORE_API FDynamicMeshAttributeSet(FDynamicMesh3* Mesh);

	GEOMETRYCORE_API FDynamicMeshAttributeSet(FDynamicMesh3* Mesh, int32 NumUVLayers, int32 NumNormalLayers);

	GEOMETRYCORE_API virtual ~FDynamicMeshAttributeSet() override;

	GEOMETRYCORE_API void Copy(const FDynamicMeshAttributeSet& Copy);

	/** returns true if the attached overlays/attributes are compact */
	GEOMETRYCORE_API bool IsCompact() const;

	/**
	 * Performs a CompactCopy of the attached overlays/attributes.
	 * Called by the parent mesh CompactCopy function.
	 *
	 * @param CompactMaps Maps indicating how vertices and triangles were changes in the parent
	 * @param Copy The attribute set to be copied
	 */
	GEOMETRYCORE_API void CompactCopy(const FCompactMaps& CompactMaps, const FDynamicMeshAttributeSet& Copy);

	/**
	 * Compacts the attribute set in place
	 * Called by the parent mesh CompactInPlace function
	 *
	 * @param CompactMaps Maps of how the vertices and triangles were compacted in the parent
	 */
	GEOMETRYCORE_API void CompactInPlace(const FCompactMaps& CompactMaps);


	/**
	 * Split all bowtie vertices in all layers
	 * @param bParallel if true, layers are processed in parallel
	 */
	GEOMETRYCORE_API void SplitAllBowties(bool bParallel = true);


	/**
	 * Enable the matching attributes and overlay layers as the ToMatch set, but do not copy any data across.
	 * If bClearExisting=true, all existing attributes are cleared, so after the function there is an exact match
	 * but any existing attribute data is lost (!)
	 * If bClearExisting=false, new attributes are added, but existing attributes and data are preserved
	 * If bDiscardExtraAttributes=true and bClearExisting=false, then attributes in the current set but not in ToMatch are discarded, but existing attributes are not cleared/reset
	 */
	GEOMETRYCORE_API void EnableMatchingAttributes(const FDynamicMeshAttributeSet& ToMatch, bool bClearExisting = true, bool bDiscardExtraAttributes = false);

	/** @return the parent mesh for this overlay */
	const FDynamicMesh3* GetParentMesh() const { return ParentMesh; }
	/** @return the parent mesh for this overlay */
	FDynamicMesh3* GetParentMesh() { return ParentMesh; }

private:
	/** @set the parent mesh for this overlay.  Only safe for use during FDynamicMesh move */
	GEOMETRYCORE_API void Reparent(FDynamicMesh3* NewParent);

public:

	/** @return true if the given edge is a seam edge in any overlay */
	GEOMETRYCORE_API virtual bool IsSeamEdge(int EdgeID) const;

	/** @return true if the given edge is the termination of a seam in any overlay*/
	GEOMETRYCORE_API virtual bool IsSeamEndEdge(int EdgeID) const;

	/** @return true if the given edge is a seam edge in any overlay. This version considers tangent seams as a type of normal seam; to consider tangent seams separately, call the 4-argument IsSeamEdge instead.  */
	UE_DEPRECATED(5.3, "Please instead call the 4 argument version of IsSeamEdge, which distinguishes between tangent and normal seam edges.")
	GEOMETRYCORE_API virtual bool IsSeamEdge(int EdgeID, bool& bIsUVSeamOut, bool& bIsNormalSeamOut, bool& bIsColorSeamOut) const;

	/** @return true if the given edge is a seam edge in any overlay */
	GEOMETRYCORE_API virtual bool IsSeamEdge(int EdgeID, bool& bIsUVSeamOut, bool& bIsNormalSeamOut, bool& bIsColorSeamOut, bool& bIsTangentSeamOut) const;

	/** @return true if the given vertex is a seam vertex in any overlay */
	GEOMETRYCORE_API virtual bool IsSeamVertex(int VertexID, bool bBoundaryIsSeam = true) const;

	/** @return true if the given vertex is a seam intersection vertex in any overlay */
	GEOMETRYCORE_API virtual bool IsSeamIntersectionVertex(int32 VertexID) const;

	/** @return true if the given edge is a material ID boundary */
	GEOMETRYCORE_API virtual bool IsMaterialBoundaryEdge(int EdgeID) const;

	//
	// UV Layers 
	//

	/** @return number of UV layers */
	virtual int NumUVLayers() const
	{
		return UVLayers.Num();
	}

	/** Set number of UV (2-vector float overlay) layers */
	GEOMETRYCORE_API virtual void SetNumUVLayers(int Num);

	/** @return the UV layer at the given Index  if exists, else nullptr*/
	FDynamicMeshUVOverlay* GetUVLayer(int Index)
	{
		if (Index < UVLayers.Num() && Index > -1)
		{ 
			return &UVLayers[Index];
		}
		return nullptr;
	}

	/** @return the UV layer at the given Index if exists, else nullptr */
	const FDynamicMeshUVOverlay* GetUVLayer(int Index) const
	{
		if (Index < UVLayers.Num() && Index > -1)
		{
			return &UVLayers[Index];
		}
		return nullptr;
	}

	/** @return the primary UV layer (layer 0) */
	FDynamicMeshUVOverlay* PrimaryUV()
	{
		return GetUVLayer(0);
	}
	/** @return the primary UV layer (layer 0) */
	const FDynamicMeshUVOverlay* PrimaryUV() const
	{
		return GetUVLayer(0);
	}


	//
	// Normal Layers 
	//

	/** @return number of Normals layers */
	virtual int NumNormalLayers() const
	{
		return NormalLayers.Num();
	}

	/** Set number of Normals (3-vector float overlay) layers */
	GEOMETRYCORE_API virtual void SetNumNormalLayers(int Num);

	/** Enable Tangents overlays (Tangent = Normal layer 1, Bitangent = Normal layer 2) */
	GEOMETRYCORE_API void EnableTangents();

	/** Disable Tangents overlays */
	GEOMETRYCORE_API void DisableTangents();

	/** @return the Normal layer at the given Index if exists, else nullptr */
	FDynamicMeshNormalOverlay* GetNormalLayer(int Index)
	{
		if (Index < NormalLayers.Num() && Index > -1)
		{ 
			return &NormalLayers[Index];
		}
		return nullptr;
	}

	/** @return the Normal layer at the given Index if exists, else nullptr */
	const FDynamicMeshNormalOverlay* GetNormalLayer(int Index) const
	{
		if (Index < NormalLayers.Num() && Index > -1)
		{
			return &NormalLayers[Index];
		}
		return nullptr;
	}


	/** @return the primary Normal layer (normal layer 0) if exists, else nullptr */
	FDynamicMeshNormalOverlay* PrimaryNormals()
	{
		return GetNormalLayer(0);
	}
	/** @return the primary Normal layer (normal layer 0) if exists, else nullptr */
	const FDynamicMeshNormalOverlay* PrimaryNormals() const
	{
		return GetNormalLayer(0);
	}
	/** @return the primary tangent layer ( normal layer 1) if exists, else nullptr */
	FDynamicMeshNormalOverlay* PrimaryTangents()
	{
		return GetNormalLayer(1);
	}
	/** @return the primary tangent layer ( normal layer 1) if exists, else nullptr */
	const FDynamicMeshNormalOverlay* PrimaryTangents() const 
	{
		return GetNormalLayer(1);
	}
	/** @return the primary biTangent layer ( normal layer 2) if exists, else nullptr */
	FDynamicMeshNormalOverlay* PrimaryBiTangents()
	{
		return GetNormalLayer(2);
	}
	/** @return the primary biTangent layer ( normal layer 2) if exists, else nullptr */
	const FDynamicMeshNormalOverlay* PrimaryBiTangents() const
	{
		return GetNormalLayer(2);
	}

	/** @return true if normal layers exist for the normal, tangent, and bitangent */
	bool HasTangentSpace() const
	{
		return (PrimaryNormals() != nullptr && PrimaryTangents()  != nullptr && PrimaryBiTangents() != nullptr);
	}

	bool HasPrimaryColors() const
	{
		return !!ColorLayer;
	}

	FDynamicMeshColorOverlay* PrimaryColors()
	{
		return ColorLayer.Get();
	}

	const FDynamicMeshColorOverlay* PrimaryColors() const
	{
		return ColorLayer.Get();
	}

	GEOMETRYCORE_API void EnablePrimaryColors();
	
	GEOMETRYCORE_API void DisablePrimaryColors();

	//
	// Polygroup layers
	//

	/** @return number of Polygroup layers */
	GEOMETRYCORE_API virtual int32 NumPolygroupLayers() const;

	/** Set the number of Polygroup layers */
	GEOMETRYCORE_API virtual void SetNumPolygroupLayers(int32 Num);

	/** @return the Polygroup layer at the given Index */
	GEOMETRYCORE_API FDynamicMeshPolygroupAttribute* GetPolygroupLayer(int Index);

	/** @return the Polygroup layer at the given Index */
	GEOMETRYCORE_API const FDynamicMeshPolygroupAttribute* GetPolygroupLayer(int Index) const;

	//
	// Weight layers
	//

	/** @return number of weight layers */
	GEOMETRYCORE_API virtual int32 NumWeightLayers() const;

	/** Set the number of weight layers */
	GEOMETRYCORE_API virtual void SetNumWeightLayers(int32 Num);

	/** Remove a weight layer at the specified index */
	GEOMETRYCORE_API virtual void RemoveWeightLayer(int32 Index);

	/** @return the weight layer at the given Index */
	GEOMETRYCORE_API FDynamicMeshWeightAttribute* GetWeightLayer(int Index);

	/** @return the weight layer at the given Index */
	GEOMETRYCORE_API const FDynamicMeshWeightAttribute* GetWeightLayer(int Index) const;

	/**
	 * Make the weight layers 1:1 with those of the ToMatch attribute set, 
	 * adding any missing layers and re-ordering existing layers to match the reference name ordering 
	 * @param ToMatch Reference attribute set whose weight layer names / ordering we must match
	 * @param bDiscardUnmatched Whether to remove unmatched weight layers. Otherwise, they will be left at the end of the weight attribute array.
	 */
	GEOMETRYCORE_API void EnableMatchingWeightLayersByNames(const FDynamicMeshAttributeSet* ToMatch, bool bDiscardUnmatched);



	//
	// Per-Triangle Material ID
	//

	bool HasMaterialID() const
	{
		return !!MaterialIDAttrib;
	}


	GEOMETRYCORE_API void EnableMaterialID();

	GEOMETRYCORE_API void DisableMaterialID();

	FDynamicMeshMaterialAttribute* GetMaterialID()
	{
		return MaterialIDAttrib.Get();
	}

	const FDynamicMeshMaterialAttribute* GetMaterialID() const
	{
		return MaterialIDAttrib.Get();
	}

	/** Triangle Label attributes */
	
	/*
	 * Attaches and registers the triangle label attribute with a given name.
	 * If an attribute with this name already exists, the existing attribute will be deleted.
	*/
	UE_INTERNAL GEOMETRYCORE_API void AttachTriangleLabelAttribute(FName InName, FDynamicMeshTriangleLabelAttribute* InAttribute);

	/* Remove a triangle label attribute matching the given name. */
	UE_INTERNAL GEOMETRYCORE_API void RemoveTriangleLabelAttribute(FName InName);

	/* Remove all triangle label attributes */
	UE_INTERNAL GEOMETRYCORE_API void RemoveAllTriangleLabelAttributes();

	/* Returns true if the list of triangle label attributes includes the given name. */
	UE_INTERNAL bool HasTriangleLabelAttribute(FName InName) const;
	
	/* Returns a pointer to the triangle label attribute of the given name or null ptr if it does not exist */
	UE_INTERNAL GEOMETRYCORE_API FDynamicMeshTriangleLabelAttribute* FindTriangleLabelAttribute(FName InName) const;
	
	/* Returns the existing triangle label attributes. */
	UE_INTERNAL GEOMETRYCORE_API const TMap<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>& GetTriangleLabelAttributes() const;
	
	/** Skin weights */

	/// Create a new skin weights attribute with a given skin weights profile name. If an attribute already exists with
	/// that name, that existing attribute will be deleted.
	GEOMETRYCORE_API void AttachSkinWeightsAttribute(FName InProfileName, FDynamicMeshVertexSkinWeightsAttribute* InAttribute);

	/// Remove a skin weights attribute matching the given profile name.
	GEOMETRYCORE_API void RemoveSkinWeightsAttribute(FName InProfileName);

	/// Remove all skin weights attributes
	GEOMETRYCORE_API void RemoveAllSkinWeightsAttributes();

	/// Returns true if the list of skin weight attributes includes the given profile name.
	bool HasSkinWeightsAttribute(FName InProfileName) const
	{
		return SkinWeightAttributes.Contains(InProfileName);
	}

	/// Returns a pointer to a skin weight attribute of the given profile name. If the attribute
	/// does not exist, a nullptr is returned.
	FDynamicMeshVertexSkinWeightsAttribute *GetSkinWeightsAttribute(FName InProfileName) const
	{
		if (SkinWeightAttributes.Contains(InProfileName))
		{
			return SkinWeightAttributes[InProfileName].Get();
		}
		else
		{
			return nullptr;
		}
	}
	
	/// Returns a map of all skin weight attributes.
	const TMap<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& GetSkinWeightsAttributes() const
	{
		return SkinWeightAttributes;
	}

	/** Morph Targets */
	
	/// Create a new morph target attribute with a given name. If an attribute already exists with
	/// that name, that existing attribute will be deleted.
	GEOMETRYCORE_API void AttachMorphTargetAttribute(FName InMorphTargetName, FDynamicMeshMorphTargetAttribute* InAttribute);

	/// Remove a morph target attribute matching the given name.
	GEOMETRYCORE_API void RemoveMorphTargetAttribute(FName InMorphTargetName);
	
	/// Remove all morph target attributes
	GEOMETRYCORE_API void RemoveAllMorphTargetAttributes();

	/// Returns true if the morph target attribute exists.
	bool HasMorphTargetAttribute(FName InMorphTargetName) const
	{
		return MorphTargetAttributes.Contains(InMorphTargetName);
	}

	/// Returns a pointer to a morph target attribute of the given name. If the attribute
	/// does not exist, a nullptr is returned.
	FDynamicMeshMorphTargetAttribute* GetMorphTargetAttribute(FName InMorphTargetName) const
	{
		if (const TUniquePtr<FDynamicMeshMorphTargetAttribute>* AttributePtr = MorphTargetAttributes.Find(InMorphTargetName))
		{
			return AttributePtr->Get();
		}
		else
		{
			return nullptr;
		}
	}
	
	/// Returns a map of all morph target attributes.
	const TMap<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& GetMorphTargetAttributes() const
	{
		return MorphTargetAttributes;
	}

	//
	// Bone Attributes
	//

	/** Copy all bone attributes from the given attribute set to this attribute set, removing any values that were there before. 
	 *  @param Copy The attribute set to copy from. 
	 */
	GEOMETRYCORE_API void CopyBoneAttributes(const FDynamicMeshAttributeSet& Copy);

	/** Copy bone attributes from the given attribute set to this attribute set, removing any values that were there before, using
	 *  the given bone hierarchy map to control which bones are copied, and how parent bone indices should be remapped. 
	 *  @param Copy The attribute set to copy from.
	 *  @param BoneHierarchy The hierarchy to use to copy. The key is the bone name to copy and value is the new parent name to
	 *    assign to it upon copy. The order of the resulting values is the same as in the Copy. A parent name of NAME_None indicates
	 *    that the bone is a root bone and will have a parent index of INDEX_NONE. If a parent name is not in the list of bones to
	 *    copy, then that bone's parent index will also be set to INDEX_NONE.
	 */
	GEOMETRYCORE_API void CopyBoneAttributesWithRemapping(const FDynamicMeshAttributeSet& Copy, const TMap<FName, FName>& BoneHierarchy); 
	
	GEOMETRYCORE_API void EnableMatchingBoneAttributes(const FDynamicMeshAttributeSet& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes);

	/** @note Only compares bone names and parent indices. */
	GEOMETRYCORE_API bool IsSameBoneAttributesAs(const FDynamicMeshAttributeSet& Other) const;

	/**
	 * The attribute is valid if either all attributes are empty or if the bone name attribute is not empty then all the other
	 * attributes must be either empty or their size is equal to the bone name attribute size.
	 */
	GEOMETRYCORE_API bool CheckBoneValidity(EValidityCheckFailMode FailMode) const;

	
	/** 
	 * Append bone attributes from another set. When appending, the bone attributes in the Other set will only be 
	 * appended if a bone with the same name does not exist in the current bone name attribute. Hence, the 
	 * order of the bones in this set is preserved.
	 * 
	 * @return true, if append was successful
	 */
	GEOMETRYCORE_API bool AppendBonesUnique(const FDynamicMeshAttributeSet& Other);

	/** Enable all bone attributes and intialize their size to the given bone number. */
	GEOMETRYCORE_API void EnableBones(const int InBonesNum);

	/** Disable all bone attributes. */
	GEOMETRYCORE_API void DisableBones();
	
	/** Get number of bones. */
	GEOMETRYCORE_API int32 GetNumBones() const;

	bool HasBones() const 
	{ 
		return !!BoneNameAttrib; 
	}

	FDynamicMeshBoneNameAttribute* GetBoneNames()
	{
		return BoneNameAttrib.Get();
	}

	const FDynamicMeshBoneNameAttribute* GetBoneNames() const
	{
		return BoneNameAttrib.Get();
	}
	
	FDynamicMeshBoneParentIndexAttribute* GetBoneParentIndices()
	{
		return BoneParentIndexAttrib.Get();
	}

	const FDynamicMeshBoneParentIndexAttribute* GetBoneParentIndices() const
	{
		return BoneParentIndexAttrib.Get();
	}

	FDynamicMeshBonePoseAttribute* GetBonePoses()
	{
		return BonePoseAttrib.Get();
	}

	const FDynamicMeshBonePoseAttribute* GetBonePoses() const
	{
		return BonePoseAttrib.Get();
	}

	FDynamicMeshBoneColorAttribute* GetBoneColors()
	{
		return BoneColorAttrib.Get();
	}

	const FDynamicMeshBoneColorAttribute* GetBoneColors() const
	{
		return BoneColorAttrib.Get();
	}

	const FDynamicMeshSculptLayers* GetSculptLayers() const
	{
		return &SculptLayers;
	}
	
	FDynamicMeshSculptLayers* GetSculptLayers()
	{
		return &SculptLayers;
	}

	// Number of sculpt layers enabled on the mesh
	inline int32 NumSculptLayers() const
	{
		return SculptLayers.Layers.Num();
	}

	// Enable sculpt layers on the mesh, if they are not already enabled.
	// @param MinLayers Number of layers to enabled. If more layers already exist, no layers will be removed.
	GEOMETRYCORE_API void EnableSculptLayers(int32 MinLayers);

	// Discard sculpt layer data from the mesh.
	GEOMETRYCORE_API void DiscardSculptLayers();


	// Attach a new attribute (and transfer ownership of it to the attribute set)
	void AttachAttribute(FName AttribName, FDynamicMeshAttributeBase* Attribute)
	{
		if (GenericAttributes.Contains(AttribName))
		{
			UnregisterExternalAttribute(GenericAttributes[AttribName].Get());
		}
		GenericAttributes.Add(AttribName, TUniquePtr<FDynamicMeshAttributeBase>(Attribute));
		RegisterExternalAttribute(Attribute);
	}

	void RemoveAttribute(FName AttribName)
	{
		if (GenericAttributes.Contains(AttribName))
		{
			UnregisterExternalAttribute(GenericAttributes[AttribName].Get());
			GenericAttributes.Remove(AttribName);
		}
	}

	FDynamicMeshAttributeBase* GetAttachedAttribute(FName AttribName)
	{
		if (TUniquePtr<FDynamicMeshAttributeBase>* AttribPtrPtr = GenericAttributes.Find(AttribName))
		{
			return AttribPtrPtr->Get();
		}
		return nullptr;
	}

	const FDynamicMeshAttributeBase* GetAttachedAttribute(FName AttribName) const
	{
		if (const TUniquePtr<FDynamicMeshAttributeBase>* AttribPtrPtr = GenericAttributes.Find(AttribName))
		{
			return AttribPtrPtr->Get();
		}
		return nullptr;
	}
	int NumAttachedAttributes() const
	{
		return GenericAttributes.Num();
	}

	bool HasAttachedAttribute(FName AttribName) const
	{
		return GenericAttributes.Contains(AttribName);
	}

	const TMap<FName, TUniquePtr<FDynamicMeshAttributeBase>>& GetAttachedAttributes() const
	{
		return GenericAttributes;
	}

	/**
	 * Returns true if this AttributeSet is the same as Other.
	 */
	GEOMETRYCORE_API bool IsSameAs(const FDynamicMeshAttributeSet& Other, bool bIgnoreDataLayout) const;

	/**
	 * Serialization operator for FDynamicMeshAttributeSet.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Set Attribute set to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FDynamicMeshAttributeSet& Set)
	{
		Set.Serialize(Ar, nullptr, false);
		return Ar;
	}

	/**
	* Serialize to and from an archive.
	*
	* @param Ar Archive to serialize with.
	* @param CompactMaps If this is not a null pointer, the mesh serialization compacted the vertex and/or triangle data using the provided mapping. 
	* @param bUseCompression Use compression for serializing bulk data.
	*/
	GEOMETRYCORE_API void Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression);

	GEOMETRYCORE_API SIZE_T GetByteCount() const;

protected:
	/** Parent mesh of this attribute set */
	FDynamicMesh3* ParentMesh;
	

	TIndirectArray<FDynamicMeshUVOverlay> UVLayers;
	TIndirectArray<FDynamicMeshNormalOverlay> NormalLayers;
	TUniquePtr<FDynamicMeshColorOverlay> ColorLayer;

	TUniquePtr<FDynamicMeshMaterialAttribute> MaterialIDAttrib;

	TIndirectArray<FDynamicMeshWeightAttribute> WeightLayers;
	TIndirectArray<FDynamicMeshPolygroupAttribute> PolygroupLayers;
	
	using TriangleLabelAttributeMap = TMap<FName, TUniquePtr<FDynamicMeshTriangleLabelAttribute>>;
	TriangleLabelAttributeMap TriangleLabelAttributes;

	using SkinWeightAttributesMap = TMap<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>;
	SkinWeightAttributesMap SkinWeightAttributes;

	using MorphTargetAttributesMap = TMap<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>;
	MorphTargetAttributesMap MorphTargetAttributes;

	// Bone attributes
	TUniquePtr<FDynamicMeshBoneNameAttribute> BoneNameAttrib;
	TUniquePtr<FDynamicMeshBoneParentIndexAttribute> BoneParentIndexAttrib;
	TUniquePtr<FDynamicMeshBonePoseAttribute> BonePoseAttrib;
	TUniquePtr<FDynamicMeshBoneColorAttribute> BoneColorAttrib;

	using GenericAttributesMap = TMap<FName, TUniquePtr<FDynamicMeshAttributeBase>>;
	GenericAttributesMap GenericAttributes;

private:

	// Class to manage sculpt layer data, separated out mainly for clearer organization
	FDynamicMeshSculptLayers SculptLayers;
	friend class FDynamicMeshSculptLayers;
	
	
protected:
	friend class FDynamicMesh3;

	/**
	 * Initialize the existing attribute layers with the given vertex and triangle sizes
	 */
	void Initialize(int MaxVertexID, int MaxTriangleID)
	{
		for (FDynamicMeshUVOverlay& UVLayer : UVLayers)
		{
			UVLayer.InitializeTriangles(MaxTriangleID);
		}
		for (FDynamicMeshNormalOverlay& NormalLayer : NormalLayers)
		{
			NormalLayer.InitializeTriangles(MaxTriangleID);
		}
	}

	// These functions are called by the FDynamicMesh3 to update the various
	// attributes when the parent mesh topology has been modified.
	// TODO: would it be better to register all the overlays and attributes with the base set and not overload these?  maybe!
	GEOMETRYCORE_API virtual void OnNewTriangle(int TriangleID, bool bInserted);
	GEOMETRYCORE_API virtual void OnNewVertex(int VertexID, bool bInserted);
	GEOMETRYCORE_API virtual void OnRemoveTriangle(int TriangleID);
	GEOMETRYCORE_API virtual void OnRemoveVertex(int VertexID);
	GEOMETRYCORE_API virtual void OnReverseTriOrientation(int TriangleID);
	GEOMETRYCORE_API virtual void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo & splitInfo);
	GEOMETRYCORE_API virtual void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo & flipInfo);
	GEOMETRYCORE_API virtual void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo & collapseInfo);
	GEOMETRYCORE_API virtual void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo & pokeInfo);
	GEOMETRYCORE_API virtual void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo & mergeInfo);
	GEOMETRYCORE_API virtual void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& mergeInfo);
	GEOMETRYCORE_API virtual void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate);

	/**
	 * Check validity of attributes
	 * 
	 * @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	 * @param FailMode Desired behavior if mesh is found invalid
	 */
	GEOMETRYCORE_API virtual bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const;

private:

	// Called by FDynamicMesh3 during a mesh append, to append a corresponding attribute set
	void Append(const FDynamicMeshAttributeSet& ToAppend, const FDynamicMesh3::FAppendInfo& AppendInfo);
	// Called by FDynamicMesh3 during a mesh append, to defaulted attributes when the ToAppend mesh did not have an attribute set
	void AppendDefaulted(const FDynamicMesh3::FAppendInfo& AppendInfo);
};



} // end namespace UE::Geometry
} // end namespace UE

