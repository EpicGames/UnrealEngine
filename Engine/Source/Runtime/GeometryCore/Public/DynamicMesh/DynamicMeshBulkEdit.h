// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/Array.h"
#include "IndexTypes.h"

namespace UE 
{
namespace Geometry
{

class FDynamicMeshBulkEdit : FNoncopyable
{
public:
	GEOMETRYCORE_API FDynamicMeshBulkEdit(FDynamicMesh3& EditMesh, bool bInBuildEdgeLists = true);
	GEOMETRYCORE_API ~FDynamicMeshBulkEdit();

	// Set the vertex, triangle and edge count.
	// 
	// If EdgeCount is set > 0, SetTriEdgeIndices and SetEdge should be called for all triangles and edges respectively.
	// 
	// If EdgeCount is set to 0, these methods do not need to be called, the edge topology will be built in Finalize.
	// 
	// Providing edges explicitly, if available, is of course the faster option.
	// 
	// SetVertex, SetTriVertexIndices should be called exactly once per vertex/triangle.
	//
	GEOMETRYCORE_API void Initialize(int VertexCount, int TriangleCount, int EdgeCount);

	// per-triangle vertex indices, which correspond to the vertices provided by SetVertex.
	// not thread-safe
	inline void SetTriVertexIndices(const int TriangleID, const FIndex3i& VertexIndices)
	{
		Mesh.Triangles[TriangleID] = VertexIndices;
		Mesh.VertexRefCounts.Increment(VertexIndices.A);
		Mesh.VertexRefCounts.Increment(VertexIndices.B);
		Mesh.VertexRefCounts.Increment(VertexIndices.C);
	}

	// per-triangle edge indices, which correspond to the edges provided by SetEdge, thread-safe
	inline void SetTriEdgeIndices(const int TriangleID, const FIndex3i& EdgeIndices)
	{
		Mesh.TriangleEdges[TriangleID] = EdgeIndices;
	}

	// thread-safe
	inline void SetVertex(const int VertexID, const FVector3d& Position)
	{
		Mesh.Vertices[VertexID] = Position;
	}

	// thread-safe
	inline void SetEdge(const int EdgeID, const FDynamicMesh3::FEdge& Edge)
	{
		Mesh.Edges[EdgeID] = Edge;
	}

	// Explicit finalization (also called in dtor), after which the DynamicMesh is usable.
	// At the minimum the vertices and vertex indices need to be provided (the topology will be built),
	// or, more efficiencly, the edges are provided directly.
	GEOMETRYCORE_API void Finalize();
	
private:

	void BuildTopology();

	void BuildEdgeLists();

	FDynamicMesh3& Mesh;
	bool bBuildEdgeLists { true };
};


template <typename VertexAccessFuncType,                //< FVector3d (*)(int VertexIndex)
          typename TriangleVertexIndicesAccessFuncType> //< FIndex3i (*)(int TriangleIndex)
inline void BulkConvertToDynamicMesh3(
	FDynamicMesh3& TargetMesh,
	int VertexCount,
	int TriangleCount,
	const VertexAccessFuncType& VertexAccessFunc,
	const TriangleVertexIndicesAccessFuncType& TriangleVertexIndicesAccessFunc)
{
	FDynamicMeshBulkEdit DynamicMeshBulkEdit(TargetMesh);
	DynamicMeshBulkEdit.Initialize(VertexCount, TriangleCount, 0);

	// parallel for would also work
	for (int VID=0; VID < VertexCount; ++VID)
	{
		DynamicMeshBulkEdit.SetVertex(VertexAccessFuncType(VID));
	}

	for (int TID=0; TID < TriangleCount; ++TID)
	{
		DynamicMeshBulkEdit.SetTriVertexIndices(TriangleVertexIndicesAccessFunc(TID));
	}
}						   

template <typename VertexAccessFuncType,                //< FVector3d (*)(int VertexIndex)
          typename TriangleVertexIndicesAccessFuncType, //< FIndex3i (*)(int TriangleIndex)
		  typename TriangleEdgeIndicesAccessFuncType,   //< FIndex3i (*)(int TriangleIndex)
		  typename EdgeAccessFuncType>                  //< FDynamicMesh3::FEdge (*)(int EdgeIdex)
inline void BulkConvertToDynamicMesh3(
	FDynamicMesh3& TargetMesh,
	int VertexCount,
	int TriangleCount,
	int EdgeCount,
	const VertexAccessFuncType& VertexAccessFunc,
	const TriangleVertexIndicesAccessFuncType& TriangleVertexIndicesAccessFunc,
	const TriangleEdgeIndicesAccessFuncType& TriangleEdgeIndicesAccessFunc,
	const EdgeAccessFuncType& EdgeAccessFunc)
{
	FDynamicMeshBulkEdit DynamicMeshBulkEdit(TargetMesh);
	DynamicMeshBulkEdit.Initialize(VertexCount, TriangleCount, EdgeCount);

	// parallel for would also work
	for (int VID=0; VID < VertexCount; ++VID)
	{
		DynamicMeshBulkEdit.SetVertex(VertexAccessFunc(VID));
	}

	for (int TID=0; TID < TriangleCount; ++TID)
	{
		DynamicMeshBulkEdit.SetTriVertexIndices(TriangleVertexIndicesAccessFunc(TID));
		DynamicMeshBulkEdit.SetTriEdgeIndices(TriangleEdgeIndicesAccessFunc(TID));
	}

	for (int EID=0; EID < EdgeCount; ++EID)
	{
		DynamicMeshBulkEdit.SetEdge(EID, EdgeAccessFunc(EID));
	}
}						   



} // namespace Geometry
} // namespace UE
