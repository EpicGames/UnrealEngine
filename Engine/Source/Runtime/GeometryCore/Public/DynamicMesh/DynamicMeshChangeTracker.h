// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "CoreTypes.h"
#include "DynamicMesh/DynamicAttribute.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "DynamicMesh/InfoTypes.h"
#include "GeometryTypes.h"
#include "HAL/PlatformCrt.h"
#include "IndexTypes.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"


//
// Implementation of mesh change tracking for FDynamicMesh3.
// 
// The top-level class is FDynamicMeshChangeTracker, found at the bottom of this file.
// You create an instance of this and then call BeginChange(), then call SaveVertex()/SaveTriangle()
// before you make modifications to a vertex/triangle, and then call EndChange() at the end.
// This function emits a FDynamicMeshChange instance, and you can call Apply() to apply/revert it.
// So this is the object you would store in a higher-level undo/redo record, for example.
// 
// Attribute overlays make everything more complicated of course. Handling of these
// follows a similar structure - the MeshChangeTracker creates a FDynamicMeshAttributeSetChangeTracker,
// which in turn creates a TDynamicMeshAttributeChange for each UV and Normal overlay
// (grouped together in a FDynamicMeshAttributeChangeSet). However you don't have to explicitly
// do anything to get Attribute support, if the initial Mesh had attributes, then
// this all happens automatically, and the attribute changes get attached to the FDynamicMeshChange.
// 
// @todo Currently the attribute support is hardcoded for UV (2-float) and Normal (3-float) overlays.
// Perhaps it would be possible to make this more flexible to avoid hardcoding these?
// Very little of the code depends on the element size or type.
// 
// 

namespace UE
{
namespace Geometry
{


/**
 * TDynamicMeshAttributeChange represents a change to an attribute overlay of a FDynamicMesh3.
 * @warning This class is meant to be used via FDynamicMeshChange and is not fully functional
 * on its own (see comments in ApplyReplaceChange)
 */
template<typename RealType, int ElementSize>
class TDynamicMeshAttributeChange
{
public:
	GEOMETRYCORE_API void SaveInitialElement(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int ElementID);
	GEOMETRYCORE_API void SaveInitialTriangle(const TDynamicMeshOverlay<RealType,ElementSize>* Overlay, int TriangleID);

	GEOMETRYCORE_API void StoreFinalElement(const TDynamicMeshOverlay<RealType, ElementSize>* Overlay, int ElementID);
	GEOMETRYCORE_API void StoreFinalTriangle(const TDynamicMeshOverlay<RealType, ElementSize>* Overlay, int TriangleID);

	GEOMETRYCORE_API bool Apply(TDynamicMeshOverlay<RealType, ElementSize>* Overlay, bool bRevert) const;

protected:
	struct FChangeElement
	{
		int ElementID;
		int DataIndex;
		int ParentVertexID;
	};

	struct FChangeTriangle
	{
		int TriangleID;
		FIndex3i Elements;
	};

	TArray<FChangeElement> OldElements;
	TArray<RealType> OldElementData;
	TArray<FChangeTriangle> OldTriangles;

	TArray<FChangeElement> NewElements;
	TArray<RealType> NewElementData;
	TArray<FChangeTriangle> NewTriangles;

	GEOMETRYCORE_API void ApplyReplaceChange(TDynamicMeshOverlay<RealType,ElementSize>* Overlay,
		const TArray<FChangeTriangle>& RemoveTris,
		const TArray<FChangeElement>& InsertElements,
		const TArray<RealType>& InsertElementData,
		const TArray<FChangeTriangle>& InsertTris) const;
};

/** Standard UV overlay change type - 2-element float */
typedef TDynamicMeshAttributeChange<float,2> FDynamicMeshUVChange;
/** Standard Normal overlay change type - 3-element float */
typedef TDynamicMeshAttributeChange<float,3> FDynamicMeshNormalChange;
/** Standard Color overlay change type - 4-element float */
typedef TDynamicMeshAttributeChange<float, 4> FDynamicMeshColorChange;
/** Standard per-triangle integer attribute change type */
typedef FDynamicMeshTriangleAttributeChange<int32,1> FDynamicMeshTriGroupChange;

/** Standard weight map change type - 1-element float */
typedef TDynamicMeshAttributeChange<float, 1> FDynamicMeshWeightChange;

/**
 * FDynamicMeshAttributeChangeSet stores a set of UV and Normal changes for a FDynamicMesh3
 */
class FDynamicMeshAttributeChangeSet
{
public:
	UE_NONCOPYABLE(FDynamicMeshAttributeChangeSet);
	FDynamicMeshAttributeChangeSet() = default;

	TArray<FDynamicMeshUVChange> UVChanges;
	TArray<FDynamicMeshNormalChange> NormalChanges;
	TOptional<FDynamicMeshColorChange> ColorChange;
	TOptional<FDynamicMeshTriGroupChange> MaterialIDAttribChange;
	TArray<FDynamicMeshTriGroupChange> PolygroupChanges;

	TArray<TUniquePtr<FDynamicMeshAttributeChangeBase>> WeightChanges;
	TArray<TUniquePtr<FDynamicMeshAttributeChangeBase>> RegisteredAttributeChanges;

	/** call ::Apply() on all the UV and Normal changes */
	GEOMETRYCORE_API bool Apply(FDynamicMeshAttributeSet* Attributes, bool bRevert) const;
};


/**
 * FDynamicMeshChange stores a "change" in a FDynamicMesh3, which in this context
 * means the replacement of one set of triangles with a second set, that may have
 * different vertices/attributes. The change can be applied and reverted via ::Apply()
 * 
 * Construction of a well-formed FDynamicMeshChange is quite complex and it is strongly
 * suggested that you do so via FDynamicMeshChangeTracker
 */
class FDynamicMeshChange
{
public:
	GEOMETRYCORE_API ~FDynamicMeshChange();

	/** Store the initial state of a vertex */
	GEOMETRYCORE_API void SaveInitialVertex(const FDynamicMesh3* Mesh, int VertexID);
	/** Store the initial state of a triangle */
	GEOMETRYCORE_API void SaveInitialTriangle(const FDynamicMesh3* Mesh, int TriangleID);

	/** Store the final state of a vertex */
	GEOMETRYCORE_API void StoreFinalVertex(const FDynamicMesh3* Mesh, int VertexID);
	/** Store the final state of a triangle */
	GEOMETRYCORE_API void StoreFinalTriangle(const FDynamicMesh3* Mesh, int TriangleID);

	/** Attach an attribute change set to this mesh change, which will the be applied/reverted automatically */
	void AttachAttributeChanges(TUniquePtr<FDynamicMeshAttributeChangeSet> AttribChanges)
	{
		this->AttributeChanges = MoveTemp(AttribChanges);
	}

	/** Apply or Revert this change using the given Mesh */
	GEOMETRYCORE_API bool Apply(FDynamicMesh3* Mesh, bool bRevert) const;

	/** Do (limited) sanity checks on this MeshChange to ensure it is well-formed */
	GEOMETRYCORE_API void VerifySaveState() const;

	/** @return true if this vertex was saved. Uses linear search. */
	GEOMETRYCORE_API bool HasSavedVertex(int VertexID);

	/** store IDs of saved triangles in TrianglesOut. if bInitial=true, old triangles are stored, otherwise new triangles */
	GEOMETRYCORE_API void GetSavedTriangleList(TArray<int>& TrianglesOut, bool bInitial) const;

	/** store IDs of affected vertices in VerticesOut -- both vertices on saved triangles, and directly saved vertices. if bRevert=true, returns old vertex IDs, otherwise new IDs are returned. */
	GEOMETRYCORE_API void GetAffectedVertices(TArray<int>& VerticesOut, bool bRevert) const;

	/** run self-validity checks on internal data structures to test if change is well-formed */
	GEOMETRYCORE_API void CheckValidity(EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check) const;

protected:

	struct FChangeVertex
	{
		int VertexID;
		FVertexInfo Info;
	};

	struct FChangeTriangle
	{
		int TriangleID;
		FIndex3i Vertices;
		FIndex3i Edges;
		int GroupID;
	};

	TArray<FChangeVertex> OldVertices;
	TArray<FChangeTriangle> OldTriangles;

	TArray<FChangeVertex> NewVertices;
	TArray<FChangeTriangle> NewTriangles;

	TUniquePtr<FDynamicMeshAttributeChangeSet> AttributeChanges;

	GEOMETRYCORE_API void ApplyReplaceChange(FDynamicMesh3* Mesh,
		const TArray<FChangeTriangle>& RemoveTris,
		const TArray<FChangeVertex>& InsertVerts,
		const TArray<FChangeTriangle>& InsertTris) const;

};







/**
 * FDynamicMeshAttributeSetChangeTracker constructs a well-formed set of TDynamicMeshAttributeChange
 * objects (stored in a FDynamicMeshAttributeChangeSet). You should not use this class
 * directly, it is intended to be used via FDynamicMeshChangeTracker
 */
class FDynamicMeshAttributeSetChangeTracker
{
public:
	GEOMETRYCORE_API explicit FDynamicMeshAttributeSetChangeTracker(const FDynamicMeshAttributeSet* Attribs);

	/** Start tracking a change */
	GEOMETRYCORE_API void BeginChange();
	/** End the change transaction and get the resulting change object */
	GEOMETRYCORE_API TUniquePtr<FDynamicMeshAttributeChangeSet> EndChange();

	/** Store the initial state of a triangle */
	GEOMETRYCORE_API void SaveInitialTriangle(int TriangleID);

	/** store the final state of a set of triangles */
	GEOMETRYCORE_API void StoreAllFinalTriangles(const TArray<int>& TriangleIDs);

	/** Store the initial state of a vertex */
	GEOMETRYCORE_API void SaveInitialVertex(int VertexID);

	/** store the final state of a set of vertices */
	UE_DEPRECATED(5.6, "Use the version of this method that takes a TSet instead")
	void StoreAllFinalVertices(const TArray<int>& VertexIDs)
	{
		ensure(false);
	}
	GEOMETRYCORE_API void StoreAllFinalVertices(const TSet<int>& VertexIDs);


protected:
	const FDynamicMeshAttributeSet* Attribs = nullptr;

	FDynamicMeshAttributeChangeSet* Change = nullptr;

	struct FElementState
	{
		int MaxElementID;
		TBitArray<> StartElements;
		TBitArray<> ChangedElements;
	};
	TArray<FElementState> UVStates;
	TArray<FElementState> NormalStates;
	FElementState ColorState;

	template<typename AttribOverlayType, typename AttribChangeType>
	void SaveElement(int ElementID, FElementState& State, const AttribOverlayType* Overlay, AttribChangeType& ChangeIn)
	{
		if (ElementID < State.MaxElementID && State.ChangedElements[ElementID] == false && State.StartElements[ElementID] == true)
		{
			ChangeIn.SaveInitialElement(Overlay, ElementID);
			State.ChangedElements[ElementID] = true;
		}
	}

};









/**
 * FDynamicMeshChangeTracker tracks changes to a FDynamicMesh and returns a
 * FDynamicMeshChange instance that represents this change and allows it to be reverted/reapplied.
 * This is the top-level class you likely want to use to track mesh changes.
 * 
 * Call BeginChange() before making any changes to the mesh, then call SaveVertex()
 * and SaveTriangle() before modifying the respective elements. Then call EndChange()
 * to construct a FDynamicMeshChange that represents the mesh delta.
 *
 * @warning Currently only vertices that are part of saved triangles will be stored in the emitted FMeshChange!
 * 
 */
class FDynamicMeshChangeTracker
{
public:
	GEOMETRYCORE_API explicit FDynamicMeshChangeTracker(const FDynamicMesh3* Mesh);
	GEOMETRYCORE_API ~FDynamicMeshChangeTracker();

	/** Initialize the change-tracking process */
	GEOMETRYCORE_API void BeginChange();
	/** Construct a change object that represents the delta between the Begin and End states */
	GEOMETRYCORE_API TUniquePtr<FDynamicMeshChange> EndChange();

	/** Save necessary information about a triangle before it is modified */
	GEOMETRYCORE_API void SaveTriangle(int32 TriangleID, bool bSaveVertices);

	/** Save necessary information about an edge before it is modified */
	inline void SaveEdge(int32 EdgeID, bool bVertices);

	/** Save necessary information about a set of triangles before they are modified */
	template<typename EnumerableType>
	void SaveTriangles(EnumerableType TriangleIDs, bool bSaveVertices);

	/** Save necessary information about a set of triangles before they are modified, and also include any direct triangle neighbours */
	template<typename EnumerableType>
	void SaveTrianglesAndNeighbourTris(EnumerableType TriangleIDs, bool bSaveVertices);

	/** Save necessary information about a set of triangles in one-ring of a vertex */
	inline void SaveVertexOneRingTriangles(int32 VertexID, bool bSaveVertices);

	/** Save necessary information about a set of triangles in one-rings of a set of vertices */
	template<typename EnumerableType>
	void SaveVertexOneRingTriangles(EnumerableType VertexIDs, bool bSaveVertices);

	/** Do (limited) sanity checking to make sure that the change is well-formed*/
	GEOMETRYCORE_API void VerifySaveState();

protected:

	//
	// Currently EndChange() only stores vertices that are part of modified triangles.
	// So calling SaveVertex() independently, on triangles that are not saved, will be lost.
	// (This needs to be fixed)
	//

	/** Save necessary information about a vertex before it is modified */
	GEOMETRYCORE_API void SaveVertex(int32 VertexID);

	/** Save necessary information about a set of vertices before they are modified */
	template<typename EnumerableType>
	void SaveVertices(EnumerableType VertexIDs);


protected:
	const FDynamicMesh3* Mesh = nullptr;

	/** Active attribute tracker, if Mesh has attribute overlays */
	FDynamicMeshAttributeSetChangeTracker* AttribChangeTracker = nullptr;

	/** Active change that is being constructed */
	FDynamicMeshChange* Change = nullptr;

	int32 MaxTriangleID;
	TBitArray<> StartTriangles;		// bit is 1 if triangle ID was in initial mesh on BeginChange()
	TBitArray<> ChangedTriangles;	// bit is set to 1 if triangle was in StartTriangles and then had SaveTriangle() called for it

	int32 MaxVertexID;
	TBitArray<> StartVertices;		// bit is 1 if vertex ID was in initial mesh on BeginChange()
	TBitArray<> ChangedVertices;	// bit is set to 1 if vertex was in StartVertices and then had SaveVertex() called for it
};



void FDynamicMeshChangeTracker::SaveEdge(int32 EdgeID, bool bVertices)
{
	const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EdgeID);
	SaveTriangle(Edge.Tri[0], bVertices);
	if (Edge.Tri[1] != FDynamicMesh3::InvalidID)
	{
		SaveTriangle(Edge.Tri[1], bVertices);
	}
}

template<typename EnumerableType>
void FDynamicMeshChangeTracker::SaveVertices(EnumerableType VertexIDs)
{
	for (int32 VertexID : VertexIDs)
	{
		SaveVertex(VertexID);
	}
}

template<typename EnumerableType>
void FDynamicMeshChangeTracker::SaveTriangles(EnumerableType TriangleIDs, bool bSaveVertices)
{
	for (int32 TriangleID : TriangleIDs)
	{
		SaveTriangle(TriangleID, bSaveVertices);
	}
}


template<typename EnumerableType>
void FDynamicMeshChangeTracker::SaveTrianglesAndNeighbourTris(EnumerableType TriangleIDs, bool bSaveVertices)
{
	for (int32 TriangleID : TriangleIDs)
	{
		SaveTriangle(TriangleID, bSaveVertices);

		FIndex3i TriNbrs = Mesh->GetTriNeighbourTris(TriangleID);
		for (int32 j = 0; j < 3; ++j)
		{
			if (TriNbrs[j] != FDynamicMesh3::InvalidID)
			{
				SaveTriangle(TriNbrs[j], bSaveVertices);
			}
		}
	}
}


void FDynamicMeshChangeTracker::SaveVertexOneRingTriangles(int32 VertexID, bool bSaveVertices)
{
	Mesh->EnumerateVertexTriangles(VertexID, [this, bSaveVertices](int32 TriangleID)
	{
		SaveTriangle(TriangleID, bSaveVertices);
	});
}

template<typename EnumerableType>
void FDynamicMeshChangeTracker::SaveVertexOneRingTriangles(EnumerableType VertexIDs, bool bSaveVertices)
{
	for (int32 VertexID : VertexIDs)
	{
		SaveVertexOneRingTriangles(VertexID, bSaveVertices);
	}
}



} // end namespace UE::Geometry
} // end namespace UE
