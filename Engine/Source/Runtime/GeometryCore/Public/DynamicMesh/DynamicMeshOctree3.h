// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/MeshAABBTree3.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "MeshQueries.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"


namespace UE
{
namespace Geometry
{


/**
 * FDynamicMeshOctree3 is an extension of FSparseDynamicOctree3 for the triangles of a FDynamicMesh3 instance.
 * This extension does several things:
 *   1) provides a simplified API based on triangle IDs to various Octree functions
 *   2) tracks ModifiedBounds box of modified areas
 *   3) support for computing/updating/querying a "Cut" of the octree, ie a set of cells which are roots of sub-branches
 *      that partition the tree. This is useful for splitting up mesh processing/rendering into spatially-coherent chunks.
 *      (This functionality should probably be extracted into a separate class...)
 */
class FDynamicMeshOctree3 : public FSparseDynamicOctree3
{
	// potential optimizations:
	//    - keep track of how many triangles are descendents of each cell? root cells?



public:
	/** parent mesh */
	const FDynamicMesh3* Mesh;

	/** bounding box of triangles that have been inserted/removed since last clear */
	FAxisAlignedBox3d ModifiedBounds;

	/** 
	 * Add all triangles of MeshIn to the octree 
	 */
	void Initialize(const FDynamicMesh3* MeshIn)
	{
		this->Mesh = MeshIn;

		for (int tid : Mesh->TriangleIndicesItr())
		{
			InsertTriangle(tid);
		}
	}

	/**
	 * Reset the internal ModifiedBounds box that tracks modified triangle bounds
	 */
	void ResetModifiedBounds()
	{
		ModifiedBounds = FAxisAlignedBox3d::Empty();
	}

	/**
	 * Insert a triangle into the tree
	 */
	void InsertTriangle(int32 TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID))
		{
			FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
			ModifiedBounds.Contain(Bounds);
			InsertObject(TriangleID, Bounds);
		}
	}

	/**
	 * Insert a list of triangles into the tree
	 */
	void InsertTriangles(const TArray<int>& Triangles)
	{
		int N = Triangles.Num();
		for (int i = 0; i < N; ++i)
		{
			if (Mesh->IsTriangle(Triangles[i]))
			{
				FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(Triangles[i]);
				ModifiedBounds.Contain(Bounds);
				InsertObject(Triangles[i], Bounds);
			}
		}
	}

	/**
	 * Insert a set of triangles into the tree
	 */
	void InsertTriangles(const TSet<int>& Triangles)
	{
		for (int TriangleID : Triangles)
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
				ModifiedBounds.Contain(Bounds);
				InsertObject(TriangleID, Bounds);
			}
		}
	}


	/**
	 * Remove a triangle from the tree
	 */
	bool RemoveTriangle(int32 TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID))
		{
			FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
			ModifiedBounds.Contain(Bounds);
		}
		return RemoveObject(TriangleID);		// will ignore if we do not contain this triangle
	}


	/**
	 * Remove a set of triangles into the tree
	 */
	template<typename EnumerableType>
	void RemoveTriangles(const EnumerableType& Triangles, bool bMarkModifiedBounds = true)
	{
		for (int TriangleID : Triangles)
		{
			if (RemoveObject(TriangleID) && bMarkModifiedBounds && Mesh->IsTriangle(TriangleID))
			{
				FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
				ModifiedBounds.Contain(Bounds);
			}
		}
	}


	/**
	 * Reinsert a set of triangles into the tree
	 */
	template<typename EnumerableType>
	void ReinsertTriangles(const EnumerableType& Triangles)
	{
		for (int TriangleID : Triangles)
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
				ModifiedBounds.Contain(Bounds);
				ReinsertObject(TriangleID, Bounds);
			}
			else
			{
				RemoveObject(TriangleID);		// can only remove, will ignore if we do not contain this triangle
			}
		}
	}


	/**
	 * Reinsert a set of triangles into the tree. Internally precomputes which triangles need
	 * re-inserting, which can be done in parallel and generally saves time as some triangles can be skipped.
	 */
	void ReinsertTrianglesParallel(const TArray<int32>& Triangles, TArray<uint32>& TempBuffer, TArray<bool>& TempFlagBuffer)
	{
		int32 NumTriangles = Triangles.Num();
		TempBuffer.SetNum(NumTriangles, EAllowShrinking::No);
		TempFlagBuffer.SetNum(NumTriangles, EAllowShrinking::No);

		// can check which triangles need reinsertion in parallel. This will also return which
		// CellID the triangle is in, which saves time in the Reinsert function
		ParallelFor(NumTriangles, [&](int k)
		{
			FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(Triangles[k]);
			TempFlagBuffer[k] = CheckIfObjectNeedsReinsert(Triangles[k], Bounds, TempBuffer[k]);
		});

		// now reinsert all necessary triangles
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			if (TempFlagBuffer[k])
			{
				ReinsertObject(Triangles[k], Mesh->GetTriBounds(Triangles[k]), TempBuffer[k]);
			}
		}
	}



	/**
	 * Include the current bounds of a triangle in the ModifiedBounds box
	 */
	void NotifyPendingModification(int TriangleID)
	{
		if (Mesh->IsTriangle(TriangleID))
		{
			FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
			ModifiedBounds.Contain(Bounds);
		}
	}

	/**
	 * Include the current bounds of a set of triangles in the ModifiedBounds box
	 */
	template<typename EnumerableType>
	void NotifyPendingModification(const EnumerableType& Triangles)
	{
		for (int TriangleID : Triangles)
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				FAxisAlignedBox3d Bounds = Mesh->GetTriBounds(TriangleID);
				ModifiedBounds.Contain(Bounds);
			}
		}
	}


	/**
	 * Find the nearest triangle of the mesh that is hit by the ray
	 */
	int32 FindNearestHitObject(const FRay3d& Ray,
		double MaxDistance = TNumericLimits<double>::Max()) const
	{
		return FSparseDynamicOctree3::FindNearestHitObject(Ray,
			[&](int tid) {return Mesh->GetTriBounds(tid); },
			[&](int tid, const FRay3d& Ray) {
			FIntrRay3Triangle3d Intr = TMeshQueries<FDynamicMesh3>::TriangleIntersection(*Mesh, tid, Ray);
			return (Intr.IntersectionType == EIntersectionType::Point) ?
				Intr.RayParameter : TNumericLimits<double>::Max();

		}, MaxDistance);
	}


	/**
	 * Find the nearest triangle of the mesh that is hit by the ray
	 * @param IncludeTriangleIDFunc predicate function that must return true for given TriangleID for it to be considered
	 */
	int32 FindNearestHitObject(const FRay3d& Ray,
		TFunctionRef<bool(int)> IncludeTriangleIDFunc,
		double MaxDistance = TNumericLimits<double>::Max()) const
	{
		return FSparseDynamicOctree3::FindNearestHitObject(Ray,
			[&](int tid) {return Mesh->GetTriBounds(tid); },
			[&](int tid, const FRay3d& Ray) {
				if (IncludeTriangleIDFunc(tid) == false)
				{
					return TNumericLimits<double>::Max();
				}
				FIntrRay3Triangle3d Intr = TMeshQueries<FDynamicMesh3>::TriangleIntersection(*Mesh, tid, Ray);
				return (Intr.IntersectionType == EIntersectionType::Point) ?
					Intr.RayParameter : TNumericLimits<double>::Max();
			}, 
			MaxDistance);
	}


	/**
	 * Check that the Octree is internally valid
	 */
	void CheckValidity(
		EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check,
		bool bVerbose = false,
		bool bFailOnMissingObjects = false) const
	{
		FSparseDynamicOctree3::CheckValidity(
			[&](int tid) {return Mesh->IsTriangle(tid); },
			[&](int tid) {return Mesh->GetTriBounds(tid); },
			FailMode, bVerbose, bFailOnMissingObjects);
	}






	//
	// Support for building "cuts" of the octree, which are sets
	// of internal nodes which can be used to decompose the tree
	// (eg into spatially-coherent chunks of triangles, for example)
	// 

	/** FCellReference is a handle to an internal cell of the octree */
	struct FCellReference
	{
		uint32 CellID;
	};

	/** FTreeCutSet is a cut of the tree, ie a set of internal cells which are "parents" of separate branches */
	class FTreeCutSet
	{
	public:
		TArray<FCellReference> CutCells;

	protected:
		TSet<uint32> CutCellIDs;			// just a set version of CutCells for internal use - maybe use set there instead, and add compare ops to FCellReference?
		int FixedCutLevel;

		friend class FDynamicMeshOctree3;
	};


	/**
	 * @return a cut of the tree at a fixed level
	 */
	FTreeCutSet BuildLevelCutSet(uint32 CutLevel = 5) const
	{
		FTreeCutSet CutSet;
		CutSet.FixedCutLevel = CutLevel;

		for (const FSparseOctreeCell& Cell : Cells)
		{
			if (Cell.Level == CutSet.FixedCutLevel)
			{
				FCellReference CellRef;
				CellRef.CellID = Cell.CellID;
				CutSet.CutCells.Add(CellRef);

				CutSet.CutCellIDs.Add(Cell.CellID);
			}
		}

		return CutSet;
	}


	/**
	 * For a fixed-level cut set created by BuildLevelCutSet, check that all current cells at that level are in the cut set
	 * (call this after adding/removing to the tree to make sure the CutSet is up to date)
	 * @param CutSet the tree cut set to check/update
	 * @param NewCutCellsOut list of new cut cells added by the update
	 */
	void UpdateLevelCutSet(FTreeCutSet& CutSet, TArray<FCellReference>& NewCutCellsOut) const
	{
		for (const FSparseOctreeCell& Cell : Cells)
		{
			if (Cell.Level == CutSet.FixedCutLevel)
			{
				if (CutSet.CutCellIDs.Contains(Cell.CellID) == false)
				{
					FCellReference CellRef;
					CellRef.CellID = Cell.CellID;
					CutSet.CutCells.Add(CellRef);
					CutSet.CutCellIDs.Add(Cell.CellID);
					NewCutCellsOut.Add(CellRef);
				}
			}
		}
	}

	/**
	 * @return true if the cell identified by CellRef intersects the Bounds box
	 */
	bool TestCellIntersection(const FCellReference& CellRef, const FAxisAlignedBox3d& Bounds) const
	{
		check(CellRefCounts.IsValid(CellRef.CellID));
		const FSparseOctreeCell& Cell = Cells[CellRef.CellID];
		return GetCellBox(Cell, MaxExpandFactor).Intersects(Bounds);
	}

	/**
	 * Call TriangleFunc on any triangles in the branch of the tree starting at CellRef
	 */
	void CollectTriangles(
		const FCellReference& CellRef, 
		TFunctionRef<void(int)> TriangleFunc) const
	{
		check(CellRefCounts.IsValid(CellRef.CellID));

		TArray<const FSparseOctreeCell*> Queue;
		Queue.Add(&Cells[CellRef.CellID]);
		while (Queue.Num() > 0)
		{
			const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

			// process elements
			for (int ObjectID : CellObjectLists.Values(CurCell->CellID))
			{
				TriangleFunc(ObjectID);
			}

			for (int k = 0; k < 8; ++k)
			{
				if (CurCell->HasChild(k))
				{
					const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
					Queue.Add(ChildCell);
				}
			}
		}
	}


	/**
	 * Call TriangleFunc for all triangles in the octree "above" the CutSet
	 * (ie at tree cells that are not children of any cut cells)
	 */
	void CollectRootTriangles(const FTreeCutSet& CutSet,
		TFunctionRef<void(int)> TriangleFunc) const
	{
		TArray<const FSparseOctreeCell*> Queue;

		// start at root cells
		RootCells.AllocatedIteration([&](const uint32* RootCellID)
		{
			const FSparseOctreeCell* RootCell = &Cells[*RootCellID];
			Queue.Add(&Cells[*RootCellID]);
		});

		while (Queue.Num() > 0)
		{
			const FSparseOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);
			if (CutSet.CutCellIDs.Contains(CurCell->CellID))
			{
				continue;
			}

			// process elements
			for (int TriangleID : CellObjectLists.Values(CurCell->CellID))
			{
				TriangleFunc(TriangleID);
			}

			for (int k = 0; k < 8; ++k)
			{
				if (CurCell->HasChild(k))
				{
					const FSparseOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
					Queue.Add(ChildCell);
				}
			}
		}
	}

	/**
	 * Call TriangleFunc for any triangles in the spill set (ie not contained in any Root cell)
	 */
	void CollectSpillTriangles(
		TFunctionRef<void(int)> TriangleFunc) const
	{
		for (int tid : SpillObjectSet)
		{
			TriangleFunc(tid);
		}
	}


};



} // end namespace UE::Geometry
} // end namespace UE
