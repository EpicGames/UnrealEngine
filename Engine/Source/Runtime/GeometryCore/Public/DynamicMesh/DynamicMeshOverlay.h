// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryTypes.h"
#include "IndexTypes.h"
#include "InfoTypes.h"
#include "IntVectorTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Util/CompactMaps.h"
#include "Util/DynamicVector.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"
#include "VectorTypes.h"

class FArchive;
namespace DynamicMeshInfo { struct FEdgeCollapseInfo; }
namespace DynamicMeshInfo { struct FEdgeFlipInfo; }
namespace DynamicMeshInfo { struct FEdgeSplitInfo; }
namespace DynamicMeshInfo { struct FMergeEdgesInfo; }
namespace DynamicMeshInfo { struct FPokeTriangleInfo; }
namespace DynamicMeshInfo { struct FVertexSplitInfo; }

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * TDynamicMeshOverlay is an add-on to a FDynamicMesh3 that allows for per-triangle storage
 * of an "element" (eg like a per-triangle UV or normal). However the elements can be shared
 * between triangles at shared vertices because the elements are stored in a separate indexable
 * list.
 *
 * Each element has one vertex in the parent mesh as its parent, whereas each vertex may be the
 * parent of multiple elements in cases where neighboring triangles are not sharing a single
 * element for that vertex. This means that there may be "seam" boundary edges in the
 * overlay topology that are not mesh boundary edges in the associated/parent mesh, but
 * the overlay topology will not connect triangles that were not connected in the parent mesh
 * or create any topologically degenerate triangles, since the parent vids of the elements
 * of a triangle will have to match up to the vids of the triangle.
 *
 * A "seam" edge is one where at least one of the elements of the triangles on either
 * side of the edge is not shared between the two triangles.
 *
 * The FDynamicMesh3 mesh topology operations (eg split/flip/collapse edge, poke face, etc) 
 * can be mirrored to the overlay via OnSplitEdge(), etc. 
 *
 * Note that although this is a template, many of the functions are defined in the .cpp file.
 * As a result you need to explicitly instantiate and export the instance of the template that 
 * you wish to use in the block at the top of DynamicMeshOverlay.cpp
 */
template<typename RealType, int ElementSize>
class TDynamicMeshOverlay
{

protected:
	/** The parent mesh this overlay belongs to */
	FDynamicMesh3* ParentMesh;

	/** Reference counts of element indices. Iterate over this to find out which elements are valid. */
	FRefCountVector ElementsRefCounts;
	/** List of element values */
	TDynamicVector<RealType> Elements;
	/** List of parent vertex indices, one per element */
	TDynamicVector<int> ParentVertices;

	/** List of triangle element-index triplets [Elem0 Elem1 Elem2]*/
	TDynamicVector<int> ElementTriangles;

	friend class FDynamicMesh3;
	friend class FDynamicMeshAttributeSet;

public:
	/** Create an empty overlay */
	TDynamicMeshOverlay()
	{
		ParentMesh = nullptr;
	}

	/** Create an overlay for the given parent mesh */
	TDynamicMeshOverlay(FDynamicMesh3* ParentMeshIn)
	{
		ParentMesh = ParentMeshIn;
	}
private:
	/** @set the parent mesh for this overlay.  Only safe for use during FDynamicMesh move */
	void Reparent(FDynamicMesh3* ParentMeshIn)
	{
		ParentMesh = ParentMeshIn;
	}
public:
	/** @return the parent mesh for this overlay */
	const FDynamicMesh3* GetParentMesh() const { return ParentMesh; }
	/** @return the parent mesh for this overlay */
	FDynamicMesh3* GetParentMesh() { return ParentMesh; }

	/** Set this overlay to contain the same arrays as the copy overlay */
	void Copy(const TDynamicMeshOverlay<RealType,ElementSize>& Copy)
	{
		ElementsRefCounts = FRefCountVector(Copy.ElementsRefCounts);
		Elements = Copy.Elements;
		ParentVertices = Copy.ParentVertices;
		ElementTriangles = Copy.ElementTriangles;
	}

	/** Copy the Copy overlay to a compact rep, also updating parent references based on the CompactMaps */
	void CompactCopy(const FCompactMaps& CompactMaps, const TDynamicMeshOverlay<RealType, ElementSize>& Copy)
	{
		ClearElements();

		// map of element IDs
		TArray<int> MapE; MapE.SetNumUninitialized(Copy.MaxElementID());

		// copy elements across
		RealType Data[ElementSize];
		for (int EID = 0; EID < Copy.MaxElementID(); EID++)
		{
			if (Copy.IsElement(EID))
			{
				Copy.GetElement(EID, Data);
				MapE[EID] = AppendElement(Data);
			}
			else
			{
				MapE[EID] = -1;
			}
		}

		// copy triangles across
		check(CompactMaps.NumTriangleMappings() == Copy.GetParentMesh()->MaxTriangleID()); // must have valid triangle map
		for (int FromTID : Copy.GetParentMesh()->TriangleIndicesItr())
		{
			if (!Copy.IsSetTriangle(FromTID))
			{
				continue;
			}
			const int ToTID = CompactMaps.GetTriangleMapping(FromTID);
			FIndex3i FromTriElements = Copy.GetTriangle(FromTID);
			SetTriangle(ToTID, FIndex3i(MapE[FromTriElements.A], MapE[FromTriElements.B], MapE[FromTriElements.C]));
		}
	}

	/** Compact overlay and update links to parent based on CompactMaps */
	void CompactInPlace(const FCompactMaps& CompactMaps)
	{
		int iLastE = MaxElementID() - 1, iCurE = 0;
		while (iLastE >= 0 && ElementsRefCounts.IsValidUnsafe(iLastE) == false)
		{
			iLastE--;
		}
		while (iCurE < iLastE && ElementsRefCounts.IsValidUnsafe(iCurE))
		{
			iCurE++;
		}

		// make a map to track element index changes, to use to update element triangles later
		// TODO: it may be faster to not construct this and to do the remapping per element as we go (by iterating the one ring of each parent vertex for each element)
		TArray<int> MapE; MapE.SetNumUninitialized(MaxElementID());
		for (int ID = 0; ID < MapE.Num(); ID++)
		{
			// mapping is 1:1 by default; sparsely re-mapped below
			MapE[ID] = ID;
			// remap all parents
			if (ParentVertices[ID] >= 0)
			{
				ParentVertices[ID] = CompactMaps.GetVertexMapping(ParentVertices[ID]);
			}
		}

		TDynamicVector<unsigned short>& ERef = ElementsRefCounts.GetRawRefCountsUnsafe();
		RealType Data[ElementSize];
		while (iCurE < iLastE)
		{
			// remap the element data
			GetElement(iLastE, Data);
			SetElement(iCurE, Data);
			ParentVertices[iCurE] = ParentVertices[iLastE];
			ERef[iCurE] = ERef[iLastE];
			ERef[iLastE] = FRefCountVector::INVALID_REF_COUNT;
			MapE[iLastE] = iCurE;

			// move cur forward one, last back one, and  then search for next valid
			iLastE--; iCurE++;
			while (iLastE >= 0 && ElementsRefCounts.IsValidUnsafe(iLastE) == false)
			{
				iLastE--;
			}
			while (iCurE < iLastE && ElementsRefCounts.IsValidUnsafe(iCurE))
			{
				iCurE++;
			}
		}
		ElementsRefCounts.Trim(ElementCount());
		Elements.Resize(ElementCount() * ElementSize);
		ParentVertices.Resize(ElementCount());

		// Remap and compact triangle element indices.
		int32 MaxNewTID = -1;
		for (int TID = 0, OldMaxTID = ElementTriangles.Num() / 3; TID < OldMaxTID; TID++)
		{
			const int32 OldStart = TID * 3;
			const int32 NewTID = CompactMaps.GetTriangleMapping(TID);
			if (NewTID == IndexConstants::InvalidID)
			{
				// skip if there's no mapping
				continue;
			}

			MaxNewTID = FMath::Max(NewTID, MaxNewTID);

			const int32 NewStart = NewTID * 3;
			if (ElementTriangles[OldStart] == IndexConstants::InvalidID)
			{
				// triangle was not set; copy back InvalidID
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					ElementTriangles[NewStart + SubIdx] = IndexConstants::InvalidID;
				}
			}
			else
			{
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					ElementTriangles[NewStart + SubIdx] = MapE[ElementTriangles[OldStart + SubIdx]];
				}
			}
		}
		// ElementTriangles should never grow during a compaction, so just resizing is ok
		// (i.e., we shouldn't need to set InvalidID on any added triangles)
		checkSlow(ElementTriangles.Num() >= (MaxNewTID + 1) * 3);
		ElementTriangles.Resize((MaxNewTID + 1) * 3);

		checkSlow(IsCompact());
	}

	void Append(const TDynamicMeshOverlay& ToAppend, const FDynamicMesh3::FAppendInfo& AppendInfo)
	{
		// We expect to be appending s.t. the intial ElementTriangles map to the pre-append triangles
		if (!ensure(ElementTriangles.Num() == AppendInfo.TriangleOffset * 3))
		{
			ElementTriangles.SetNum(AppendInfo.TriangleOffset * 3);
		}
		check(ToAppend.ElementTriangles.Num() == AppendInfo.NumTriangle * 3);

		int32 ElementIDOffset = ElementsRefCounts.GetMaxIndex();
		int32 ElementTrianglesOffset = ElementTriangles.Num();
		ElementTriangles.Add(ToAppend.ElementTriangles);
		for (int32 Idx = ElementTrianglesOffset, N = ElementTriangles.Num(); Idx < N; ++Idx)
		{
			int32& ElID = ElementTriangles[Idx];
			if (ElID != IndexConstants::InvalidID)
			{
				ElID += ElementIDOffset;
			}
		}

		checkSlow(ElementIDOffset == ParentVertices.Num()); // ParentVertices must be 1:1 with Element IDs
		ParentVertices.Add(ToAppend.ParentVertices);
		for (int32 Idx = 0; Idx < ToAppend.ParentVertices.Num(); ++Idx)
		{
			int32& Parent = ParentVertices[Idx + ElementIDOffset];
			if (Parent != IndexConstants::InvalidID)
			{
				Parent += AppendInfo.VertexOffset;
			}
		}
		ElementsRefCounts.Append(ToAppend.ElementsRefCounts);
		Elements.Add(ToAppend.Elements);
	}
	void AppendDefaulted(const FDynamicMesh3::FAppendInfo& AppendInfo)
	{
		// Note: ElementRefCounts, Elements and ParentVertices remain unchanged, since the new triangles are unset
		checkSlow(ElementTriangles.Num() == AppendInfo.TriangleOffset * 3);
		ElementTriangles.Resize(3 * (AppendInfo.TriangleOffset + AppendInfo.NumTriangle), IndexConstants::InvalidID);
	}

	/** Discard all elements. */
	void ClearElements();

	/** Discard elements for given triangles. */
	template <typename EnumerableIntType>
	void ClearElements(const EnumerableIntType& Triangles)
	{
		for (int32 TriID : Triangles)
		{
			UnsetTriangle(TriID);
		}
	}

	/** @return the number of in-use Elements in the overlay */
	int ElementCount() const { return (int)ElementsRefCounts.GetCount(); }
	/** @return the maximum element index in the overlay. This may be larger than the count if Elements have been deleted. */
	int MaxElementID() const { return (int)ElementsRefCounts.GetMaxIndex(); }
	/** @return true if this element index is in use */
	inline bool IsElement(int vID) const { return ElementsRefCounts.IsValid(vID); }

	/** @return true if the elements are compact */
	bool IsCompact() const { return ElementsRefCounts.IsDense(); }


	typedef typename FRefCountVector::IndexEnumerable element_iterator;

	/** @return enumerator for valid element indices suitable for use with range-based for */
	element_iterator ElementIndicesItr() const { return ElementsRefCounts.Indices(); }


	/** Allocate a new element with the given constant value */
	int AppendElement(RealType ConstantValue);
	/** Allocate a new element with the given value */
	int AppendElement(const RealType* Value);

	void SetParentVertex(int ElementIndex, int ParentVertexIndex)
	{
		ParentVertices[ElementIndex] = ParentVertexIndex;
	}

	/** Initialize the triangle list to the given size, and set all triangles to InvalidID */
	void InitializeTriangles(int MaxTriangleID);

	/**
	 * Set the triangle to the given Element index tuple, and increment element reference counts 
	 * 
	 * @param bAllowElementFreeing If true, then any elements that were only referenced by this triangle
	 *  become immediately unallocated if the triangle no longer references them. This can be set to false
	 *  when remeshing across existing elements to avoid them being freed while temporarily unreferenced,
	 *  but then it should eventually be followed by a call to FreeUnusedElements().
	 */
	EMeshResult SetTriangle(int TriangleID, const FIndex3i& TriElements, bool bAllowElementFreeing = true);

	/**
	 * Goes through elements and frees any whose reference counts indicate that they are not being used. This
	 * is usually not necessary since most operations that remove references will go ahead and do this, but
	 * it may be used, for instance, after SetTriangle is called with bAllowElementFreeing set to false.
	 * 
	 * @param ElementsToCheck If provided, only these element ID's will be checked.
	 */
	void FreeUnusedElements(const TSet<int>* ElementsToCheck = nullptr);

	/**
	 * Set the triangle to have InvalidID element IDs, decrementing element reference counts if needed.
	 * 
	 * @param bAllowElementFreeing If true, then any elements that were only referenced by this triangle
	 *  become immediately unallocated. This can be set to false as part of a remeshing, but then it 
	 *  should eventually be followed by a call to FreeUnusedElements().
	 */
	void UnsetTriangle(int TriangleID, bool bAllowElementFreeing = true);

	/** @return true if this triangle was set */
	bool IsSetTriangle(int TID) const
	{
		bool bIsSet = ElementTriangles[3 * TID] >= 0;
		// we require that triangle elements either be all set or all unset
		checkSlow(ElementTriangles[3 * TID + 1] >= 0 == bIsSet);
		checkSlow(ElementTriangles[3 * TID + 2] >= 0 == bIsSet);
		return bIsSet;
	}

	/**
	 * @return true if this overlay's per-triangle storage is large enough to cover every triangle
	 *  in the parent mesh's ID space. When false, the overlay was not grown alongside the parent
	 *  (e.g., triangles were added to the parent through a path that did not fire OnNewTriangle,
	 *  or the on-disk data was deserialized with mismatched element-triangle counts) and any
	 *  IsSetTriangle / GetTri* call for a triangle ID near MaxTriangleID() will read past the end
	 *  of ElementTriangles.
	 */
	bool IsTriangleStorageValid() const
	{
		return ParentMesh != nullptr
			&& static_cast<int32>(ElementTriangles.Num()) >= 3 * ParentMesh->MaxTriangleID();
	}

	/**
	 * Build overlay topology from a predicate function, e.g. to build topology for sharp normals
	 * 
	 * @param TrisCanShareVertexPredicate Indicator function returns true if the given vertex can be shared for the given pair of triangles
	 *									  Note if a vertex can be shared between tris A and B, and B and C, it will be shared between all three
	 * @param InitElementValue Initial element value, copied into all created elements
	 */
	void CreateFromPredicate(TFunctionRef<bool(int ParentVertexIdx, int TriIDA, int TriIDB)> TrisCanShareVertexPredicate, RealType InitElementValue);

	/**
	 * Build overlay topology with one element per vertex.
	 * Note: Faster-but-equivalent-to calling CreateFromPredicate() with a CanShareVertex predicate that always returns true
	 * @param bExactMapIDs If true, ElementIDs will be constructed to exactly match the corresponding vertex IDs, even if the mesh is non-compact.
	 */
	void CreatePerVertex(RealType InitElementValue, bool bExactMapIDs = false);

	/**
	 * Refine an existing overlay topology.  For any element on a given triangle, if the predicate returns true, it gets topologically split out so it isn't shared by any other triangle.
	 * Used for creating sharp vertices in the normals overlay.
	 *
	 * @param ShouldSplitOutVertex predicate returns true of the element should be split out and not shared w/ any other triangle
	 * @param GetNewElementValue function to assign a new value to any element that is split out
	 */
	void SplitVerticesWithPredicate(TFunctionRef<bool(int ElementIdx, int TriID)> ShouldSplitOutVertex, TFunctionRef<void(int ElementIdx, int TriID, RealType* FillVect)> GetNewElementValue);

	/**
	 * Collapse SourceElementID into TargetElementID, resulting in connecting any containing triangles and reducing the total elements in the overlay.
	 *
	 * @param SourceElementID the element to merge away
	 * @param TargetElementID the element to merge into
	 * @return If the operation completed successfully, returns true
	 */
	bool MergeElement(int SourceElementID, int TargetElementID);

	/**
	 * Create a new copy of ElementID, and update connected triangles in the TrianglesToUpdate array to reference the copy of ElementID where they used to reference ElementID
	 * (Note: This just calls "SplitElementWithNewParent" with the existing element's parent id.)
	 *
	 * @param ElementID the element to copy
	 * @param TrianglesToUpdate the triangles that should now reference the new element
	 * @return the ID of the newly created element
	 */
	int SplitElement(int ElementID, const TArrayView<const int>& TrianglesToUpdate);

	/**
	* Create a new copy of ElementID, and update connected triangles in the TrianglesToUpdate array to reference the copy of ElementID where they used to reference ElementID.  The new element will have the given parent vertex ID.
	* Deletes any elements that are no longer used after the triangles are changed.
	*
	* @param ElementID the element to copy
	* @param SplitParentVertexID the new parent vertex for copied elements
	* @param TrianglesToUpdate the triangles that should now reference the new element.  Note: this is allowed to include triangles that do not have the element at all; sometimes you may want to do so to avoid creating a new array for each call.
	* @return the ID of the newly created element
	*/
	int SplitElementWithNewParent(int ElementID, int SplitParentVertexID, const TArrayView<const int>& TrianglesToUpdate);

	/**
	 * Split any bowties at given vertex.
	 * 
	 * @param NewElementIDs If not null, newly created element IDs are placed here. Note that this array is
	 *   intentionally not cleared before appending to it.
	 */
	void SplitBowtiesAtVertex(int32 Vid, TArray<int32>* NewElementIDs = nullptr);

	/**
	* Refine an existing overlay topology by splitting any bowties
	* @param bParallel Whether to run bowtie detection in parallel
	*/
	void SplitBowties(bool bParallel = true);


	//
	// Support for inserting element at specific ID. This is a bit tricky
	// because we likely will need to update the free list in the RefCountVector, which
	// can be expensive. If you are going to do many inserts (eg inside a loop), wrap in
	// BeginUnsafe / EndUnsafe calls, and pass bUnsafe = true to the InsertElement() calls,
	// to the defer free list rebuild until you are done.
	//

	/** Call this before a set of unsafe InsertVertex() calls */
	void BeginUnsafeElementsInsert()
	{
		// do nothing...
	}

	/** Call after a set of unsafe InsertVertex() calls to rebuild free list */
	void EndUnsafeElementsInsert()
	{
		ElementsRefCounts.RebuildFreeList();
	}

	/**
	 * Insert element at given index, assuming it is unused.
	 * If bUnsafe, we use fast id allocation that does not update free list.
	 * You should only be using this between BeginUnsafeElementsInsert() / EndUnsafeElementsInsert() calls
	 */
	EMeshResult InsertElement(int ElementID, const RealType* Value, bool bUnsafe = false);


	//
	// Accessors/Queries
	//  


	/** Get the element at a given index */
	inline void GetElement(int ElementID, RealType* Data) const
	{
		int k = ElementID * ElementSize;
		for (int i = 0; i < ElementSize; ++i) 
		{
			Data[i] = Elements[k + i];
		}
	}

	/** Get the element at a given index */
	template<typename AsType>
	void GetElement(int ElementID, AsType& Data) const
	{
		int k = ElementID * ElementSize;
		for (int i = 0; i < ElementSize; ++i) 
		{
			Data[i] = Elements[k + i];
		}
	}
	
	/**
	 * Get the Element value associated with a vertex of a triangle.
	 * 
	 * @param TriangleID ID of a triangle containing the Element
	 * @param VertexID ID of the Element's parent vertex 
	 * @param Data Value contained at the Element
	 */
	template<typename AsType>
	inline void GetElementAtVertex(int TriangleID, int VertexID, AsType& Data) const
	{
		int ElementID = GetElementIDAtVertex(TriangleID, VertexID);
		
		checkSlow(ElementID != IndexConstants::InvalidID);
		if (ElementID != IndexConstants::InvalidID) 
		{
			GetElement(ElementID, Data);
		}
	}	

	/** Get the parent vertex id for the element at a given index */
	inline int GetParentVertex(int ElementID) const
	{
		return ParentVertices[ElementID];
	}


	/** Get the element index tuple for a triangle */
	inline FIndex3i GetTriangle(int TriangleID) const 
	{
		int i = 3 * TriangleID;
		return FIndex3i(ElementTriangles[i], ElementTriangles[i + 1], ElementTriangles[i + 2]);
	}

	/** If the triangle is set to valid element indices, return the indices in TriangleOut and return true, otherwise return false */
	inline bool GetTriangleIfValid(int TriangleID, FIndex3i& TriangleOut) const 
	{
		int i = 3 * TriangleID;
		int a = ElementTriangles[i];
		if ( a >= 0 )
		{
			TriangleOut = FIndex3i(a, ElementTriangles[i+1], ElementTriangles[i+2]);
			checkSlow(TriangleOut.B >= 0 && TriangleOut.C >= 0);
			return true;
		}
		return false;
	}

	/** Set the element at a given index */
	inline void SetElement(int ElementID, const RealType* Data)
	{
		int k = ElementID * ElementSize;
		for (int i = 0; i < ElementSize; ++i)
		{
			Elements[k + i] = Data[i];
		}
	}

	/** Set the element at a given index */
	template<typename AsType>
	void SetElement(int ElementID, const AsType& Data)
	{
		int k = ElementID * ElementSize;
		for (int i = 0; i < ElementSize; ++i)
		{
			Elements[k + i] = Data[i];
		}
	}

	/** @return true if triangle contains element */
	inline bool TriangleHasElement(int TriangleID, int ElementID) const
	{
		int i = 3 * TriangleID;
		return (ElementTriangles[i] == ElementID || ElementTriangles[i+1] == ElementID || ElementTriangles[i+2] == ElementID);
	}


	/** Returns true if the parent-mesh edge is a "Seam" in this overlay.  
	*   If present, bIsNonIntersectingOut will be true only if this is a seam edge 
	*   that does not intersect with another seam or the end of the seam. 
	*/
	bool IsSeamEdge(int EdgeID, bool* bIsNonIntersectingOut = nullptr) const;
	/** Returns true if the parent-mesh edge is a "Seam End" in this overlay, meaning the adjacent element triangles share one element, not two */
	bool IsSeamEndEdge(int EdgeID) const;
	/** Returns true if the parent-mesh vertex is connected to any seam edges */
	bool IsSeamVertex(int VertexID, bool bBoundaryIsSeam = true) const;
	/** Returns true if the parent-mesh vertex is at a seam 'intersection' -- i.e., the end of a seam, or the intersection w/ another seam. */
	bool IsSeamIntersectionVertex(int32 VertexID) const;

	/**
	 * Determines whether the base-mesh vertex has "bowtie" topology in the Overlay.
	 * Bowtie topology means that one or more elements at the vertex are shared across disconnected UV-components.
	 * @return true if the base-mesh vertex has "bowtie" topology in the overlay
	 */
	bool IsBowtieInOverlay(int32 VertexID) const;

	/** @return true if the two triangles are connected, ie shared edge exists and is not a seam edge */
	bool AreTrianglesConnected(int TriangleID0, int TriangleID1) const;

	/** find the elements associated with a given parent-mesh vertex */
	void GetVertexElements(int VertexID, TArray<int>& OutElements) const;
	/** Count the number of unique elements for a given parent-mesh vertex */
	int CountVertexElements(int VertexID, bool bBruteForce = false) const;

	/** find the triangles connected to an element */
	void GetElementTriangles(int ElementID, TArray<int>& OutTriangles) const;

	/** 
	 * Find the element ID at a vertex of a triangle. 
	 * 
	 * @return Returns the element ID or FDynamicMesh3::InvalidID if the vertex is not a parent of any element contained in the triangle. 
	 */
	int GetElementIDAtVertex(int TriangleID, int VertexID) const;

	/**
	 * Find an element ID associated with the given VertexID
	 * 
	 * @param VertexID The vertex ID to search
	 * @param OutElementID Will be set to the found element ID, or FDynamicMesh3::InvalidID if none found
	 * @return true if an element ID was found, false otherwise
	 */
	bool FindAnyElementIDAtVertex(int32 VertexID, int32& OutElementID) const;

	/** @return true if overlay has any interior seam edges. This requires an O(N) search unless it early-outs. */
	bool HasInteriorSeamEdges() const;

	/**
	 * Compute interpolated parameter value inside triangle using barycentric coordinates
	 * @param TriangleID index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @param DataOut resulting interpolated overlay parameter value (of size ElementSize)
	 */
	template<typename AsType>
	void GetTriBaryInterpolate(int32 TriangleID, const AsType* BaryCoords, AsType* DataOut) const
	{
		int32 TriIndex = 3 * TriangleID;
		int32 ElemIndex0 = ElementTriangles[TriIndex] * ElementSize;
		int32 ElemIndex1 = ElementTriangles[TriIndex+1] * ElementSize;
		int32 ElemIndex2 = ElementTriangles[TriIndex+2] * ElementSize;
		const AsType Bary0 = (AsType)BaryCoords[0], Bary1 = (AsType)BaryCoords[1], Bary2 = (AsType)BaryCoords[2];
		for (int32 i = 0; i < ElementSize; ++i)
		{
			DataOut[i] = Bary0*(AsType)Elements[ElemIndex0+i] + Bary1*(AsType)Elements[ElemIndex1+i] + Bary2*(AsType)Elements[ElemIndex2+i];
		}
	}

	/**
	 * Checks that the overlay mesh is well-formed, ie all internal data structures are consistent
	 */
	bool CheckValidity(bool bAllowNonManifoldVertices = true, EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check) const;

	/**
	 * Returns true if this overlay is the same as Other.
	 */
	bool IsSameAs(const TDynamicMeshOverlay<RealType, ElementSize>& Other, bool bIgnoreDataLayout) const;

	/**
	 * Serialization operator for FDynamicMeshOverlay.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Overlay Mesh overlay to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, TDynamicMeshOverlay<RealType,ElementSize>& Overlay)
	{
		Overlay.Serialize(Ar, nullptr, false);
		return Ar;
	}

	/**
	 * Serialize to and from an archive.
	 *
	 * @param Ar Archive to serialize with.
	 * @param CompactMaps If this is not a null pointer, the mesh serialization compacted the vertex and/or triangle data using the provided mapping. 
	 * @param bUseCompression Use compression for serializing bulk data.
	 */
	void Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression);

	SIZE_T GetByteCount() const
	{
		return ElementsRefCounts.GetByteCount() + Elements.GetByteCount() + ParentVertices.GetByteCount() + ElementTriangles.GetByteCount();
	}

public:
	/** Set a triangle's element indices to InvalidID */
	void InitializeNewTriangle(int TriangleID);
	/** Remove a triangle from the overlay */
	void OnRemoveTriangle(int TriangleID);
	/** Reverse the orientation of a triangle's elements */
	void OnReverseTriOrientation(int TriangleID);
	/** Update the overlay to reflect an edge split in the parent mesh */
	void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo& SplitInfo);
	/** Update the overlay to reflect an edge flip in the parent mesh */
	void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo);
	/** Update the overlay to reflect an edge collapse in the parent mesh */
	void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo& CollapseInfo);
	/** Update the overlay to reflect a face poke in the parent mesh */
	void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo& PokeInfo);
	/** Update the overlay to reflect an edge merge in the parent mesh */
	void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo& MergeInfo);
	/** Update the overlay to reflect a vertex merge in the parent mesh */
	void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& MergeInfo);
	/** Update the overlay to reflect a vertex split in the parent mesh */
	void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate);

protected:
	/** Set the value at an Element to be a linear interpolation of two other Elements */
	void SetElementFromLerp(int SetElement, int ElementA, int ElementB, double Alpha);
	/** Set the value at an Element to be a barycentric interpolation of three other Elements */
	void SetElementFromBary(int SetElement, int ElementA, int ElementB, int ElementC, const FVector3d& BaryCoords);

	/** updates the triangles array and optionally the element reference counts */
	void InternalSetTriangle(int TriangleID, const FIndex3i& TriElements, bool bUpdateRefCounts, bool bAllowElementFreeing = true);

};






/**
 * TDynamicMeshVectorOverlay is an convenient extension of TDynamicMeshOverlay that adds
 * a specific N-element Vector type to the template, and adds accessor functions
 * that convert between that N-element vector type and the N-element arrays used by TDynamicMeshOverlay.
 */
template<typename RealType, int ElementSize, typename InVectorType>
class TDynamicMeshVectorOverlay : public TDynamicMeshOverlay<RealType, ElementSize>
{
public:
	using BaseType = TDynamicMeshOverlay<RealType, ElementSize>;
	using VectorType = InVectorType;

	TDynamicMeshVectorOverlay()
		: TDynamicMeshOverlay<RealType, ElementSize>()
	{
	}

	TDynamicMeshVectorOverlay(FDynamicMesh3* parentMesh) 
		: TDynamicMeshOverlay<RealType, ElementSize>(parentMesh)
	{
	}

	/**
	 * Append a new Element to the overlay
	 */
	inline int AppendElement(const VectorType& Value)
	{
		// Cannot use cast operator here because Core Vector types do not define it.
		// However assuming that vector has .X member is also not good...
		//return BaseType::AppendElement((const RealType*)Value);
		return BaseType::AppendElement(&Value.X);
	}

	/**
	 * Append a new Element to the overlay
	 */
	inline int AppendElement(const RealType* Value)
	{
		return BaseType::AppendElement(Value);
	}

	/**
	 * Get Element at a specific ID
	 */
	inline VectorType GetElement(int ElementID) const
	{
		VectorType V;
		BaseType::GetElement(ElementID, V);
		return V;
	}

	/**
	 * Get Element at a specific ID
	 */
	inline void GetElement(int ElementID, VectorType& V) const
	{
		BaseType::GetElement(ElementID, V);
	}

	/**
	 * Get the Element value associated with a vertex of a triangle.
	 */
	inline VectorType GetElementAtVertex(int TriangleID, int VertexID) const
	{
		VectorType V;
		BaseType::GetElementAtVertex(TriangleID, VertexID, V);
		return V;
	}

	/**
	 * Get the Element value associated with a vertex of a triangle.
	 */
	inline void GetElementAtVertex(int TriangleID, int VertexID, VectorType& V) const
	{
		BaseType::GetElementAtVertex(TriangleID, VertexID, V);
	}

	/**
	 * Get the Element associated with a vertex of a triangle
	 * @param TriVertexIndex index of vertex in triangle, valid values are 0,1,2
	 */
	inline void GetTriElement(int TriangleID, int32 TriVertexIndex, VectorType& Value) const
	{
		checkSlow(TriVertexIndex >= 0 && TriVertexIndex <= 2);
		GetElement(BaseType::ElementTriangles[(3 * TriangleID) + TriVertexIndex], Value);
	}

	/**
	 * Get the three Elements associated with a triangle
	 */
	inline void GetTriElements(int TriangleID, VectorType& A, VectorType& B, VectorType& C) const
	{
		int i = 3 * TriangleID;
		GetElement(BaseType::ElementTriangles[i], A);
		GetElement(BaseType::ElementTriangles[i+1], B);
		GetElement(BaseType::ElementTriangles[i+2], C);
	}


	/**
	 * Set Element at a specific ID
	 */
	inline void SetElement(int ElementID, const VectorType& Value)
	{
		BaseType::SetElement(ElementID, Value);
	}

	/**
	 * Iterate through triangles connected to VertexID and call ProcessFunc for each per-triangle-vertex Element with its Value.
	 * ProcessFunc must return true to continue the enumeration, or false to early-terminate it
	 * 
	 * @param bFindUniqueElements if true, ProcessFunc is only called once for each ElementID, otherwise it is called once for each Triangle
	 * @return true if at least one valid Element was found, ie if ProcessFunc was called at least one time
	 */
	bool EnumerateVertexElements(
		int VertexID,
		TFunctionRef<bool(int TriangleID, int ElementID, const VectorType& Value)> ProcessFunc,
		bool bFindUniqueElements = true) const;
};


#if PLATFORM_COMPILER_CLANG
#define UE_EXTERN_TEMPLATE_API GEOMETRYCORE_API
#else
#define UE_EXTERN_TEMPLATE_API
#endif

#if !UE_MERGED_MODULES
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<float, 1>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<double, 1>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<int, 1>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<float, 2>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<double, 2>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<int, 2>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<float, 3>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<double, 3>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<int, 3>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<float, 4>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshOverlay<double, 4>;

extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<float, 2, FVector2f>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<double, 2, FVector2d>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<int, 2, FVector2i>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<float, 3, FVector3f>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<double, 3, FVector3d>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<int, 3, FVector3i>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<float, 4, FVector4f>;
extern template class UE_EXTERN_TEMPLATE_API TDynamicMeshVectorOverlay<double, 4, FVector4d>;
#endif

#undef UE_EXTERN_TEMPLATE_API

} // end namespace UE::Geometry
} // end namespace UE