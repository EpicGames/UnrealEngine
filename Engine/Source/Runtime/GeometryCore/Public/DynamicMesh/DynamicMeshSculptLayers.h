// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/IndirectArray.h"
#include "DynamicMesh/DynamicAttribute.h"
#include "DynamicMesh/DynamicVertexAttribute.h"


namespace UE::Geometry
{


/** Per-vertex position offsets */
typedef TDynamicMeshVertexAttribute<double, 3> FDynamicMeshSculptLayerAttribute;


// Manages a dynamic mesh attribute set's sculpt layer data
class FDynamicMeshSculptLayers final
{
public:

	inline int32 NumLayers() const
	{
		return Layers.Num();
	}

	inline bool IsEnabled() const
	{
		return !Layers.IsEmpty();
	}

	inline int32 GetActiveLayer() const
	{
		return ActiveLayer;
	}

	// Attempt to set the active sculpt layer to the requested index. Will clamp to a valid layer range. Note: If the requested layer has zero weight, will attempt to use a layer with non-zero weight instead.
	// @return The actual current active layer (may be different from the requested layer!)
	GEOMETRYCORE_API int32 SetActiveLayer(int32 LayerIndex);

	// Remove the given sculpt layer, discarding its contribution to the shape.
	// Note: Shifts all sculpt layers above this index downward, which may invalidate externally-held sculpt layer indices. May also change the active layer.
	GEOMETRYCORE_API bool DiscardSculptLayer(int32 LayerIndex);
	
	// Merge the contribution of a range of layers.
	// Note: Shifts all sculpt layers above this range downward, which may invalidate externally-held sculpt layer indices. May also change the active layer.
	// @param StartIndex Index of the first layer to merge
	// @param EndIndex Index of the last layer to merge
	// @param bUseWeights Whether to merge layers based on their current weight strength. Requires StartIndex have non-zero weight. If false, the mesh vertices may move after merge.
	GEOMETRYCORE_API bool MergeSculptLayers(int32 StartIndex, int32 EndIndex, bool bUseWeights);

	// Set new sculpt layer weights
	GEOMETRYCORE_API void UpdateLayerWeights(TConstArrayView<double> InLayerWeights);

	// moves the layer at the first provided index to the location given by the second provided index
	GEOMETRYCORE_API bool MoveLayer(const int32 InitLayerIndex, const int32 TargetIndex);

	// Directly access the sculpt layer data
	const FDynamicMeshSculptLayerAttribute* GetLayer(int32 LayerIndex) const
	{
		return Layers.IsValidIndex(LayerIndex) ? &Layers[LayerIndex] : nullptr;
	}
	
	// Directly access the sculpt layer data
	// Note: Must explicitly call "RebuildMesh" for modifications to be applied to the mesh vertex positions
	FDynamicMeshSculptLayerAttribute* GetLayer(int32 LayerIndex)
	{
		return Layers.IsValidIndex(LayerIndex) ? &Layers[LayerIndex] : nullptr;
	}

	// Get the current sculpt layer weights
	TConstArrayView<double> GetLayerWeights() const
	{
		return TConstArrayView<double>(LayerWeights);
	}

	// Rebuild mesh from sculpt layer offsets + weights, ignoring current mesh positions
	void RebuildMesh()
	{
		UpdateMeshFromLayers();
	}

	// Update the active layer's sculpt offsets so that the sum of sculpt layers w/ current weights gives the current mesh vertex positions.
	// Will fail if the active layer has zero weight, or if there are no sculpt layers.
	// @return true on success
	GEOMETRYCORE_API bool UpdateLayersFromMesh();

	// Copy across sculpt layer data via a vertex mapping, for all layers that exist on both this and the other layers
	GEOMETRYCORE_API void CopyThroughMapping(const FDynamicMeshSculptLayers& Other, const FMeshIndexMappings& Mapping);

private:

	// Sculpt layers are stored as vertex position offsets from the previous layer, with layer zero storing initial positions
	TIndirectArray<FDynamicMeshSculptLayerAttribute> Layers;
	// Weights per sculpt layer
	TArray<double> LayerWeights;
	// Indicates which layer is currently being edited (and is therefore reflected in the mesh vertices, rather than the layer data)
	int32 ActiveLayer = -1;

	friend class FDynamicMeshAttributeSet;


	// Internal helpers used by FDynamicMeshAttributeSet to manage layers

	GEOMETRYCORE_API void Copy(FDynamicMeshAttributeSet* AttributeSet, const FDynamicMeshSculptLayers& Copy);
	GEOMETRYCORE_API void CompactCopy(FDynamicMeshAttributeSet* AttributeSet, const FCompactMaps& CompactMaps, const FDynamicMeshSculptLayers& Copy);
	GEOMETRYCORE_API void EnableMatching(FDynamicMeshAttributeSet* AttributeSet, const FDynamicMeshSculptLayers& ToMatch, bool bClearExisting, bool bDiscardExtraAttributes);
	GEOMETRYCORE_API void Enable(FDynamicMeshAttributeSet* AttributeSet, int32 MinLayerCount = -1);
	GEOMETRYCORE_API void Discard(FDynamicMeshAttributeSet* AttributeSet);
	GEOMETRYCORE_API bool CheckValidity(const FDynamicMeshAttributeSet* AttributeSet, bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const;
	GEOMETRYCORE_API bool UpdateMeshFromLayers();
	GEOMETRYCORE_API bool ValidateActiveLayer();

	inline bool HasValidLayers() const
	{
		return Layers.Num() == LayerWeights.Num() && Layers.IsValidIndex(ActiveLayer);
	}
};


} // namespace UE::Geometry
