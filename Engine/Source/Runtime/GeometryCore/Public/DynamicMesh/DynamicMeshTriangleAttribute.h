// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicAttribute.h"

namespace UE
{
namespace Geometry
{

template<typename AttribValueType, int AttribDimension>
class TDynamicMeshTriangleAttribute;


template<typename AttribValueType, int AttribDimension>
class FDynamicMeshTriangleAttributeChange : public FDynamicMeshAttributeChangeBase
{
private:
	struct FChangeTriangleAttribute
	{
		AttribValueType Data[AttribDimension];
		int TriangleID;
	};

	TArray<FChangeTriangleAttribute> OldTriangleAttributes, NewTriangleAttributes;

public:
	FDynamicMeshTriangleAttributeChange()
	{}

	virtual ~FDynamicMeshTriangleAttributeChange() = default;

	inline virtual void SaveInitialTriangle(const FDynamicMeshAttributeBase* Attribute, int TriangleID) override;

	inline virtual void StoreAllFinalTriangles(const FDynamicMeshAttributeBase* Attribute, const TArray<int>& TriangleIDs) override;

	inline virtual bool Apply(FDynamicMeshAttributeBase* Attribute, bool bRevert) const override;
};



/**
 * TDynamicMeshTriangleAttribute is an add-on to a FDynamicMesh3 that allows for 
 * per-triangle storage of an attribute value.
 *
 * The FDynamicMesh3 mesh topology operations (eg split/flip/collapse edge, poke face, etc)
 * can be mirrored to the overlay via OnSplitEdge(), etc.
 */
template<typename AttribValueType, int AttribDimension>
class TDynamicMeshTriangleAttribute : public FDynamicMeshAttributeBase
{

protected:
	/** The parent mesh this overlay belongs to */
	FDynamicMesh3* ParentMesh;

	/** List of per-triangle attribute values */
	TDynamicVector<AttribValueType> AttribValues;

	using Super = FDynamicMeshAttributeBase;

	friend class FDynamicMesh3;
	friend class FDynamicMeshAttributeSet;

public:
	/** Create an empty overlay */
	TDynamicMeshTriangleAttribute()
	{
		ParentMesh = nullptr;
	}

	/** Create an overlay for the given parent mesh */
	TDynamicMeshTriangleAttribute(FDynamicMesh3* ParentMeshIn, bool bAutoInit = true)
	{
		ParentMesh = ParentMeshIn;
		if (bAutoInit)
		{
			Initialize();
		}
	}

private:
	/** @set the parent mesh for this overlay.  Only safe for use during FDynamicMesh move */
	void Reparent(FDynamicMesh3* ParentMeshIn)
	{
		ParentMesh = ParentMeshIn;
	}

public:
	/** @return the parent mesh for this overlay */
	const FDynamicMesh3* GetParentMesh() const { return ParentMesh; }
	/** @return the parent mesh for this overlay */
	FDynamicMesh3* GetParentMesh() { return ParentMesh; }

	virtual FDynamicMeshAttributeBase* MakeNew(FDynamicMesh3* ParentMeshIn) const override
	{
		TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* Matching = new TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>(ParentMeshIn);
		Matching->Initialize();
		return Matching;
	}

	virtual FDynamicMeshAttributeBase* MakeCopy(FDynamicMesh3* ParentMeshIn) const override
	{
		TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* ToFill = new TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>(ParentMeshIn);
		ToFill->Copy(*this);
		return ToFill;
	}

	/** Set this overlay to contain the same arrays as the copy overlay */
	void Copy(const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>& Copy)
	{
		CopyParentClassData(Copy);
		AttribValues = Copy.AttribValues;
	}

	virtual FDynamicMeshAttributeBase* MakeCompactCopy(const FCompactMaps& CompactMaps, FDynamicMesh3* ParentMeshIn) const override
	{
		TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* ToFill = new TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>(ParentMeshIn);
		ToFill->Initialize();
		ToFill->CompactCopy(CompactMaps, *this);
		return ToFill;
	}

	void CompactInPlace(const FCompactMaps& CompactMaps)
	{
		for (int TID = 0, NumTID = CompactMaps.NumTriangleMappings(); TID < NumTID; TID++)
		{
			const int ToTID = CompactMaps.GetTriangleMapping(TID);
			if (ToTID == FCompactMaps::InvalidID)
			{
				continue;
			}
			if (ensure(ToTID <= TID))
			{
				CopyValue(TID, ToTID);
			}
		}
		AttribValues.Resize(ParentMesh->MaxTriangleID() * AttribDimension);
	}

	void CompactCopy(const FCompactMaps& CompactMaps, const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>& ToCopy)
	{
		CopyParentClassData(ToCopy);
		check(CompactMaps.NumTriangleMappings() <= int(ToCopy.AttribValues.Num() / AttribDimension));
		AttribValueType Data[AttribDimension];
		for (int TID = 0, NumTID = CompactMaps.NumTriangleMappings(); TID < NumTID; TID++)
		{
			const int ToTID = CompactMaps.GetTriangleMapping(TID);
			if (ToTID == FCompactMaps::InvalidID)
			{
				continue;
			}
			ToCopy.GetValue(TID, Data);
			SetValue(ToTID, Data);
		}
	}


	/** Initialize the attribute values with InitialValue, and resize to the parent mesh's max triangle ID */
	void Initialize(AttribValueType InitialValue)
	{
		check(ParentMesh != nullptr);
		AttribValues.Resize(ParentMesh->MaxTriangleID() * AttribDimension);
		AttribValues.Fill(InitialValue);
	}

	void Initialize()
	{
		Initialize(GetDefaultAttributeValue());
	}

	void SetNewValue(int NewTriangleID, const AttribValueType* Data)
	{
		int k = NewTriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues.InsertAt(Data[i], k + i);
		}
	}



	//
	// Accessors/Queries
	//  

	virtual bool CopyThroughMapping(const TDynamicAttributeBase<FDynamicMesh3>* Source, const FMeshIndexMappings& Mapping) override
	{
		AttribValueType BufferData[AttribDimension];
		int BufferSize = sizeof(BufferData);
		for (const TPair<int32, int32>& MapTID : Mapping.GetTriangleMap().GetForwardMap())
		{
			if (!ensure(Source->CopyOut(MapTID.Key, BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(MapTID.Value, BufferData);
		}
		return true;
	}

	virtual bool Append(const TDynamicAttributeBase& Source, const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		int32 NewMaxID = Info.NumTriangle + Info.TriangleOffset;
		if (NewMaxID * AttribDimension > AttribValues.Num())
		{
			AttribValues.SetNum(NewMaxID * AttribDimension);
		}

		AttribValueType BufferData[AttribDimension];
		int BufferSize = sizeof(BufferData);
		for (int32 Idx = 0; Idx < Info.NumTriangle; ++Idx)
		{
			if (!ensure(Source.CopyOut(Idx, BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(Idx + Info.TriangleOffset, BufferData);
		}
		return true;
	}

	virtual void AppendDefaulted(const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		int32 NewMaxID = Info.NumTriangle + Info.TriangleOffset;
		AttribValues.SetMinimumSize(NewMaxID * AttribDimension, GetDefaultAttributeValue());
	}

	virtual bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		if (sizeof(AttribValueType) * AttribDimension != BufferSize)
		{
			return false;
		}
		AttribValueType* BufferData = static_cast<AttribValueType*>(Buffer);
		int k = RawID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			BufferData[i] = AttribValues[k + i];
		}
		return true;
	}
	virtual bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		if (sizeof(AttribValueType) * AttribDimension != BufferSize)
		{
			return false;
		}
		AttribValueType* BufferData = static_cast<AttribValueType*>(Buffer);
		int k = RawID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = BufferData[i];
		}
		return true;
	}

	/** Get the element at a given index */
	inline void GetValue(int TriangleID, AttribValueType* Data) const
	{
		int k = TriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			Data[i] = AttribValues[k + i];
		}
	}

	/** Get the element at a given index */
	template<typename AsType>
	void GetValue(int TriangleID, AsType& Data) const
	{
		int k = TriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			Data[i] = AttribValues[k + i];
		}
	}


	/** Set the element at a given index */
	inline void SetValue(int TriangleID, const AttribValueType* Data)
	{
		int k = TriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = Data[i];
		}
	}

	/** Set the element at a given index */
	template<typename AsType>
	void SetValue(int TriangleID, const AsType& Data)
	{
		int k = TriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = Data[i];
		}
	}

	/** Set the element at a given index with a scalar value (the same value for each dimension) */
	inline void SetScalarValue(int TriangleID, const AttribValueType& SingleValue)
	{
		int k = TriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = SingleValue;
		}
	}

	/**
	 * Copy the attribute value at FromTriangleID to ToTriangleID
	 */
	inline void CopyValue(int FromTriangleID, int ToTriangleID)
	{
		int kA = FromTriangleID * AttribDimension;
		int kB = ToTriangleID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues.InsertAt(AttribValues[kA+i], kB+i);
		}
	}


	/** Returns true if the parent-mesh edge is a "Seam" in this overlay */
	bool IsBorderEdge(int EdgeID, bool bMeshBoundaryIsBorder = true) const
	{
		FIndex2i EdgeTris = ParentMesh->GetEdgeT(EdgeID);
		if (EdgeTris.B == FDynamicMesh3::InvalidID)
		{
			return bMeshBoundaryIsBorder;
		}
		int kA = EdgeTris.A * AttribDimension;
		int kB = EdgeTris.B * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			if (AttribValues[kA+i] != AttribValues[kB+i])
			{
				return true;
			}
		}
		return false;
	}


	virtual TUniquePtr<FDynamicMeshAttributeChangeBase> NewBlankChange() const override
	{
		return MakeUnique<FDynamicMeshTriangleAttributeChange<AttribValueType, AttribDimension>>();
	}


public:

	/** Update the overlay to reflect an edge split in the parent mesh */
	void OnSplitEdge(const DynamicMeshInfo::FEdgeSplitInfo& SplitInfo) override
	{
		CopyValue(SplitInfo.OriginalTriangles.A, SplitInfo.NewTriangles.A);
		if (SplitInfo.OriginalTriangles.B != FDynamicMesh3::InvalidID)
		{
			CopyValue(SplitInfo.OriginalTriangles.B, SplitInfo.NewTriangles.B);
		}
	}

	/** Update the overlay to reflect an edge flip in the parent mesh */
	void OnFlipEdge(const DynamicMeshInfo::FEdgeFlipInfo& FlipInfo) override
	{
		// yikes! triangles did not actually change so we will leave attrib unmodified
	}

	/** Update the overlay to reflect an edge collapse in the parent mesh */
	void OnCollapseEdge(const DynamicMeshInfo::FEdgeCollapseInfo& CollapseInfo) override
	{
		// nothing to do here, triangles were only deleted
	}

	/** Update the overlay to reflect a face poke in the parent mesh */
	void OnPokeTriangle(const DynamicMeshInfo::FPokeTriangleInfo& PokeInfo) override
	{
		CopyValue(PokeInfo.OriginalTriangle, PokeInfo.NewTriangles.A);
		CopyValue(PokeInfo.OriginalTriangle, PokeInfo.NewTriangles.B);
	}

	/** Update the overlay to reflect an edge merge in the parent mesh */
	void OnMergeEdges(const DynamicMeshInfo::FMergeEdgesInfo& MergeInfo) override
	{
		// nothing to do here because triangles did not change
	}

	void OnMergeVertices(const DynamicMeshInfo::FMergeVerticesInfo& MergeInfo) override
	{
		// This resolves as either an edge collapse, edge weld, or merge of disconnected vertices. 
		//  The triangles either get removed or unchanged- nothing more to do here.
	}

	/** Update the overlay to reflect a vertex split in the parent */
	void OnSplitVertex(const DynamicMeshInfo::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate) override
	{
		// nothing to do here because triangles did not change
	}

	virtual AttribValueType GetDefaultAttributeValue()
	{
		return AttribValueType();
	}

	inline void ResizeAttribStoreIfNeeded(int TriangleID)
	{
		if (!ensure(TriangleID >= 0))
		{
			return;
		}
		size_t NeededSize = (((size_t)TriangleID+1) * AttribDimension);
		if (NeededSize > AttribValues.Num())
		{
			AttribValues.Resize(NeededSize, GetDefaultAttributeValue());
		}
	}

	virtual void OnNewTriangle(int TriangleID, bool bInserted) override
	{
		ResizeAttribStoreIfNeeded(TriangleID);
		SetScalarValue(TriangleID, GetDefaultAttributeValue());
	}

public:
	/**
	 * Returns true if this AttributeSet is the same as Other.
	 */
	bool IsSameAs(const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>& Other, bool bIgnoreDataLayout) const
	{
		if (!bIgnoreDataLayout)
		{
			if (AttribValues.Num() != Other.AttribValues.Num())
			{
				return false;
			}

			for (int Idx = 0, NumValues = AttribValues.Num(); Idx < NumValues; Idx++)
			{
				if (AttribValues[Idx] != Other.AttribValues[Idx])
				{
					return false;
				}
			}
		}
		else
		{
			FRefCountVector::IndexIterator ItTid = ParentMesh->GetTrianglesRefCounts().BeginIndices();
			const FRefCountVector::IndexIterator ItTidEnd = ParentMesh->GetTrianglesRefCounts().EndIndices();
			FRefCountVector::IndexIterator ItTidOther = Other.ParentMesh->GetTrianglesRefCounts().BeginIndices();
			const FRefCountVector::IndexIterator ItTidEndOther = Other.ParentMesh->GetTrianglesRefCounts().EndIndices();

			while (ItTid != ItTidEnd && ItTidOther != ItTidEndOther)
			{
				for (int32 i = 0; i < AttribDimension; ++i)
				{
					const AttribValueType AttribValue = AttribValues[*ItTid * AttribDimension + i];
					const AttribValueType AttribValueOther = Other.AttribValues[*ItTidOther * AttribDimension + i];
					if (AttribValue != AttribValueOther)
					{
						// Triangle attribute value is not the same.
						return false;
					}
				}
				++ItTid;
				++ItTidOther;
			}

			if (ItTid != ItTidEnd || ItTidOther != ItTidEndOther)
			{
				// Number of triangle attribute values is not the same.
				return false;
			}
		}

		return true;
	}

	/**
	* Check validity of attribute
	* 
	* @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	* @param FailMode Desired behavior if mesh is found invalid
	*/
	virtual bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const override
	{
		// just check that the values buffer is big enough
		if (!ParentMesh || ParentMesh->MaxTriangleID() < 0 || static_cast<size_t>(ParentMesh->MaxTriangleID())*AttribDimension > AttribValues.Num())
		{
			switch (FailMode)
			{
			case EValidityCheckFailMode::Check:
				check(false);
				return false;
			case EValidityCheckFailMode::Ensure:
				ensure(false);
				return false;
			default:
				return false;
			}
		}

		return true;
	}
	
	/**
	 * Serialization operator for TDynamicMeshTriangleAttribute.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Attr Mesh triangle attribute to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>& Attr)
	{
		Attr.Serialize(Ar, nullptr, false);
		return Ar;
	}

	/**
	* Serialize to and from an archive.
	*
	* @param Ar Archive to serialize with.
	* @param CompactMaps If this is not a null pointer, the mesh serialization compacted the vertex and/or triangle data using the provided mapping. 
	* @param bUseCompression Use compression for serializing bulk data.
	*/
	void Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
	{
		Super::Serialize(Ar);

		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
		{
			Ar << AttribValues;
		}
		else
		{
			auto SerializeVector = [](FArchive& Ar, auto& Vector, bool bUseCompression)
			{
				if (bUseCompression)
				{
					Vector.template Serialize<true, true>(Ar);
				}
				else
				{
					Vector.template Serialize<true, false>(Ar);
				}
			};

			Ar << bUseCompression;

			if (CompactMaps == nullptr || !CompactMaps->TriangleMapIsSet())
			{
				SerializeVector(Ar, AttribValues, bUseCompression);
			}
			else
			{
				TDynamicVector<AttribValueType> AttribValuesCompact;
				AttribValuesCompact.SetNum(ParentMesh->TriangleCount() * AttribDimension);

				int32 TidCompact = 0;
				for (int32 Tid = 0, Num = AttribValues.Num() / AttribDimension; Tid < Num; ++Tid)
				{
					const int32 TidMapping = CompactMaps->GetTriangleMapping(Tid);
					if (TidMapping != FCompactMaps::InvalidID)
					{
						for (int32 i = 0; i < AttribDimension; ++i)
						{
							AttribValuesCompact[TidCompact++] = AttribValues[Tid * AttribDimension + i];
						}
					}
				}

				SerializeVector(Ar, AttribValuesCompact, bUseCompression);
			}
		}
	}

	virtual SIZE_T GetByteCount() const
	{
		return AttribValues.GetByteCount();
	}
};



template<typename AttribValueType, int AttribDimension>
void FDynamicMeshTriangleAttributeChange<AttribValueType, AttribDimension>::SaveInitialTriangle(const FDynamicMeshAttributeBase* Attribute, int TriangleID)
{
	FChangeTriangleAttribute& Change = OldTriangleAttributes.Emplace_GetRef();
	Change.TriangleID = TriangleID;
	const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* AttribCast = static_cast<const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>*>(Attribute);
	AttribCast->GetValue(TriangleID, Change.Data);
}

template<typename AttribValueType, int AttribDimension>
void FDynamicMeshTriangleAttributeChange<AttribValueType, AttribDimension>::StoreAllFinalTriangles(const FDynamicMeshAttributeBase* Attribute, const TArray<int>& TriangleIDs)
{
	const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* AttribCast = static_cast<const TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>*>(Attribute);
	NewTriangleAttributes.Reserve(NewTriangleAttributes.Num() + TriangleIDs.Num());
	for (int TriangleID : TriangleIDs)
	{
		FChangeTriangleAttribute& Change = NewTriangleAttributes.Emplace_GetRef();
		Change.TriangleID = TriangleID;
		AttribCast->GetValue(TriangleID, Change.Data);
	}
}

template<typename AttribValueType, int AttribDimension>
bool FDynamicMeshTriangleAttributeChange<AttribValueType, AttribDimension>::Apply(FDynamicMeshAttributeBase* Attribute, bool bRevert) const
{
	TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>* AttribCast = static_cast<TDynamicMeshTriangleAttribute<AttribValueType, AttribDimension>*>(Attribute);
	const TArray<FChangeTriangleAttribute> *Changes = bRevert ? &OldTriangleAttributes : &NewTriangleAttributes;
	for (const FChangeTriangleAttribute& Change : *Changes)
	{
		bool bIsTriangle = AttribCast->GetParentMesh()->IsTriangle(Change.TriangleID);
		checkSlow(bIsTriangle);
		if (bIsTriangle)
		{
			AttribCast->SetValue(Change.TriangleID, Change.Data);
		}
	}
	return true;
}



/**
 * TDynamicMeshScalarTriangleAttribute is an extension of TDynamicMeshTriangleAttribute for scalar-valued attributes.
 * Adds some convenience functions to simplify get/set code.
 */
template<typename RealType>
class TDynamicMeshScalarTriangleAttribute : public TDynamicMeshTriangleAttribute<RealType, 1>
{
public:
	using BaseType = TDynamicMeshTriangleAttribute<RealType, 1>;
	using BaseType::SetNewValue;
	using BaseType::GetValue;
	using BaseType::SetValue;

	TDynamicMeshScalarTriangleAttribute() : BaseType()
	{
	}

	TDynamicMeshScalarTriangleAttribute(FDynamicMesh3* ParentMeshIn) : BaseType(ParentMeshIn)
	{
	}

	inline void SetNewValue(int NewTriangleID, RealType Value)
	{
		this->AttribValues.InsertAt(Value, NewTriangleID);
	}

	inline RealType GetValue(int TriangleID) const
	{
		return this->AttribValues[TriangleID];
	}

	inline void SetValue(int TriangleID, RealType Value)
	{
		this->AttribValues[TriangleID] = Value;
	}
};

template<typename AttribValueType>
class TDynamicMeshSingleTriangleAttribute : public TDynamicMeshTriangleAttribute<AttribValueType, 1>
{
	using BaseType = TDynamicMeshTriangleAttribute<AttribValueType, 1>;
	
public:
	TDynamicMeshSingleTriangleAttribute() = default;

	TDynamicMeshSingleTriangleAttribute(FDynamicMesh3* ParentMeshIn, bool bAutoInit = true)
		: BaseType(ParentMeshIn, bAutoInit)
	{}

	inline AttribValueType GetValue(int TriangleID) const
	{
		return this->AttribValues[TriangleID];
	}
	
	inline void SetValue(int TriangleID, const AttribValueType& Value)
	{
		this->AttribValues[TriangleID] = Value;
	}
};

using FDynamicMeshTriangleLabelAttribute = TDynamicMeshSingleTriangleAttribute<FName>;

} // end namespace UE::Geometry
} // end namespace UE
