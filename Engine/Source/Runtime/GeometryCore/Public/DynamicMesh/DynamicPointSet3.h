// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "GeometryTypes.h"
#include "MathUtil.h"
#include "Util/DynamicVector.h"
#include "Util/IndexUtil.h"
#include "Util/IteratorUtil.h"
#include "Util/RefCountVector.h"
#include "VectorTypes.h"
#include "VectorUtil.h"
#include "DynamicMesh/DynamicAttribute.h"

class FDynamicAttributeSetBase;

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TDynamicPointSet3 implements a dynamic 3D point set, templated on real-value type (float or double).
 * The points are indexed and the class allows for gaps in the index space.
 * 
 * The points are referred to as "Vertices" for consistency with higher-level Mesh and Graph classes.
 * This class can be used interchangeably with the Vertices of those classes in some cases
 * (for example in spatial data structures like TDynamicVerticesOctree3)
 *
 * The point data is stored in via TDynamicVector<RealType>, which internally uses chunked storage
 * and so is relatively efficient to grow/shrink/etc
 *
 * Reference counts for the points are stored via a FRefCountVector instance
 *
 * A per-point attribute set can optionally be attached. Currently the client is
 * responsible for managing the memory for this attribute set. The attribute set
 * will be automatically updated when points are added or removed.
 */
template<typename RealType>
class TDynamicPointSet3
{
protected:
	/** Reference counts of vertex indices. Iterate over this to find out which vertex indices are valid. */
	FRefCountVector VertexRefCounts;
	/** List of vertex positions */
	TDynamicVector<RealType> Vertices;


	/** Base attribute set */
	TDynamicAttributeSetBase<TDynamicPointSet3<RealType>>* BaseAttributeSet = nullptr;


	//
	// Attribute Set Support. At the level of this class, we do not have a specific attribute set.
	// Clients can provide a pointer to their attribute set so that functions like AppendVertex()
	// or DeleteVertex() will work.
	//
public:

	void SetExternallyManagedAttributes(TDynamicAttributeSetBase<TDynamicPointSet3<RealType>>* AttributeSet)
	{
		BaseAttributeSet = AttributeSet;
	}

	TDynamicAttributeSetBase<TDynamicPointSet3<RealType>>* GetBaseAttributeSet() const
	{
		return BaseAttributeSet;
	}



	//
	// Indices
	//
public:

	/** @return number of vertices in the point set */
	int VertexCount() const
	{
		return (int)VertexRefCounts.GetCount();
	}

	/** @return upper bound on vertex IDs used in the point set, ie all vertex IDs in use are < MaxVertexID */
	int MaxVertexID() const
	{
		return (int)VertexRefCounts.GetMaxIndex();
	}


	/** @return true if VertexID is a valid vertex in this point set */
	inline bool IsVertex(int VertexID) const
	{
		return VertexRefCounts.IsValid(VertexID);
	}



	//
	// Construction
	//
public:

	/** Discard all data */
	void Clear()
	{
		Vertices.Clear();
		VertexRefCounts = FRefCountVector();
		BaseAttributeSet = nullptr;
	}


	/** Append vertex at position, returns vid */
	int AppendVertex(const TVector<RealType>& Position)
	{
		int vid = VertexRefCounts.Allocate();
		int i = 3 * vid;
		Vertices.InsertAt(Position[2], i + 2);
		Vertices.InsertAt(Position[1], i + 1);
		Vertices.InsertAt(Position[0], i);

		if (GetBaseAttributeSet() != nullptr)
		{
			GetBaseAttributeSet()->OnNewVertex(vid, false);
		}

		return vid;
	}

	/** Call this before a set of unsafe InsertVertex() calls */
	void BeginUnsafeVerticesInsert()
	{
		// do nothing...
	}

	/** Call after a set of unsafe InsertVertex() calls to rebuild free list */
	void EndUnsafeVerticesInsert()
	{
		VertexRefCounts.RebuildFreeList();
	}

	/**
	 * Insert vertex at given index, assuming it is unused.
	 * If bUnsafe, we use fast id allocation that does not update free list.
	 * You should only be using this between BeginUnsafeVerticesInsert() / EndUnsafeVerticesInsert() calls
	 */
	EMeshResult InsertVertex(int VertexID, const TVector<RealType>& Position, bool bUnsafe = false)
	{
		if (VertexRefCounts.IsValid(VertexID))
		{
			return EMeshResult::Failed_VertexAlreadyExists;
		}

		bool bOK = (bUnsafe) ? VertexRefCounts.AllocateAtUnsafe(VertexID) : VertexRefCounts.AllocateAt(VertexID);
		if (bOK == false)
		{
			return EMeshResult::Failed_CannotAllocateVertex;
		}

		int i = 3 * VertexID;
		Vertices.InsertAt(Position[2], i + 2);
		Vertices.InsertAt(Position[1], i + 1);
		Vertices.InsertAt(Position[0], i);

		if (GetBaseAttributeSet() != nullptr)
		{
			GetBaseAttributeSet()->OnNewVertex(VertexID, true);
		}

		return EMeshResult::Ok;
	}



	//
	// get/set
	//
public:

	/** @return the vertex position */
	inline TVector<RealType> GetVertex(int VertexID) const
	{
		check(IsVertex(VertexID));
		int i = 3 * VertexID;
		return TVector<RealType>(Vertices[i], Vertices[i + 1], Vertices[i + 2]);
	}

	/** Set vertex position */
	inline void SetVertex(int VertexID, const TVector<RealType>& vNewPos)
	{
		//check(VectorUtil::IsFinite(vNewPos));
		check(IsVertex(VertexID));
		int i = 3 * VertexID;
		Vertices[i] = vNewPos.X;
		Vertices[i + 1] = vNewPos.Y;
		Vertices[i + 2] = vNewPos.Z;
	}



	//
	// iterators
	//
public:
	/** @return enumerable object for valid vertex indices suitable for use with range-based for, ie for ( int i : VertexIndicesItr() ) */
	FRefCountVector::IndexEnumerable VertexIndicesItr() const
	{
		return VertexRefCounts.Indices();
	}

	/** Enumerate positions of all points */
	FRefCountVector::MappedEnumerable<TVector<RealType>> VerticesItr() const
	{
		return VertexRefCounts.MappedIndices<TVector<RealType>>([this](int VertexID) {
			int i = 3 * VertexID;
			return TVector<RealType>(Vertices[i], Vertices[i + 1], Vertices[i + 2]);
		});
	}




	/** @return true if vertex count == max vertex id */
	bool IsCompact() const
	{
		return VertexRefCounts.IsDense();
	}

	/** returns measure of compactness in range [0,1], where 1 is fully compacted */
	RealType CompactMetric() const
	{
		return ((RealType)VertexCount() / (RealType)MaxVertexID());
	}

	//
	// Geometric queries
	//
public:

	/** Returns bounding box of all points */
	TAxisAlignedBox3<RealType> GetBounds() const
	{
		TAxisAlignedBox3<RealType> Box = TAxisAlignedBox3<RealType>::Empty();
		for (int vi : VertexIndicesItr())
		{
			int k = 3 * vi;
			Box.Contain( TVector<RealType>(Vertices[k], Vertices[k + 1], Vertices[k + 2]) );
		}
		return Box;
	}



	//
	// direct buffer access  (not a good idea to use)
	//
public:
	const TDynamicVector<double>& GetVerticesBuffer()
	{
		return Vertices;
	}
	const FRefCountVector& GetVerticesRefCounts()
	{
		return VertexRefCounts;
	}



	//
	// Edit operations
	//
public:

	// @todo port this from DynamicMesh3
	//void CompactInPlace(FCompactMaps* CompactInfo = nullptr);

	/**
	 * Remove point
	 */
	EMeshResult RemoveVertex(int VertexID)
	{
		if (VertexRefCounts.IsValid(VertexID) == false)
		{
			return EMeshResult::Failed_NotAVertex;
		}
		VertexRefCounts.Decrement(VertexID);
		check(VertexRefCounts.IsValid(VertexID) == false);

		if (GetBaseAttributeSet() != nullptr)
		{
			GetBaseAttributeSet()->OnRemoveVertex(VertexID);
		}

		return EMeshResult::Ok;
	}

};



typedef TDynamicPointSet3<float> FDynamicPointSet3f;
typedef TDynamicPointSet3<double> FDynamicPointSet3d;


} // end namespace UE::Geometry
} // end namespace UE