// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicAttribute.h"

namespace UE
{
namespace Geometry
{

template<typename AttribValueType, int AttribDimension, typename ParentType>
class TDynamicVertexAttribute;


template<typename AttribValueType, int AttribDimension, typename ParentType>
class TDynamicVertexAttributeChange : public TDynamicAttributeChangeBase<ParentType>
{
private:
	struct FChangeVertexAttribute
	{
		AttribValueType Data[AttribDimension];
		int VertexID;
	};

	TArray<FChangeVertexAttribute> OldVertexAttributes, NewVertexAttributes;

public:
	TDynamicVertexAttributeChange()
	{}

	virtual ~TDynamicVertexAttributeChange() = default;

	inline virtual void SaveInitialVertex(const TDynamicAttributeBase<ParentType>* Attribute, int VertexID) override;

	inline virtual void StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TSet<int>& VertexIDs) override;

	inline virtual bool Apply(TDynamicAttributeBase<ParentType>* Attribute, bool bRevert) const override;
};


/**
 * TDynamicVertexAttribute provides per-vertex storage of an attribute value
 */
template<typename AttribValueType, int AttribDimension, typename ParentType>
class TDynamicVertexAttribute : public TDynamicAttributeBase<ParentType>
{

protected:
	/** The parent object (e.g. mesh, point set) this attribute belongs to */
	ParentType* Parent;

	/** List of per-vertex attribute values */
	TDynamicVector<AttribValueType> AttribValues;

	using Super = TDynamicAttributeBase<ParentType>;

	friend class FDynamicMeshAttributeSet;

public:
	/** Create an empty overlay */
	TDynamicVertexAttribute() : Parent(nullptr)
	{
	}

	/** Create an attribute for the given parent */
	TDynamicVertexAttribute(ParentType* ParentIn, bool bAutoInit = true) : Parent(ParentIn)
	{
		if (bAutoInit)
		{
			Initialize();
		}
	}

	virtual ~TDynamicVertexAttribute() = default;

	/** @return the parent for this attribute */
	const ParentType* GetParent() const { return Parent; }
	/** @return the parent for this attribute */
	ParentType* GetParent() { return Parent; }
private:
	void Reparent(ParentType* NewParent) override { Parent = NewParent;  }
public:
	virtual TDynamicAttributeBase<ParentType>* MakeNew(ParentType* ParentIn) const override
	{
		TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* Matching = new TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>(ParentIn);
		Matching->Initialize();
		return Matching;
	}
	virtual TDynamicAttributeBase<ParentType>* MakeCopy(ParentType* ParentIn) const override
	{
		TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* ToFill = new TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>(ParentIn);
		ToFill->Copy(*this);
		return ToFill;
	}

	/** Set this overlay to contain the same arrays as the copy overlay */
	void Copy(const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>& Copy)
	{
		TDynamicAttributeBase<ParentType>::CopyParentClassData(Copy);
		AttribValues = Copy.AttribValues;
	}

	virtual TDynamicAttributeBase<ParentType>* MakeCompactCopy(const FCompactMaps& CompactMaps, ParentType* ParentTypeIn) const override
	{
		TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* ToFill = new TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>(ParentTypeIn);
		ToFill->Initialize();
		ToFill->CompactCopy(CompactMaps, *this);
		return ToFill;
	}

	void CompactInPlace(const FCompactMaps& CompactMaps)
	{
		for (int VID = 0, NumVID = CompactMaps.NumVertexMappings(); VID < NumVID; VID++)
		{
			const int ToVID = CompactMaps.GetVertexMapping(VID);
			if (ToVID == FCompactMaps::InvalidID)
			{
				continue;
			}
			if (ensure(ToVID <= VID))
			{
				CopyValue(VID, ToVID);
			}
		}
		AttribValues.Resize(Parent->MaxVertexID() * AttribDimension);
	}

	void CompactCopy(const FCompactMaps& CompactMaps, const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>& ToCopy)
	{
		TDynamicAttributeBase<ParentType>::CopyParentClassData(ToCopy);
		check(CompactMaps.NumVertexMappings() <= int(ToCopy.AttribValues.Num() / AttribDimension));

		AttribValueType Data[AttribDimension];
		for (int VID = 0, NumVID = CompactMaps.NumVertexMappings(); VID < NumVID; VID++)
		{
			int ToVID = CompactMaps.GetVertexMapping(VID);
			if (ToVID == FCompactMaps::InvalidID)
			{
				continue;
			}
			ToCopy.GetValue(VID, Data);
			SetValue(ToVID, Data);
		}
	}

	/** Initialize the attribute values with InitialValue, and resize to the parent mesh's max vertex ID */
	void Initialize(AttribValueType InitialValue = (AttribValueType)0)
	{
		check(Parent != nullptr);
		AttribValues.Resize(Parent->MaxVertexID() * AttribDimension);
		AttribValues.Fill(InitialValue);
	}

	void SetNewValue(int NewVertexID, const AttribValueType* Data)
	{
		int k = NewVertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues.InsertAt(Data[i], k + i);
		}
	}



	//
	// Accessors/Queries
	//

	virtual bool CopyThroughMapping(const TDynamicAttributeBase<ParentType>* Source, const FMeshIndexMappings& Mapping) override
	{
		AttribValueType BufferData[AttribDimension];
		int BufferSize = sizeof(BufferData);
		for (const TPair<int32, int32>& MapVID : Mapping.GetVertexMap().GetForwardMap())
		{
			if (!ensure(Source->CopyOut(MapVID.Key, BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(MapVID.Value, BufferData);
		}
		return true;
	}

	virtual bool Append(const TDynamicAttributeBase<ParentType>& Source, const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		AttribValueType BufferData[AttribDimension];
		int BufferSize = sizeof(BufferData);

		int32 NewMaxID = Info.NumVertex + Info.VertexOffset;
		if (NewMaxID * AttribDimension > AttribValues.Num())
		{
			AttribValues.SetNum(NewMaxID * AttribDimension);
		}
		for (int32 Idx = 0; Idx < Info.NumVertex; ++Idx)
		{
			if (!ensure(Source.CopyOut(Idx, BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(Idx + Info.VertexOffset, BufferData);
		}
		return true;
	}

	virtual void AppendDefaulted(const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		AttribValues.SetMinimumSize((Info.NumVertex + Info.VertexOffset) * AttribDimension, GetDefaultAttributeValue());
	}

	virtual bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		if (sizeof(AttribValueType)*AttribDimension != BufferSize)
		{
			return false;
		}
		AttribValueType* BufferData = static_cast<AttribValueType*>(Buffer);
		int k = RawID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			BufferData[i] = AttribValues[k + i];
		}
		return true;
	}
	virtual bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		if (sizeof(AttribValueType) * AttribDimension != BufferSize)
		{
			return false;
		}
		AttribValueType* BufferData = static_cast<AttribValueType*>(Buffer);
		int k = RawID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = BufferData[i];
		}
		return true;
	}

	/** Get the element at a given index */
	inline void GetValue(int VertexID, AttribValueType* Data) const
	{
		int k = VertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			Data[i] = AttribValues[k + i];
		}
	}

	/** Get the element at a given index. For scalar access, use GetValue<T*const>(&Value) for a Value of type T. */
	template<typename AsType>
	void GetValue(int VertexID, AsType& Data) const
	{
		int k = VertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			Data[i] = AttribValues[k + i];
		}
	}


	/** Set the element at a given index */
	inline void SetValue(int VertexID, const AttribValueType* Data)
	{
		int k = VertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = Data[i];
		}
	}

	/** Set the element at a given index */
	template<typename AsType>
	void SetValue(int VertexID, const AsType& Data)
	{
		int k = VertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = Data[i];
		}
	}

	/** Set the element at a given index with a scalar value (the same value for each dimension) */
	inline void SetScalarValue(int32 VertexID, const AttribValueType& SingleValue)
	{
		int k = VertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[k + i] = SingleValue;
		}
	}

	/**
	 * Copy the attribute value at FromVertexID to ToVertexID
	 */
	inline void CopyValue(int FromVertexID, int ToVertexID)
	{
		int kA = FromVertexID * AttribDimension;
		int kB = ToVertexID * AttribDimension;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues.InsertAt(AttribValues[kA+i], kB+i);
		}
	}


public:

	/** Update the overlay to reflect an edge split in the parent */
	void OnSplitEdge(const FDynamicMesh3::FEdgeSplitInfo& SplitInfo) override
	{
		ResizeAttribStoreIfNeeded(SplitInfo.NewVertex);
		SetAttributeFromLerp(SplitInfo.NewVertex, SplitInfo.OriginalVertices.A, SplitInfo.OriginalVertices.B, SplitInfo.SplitT);
	}

	/** Update the overlay to reflect an edge flip in the parent */
	void OnFlipEdge(const FDynamicMesh3::FEdgeFlipInfo& FlipInfo) override
	{
		// vertices unchanged
	}

	/** Update the overlay to reflect an edge collapse in the parent */
	void OnCollapseEdge(const FDynamicMesh3::FEdgeCollapseInfo& CollapseInfo) override
	{
		SetAttributeFromLerp(CollapseInfo.KeptVertex, CollapseInfo.KeptVertex, CollapseInfo.RemovedVertex, CollapseInfo.CollapseT);
	}

	virtual AttribValueType GetDefaultAttributeValue()
	{
		return (AttribValueType)0;
	}

	inline void ResizeAttribStoreIfNeeded(int VertexID)
	{
		if (!ensure(VertexID >= 0))
		{
			return;
		}
		size_t NeededSize = (size_t(VertexID)+1) * AttribDimension;
		if (NeededSize > AttribValues.Num())
		{
			AttribValues.Resize(NeededSize, GetDefaultAttributeValue());
		}
	}

	virtual void OnNewVertex(int VertexID, bool bInserted) override
	{
		ResizeAttribStoreIfNeeded(VertexID);
		SetScalarValue(VertexID, GetDefaultAttributeValue());
	}

	/** Update the overlay to reflect a face poke in the parent */
	void OnPokeTriangle(const FDynamicMesh3::FPokeTriangleInfo& PokeInfo) override
	{
		FIndex3i Tri = PokeInfo.TriVertices;
		ResizeAttribStoreIfNeeded(PokeInfo.NewVertex);
		SetAttributeFromBary(PokeInfo.NewVertex, Tri.A, Tri.B, Tri.C, PokeInfo.BaryCoords);
	}

	/** Update the overlay to reflect an edge merge in the parent */
	void OnMergeEdges(const FDynamicMesh3::FMergeEdgesInfo& MergeInfo) override
	{
		// just blend the attributes?
		if (MergeInfo.RemovedVerts.A != FDynamicMesh3::InvalidID)
		{
			SetAttributeFromLerp(MergeInfo.KeptVerts.A, MergeInfo.KeptVerts.A, MergeInfo.RemovedVerts.A, MergeInfo.InterpolationT);
		}
		if (MergeInfo.RemovedVerts.B != FDynamicMesh3::InvalidID)
		{
			SetAttributeFromLerp(MergeInfo.KeptVerts.B, MergeInfo.KeptVerts.B, MergeInfo.RemovedVerts.B, MergeInfo.InterpolationT);
		}
	}

	void OnMergeVertices(const FDynamicMesh3::FMergeVerticesInfo& MergeInfo) override
	{
		SetAttributeFromLerp(MergeInfo.KeptVertex, MergeInfo.KeptVertex, MergeInfo.RemovedVertex, MergeInfo.InterpolationT);
	}

	/** Update the overlay to reflect a vertex split in the parent */
	void OnSplitVertex(const FDynamicMesh3::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate) override
	{
		CopyValue(SplitInfo.OriginalVertex, SplitInfo.NewVertex);
	}

	virtual TUniquePtr<TDynamicAttributeChangeBase<ParentType>> NewBlankChange() const override
	{
		return MakeUnique<TDynamicVertexAttributeChange<AttribValueType, AttribDimension, ParentType>>();
	}

	/**
	* Check validity of attribute
	* 
	* @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	* @param FailMode Desired behavior if mesh is found invalid
	*/
	virtual bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const override
	{
		// just check that the values buffer is big enough
		if (Parent->MaxVertexID() < 0 || static_cast<size_t>(Parent->MaxVertexID())*AttribDimension > AttribValues.Num())
		{
			switch (FailMode)
			{
			case EValidityCheckFailMode::Check:
				check(false);
			case EValidityCheckFailMode::Ensure:
				ensure(false);
			default:
				return false;
			}
		}

		return true;
	}

	/**
	 * Returns true if this AttributeSet is the same as Other.
	 * @param bIgnoreDataLayout Ignore gaps and padding in the data layout, i.e. the exact index is irrelevant as long as valid entries are equal in value and in the same order.
	 */
	bool IsSameAs(const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>& Other, bool bIgnoreDataLayout) const
	{
		if (!bIgnoreDataLayout)
		{
			if (AttribValues.Num() != Other.AttribValues.Num())
			{
				return false;
			}

			for (int Idx = 0, NumValues = AttribValues.Num(); Idx < NumValues; Idx++)
			{
				if (AttribValues[Idx] != Other.AttribValues[Idx])
				{
					return false;
				}
			}
		}
		else
		{
			// bIgnoreDataLayout

			FRefCountVector::IndexIterator ItVid = Parent->GetVerticesRefCounts().BeginIndices();
			const FRefCountVector::IndexIterator ItVidEnd = Parent->GetVerticesRefCounts().EndIndices();
			FRefCountVector::IndexIterator ItVidOther = Other.Parent->GetVerticesRefCounts().BeginIndices();
			const FRefCountVector::IndexIterator ItVidEndOther = Other.Parent->GetVerticesRefCounts().EndIndices();

			while (ItVid != ItVidEnd && ItVidOther != ItVidEndOther)
			{
				for (int32 i = 0; i < AttribDimension; ++i)
				{
					const AttribValueType AttribValue = AttribValues[*ItVid * AttribDimension + i];
					const AttribValueType AttribValueOther = Other.AttribValues[*ItVidOther * AttribDimension + i];
					if (AttribValue != AttribValueOther)
					{
						// Vertex attribute value is not the same.
						return false;
					}
				}
				++ItVid;
				++ItVidOther;
			}

			if (ItVid != ItVidEnd || ItVidOther != ItVidEndOther)
			{
				// Number of vertex attribute values is not the same.
				return false;
			}
		}

		return true;
	}


	/**
	 * Serialization operator for TDynamicVertexAttribute.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Attr Vertex attribute to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>& Attr)
	{
		Attr.Serialize(Ar, nullptr, false);
		return Ar;
	}

	/**
	* Serialize to and from an archive.
	*
	* @param Ar Archive to serialize with.
	* @param CompactMaps If this is not a null pointer, the mesh serialization compacted the vertex and/or triangle data using the provided mapping.
	* @param bUseCompression Use compression for serializing bulk data.
	*/
	void Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
	{
		Super::Serialize(Ar);

		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
		{
			Ar << AttribValues;
		}
		else
		{
			auto SerializeVector = [](FArchive& Ar, auto& Vector, bool bUseCompression)
			{
				if (bUseCompression)
				{
					Vector.template Serialize<true, true>(Ar);
				}
				else
				{
					Vector.template Serialize<true, false>(Ar);
				}
			};

			Ar << bUseCompression;

			if (CompactMaps == nullptr || !CompactMaps->VertexMapIsSet())
			{
				SerializeVector(Ar, AttribValues, bUseCompression);
			}
			else
			{
				TDynamicVector<AttribValueType> AttribValuesCompact;
				AttribValuesCompact.SetNum(Parent->VertexCount() * AttribDimension);

				int32 VidCompact = 0;
				for (int32 Vid = 0, Num = AttribValues.Num() / AttribDimension; Vid < Num; ++Vid)
				{
					const int32 VidMapping = CompactMaps->GetVertexMapping(Vid);
					if (VidMapping != FCompactMaps::InvalidID)
					{
						for (int32 i = 0; i < AttribDimension; ++i)
						{
							AttribValuesCompact[VidCompact++] = AttribValues[Vid * AttribDimension + i];
						}
					}
				}

				SerializeVector(Ar, AttribValuesCompact, bUseCompression);
			}
		}
	}

	virtual SIZE_T GetByteCount() const override
	{
		return AttribValues.GetByteCount();
	}


protected:

	// interpolation functions; default implementation assumes your attributes can be interpolated as reals

	/** Set the value at an Attribute to be a linear interpolation of two other Attributes */
	virtual void SetAttributeFromLerp(int SetAttribute, int AttributeA, int AttributeB, double Alpha)
	{
		int IndexSet = AttribDimension * SetAttribute;
		int IndexA = AttribDimension * AttributeA;
		int IndexB = AttribDimension * AttributeB;
		double Beta = (1. - Alpha);
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[IndexSet+i] = AttribValueType(Beta*AttribValues[IndexA+i] + Alpha*AttribValues[IndexB+i]);
		}
	}

	/** Set the value at an Attribute to be a barycentric interpolation of three other Attributes */
	virtual void SetAttributeFromBary(int SetAttribute, int AttributeA, int AttributeB, int AttributeC, const FVector3d& BaryCoords)
	{
		int IndexSet = AttribDimension * SetAttribute;
		int IndexA = AttribDimension * AttributeA;
		int IndexB = AttribDimension * AttributeB;
		int IndexC = AttribDimension * AttributeC;
		for (int i = 0; i < AttribDimension; ++i)
		{
			AttribValues[IndexSet + i] = AttribValueType(
				BaryCoords.X*AttribValues[IndexA+i] + BaryCoords.Y*AttribValues[IndexB+i] + BaryCoords.Z*AttribValues[IndexC+i]);
		}
	}

};


template<typename AttribValueType, int AttribDimension>
using TDynamicMeshVertexAttribute = TDynamicVertexAttribute<AttribValueType, AttribDimension, FDynamicMesh3>;


template<typename AttribValueType, int AttribDimension, typename ParentType>
void TDynamicVertexAttributeChange<AttribValueType, AttribDimension, ParentType>::SaveInitialVertex(const TDynamicAttributeBase<ParentType>* Attribute, int VertexID)
{
	FChangeVertexAttribute& Change = OldVertexAttributes.Emplace_GetRef();
	Change.VertexID = VertexID;
	const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* AttribCast = static_cast<const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>*>(Attribute);
	AttribCast->GetValue(VertexID, Change.Data);
}

template<typename AttribValueType, int AttribDimension, typename ParentType>
void TDynamicVertexAttributeChange<AttribValueType, AttribDimension, ParentType>::StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TSet<int>& VertexIDs)
{
	NewVertexAttributes.Reserve(NewVertexAttributes.Num() + VertexIDs.Num());
	const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* AttribCast = static_cast<const TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>*>(Attribute);
	for (int VertexID : VertexIDs)
	{
		FChangeVertexAttribute& Change = NewVertexAttributes.Emplace_GetRef();
		Change.VertexID = VertexID;
		AttribCast->GetValue(VertexID, Change.Data);
	}
}

template<typename AttribValueType, int AttribDimension, typename ParentType>
bool TDynamicVertexAttributeChange<AttribValueType, AttribDimension, ParentType>::Apply(TDynamicAttributeBase<ParentType>* Attribute, bool bRevert) const
{
	const TArray<FChangeVertexAttribute> *Changes = bRevert ? &OldVertexAttributes : &NewVertexAttributes;
	TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>* AttribCast = static_cast<TDynamicVertexAttribute<AttribValueType, AttribDimension, ParentType>*>(Attribute);
	for (const FChangeVertexAttribute& Change : *Changes)
	{
		bool bIsVertex = AttribCast->GetParent()->IsVertex(Change.VertexID);
		check(bIsVertex);
		if (bIsVertex)
		{
			AttribCast->SetValue(Change.VertexID, Change.Data);
		}
	}
	return true;
}


} // end namespace UE::Geometry
} // end namespace UE
