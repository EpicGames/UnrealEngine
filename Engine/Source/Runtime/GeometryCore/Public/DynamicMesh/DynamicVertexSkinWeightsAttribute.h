// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicAttribute.h"

#include "BoneWeights.h"
#include "HAL/UnrealMemory.h"
#include "Misc/MemStack.h"

namespace UE
{
namespace Geometry
{

template<typename ParentType>
class TDynamicVertexSkinWeightsAttributeChange final : 
	public TDynamicAttributeChangeBase<ParentType>
{
	struct FChangeVertexBoneWeights
	{
		UE::AnimationCore::FBoneWeights Weights;
		int VertexID;
	};

	TArray<FChangeVertexBoneWeights> OldVertexBoneWeights, NewVertexBoneWeights;

public:
	TDynamicVertexSkinWeightsAttributeChange() = default;

	virtual ~TDynamicVertexSkinWeightsAttributeChange() override = default;

	void SaveInitialVertex(const TDynamicAttributeBase<ParentType>* Attribute, int VertexID) override;

	void StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TSet<int>& VertexIDs) override;

	bool Apply(TDynamicAttributeBase<ParentType>* Attribute, bool bRevert) const override;
};


/**
 * TDynamicVertexAttribute provides per-vertex storage of bone weights (skin weights)
 */
template<typename ParentType>
class TDynamicVertexSkinWeightsAttribute final : 
	public TDynamicAttributeBase<ParentType>
{
public:
	using FBoneWeights = UE::AnimationCore::FBoneWeights;
	using FBoneWeight  = UE::AnimationCore::FBoneWeight;

protected:
	friend class FDynamicMeshAttributeSet;

	/** The parent object (e.g. mesh, point set) this attribute belongs to */
	ParentType* Parent = nullptr;

	/** List of per-vertex bone weights values */
	// FIXME: Replace with a local slab allocator storage to avoid per-item TInlineAllocator
	TDynamicVector<FBoneWeights> VertexBoneWeights;

public:

	/** Create an empty overlay */
	TDynamicVertexSkinWeightsAttribute() = default;

	/** Create an attribute for the given parent */
	TDynamicVertexSkinWeightsAttribute(ParentType* ParentIn, bool bAutoInit = true) : 
		Parent(ParentIn)
	{
		if (bAutoInit)
		{
			Initialize();
		}
	}

	virtual ~TDynamicVertexSkinWeightsAttribute() = default;

	/** @return the parent for this attribute */
	const ParentType* GetParent() const { return Parent; }
	/** @return the parent for this attribute */
	ParentType* GetParent() { return Parent; }
private:
	void Reparent(ParentType* NewParent) override { Parent = NewParent;  }
public:
	TDynamicAttributeBase<ParentType>* MakeNew(ParentType* ParentIn) const override
	{
		TDynamicVertexSkinWeightsAttribute<ParentType>* Matching = new TDynamicVertexSkinWeightsAttribute<ParentType>(ParentIn);
		Matching->Initialize();
		return Matching;
	}
	TDynamicAttributeBase<ParentType>* MakeCopy(ParentType* ParentIn) const override
	{
		TDynamicVertexSkinWeightsAttribute<ParentType>* ToFill = new TDynamicVertexSkinWeightsAttribute<ParentType>(ParentIn);
		ToFill->Copy(*this);
		return ToFill;
	}

	/** Set this overlay to contain the same arrays as the copy overlay */
	void Copy(const TDynamicVertexSkinWeightsAttribute<ParentType>& Copy)
	{
		TDynamicAttributeBase<ParentType>::CopyParentClassData(Copy);
		VertexBoneWeights = Copy.VertexBoneWeights;
	}

	TDynamicAttributeBase<ParentType>* MakeCompactCopy(const FCompactMaps& CompactMaps, ParentType* ParentTypeIn) const override
	{
		TDynamicVertexSkinWeightsAttribute<ParentType>* ToFill = new TDynamicVertexSkinWeightsAttribute<ParentType>(ParentTypeIn);
		ToFill->Initialize();
		ToFill->CompactCopy(CompactMaps, *this);
		return ToFill;
	}

	void CompactInPlace(const FCompactMaps& CompactMaps) override
	{
		for (int32 VID = 0, NumVID = CompactMaps.NumVertexMappings(); VID < NumVID; VID++)
		{
			const int32 ToVID = CompactMaps.GetVertexMapping(VID);
			if (ToVID == FCompactMaps::InvalidID)
			{
				continue;
			}
			if (ensure(ToVID <= VID))
			{
				CopyValue(VID, ToVID);
			}
		}
		VertexBoneWeights.Resize(Parent->MaxVertexID());
	}

	void CompactCopy(const FCompactMaps& CompactMaps, const TDynamicVertexSkinWeightsAttribute<ParentType>& ToCopy)
	{
		TDynamicAttributeBase<ParentType>::CopyParentClassData(ToCopy);
		check(CompactMaps.NumVertexMappings() >= VertexBoneWeights.Num());
		FBoneWeights Data;
		for (int32 VID = 0, NumVID = CompactMaps.NumVertexMappings(); VID < NumVID; VID++)
		{
			const int32 ToVID = CompactMaps.GetVertexMapping(VID);
			if (ToVID == FCompactMaps::InvalidID)
			{
				continue;
			}
			ToCopy.GetValue(VID, Data);
			SetValue(ToVID, Data);
		}
	}

	/** Initialize the attribute values to the given max vertex ID */
	void Initialize(const FBoneWeights InitialValue = {})
	{
		check(Parent != nullptr);
		VertexBoneWeights.Resize(Parent->MaxVertexID());
		VertexBoneWeights.Fill(InitialValue);
	}

	void SetNewValue(int32 InNewVertexID, const FBoneWeights InBoneWeights)
	{
		VertexBoneWeights.InsertAt(InBoneWeights, InNewVertexID);
	}

	
	/** 
	 * Re-index bone indices of the skin weights from one reference skeleton (represented by an array of bone names) 
	 * to another.
	 * 
	 * @param FromRefSkeleton The reference skeleton that the current skin weights are indexed against.
	 * @param ToRefSkeleton The new reference skeleton that should be used to reindex the weights.
	 * 
	 * @return true on success, false otherwise
	 * 
	 * @note Reasons for failure are:
	 * 		 - bone index of one of the bone weights is larger than the number of bones in FromRefSkeleton, meaning
	 * 		   the weights most likely were not computed with respect to the FromRefSkeleton.
	 * 		 - one of the bone weights references a bone in FromRefSkeleton that doesn't exist in ToRefSkeleton.
	 * 		 - ToRefSkeleton contains duplicates.
	 *
	 * @note FromRefSkeleton can contain bones that are not present in ToRefSkeleton as long as those bones are not 
			 referenced by any of the weights.
	 */
	bool ReindexBoneIndicesToSkeleton(const TArray<FName>& FromRefSkeleton, const TArray<FName>& ToRefSkeleton)
	{	
		if (VertexBoneWeights.Num() == 0)
		{
			return true; // empty weights, nothing to do
		}

		if (FromRefSkeleton == ToRefSkeleton)
		{
			return true; // skeletons are the same, nothing to do
		}

		// Create a map from bone name to bone index for the ToRefSkeleton so we can
		// check if bone with a given name in FromRefSkeleton exists in ToRefSkeleton 
		// and at what index
		TMap<FName, int32> NameToIndex;
		NameToIndex.Reserve(ToRefSkeleton.Num());
		for (int BoneID = 0; BoneID < ToRefSkeleton.Num(); ++BoneID)
		{
			const FName& BoneName = ToRefSkeleton[BoneID];
			if (NameToIndex.Contains(BoneName))
			{
				return false; // there should be no duplicates in ToRefSkeleton
			}
			NameToIndex.Add(BoneName, BoneID);
		}

		// Store temporary results of reindexing in a new dynamic vector
		TDynamicVector<FBoneWeights> NewVertexBoneWeights;
		NewVertexBoneWeights.Resize(VertexBoneWeights.Num());

		// Since we are only changing indicies, make sure we don't do any re-normalization on the weights
		UE::AnimationCore::FBoneWeightsSettings BoneSettings;
		BoneSettings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::None); 

		// Iterate over every skin weight of every vertex and remap it's bone index from FromRefSkeleton to ToRefSkeleton
		for (int VertexID = 0; VertexID < VertexBoneWeights.Num(); ++VertexID)
		{
			FBoneWeights BoneWeights = VertexBoneWeights[VertexID];
			if (BoneWeights.Num() == 0)
			{
				continue; // vertex has no weights
			}

			FBoneWeights NewWeights;
			for (int32 Idx = 0; Idx < BoneWeights.Num(); ++Idx)
			{
				const FBoneWeight& BoneWeight = BoneWeights[Idx];
				if (BoneWeight.GetBoneIndex() >= FromRefSkeleton.Num())
				{
					return false; // invalid FromRefSkeleton
				}
				
				const FName& FromName = FromRefSkeleton[BoneWeight.GetBoneIndex()];
				const uint16 FromWeight = BoneWeight.GetRawWeight();

				const int32* ToIdx = NameToIndex.Find(FromName);
				if (ToIdx)
				{
					NewWeights.SetBoneWeight(FBoneWeight(static_cast<FBoneIndexType>(*ToIdx), FromWeight), BoneSettings);
					NewVertexBoneWeights[VertexID] = NewWeights;
				}
				else
				{
					return false; // ToRefSkeleton must be a superset of all the bones referenced in FromRefSkeleton by the skinning weights
				}
			}
		}

		VertexBoneWeights = MoveTemp(NewVertexBoneWeights);

		return true;
	}

	//
	// Accessors/Queries
	//

	/** Returns a list of all unique bone indices that are used by this skin weight attribute.
	 *  \return A unique list of bone indices used by all bone weights across all vertices. 
	 */
	TSet<int32> GetBoundBoneIndices() const
	{
		if (VertexBoneWeights.Num() == 0)
		{
			return {};	// No weights defined, return the empty set.
		}

		TSet<int32> UniqueBoneIndices;
		for (int VertexID = 0; VertexID < VertexBoneWeights.Num(); ++VertexID)
		{
			for (FBoneWeight BoneWeight: VertexBoneWeights[VertexID])
			{
				UniqueBoneIndices.Add(BoneWeight.GetBoneIndex());
			}
		}
		return UniqueBoneIndices;
	}

	bool CopyThroughMapping(const TDynamicAttributeBase<ParentType>* Source, const FMeshIndexMappings& Mapping) override
	{
		// Don't snarf the FBoneWeight as a concrete object, since it _may_ contain a pointer
		// and we don't want it to destruct.
		constexpr int32 BufferSize = sizeof(FBoneWeights);
		int8 BufferData[BufferSize];
		for (const TPair<int32, int32>& MapVID : Mapping.GetVertexMap().GetForwardMap())
		{
			if (!ensure(Source->CopyOut(MapVID.Key, &BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(MapVID.Value, *reinterpret_cast<FBoneWeights *>(BufferData));
		}
		return true;
	}

	virtual bool Append(const TDynamicAttributeBase<ParentType>& Source, const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		// Don't snarf the FBoneWeight as a concrete object, since it _may_ contain a pointer
		// and we don't want it to destruct.
		constexpr int32 BufferSize = sizeof(FBoneWeights);
		int8 BufferData[BufferSize];

		int32 NewMaxID = Info.NumVertex + Info.VertexOffset;
		if (NewMaxID > VertexBoneWeights.Num())
		{
			VertexBoneWeights.SetNum(NewMaxID);
		}
		for (int32 Idx = 0; Idx < Info.NumVertex; ++Idx)
		{
			if (!ensure(Source.CopyOut(Idx, BufferData, BufferSize)))
			{
				return false;
			}
			SetValue(Idx + Info.VertexOffset, *reinterpret_cast<FBoneWeights*>(BufferData));
		}
		return true;
	}

	virtual void AppendDefaulted(const UE::Geometry::FDynamicMesh3::FAppendInfo& Info) override
	{
		int32 NewMaxID = Info.NumVertex + Info.VertexOffset;
		VertexBoneWeights.SetMinimumSize(NewMaxID, FBoneWeights{});
	}

	bool CopyOut(int RawID, void* Buffer, int BufferSize) const override
	{
		if (sizeof(FBoneWeights) != BufferSize)
		{
			return false;
		}

		// Avoid the copy constructor.
		FMemory::Memcpy(Buffer, &VertexBoneWeights[RawID], sizeof(FBoneWeights));
		return true;
	}
	bool CopyIn(int RawID, void* Buffer, int BufferSize) override
	{
		if (sizeof(FBoneWeights) != BufferSize)
		{
			return false;
		}
		
		// Ensure the copy constructor is called.
		VertexBoneWeights[RawID] = *static_cast<FBoneWeights*>(Buffer);
		return true;
	}

	/** Get the element at a given index */
	void GetValue(int VertexID, FBoneWeights& Data) const
	{
		Data = VertexBoneWeights[VertexID];
	}

	/** Get the element at a given index */
	template<typename AsType>
	void GetValue(int VertexID, AsType& Data) const
	{
		Data = VertexBoneWeights[VertexID];
	}

	/** 
	 * Get the element at a given index via a pair of bone index and weight arrays. 
	 * The number of influences is equal to the array size. 
	 */
	template<typename BoneIndexType, typename BoneFloatWeightType>
	void GetValue(int VertexID, TArray<BoneIndexType>& OutBones, TArray<BoneFloatWeightType>& OutWeights) const
	{
		FBoneWeights BoneWeights;
		GetValue(VertexID, BoneWeights);

		const int32 NumEntries = BoneWeights.Num();

		OutBones.SetNum(NumEntries, EAllowShrinking::No);
		OutWeights.SetNum(NumEntries, EAllowShrinking::No);

		for (int32 BoneIdx = 0; BoneIdx < NumEntries; ++BoneIdx)
		{
			OutBones[BoneIdx] = static_cast<BoneIndexType>(BoneWeights[BoneIdx].GetBoneIndex());
			OutWeights[BoneIdx] = static_cast<BoneFloatWeightType>(BoneWeights[BoneIdx].GetWeight());
		}
	}

	/** Set the element at a given index */
	void SetValue(int VertexID, const FBoneWeights& Data)
	{
		VertexBoneWeights[VertexID] = Data;
	}

	/** Set the element at a given index */
	template<typename ContainerAdapter>
	void SetValue(int VertexID, const UE::AnimationCore::TBoneWeights<ContainerAdapter>& Data)
	{
		VertexBoneWeights[VertexID] = FBoneWeights::Create(Data);
	}

	/** Set the element at a given index */
	template<typename BoneIndexType, typename BoneFloatWeightType>
	void SetValue(int VertexID, const TArray<BoneIndexType>& InBones, const TArray<BoneFloatWeightType>& InWeights, int32 InNumEntries)
	{
		checkSlow(InBones.Num() == InNumEntries && InWeights.Num() == InNumEntries);
		
		TArray<FBoneIndexType, TMemStackAllocator<>> Bones;
		TArray<float, TMemStackAllocator<>> Weights;
 
		Bones.SetNumUninitialized(InNumEntries);
		Weights.SetNumUninitialized(InNumEntries);

		for (int32 Idx = 0; Idx < InNumEntries; ++Idx)
		{
			Bones[Idx] = static_cast<FBoneIndexType>(InBones[Idx]);
			Weights[Idx] = static_cast<float>(InWeights[Idx]);
		}

		VertexBoneWeights[VertexID] = FBoneWeights::Create(Bones.GetData(), Weights.GetData(), InNumEntries);
	}

	/**
	 * Copy the attribute value at FromVertexID to ToVertexID
	 */
	void CopyValue(int FromVertexID, int ToVertexID)
	{
		VertexBoneWeights.InsertAt(VertexBoneWeights[FromVertexID], ToVertexID);
	}


public:

	/** Update the overlay to reflect an edge split in the parent */
	void OnSplitEdge(const FDynamicMesh3::FEdgeSplitInfo& SplitInfo) override
	{
		ResizeAttribStoreIfNeeded(SplitInfo.NewVertex);
		SetBoneWeightsFromLerp(SplitInfo.NewVertex, SplitInfo.OriginalVertices.A, SplitInfo.OriginalVertices.B, SplitInfo.SplitT);
	}

	/** Update the overlay to reflect an edge flip in the parent */
	void OnFlipEdge(const FDynamicMesh3::FEdgeFlipInfo& FlipInfo) override
	{
		// vertices unchanged
	}

	/** Update the overlay to reflect an edge collapse in the parent */
	void OnCollapseEdge(const FDynamicMesh3::FEdgeCollapseInfo& CollapseInfo) override
	{
		SetBoneWeightsFromLerp(CollapseInfo.KeptVertex, CollapseInfo.KeptVertex, CollapseInfo.RemovedVertex, CollapseInfo.CollapseT);
	}

	void ResizeAttribStoreIfNeeded(int VertexID)
	{
		if (!ensure(VertexID >= 0))
		{
			return;
		}
		size_t NeededSize = (size_t(VertexID)+1);
		if (NeededSize > VertexBoneWeights.Num())
		{
			VertexBoneWeights.Resize(NeededSize, FBoneWeights{});
		}
	}

	void OnNewVertex(int VertexID, bool bInserted) override
	{
		ResizeAttribStoreIfNeeded(VertexID);
	}

	/** Update the overlay to reflect a face poke in the parent */
	void OnPokeTriangle(const FDynamicMesh3::FPokeTriangleInfo& PokeInfo) override
	{
		FIndex3i Tri = PokeInfo.TriVertices;
		ResizeAttribStoreIfNeeded(PokeInfo.NewVertex);
		SetBoneWeightsFromBary(PokeInfo.NewVertex, Tri.A, Tri.B, Tri.C, PokeInfo.BaryCoords);
	}

	/** Update the overlay to reflect an edge merge in the parent */
	void OnMergeEdges(const FDynamicMesh3::FMergeEdgesInfo& MergeInfo) override
	{
		// just blend the attributes?
		if (MergeInfo.RemovedVerts.A != FDynamicMesh3::InvalidID)
		{
			SetBoneWeightsFromLerp(MergeInfo.KeptVerts.A, MergeInfo.KeptVerts.A, MergeInfo.RemovedVerts.A, MergeInfo.InterpolationT);
		}
		if (MergeInfo.RemovedVerts.B != FDynamicMesh3::InvalidID)
		{
			SetBoneWeightsFromLerp(MergeInfo.KeptVerts.B, MergeInfo.KeptVerts.B, MergeInfo.RemovedVerts.B, MergeInfo.InterpolationT);
		}
	}

	/** Update the overlay to reflect a vertex merge in the parent */
	void OnMergeVertices(const FDynamicMesh3::FMergeVerticesInfo& MergeInfo) override
	{
		SetBoneWeightsFromLerp(MergeInfo.KeptVertex, MergeInfo.KeptVertex, MergeInfo.RemovedVertex, MergeInfo.InterpolationT);
	}

	/** Update the overlay to reflect a vertex split in the parent */
	void OnSplitVertex(const FDynamicMesh3::FVertexSplitInfo& SplitInfo, const TArrayView<const int>& TrianglesToUpdate) override
	{
		CopyValue(SplitInfo.OriginalVertex, SplitInfo.NewVertex);
	}

	TUniquePtr<TDynamicAttributeChangeBase<ParentType>> NewBlankChange() const override
	{
		return MakeUnique<TDynamicVertexSkinWeightsAttributeChange<ParentType>>();
	}

	/**
	* Check validity of attribute
	* 
	* @param bAllowNonmanifold Accept non-manifold topology as valid. Note that this should almost always be true for attributes; non-manifold overlays are generally valid.
	* @param FailMode Desired behavior if mesh is found invalid
	*/
	bool CheckValidity(bool bAllowNonmanifold, EValidityCheckFailMode FailMode) const override
	{
		// just check that the values buffer is big enough
		if (Parent->MaxVertexID() > VertexBoneWeights.Num())
		{
			switch (FailMode)
			{
			case EValidityCheckFailMode::Check:
				check(false);
			case EValidityCheckFailMode::Ensure:
				ensure(false);
			default:
				return false;
			}
		}

		return true;
	}

	void Serialize(FArchive& Ar, const FCompactMaps* CompactMaps, bool bUseCompression)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

		Ar << bUseCompression;

		const bool bUseVertexCompactMap = CompactMaps && CompactMaps->VertexMapIsSet();

		if (!bUseCompression)
		{
			if (Ar.IsLoading() || !bUseVertexCompactMap)
			{
				VertexBoneWeights.Serialize<false, false>(Ar);
			}
			else
			{
				TDynamicVector<FBoneWeights> VertexBoneWeightsCompact;
				VertexBoneWeightsCompact.Resize(Parent->VertexCount());

				for (int32 Vid = 0, Num = VertexBoneWeights.Num(); Vid < Num; ++Vid)
				{
					const int32 VidCompact = CompactMaps->GetVertexMapping(Vid);
					if (VidCompact != FCompactMaps::InvalidID)
					{
						VertexBoneWeightsCompact[VidCompact] = VertexBoneWeights[Vid];
					}
				}

				VertexBoneWeightsCompact.Serialize<false, false>(Ar);
			}
		}
		else
		{
			// To achieve better compression performance both w.r.t. size and speed, we copy everything into one big flat buffer.
			// We take advantage of the fact that we can effectively store everything as int32 for both the vector/array sizes as well as the bone weights.
			TArray<int32> Buffer;

			if (Ar.IsLoading())
			{
				// Serialize size of decompressed buffer to be able to hold all the compressed data in the archive, and allocate accordingly.
				int32 BufferSize;
				Ar << BufferSize;
				Buffer.SetNumUninitialized(BufferSize);

				// Decompress buffer from archive.
				Ar.SerializeCompressedNew(Buffer.GetData(), Buffer.Num() * sizeof(int32), NAME_Oodle, NAME_Oodle, COMPRESS_NoFlags, false, nullptr);

				// Restore bone weights arrays from decompressed buffer.
				int32* BufferPtr = Buffer.GetData();
				VertexBoneWeights.Resize(*BufferPtr++);
				AnimationCore::FBoneWeightsSettings BoneWeightsSettings;
				BoneWeightsSettings.SetNormalizeType(AnimationCore::EBoneWeightNormalizeType::None);
				for (FBoneWeights& BoneWeights : VertexBoneWeights)
				{
					const int32 Num = *BufferPtr++;
					for (int32 i = 0; i < Num; ++i)
					{
						BoneWeights.SetBoneWeight(reinterpret_cast<AnimationCore::FBoneWeight&>(*BufferPtr++), BoneWeightsSettings);
					}
				}
				checkSlow(BufferPtr == Buffer.GetData() + Buffer.Num());
			}
			else
			{
				// Determine total number of individual bone weights.

				auto CountBoneWeights = [](const FBoneWeights& BoneWeights, SIZE_T& NumBoneWeights)
				{
					NumBoneWeights += BoneWeights.Num();
				};

				SIZE_T NumBoneWeights = 0;
				if (!bUseVertexCompactMap)
				{
					for (const FBoneWeights& BoneWeights : VertexBoneWeights)
					{
						CountBoneWeights(BoneWeights, NumBoneWeights);
					}
				}
				else
				{
					for (int32 Vid = 0, Num = VertexBoneWeights.Num(); Vid < Num; ++Vid)
					{
						const int32 VidCompact = CompactMaps->GetVertexMapping(Vid);
						if (VidCompact != FCompactMaps::InvalidID)
						{
							CountBoneWeights(VertexBoneWeights[Vid], NumBoneWeights);
						}
					}
				}

				// Set buffer size to hold number of vertex bone weight arrays, size for vertex each bone weight array, and all individual bone weights.
				// We also serialize out size of uncompressed buffer to allow for allocation to correct size during loading.
				const uint32 NumVertexBoneWeightArrays = bUseVertexCompactMap ? Parent->VertexCount() : VertexBoneWeights.Num();
				uint32 BufferSize = 1 + NumVertexBoneWeightArrays + IntCastChecked<uint32>(NumBoneWeights);
				Buffer.SetNumUninitialized(BufferSize);
				Ar << BufferSize;

				// Write everything into the buffer.

				auto WriteBoneWeights = [](const FBoneWeights& BoneWeights, int32*& BufferPtr)
				{
					const int32 Num = BoneWeights.Num();
					*BufferPtr++ = Num;
					if (Num > 0)
					{
						FMemory::Memcpy(BufferPtr, &BoneWeights[0], Num * sizeof(int32));
						BufferPtr += Num;
					}
				};

				int32* BufferPtr = Buffer.GetData();
				*BufferPtr++ = NumVertexBoneWeightArrays;
				if (!bUseVertexCompactMap)
				{
					for (const FBoneWeights& BoneWeights : VertexBoneWeights)
					{
						WriteBoneWeights(BoneWeights, BufferPtr);
					}
				}
				else
				{
					for (int32 Vid = 0, Num = VertexBoneWeights.Num(); Vid < Num; ++Vid)
					{
						const int32 VidCompact = CompactMaps->GetVertexMapping(Vid);
						if (VidCompact != FCompactMaps::InvalidID)
						{
							WriteBoneWeights(VertexBoneWeights[Vid], BufferPtr);
						}
					}
				}
				checkSlow(BufferPtr == Buffer.GetData() + Buffer.Num());

				// Compress buffer to archive.
				Ar.SerializeCompressedNew(Buffer.GetData(), Buffer.Num() * sizeof(int32), NAME_Oodle, NAME_Oodle, COMPRESS_NoFlags, false, nullptr);
			}
		}
	}

	virtual SIZE_T GetByteCount() const override
	{
		return VertexBoneWeights.GetByteCount();
	}

protected:

	// interpolation functions; default implementation assumes your attributes can be interpolated as reals

	/** Set the value at an Attribute to be a linear interpolation of two other Attributes */
	void SetBoneWeightsFromLerp(int SetAttribute, int AttributeA, int AttributeB, double Alpha)
	{
		Alpha = FMath::Clamp(Alpha, 0.0, 1.0);
		VertexBoneWeights[SetAttribute] = FBoneWeights::Blend(VertexBoneWeights[AttributeA], VertexBoneWeights[AttributeB], (float)Alpha);
	}

	/** Set the value at an Attribute to be a barycentric interpolation of three other Attributes. If one of the 
	 *  barycentric coordinates is nearly one (within UE_SMALL_NUMBER tolerance) then simply copy over the value of the 
	 *  corresponding Attribute.
	 */
	void SetBoneWeightsFromBary(int SetAttribute, int AttributeA, int AttributeB, int AttributeC, const FVector3d& BaryCoords)
	{
		const float BaryX = static_cast<float>(BaryCoords.X);
		const float BaryY  = static_cast<float>(BaryCoords.Y);
		const float BaryZ = static_cast<float>(BaryCoords.Z);

		if (FMath::IsNearlyEqual(BaryX, 1.0f))
		{
			if (SetAttribute != AttributeA) // avoid unnecessary copy
			{
				VertexBoneWeights[SetAttribute] = VertexBoneWeights[AttributeA];
			}
		}
		else if (FMath::IsNearlyEqual(BaryY, 1.0f)) 
		{
			if (SetAttribute != AttributeB) 
			{
				VertexBoneWeights[SetAttribute] = VertexBoneWeights[AttributeB];
			}
		}
		else if (FMath::IsNearlyEqual(BaryZ, 1.0f)) 
		{
			if (SetAttribute != AttributeC) 
			{
				VertexBoneWeights[SetAttribute] = VertexBoneWeights[AttributeC];
			}
		}
		else 
		{
			VertexBoneWeights[SetAttribute] = FBoneWeights::Blend(VertexBoneWeights[AttributeA], 
																  VertexBoneWeights[AttributeB], 
																  VertexBoneWeights[AttributeC], 
																  BaryX,
																  BaryY,
																  BaryZ);
		}
	}
};


class FDynamicMesh3;
using FDynamicMeshVertexSkinWeightsAttribute = TDynamicVertexSkinWeightsAttribute<FDynamicMesh3>;


template<typename ParentType>
void TDynamicVertexSkinWeightsAttributeChange<ParentType>::SaveInitialVertex(const TDynamicAttributeBase<ParentType>* Attribute, int VertexID)
{
	FChangeVertexBoneWeights& Change = OldVertexBoneWeights.Emplace_GetRef();
	Change.VertexID = VertexID;
	const TDynamicVertexSkinWeightsAttribute<ParentType>* AttribCast = static_cast<const TDynamicVertexSkinWeightsAttribute<ParentType>*>(Attribute);
	AttribCast->GetValue(VertexID, Change.Weights);
}

template<typename ParentType>
void TDynamicVertexSkinWeightsAttributeChange<ParentType>::StoreAllFinalVertices(const TDynamicAttributeBase<ParentType>* Attribute, const TSet<int>& VertexIDs)
{
	NewVertexBoneWeights.Reserve(NewVertexBoneWeights.Num() + VertexIDs.Num());
	const TDynamicVertexSkinWeightsAttribute<ParentType>* AttribCast = static_cast<const TDynamicVertexSkinWeightsAttribute<ParentType>*>(Attribute);
	for (int VertexID : VertexIDs)
	{
		FChangeVertexBoneWeights& Change = NewVertexBoneWeights.Emplace_GetRef();
		Change.VertexID = VertexID;
		AttribCast->GetValue(VertexID, Change.Weights);
	}
}

template<typename ParentType>
bool TDynamicVertexSkinWeightsAttributeChange<ParentType>::Apply(TDynamicAttributeBase<ParentType>* Attribute, bool bRevert) const
{
	const TArray<FChangeVertexBoneWeights> *Changes = bRevert ? &OldVertexBoneWeights : &NewVertexBoneWeights;
	TDynamicVertexSkinWeightsAttribute<ParentType>* AttribCast = static_cast<TDynamicVertexSkinWeightsAttribute<ParentType>*>(Attribute);
	for (const FChangeVertexBoneWeights& Change : *Changes)
	{
		check(AttribCast->GetParent()->IsVertex(Change.VertexID));
		AttribCast->SetValue(Change.VertexID, Change.Weights);
	}
	return true;
}

}
}
