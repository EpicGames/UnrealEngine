// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/MeshAABBTree3.h"
#include "Spatial/SparseDynamicPointOctree3.h"
#include "MeshQueries.h"
#include "Intersection/IntersectionUtil.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
namespace Geometry
{

/**
 * TDynamicVerticesOctree3 is an extension of FSparseDynamicPointOctree3 for the vertices of 
 * a FDynamicXYZ3 instance (eg FDynamicMesh3, TDynamicPointSet3, FDynamicGraph, etc)
 * This extension does several things:
 *   1) provides a simplified API based on vertex IDs to various Octree functions
 *   2) tracks ModifiedBounds box of modified areas
 *
 * The template expansion requires that FDynamicXYZ3 has the following API:
 *   1) a function int MaxVertexID() that returns the maximum vertex ID/index
 *   2) a function bool IsVertex(int) that returns true if the index is valid
 *   3) a function FVector3d GetVertex(int index) that returns the position of a vertex
 *   4) a function FAxisAlignedBox3d GetBounds() that returns a bounding-box of the point set
 */
template<typename SourceType>
class TDynamicVerticesOctree3 : public FSparseDynamicPointOctree3
{
	// potential optimizations:
	//    - keep track of how many vertices are descendents of each cell? root cells?



public:
	/** parent mesh */
	const SourceType* VertexSource;

	/** bounding box of vertices that have been inserted/removed since last clear */
	FAxisAlignedBox3d ModifiedBounds;

	double HitSphereRadius = 1.0f;

	/** 
	 * Add all vertices of MeshIn to the octree 
	 */
	void Initialize(const SourceType* VertexSourceIn, bool bDynamicExpand)
	{
		this->VertexSource = VertexSourceIn;

		HitSphereRadius = VertexSourceIn->GetBounds().Diagonal().Length() * 0.001;

		int MaxVertexID = VertexSource->MaxVertexID();
		for ( int VertexID = 0; VertexID < MaxVertexID; VertexID++ )
		{
			if (VertexSource->IsVertex(VertexID))
			{
				if (bDynamicExpand)
				{
					ModifiedBounds.Contain(VertexSource->GetVertex(VertexID));
					const SourceType* VtxSource = VertexSource;
					FSparseDynamicPointOctree3::InsertPoint_DynamicExpand(
						VertexID,
						[VtxSource](int k) { return VtxSource->GetVertex(k); });
				}
				else
				{
					InsertVertex(VertexID);
				}
			}
		}
	}

	/**
	 * Reset the internal ModifiedBounds box that tracks modified triangle bounds
	 */
	void ResetModifiedBounds()
	{
		ModifiedBounds = FAxisAlignedBox3d::Empty();
	}

	/**
	 * Insert a triangle into the tree
	 */
	void InsertVertex(int32 VertexID)
	{
		FVector3d Point = VertexSource->GetVertex(VertexID);
		ModifiedBounds.Contain(Point);
		FSparseDynamicPointOctree3::InsertPoint(VertexID, Point);
	}

	/**
	 * Insert a list of vertices into the tree
	 */
	void InsertVertices(const TArray<int>& Vertices)
	{
		int N = Vertices.Num();
		for (int i = 0; i < N; ++i)
		{
			FVector3d Point = VertexSource->GetVertex(Vertices[i]);
			ModifiedBounds.Contain(Point);
			FSparseDynamicPointOctree3::InsertPoint(Vertices[i], Point);
		}
	}

	/**
	 * Insert a set of vertices into the tree
	 */
	void InsertVertices(const TSet<int>& Vertices)
	{
		for (int VertexID : Vertices)
		{
			FVector3d Point = VertexSource->GetVertex(VertexID);
			ModifiedBounds.Contain(Point);
			FSparseDynamicPointOctree3::InsertPoint(VertexID, Point);
		}
	}


	/**
	 * Remove a triangle from the tree
	 */
	bool RemovePoint(int32 VertexID)
	{
		FVector3d Point = VertexSource->GetVertex(VertexID);
		ModifiedBounds.Contain(Point);
		return FSparseDynamicPointOctree3::RemovePoint(VertexID);
	}

	/**
	 * Remove a list of vertices from the tree
	 */
	void RemoveVertices(const TArray<int>& Vertices)
	{
		int N = Vertices.Num();
		for ( int i = 0; i < N; ++i )
		{
			FVector3d Point = VertexSource->GetVertex(Vertices[i]);
			ModifiedBounds.Contain(Point);
			FSparseDynamicPointOctree3::RemovePoint(Vertices[i]);
		}
	}

	/**
	 * Remove a set of vertices from the tree
	 */
	void RemoveVertices(const TSet<int>& Vertices)
	{
		for (int VertexID : Vertices)
		{
			FVector3d Point = VertexSource->GetVertex(VertexID);
			ModifiedBounds.Contain(Point);
			FSparseDynamicPointOctree3::RemovePoint(VertexID);
		}
	}


	/**
	 * Reinsert Vertices into the tree
	 */
	template<typename EnumerableType>
	void ReinsertVertices(const EnumerableType& Vertices)
	{
		for (int VertexID : Vertices)
		{
			FVector3d Point = VertexSource->GetVertex(VertexID);
			ModifiedBounds.Contain(Point);
			FSparseDynamicPointOctree3::ReinsertPoint(VertexID, Point);
		}
	}

	/**
	 * Reinsert Vertices into the tree, unless ShouldSkip(VertexID) returns true
	 */
	template<typename EnumerableType>
	void ReinsertFilteredVertices(const EnumerableType& Vertices, TFunctionRef<bool(int32)> ShouldSkip)
	{
		for (int32 VertexID : Vertices)
		{
			if (!ShouldSkip(VertexID))
			{
				FVector3d Point = VertexSource->GetVertex(VertexID);
				ModifiedBounds.Contain(Point);
				FSparseDynamicPointOctree3::ReinsertPoint(VertexID, Point);
			}
		}
	}


	/**
	 * Include the current bounds of a triangle in the ModifiedBounds box
	 */
	void NotifyPendingModification(int VertexID)
	{
		FVector3d Point = VertexSource->GetVertex(VertexID);
		ModifiedBounds.Contain(Point);
	}

	/**
	 * Include the current bounds of a set of vertices in the ModifiedBounds box
	 */
	void NotifyPendingModification(const TSet<int>& Vertices)
	{
		for (int VertexID : Vertices)
		{
			FVector3d Point = VertexSource->GetVertex(VertexID);
			ModifiedBounds.Contain(Point);
		}
	}


	/**
	 * Find the nearest triangle of the VertexSource that is hit by the ray
	 */
	int32 FindNearestHitVertex(const FRay3d& Ray,
		double MaxDistance = TNumericLimits<double>::Max()) const
	{
		return FSparseDynamicPointOctree3::FindNearestHitPoint(Ray,
			[this](int vid, const FRay3d& Ray) {
				FVector3d Point = VertexSource->GetVertex(vid);
				FLinearIntersection Hit = IntersectionUtil::RaySphereIntersection(Ray.Origin, Ray.Direction, Point, HitSphereRadius);
				return (Hit.intersects) ? Hit.parameter.Min : TNumericLimits<double>::Max();
			}, MaxDistance);
	}


	/**
	 * Check that the Octree is internally valid
	 */
	void CheckValidity(
		EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check,
		bool bVerbose = false,
		bool bFailOnMissingPoints = false) const
	{
		FSparseDynamicPointOctree3::CheckValidity(
			[&](int vid) { return VertexSource->IsVertex(vid); },
			[this](int vid) { return VertexSource->GetVertex(vid); },
			FailMode, bVerbose, bFailOnMissingPoints);
	}


};



} // end namespace UE::Geometry
} // end namespace UE
