// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexTypes.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * FVertexInfo stores information about vertex attributes - position, normal, color, UV
 */
struct FVertexInfo
{
	FVector3d Position{ FVector3d::Zero() };
	FVector3f Normal{ FVector3f::Zero() };
	FVector3f Color{ FVector3f::Zero() };
	FVector2f UV{ FVector2f::Zero() };
	bool bHaveN{}, bHaveC{}, bHaveUV{};

	FVertexInfo() = default;
	FVertexInfo(const FVector3d& PositionIn)
		: Position{ PositionIn }{}
	FVertexInfo(const FVector3d& PositionIn, const FVector3f& NormalIn)
		: Position{ PositionIn }, Normal{ NormalIn }, bHaveN{ true }{}
	FVertexInfo(const FVector3d& PositionIn, const FVector3f& NormalIn, const FVector3f& ColorIn)
		: Position{ PositionIn }, Normal{ NormalIn }, Color{ ColorIn }, bHaveN{ true }, bHaveC{true}{}
	FVertexInfo(const FVector3d& PositionIn, const FVector3f& NormalIn, const FVector3f& ColorIn, const FVector2f& UVIn)
		: Position{ PositionIn }, Normal{ NormalIn }, Color{ ColorIn }, UV{ UVIn }, bHaveN{ true }, bHaveC{true}, bHaveUV{true}{}
};



/**
 * FMeshTriEdgeID identifies an edge in a triangle mesh based on 
 * the triangle ID/Index and the "edge index" 0/1/2 in the triangle.
 * If the ordered triangle vertices are [A,B,C], then [A,B]=0, [B,C]=1, and [C,A]=2.
 * 
 * This type of edge identifier is applicable on any indexed mesh, even if 
 * the mesh does not store explicit edge IDs. 
 * 
 * Values are stored unsigned, so it is currently *not* possible to store an "invalid"
 * edge identifier as a FMeshTriEdgeID (0xFFFFFFFF could potentially be used as such an identifier).
 * 
 * The TriangleID is stored in 30 bits, while FDynamicMesh3 stores (valid) triangle IDs in 31 bits.
 * So, only ~1 billion triangles are allowed in a mesh when using FMeshTriEdgeID, vs ~2 billion in FDynamicMesh3.
 * This limit has not been encountered in practice, to date.
 * 
 * Note that cycling or permuting the vertices of a triangle will change these indices.
 */
struct FMeshTriEdgeID
{
	/** The 0/1/2 index of the edge in the triangle's tuple of edges */
	unsigned TriEdgeIndex : 2;
	/** The index of the mesh Triangle */
	unsigned TriangleID : 30;

	FMeshTriEdgeID()
	{
		TriEdgeIndex = 0;
		TriangleID = 0;
	}

	/**
	 * Construct a FMeshTriEdgeID for the given TriangleID and Edge Index in range
	 * @param EdgeIndexIn index in range 0,1,2
	 */
	FMeshTriEdgeID(int32 TriangleIDIn, int32 EdgeIndexIn)
	{
		checkSlow(EdgeIndexIn >= 0 && EdgeIndexIn <= 2);
		checkSlow(TriangleIDIn >= 0 && TriangleIDIn < (1<<30));
		TriEdgeIndex = (unsigned int)EdgeIndexIn;
		TriangleID = (unsigned int)TriangleIDIn;
	}

	/**
	 * Decode an encoded FMeshTriEdgeID from a packed uint32 created by the Encoded() function
	 */
	explicit FMeshTriEdgeID(uint32 EncodedEdgeKey)
	{
		TriangleID = EncodedEdgeKey & 0x8FFFFFFF;
		TriEdgeIndex = (EncodedEdgeKey & 0xC0000000) >> 30;
	}

	/**
	 * @return the (TriangleID, TriEdgeIndex) values packed into a 32 bit integer
	 */
	uint32 Encoded() const
	{
		return (TriEdgeIndex << 30) | TriangleID;
	}
};


/**
 * FMeshTriOrderedEdgeID identifies an oriented edge in a triangle mesh based on indices
 * into the triangle vertices. IE if a triangle has vertices [A,B,C], then an
 * oriented edge could be (A,B) or (B,A), or any of the other 4 permutations.
 * So the ordered edge in the triangle can be represented as two vertex indices, 
 * and the full encoding is (TriangleID, J, K) where J and K are in range 0/1/2.
 * 
 * This type of edge identifier is applicable on any indexed mesh, even if 
 * the mesh does not store explicit edge IDs. In addition, this identifier is stable
 * across mesh topological changes, ie if two connected triangles are unlinked (ie
 * the shared edge becomes two edges, and the 2 vertices become 4), the FMeshTriOrderedEdgeID
 * will still refer to the correct oriented edge, as it does not explicitly depend on
 * the Vertex or Edge IDs, only the ordering within the triangle.
 * 
 * Note that cycling or permuting the vertices of a triangle will change/break these indices.
 */
struct FMeshTriOrderedEdgeID
{
	/** The index of the mesh Triangle */
	int32 TriangleID;
	/** The 0/1/2 index of the first vertex in the triangles tuple of vertices */
	unsigned VertIndexA : 2;
	/** The 0/1/2 index of the second vertex in the triangles tuple of vertices */
	unsigned VertIndexB : 2;


	FMeshTriOrderedEdgeID()
	{
		TriangleID = IndexConstants::InvalidID;
		VertIndexA = 0;
		VertIndexB = 0;
	}

	FMeshTriOrderedEdgeID(int32 TriangleIDIn, int32 VertexIndexA, int32 VertexIndexB)
	{
		checkSlow(VertexIndexA >= 0 && VertexIndexA <= 2);
		checkSlow(VertexIndexB >= 0 && VertexIndexB <= 2);
		TriangleID = TriangleIDIn;
		VertIndexA = VertexIndexA;
		VertIndexB = VertexIndexB;
	}
};




} // end namespace UE::Geometry
} // end namespace UE



namespace DynamicMeshInfo
{
using namespace UE::Geometry;

/** Information about the mesh elements created by a call to SplitEdge() */
struct FEdgeSplitInfo
{
	int OriginalEdge;					// the edge that was split
	FIndex2i OriginalVertices;			// original edge vertices [a,b]
	FIndex2i OtherVertices;				// original opposing vertices [c,d] - d is InvalidID for boundary edges
	FIndex2i OriginalTriangles;			// original edge triangles [t0,t1]
	bool bIsBoundary;					// was the split edge a boundary edge?  (redundant)

	int NewVertex;						// new vertex f that was created
	FIndex2i NewTriangles;				// new triangles [t2,t3], oriented as explained in SplitEdge() header comment
	FIndex3i NewEdges;					// new edges are [f,b], [f,c] and [f,d] if this is not a boundary edge

	double SplitT;						// parameter value for NewVertex along original edge
};

/** Information about the mesh elements modified by a call to FlipEdge() */
struct FEdgeFlipInfo
{
	int EdgeID;						// the edge that was flipped
	FIndex2i OriginalVerts;			// original verts of the flipped edge, that are no longer connected
	FIndex2i OpposingVerts;			// the opposing verts of the flipped edge, that are now connected
	FIndex2i Triangles;				// the two triangle IDs. Original tris vert [Vert0,Vert1,OtherVert0] and [Vert1,Vert0,OtherVert1].
										// New triangles are [OtherVert0, OtherVert1, Vert1] and [OtherVert1, OtherVert0, Vert0]
};

/** Information about mesh elements modified/removed by CollapseEdge() */
struct FEdgeCollapseInfo
{
	int KeptVertex;					// the vertex that was kept (ie collapsed "to")
	int RemovedVertex;				// the vertex that was removed
	FIndex2i OpposingVerts;			// the opposing vertices [c,d]. If the edge was a boundary edge, d is InvalidID
	bool bIsBoundary;				// was the edge a boundary edge

	int CollapsedEdge;				// the edge that was collapsed/removed
	FIndex2i RemovedTris;			// the triangles that were removed in the collapse (second is InvalidID for boundary edge)
	FIndex2i RemovedEdges;			// the edges that were removed (second is InvalidID for boundary edge)
	FIndex2i KeptEdges;				// the edges that were kept (second is InvalidID for boundary edge)

	double CollapseT;				// interpolation parameter along edge for new vertex in range [0,1] where 0 => KeptVertex and 1 => RemovedVertex
};

/** Information about mesh elements modified by MergeEdges() */
struct FMergeEdgesInfo
{
	int KeptEdge;				// the edge that was kept
	int RemovedEdge;			// the edge that was removed

	FIndex2i KeptVerts;			// The two vertices that were kept (redundant w/ KeptEdge?)
	FIndex2i RemovedVerts;		// The removed vertices of RemovedEdge. Either may be InvalidID if it was same as the paired KeptVert

	FIndex2i ExtraRemovedEdges; // extra removed edges, see description below. Either may be or InvalidID
	FIndex2i ExtraKeptEdges;	// extra kept edges, paired with ExtraRemovedEdges

	// Even more Removed and Kept edges, in cases where there were multiple such edges on one or both sides of the merged edge
	// Only possible if the pre-merge mesh had non-manifold vertices (aka bowties), in almost all meshes these arrays will be empty
	TArray<int, TInlineAllocator<4>> BowtiesRemovedEdges, BowtiesKeptEdges;

	double InterpolationT = 0;	// Interpolation parameter for each kept vertex in range [0,1] where 0 => KeptVertex and 1 => RemovedVertex
};

/** Information about mesh elements modified by MergeVertices() */
struct FMergeVerticesInfo
{
	int KeptVertex;				// the vertex that was kept
	int RemovedVertex;			// the vertex that was removed
	double InterpolationT = 0;	// Interpolation parameter for the kept vertex in range [0,1] where 0 => KeptVertex and 1 => RemovedVertex

	// If the merge resolves as an edge collapse, the information is stored here
	TOptional<FEdgeCollapseInfo> EdgeCollapseInfo;
	// If the merge resolves as an edge weld, the information is stored here
	TOptional<FMergeEdgesInfo> MergeEdgesInfo;
};

/** Information about mesh elements modified/created by PokeTriangle() */
struct FPokeTriangleInfo
{
	int OriginalTriangle;				// the triangle that was poked
	FIndex3i TriVertices;				// vertices of the original triangle

	int NewVertex;						// the new vertex that was inserted
	FIndex2i NewTriangles;				// the two new triangles that were added (OriginalTriangle is re-used, see code for vertex orders)
	FIndex3i NewEdges;					// the three new edges connected to NewVertex

	FVector3d BaryCoords;				// barycentric coords that NewVertex was inserted at
};

/** Information about mesh elements modified/created by SplitVertex() */
struct FVertexSplitInfo
{
	int OriginalVertex;
	int NewVertex;
	// if needed could possibly add information about added edges?  but it would be a dynamic array, and there is no use for it yet.
	// modified triangles are passed as input to the function, no need to store those here.
};
}
