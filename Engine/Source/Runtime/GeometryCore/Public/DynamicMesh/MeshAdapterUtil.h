// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicPointSet3.h"
#include "Math/Transform.h"
#include "MeshAdapter.h"
#include "PointSetAdapter.h"

/**
 * Utility functions for constructing various PointSetAdapter and MeshAdapter instances from dynamic meshes
 */
namespace UE
{
namespace Geometry
{
	class FDynamicMesh3;

	/**
	 * @return Transformed adapter of a FDynamicMesh
	 */
	FTriangleMeshAdapterd GEOMETRYCORE_API MakeTransformedDynamicMeshAdapter(const FDynamicMesh3* Mesh, FTransform3d Transform);

	/**
	 * @return 1:1 adapter of a FDynamicMesh; can be used as a starting point to create other adapters
	 */
	FTriangleMeshAdapterd GEOMETRYCORE_API MakeDynamicMeshAdapter(const FDynamicMesh3* Mesh);

	/**
	 * @return Mesh vertices as a point set
	 */
	FPointSetAdapterd GEOMETRYCORE_API MakeVerticesAdapter(const FDynamicMesh3* Mesh);

	/**
	 * @return PointSet points as a point set
	 */
	FPointSetAdapterd GEOMETRYCORE_API MakePointsAdapter(const FDynamicPointSet3d* PointSet);

	/**
	 * @return Mesh triangle centroids as a point set
	 */
	FPointSetAdapterd GEOMETRYCORE_API MakeTriCentroidsAdapter(const FDynamicMesh3* Mesh);

	/**
	 * @return mesh edge midpoints as a point set
	 */
	FPointSetAdapterd GEOMETRYCORE_API MakeEdgeMidpointsAdapter(const FDynamicMesh3* Mesh);

	/**
	 * @return Mesh boundary edge midpoints as a point set
	 */
	FPointSetAdapterd GEOMETRYCORE_API MakeBoundaryEdgeMidpointsAdapter(const FDynamicMesh3* Mesh);

}
}
