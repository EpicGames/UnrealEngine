// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "HAL/Platform.h"
#include "IntBoxTypes.h"


/**
 * Utility functions for manipulating DynamicMesh attribute sets
 */
namespace UE
{
namespace Geometry
{
	struct FInterval1i;
	template <typename RealType> class TDynamicMeshScalarTriangleAttribute;

	/**
	 * Compact the values of an integer Triangle Attribute, ie so that the attribute values are dense in range 0..N.
	 * Useful for (eg) compacting MaterialIDs or Polygroups.
	 * Note that this function cannot tell if values were removed from the "end" of the attribute value list. Some care must 
	 * be taken in calling code to handle this case if using this function to determine whether compaction of other data structures is needed.
	 * Currently OldToNewMapOut and NewToOldMapOut are always populated even if the input is compact. 
	 * @param OldMaxAttributeRangeOut min and max input attribute values, ie pre-compaction
	 * @param NewMaxAttributeValueOut new max attribute value after compacting.
	 * @param OldToNewMapOut generated mapping from previous IDs to new IDs (ie size is maximum input attrib value (+1), will contain InvalidID for any compacted values )
	 * @param NewToOldMapOut generated mapping from new IDs to previous IDs (ie size is maximum output attribute value (+1), no invalid values)
	 * @param bWasCompactOut set to true if the attribute set was already compact, IE was not modified by the operation
	 * @return true on success (including if already compact). false if any Attribute Values were negative/InvalidID, cannot compact in this case
	 */
	GEOMETRYCORE_API bool CompactAttributeValues(
		const FDynamicMesh3& Mesh,
		TDynamicMeshScalarTriangleAttribute<int32>& TriangleAttrib,
		FInterval1i& OldMaxAttributeRangeOut,
		int& NewMaxAttributeValueOut,
		TArray<int32>& OldToNewMapOut,
		TArray<int32>& NewToOldMapOut,
		bool& bWasCompactOut);

	/**
	 * Copies vertex UVs to a given overlay (clearing it before use if it has any elements).
	 * 
	 * @param bCompactElements If true, elements are compacted in the overlay
	 * @return true if successful
	 */
	GEOMETRYCORE_API bool CopyVertexUVsToOverlay(
		const FDynamicMesh3& Mesh,
		FDynamicMeshUVOverlay& UVOverlayOut,
		bool bCompactElements = false);

	/**
	 * Copies vertex normals to a given overlay (clearing it before use if it has any elements).
	 *
	 * @param bCompactElements If true, elements are compacted in the overlay
	 * @return true if successful
	 */
	GEOMETRYCORE_API bool CopyVertexNormalsToOverlay(
		const FDynamicMesh3& Mesh,
		FDynamicMeshNormalOverlay& NormalOverlayOut,
		bool bCompactElements = false);
}
}
