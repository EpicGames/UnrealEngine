// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"

// Forward declarations
namespace UE::Geometry 
{ 
    class FDynamicMesh3; 
}
class FName;

namespace UE
{
namespace Geometry
{

/**
 * FMeshBones is a utility class for manipulating mesh bone attributes
 */
class FMeshBones
{
public:

	/**
     * Return an array of indices into the bone attribute arrays of all the children of the bone. Optionally, recursively
     * add all grandchildren via a breadth-first search.
     *
     * @param BoneIndex the index of the bone whose children we are requesting
     * @param bRecursive if true, recursively add all grandchildren
     */
	static GEOMETRYCORE_API bool GetBoneChildren(const FDynamicMesh3& Mesh, int32 BoneIndex, TArray<int32>& ChildrenIndices, bool bRecursive = false);

	/**
     * Create bone data arrays that can be used to create a FReferenceSkeleton.
     *
     * @param bOrderChanged true, if the order of the bones in the output data arrays is different than
     *                      the order in the mesh bone attribute
     *
     * @return false if the dynamic mesh has no bone attributes
     *
     * @note the order of the bones in the output arrays might be different than the order of the bones in
     * the mesh bone attribute. The reference skeleton expects the parent bone to always have a lower index than the
     * children. This doesn't have to be true for dynamic mesh bone attributes. You can use
     * TDynamicVertexSkinWeightsAttribute::ReindexBoneIndicesToSkeleton method to reindex the skinning weights if the
     * order changed (bOutOrderChanged == true).
     */
	static GEOMETRYCORE_API bool GetBonesInIncreasingOrder(const FDynamicMesh3& Mesh, 
                                                           TArray<FName>& BoneNames,
                                                           TArray<int32>& BoneParentIdx,
                                                           TArray<FTransform>& BonePose, 
                                                           bool& bOrderChanged);


	/** 
     * Create a superset reference skeleton out of all the bones in the mesh Lods. 
     * 
     * @param bOrderChanged true, if all Lods have the same bone attributes and they are ordered correctly such that 
     *                      the parent bone always has a lower index than the children. Hence the arrays can be directly
     *                      added to an empty FReferenceSkeleton instance.
     *
     * @return false if one of the Lods has no bone attributes or Lods have incompatible skeletons
     */
	static GEOMETRYCORE_API bool CombineLodBonesToReferenceSkeleton(const TArray<FDynamicMesh3>& Meshes, 
                                                                    TArray<FName>& BoneNames,
                                                                    TArray<int32>& BoneParentIdx,
                                                                    TArray<FTransform>& BonePose, 
                                                                    bool& bOrderChanged);

};

} // end namespace UE::Geometry
} // end namespace UE
