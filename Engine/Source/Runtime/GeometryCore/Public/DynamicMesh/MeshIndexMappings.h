// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryTypes.h"

namespace UE
{
namespace Geometry
{
class FDynamicMesh3;


/**
 * FMeshIndexMappings stores a set of integer IndexMaps for a mesh
 * This is a convenient object to have, to avoid passing around large numbers of separate maps.
 * The individual maps are not necessarily all filled by every operation.
 */
struct FMeshIndexMappings
{
protected:
	FIndexMapi VertexMap;
	FIndexMapi TriangleMap;
	FIndexMapi GroupMap;

	FIndexMapi ColorMap;
	TArray<FIndexMapi> UVMaps;
	TArray<FIndexMapi> NormalMaps;

public:
	/** Size internal arrays-of-maps to be suitable for this Mesh */
	GEOMETRYCORE_API void Initialize(FDynamicMesh3* Mesh);

	/** @return the value used to indicate "invalid" in the mapping */
	constexpr int InvalidID() const { return VertexMap.UnmappedID(); }

	void Reset()
	{
		VertexMap.Reset();
		TriangleMap.Reset();
		GroupMap.Reset();
		ColorMap.Reset();
		for (FIndexMapi& UVMap : UVMaps)
		{
			UVMap.Reset();
		}
		for (FIndexMapi& NormalMap : NormalMaps)
		{
			NormalMap.Reset();
		}
	}

	void ResetTriangleMap()
	{
		TriangleMap.Reset();
	}

	FIndexMapi& GetVertexMap() { return VertexMap; }
	const FIndexMapi& GetVertexMap() const { return VertexMap; }
	inline void SetVertex(int FromID, int ToID) { VertexMap.Add(FromID, ToID); }
	inline int GetNewVertex(int FromID) const { return VertexMap.GetTo(FromID); }
	inline bool ContainsVertex(int FromID) const { return VertexMap.ContainsFrom(FromID); }

	FIndexMapi& GetTriangleMap() { return TriangleMap; }
	const FIndexMapi& GetTriangleMap() const { return TriangleMap; }
	void SetTriangle(int FromID, int ToID) { TriangleMap.Add(FromID, ToID); }
	int GetNewTriangle(int FromID) const { return TriangleMap.GetTo(FromID); }
	inline bool ContainsTriangle(int FromID) const { return TriangleMap.ContainsFrom(FromID); }

	FIndexMapi& GetGroupMap() { return GroupMap; }
	const FIndexMapi& GetGroupMap() const { return GroupMap; }
	void SetGroup(int FromID, int ToID) { GroupMap.Add(FromID, ToID); }
	int GetNewGroup(int FromID) const { return GroupMap.GetTo(FromID); }
	inline bool ContainsGroup(int FromID) const { return GroupMap.ContainsFrom(FromID); }

	FIndexMapi& GetUVMap(int UVLayer) { return UVMaps[UVLayer]; }
	const FIndexMapi& GetUVMap(int UVLayer) const { return UVMaps[UVLayer]; }
	void SetUV(int UVLayer, int FromID, int ToID) { UVMaps[UVLayer].Add(FromID, ToID); }
	int GetNewUV(int UVLayer, int FromID) const { return UVMaps[UVLayer].GetTo(FromID); }
	inline bool ContainsUV(int UVLayer, int FromID) const { return UVMaps[UVLayer].ContainsFrom(FromID); }

	FIndexMapi& GetNormalMap(int NormalLayer) { return NormalMaps[NormalLayer]; }
	const FIndexMapi& GetNormalMap(int NormalLayer) const { return NormalMaps[NormalLayer]; }
	void SetNormal(int NormalLayer, int FromID, int ToID) { NormalMaps[NormalLayer].Add(FromID, ToID); }
	int GetNewNormal(int NormalLayer, int FromID) const { return NormalMaps[NormalLayer].GetTo(FromID); }
	inline bool ContainsNormal(int NormalLayer, int FromID) const { return NormalMaps[NormalLayer].ContainsFrom(FromID); }

	FIndexMapi& GetColorMap() { return ColorMap; }
	const FIndexMapi& GetColorMap() const { return ColorMap; }
	void SetColor( int FromID, int ToID) { ColorMap.Add(FromID, ToID); }
	int GetNewColor(int FromID) const { return ColorMap.GetTo(FromID); }
	inline bool ContainsColor(int FromID) const { return ColorMap.ContainsFrom(FromID); }

};

} // end namespace UE::Geometry
} // end namespace UE
