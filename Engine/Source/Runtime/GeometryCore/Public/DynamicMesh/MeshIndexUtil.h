// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "HAL/Platform.h"
#include "IndexTypes.h"
#include "Misc/AssertionMacros.h"

namespace UE
{
namespace Geometry
{
	/**
	 * Find list of unique vertices that are contained in one or more triangles
	 * @param Mesh input mesh
	 * @param TriangleIDs list of triangle IDs of Mesh
	 * @param VertexIDsOut list of vertices contained by triangles
	 */
	GEOMETRYCORE_API void TriangleToVertexIDs(const FDynamicMesh3* Mesh, TConstArrayView<int> TriangleIDs, TArray<int>& VertexIDsOut);


	/**
	 * Find all the triangles in all the one rings of a set of vertices
	 * @param Mesh input mesh
	 * @param VertexIDs list of Vertex IDs of Mesh
	 * @param TriangleIDsOut list of triangle IDs where any vertex of triangle touches an element of VertexIDs
	 */
	GEOMETRYCORE_API void VertexToTriangleOneRing(const FDynamicMesh3* Mesh, TConstArrayView<int> VertexIDs, TSet<int>& TriangleIDsOut);


	/**
	 * For a given list/enumeration of Triangles, and an Overlay, find all the Elements in all the Triangles.
	 * Note that if ElementsOut is a TArray, it may include an Element multiple times.
	 * @param Overlay source overlay
	 * @param TriangleEnumeration list of triangles
	 * @param ElementsOut list of elements
	 */
	template<typename OverlayType, typename EnumeratorType, typename OutputSetType>
	void TrianglesToOverlayElements(const OverlayType* Overlay, EnumeratorType TriangleEnumeration, OutputSetType& ElementsOut);



	/**
	 * Find the two edges in a Triangle that are connected to a given Vertex of the triangle
	 * @return edge pair, or (IndexConstants::InvalidID, IndexConstants::InvalidID) on error
	 */
	GEOMETRYCORE_API FIndex2i FindVertexEdgesInTriangle(const FDynamicMesh3& Mesh, int32 TriangleID, int32 VertexID);


	/**
	 * @return the ID of the shared edge betwen two given triangles, or IndexConstants::InvalidID if not found / invalid input
	 */
	GEOMETRYCORE_API int32 FindSharedEdgeInTriangles(const FDynamicMesh3& Mesh, int32 Triangle0, int32 Triangle1);


	/**
	 * Call SetType.Add(VertexPosition) for all valid vertex indices in for_each(Enumeration)
	 */
	template<typename EnumeratorType, typename SetType>
	void CollectVertexPositions(const FDynamicMesh3& Mesh, EnumeratorType Enumeration, SetType& Output);


	/**
	 * Walk around VertexID from FromTriangleID to next connected triangle if it exists, walking "away" from PrevTriangleID.
	 * @param TrisConnectedFunc returns true if two triangles should be considered connected, to support breaking at seams/etc that are not in base mesh topology
	 * @return triplet of values (FoundTriangleID, SharedEdgeID, IndexOfEdgeInFromTri), or all IndexConstants::InvalidID if not found
	 */
	template<typename TrisConnectedPredicate>
	FIndex3i FindNextAdjacentTriangleAroundVtx(const FDynamicMesh3* Mesh,
		int32 VertexID, int32 FromTriangleID, int32 PrevTriangleID,
		TrisConnectedPredicate TrisConnectedTest);

	/**
	 * Split the triangle one-ring at VertexID into two triangle sets by "cutting" it with edge SplitEdgeID that is connected to VertexID.
	 * VertexID must be a boundary-vertex of the mesh, otherwise one edge is not enough to "split" it. Bowtie vertices are not supported.
	 * @return false if the split failed, ie the inputs are invalid or one of the output sets is empty 
	 */
	GEOMETRYCORE_API bool SplitBoundaryVertexTrianglesIntoSubsets(
		const FDynamicMesh3* Mesh,
		int32 VertexID,
		int32 SplitEdgeID,
		TArray<int32>& TriangleSet0, TArray<int32>& TriangleSet1);

	/**
	* Split the triangle one-ring at VertexID into two triangle sets by "cutting" it with two edges SplitEdgeID0 and SplitEdgeID1 that are both connected to VertexID.
	* VertexID must be an "interior" vertex of the mesh (ie not on boundary). Bowtie vertices are not supported.
	* @return false if the split failed, ie the inputs are invalid or one of the output sets is empty
	*/
	GEOMETRYCORE_API bool SplitInteriorVertexTrianglesIntoSubsets(
		const FDynamicMesh3* Mesh,
		int32 VertexID,
		int32 SplitEdgeID0, int32 SplitEdgeID1,
		TArray<int32>& TriangleSet0, TArray<int32>& TriangleSet1);

}
}



// implementations of functions declared above

namespace UE
{
namespace Geometry
{

template<typename EnumeratorType, typename SetType>
void CollectVertexPositions(const FDynamicMesh3& Mesh, EnumeratorType Enumeration, SetType& Output)
{
	for (int32 vid : Enumeration)
	{
		if (Mesh.IsVertex(vid))
		{
			Output.Add(Mesh.GetVertex(vid));
		}
	}
}


template<typename TrisConnectedPredicate>
FIndex3i FindNextAdjacentTriangleAroundVtx(const FDynamicMesh3* Mesh, 
	int32 VertexID, int32 FromTriangleID, int32 PrevTriangleID,
	TrisConnectedPredicate TrisConnectedTest)
{
	check(Mesh);

	// find neigbour edges and tris for triangle
	FIndex3i TriEdges = Mesh->GetTriEdges(FromTriangleID);
	FIndex3i TriNbrTris;
	for (int32 j = 0; j < 3; ++j)
	{
		FIndex2i EdgeT = Mesh->GetEdgeT(TriEdges[j]);
		TriNbrTris[j] = (EdgeT.A == FromTriangleID) ? EdgeT.B : EdgeT.A;
	}

	// Search for the neighbour tri that is not PrevTriangleID, and is also connected to VertexID.
	// This is our next triangle around the ring
	for (int32 j = 0; j < 3; ++j)
	{
		if (TriNbrTris[j] != PrevTriangleID && Mesh->IsTriangle(TriNbrTris[j]))
		{
			FIndex3i TriVerts = Mesh->GetTriangle(TriNbrTris[j]);
			if (TriVerts.A == VertexID || TriVerts.B == VertexID || TriVerts.C == VertexID)
			{
				// test if predicate allows this connection
				if (TrisConnectedTest(FromTriangleID, TriNbrTris[j], TriEdges[j]) == false)
				{
					break;
				}
				return FIndex3i(TriNbrTris[j], TriEdges[j], j);
			}
		}
	}

	return FIndex3i(IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID);
}



template<typename OverlayType, typename EnumeratorType, typename OutputSetType>
void TrianglesToOverlayElements(const OverlayType* Overlay, EnumeratorType TriangleEnumeration, OutputSetType& ElementsOut)
{
	for (int32 TriangleID : TriangleEnumeration)
	{
		if ( Overlay->IsSetTriangle(TriangleID) )
		{
			FIndex3i TriElements = Overlay->GetTriangle(TriangleID);
			ElementsOut.Add(TriElements.A);
			ElementsOut.Add(TriElements.B);
			ElementsOut.Add(TriElements.C);
		}
	}
}



}
}
