// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshNormals

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh3.h"
#include "DynamicMeshAttributeSet.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"

template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Geometry
{


/**
 * FMeshNormals is a utility class that can calculate and store various types of
 * normal vectors for a FDynamicMesh. 
 */
class FMeshNormals
{
protected:
	/** Target Mesh */
	const FDynamicMesh3* Mesh;
	/** Set of computed normals */
	TArray<FVector3d> Normals;

public:
	FMeshNormals()
	{
		Mesh = nullptr;
	}

	FMeshNormals(const FDynamicMesh3* Mesh)
	{
		SetMesh(Mesh);
	}


	void SetMesh(const FDynamicMesh3* MeshIn)
	{
		this->Mesh = MeshIn;
	}

	const TArray<FVector3d>& GetNormals() const { return Normals; }

	/*
	 * @return the normals array by a move
	 */
	TArray<FVector3d>&& MoveNormals()
	{ 
		return MoveTemp(Normals); 
	}

	FVector3d& operator[](int i) { return Normals[i]; }
	const FVector3d& operator[](int i) const { return Normals[i]; }


	/**
	 * Set the size of the Normals array to Count, and optionally clear all values to (0,0,0)
	 */
	GEOMETRYCORE_API void SetCount(int Count, bool bClearToZero);

	/**
	 * Compute standard per-vertex normals by averaging one-ring face normals
	 */
	void ComputeVertexNormals(bool bWeightByArea = true, bool bWeightByAngle = true)
	{
		Compute_FaceAvg(bWeightByArea, bWeightByAngle);
	}
	
	// Method to use when copying split normals to per-vertex normals
	enum class ECombineSplitNormalsMethod
	{
		// Average the split normals
		Average,
		// Arbitrarily choose one of the normals to exactly copy
		CopyAny
	};

	/**
	 * Fill in per-vertex normals in the FMeshNormals::Normals array by copying normals from the overlay
	 * @param CombineSplitNormals Method to use when copying split normals to per-vertex normals.
	 */
	GEOMETRYCORE_API void GetVertexNormalsFromOverlayNormals(ECombineSplitNormalsMethod CombineSplitNormals = ECombineSplitNormalsMethod::Average);

	/**
	 * Compute per-triangle normals
	 */
	void ComputeTriangleNormals()
	{
		Compute_Triangle();
	}

	/**
	 * Assumes that ComputeTriangleNormals() has already been run.
	 * 
	 * Looks through the normals for any that are zero, and tries to set them to the normal of 
	 * some neighboring triangle, preferring the triangle on its longer side if possible. This is 
	 * useful for finding connected components based on normals, as it prevents degenerate triangles
	 * with 0 normals from connecting otherwise disconnected components.
	 * This won't help if a whole mesh componenent is entirely made of such degenerate triangles
	 * (which is usually ok).
	 */
	GEOMETRYCORE_API void SetDegenerateTriangleNormalsToNeighborNormal();

	/**
	 * Recompute the per-element normals of the given overlay by averaging one-ring face normals
	 * @warning NormalOverlay must be attached to ParentMesh or an exact copy
	 */
	void RecomputeOverlayNormals(const FDynamicMeshNormalOverlay* NormalOverlay, bool bWeightByArea = true, bool bWeightByAngle = true)
	{
		Compute_Overlay_FaceAvg(NormalOverlay, bWeightByArea, bWeightByAngle);
	}


	/**
	 * Copy the current set of normals to the vertex normals of SetMesh
	 * @warning assumes that the computed normals are vertex normals
	 * @param bInvert if true, normals are flipped
	 */
	GEOMETRYCORE_API void CopyToVertexNormals(FDynamicMesh3* SetMesh, bool bInvert = false) const;

	/**
	 * Copy the current set of normals to the NormalOverlay attribute layer
	 * @warning assumes that the computed normals are attribute normals
	 * @param bInvert if true, normals are flipped
	 */
	GEOMETRYCORE_API void CopyToOverlay(FDynamicMeshNormalOverlay* NormalOverlay, bool bInvert = false) const;


	/**
	 * Compute per-vertex normals for the given Mesh
	 * @param bInvert if true, normals are flipped
	 */
	static GEOMETRYCORE_API void QuickComputeVertexNormals(FDynamicMesh3& Mesh, bool bInvert = false);


	/**
	 * Apply rounds of explicit uniform-weighted normal smoothing to the VertexNormals attribute of the given Mesh.
	 */
	static GEOMETRYCORE_API void SmoothVertexNormals(FDynamicMesh3& Mesh, int32 SmoothingRounds, double SmoothingAlpha);


	/**
	 * Compute per-vertex normals for the vertices of a set of triangles of a Mesh
	 * @param bWeightByArea weight neighbor triangles by area
	 * @param bWeightByAngle weight neighbor triangles by angle
	 * @param bInvert if true, normals are flipped
	 */
	static GEOMETRYCORE_API void QuickComputeVertexNormalsForTriangles(FDynamicMesh3& Mesh, const TArray<int32>& Triangles, bool bWeightByArea = true, bool bWeightByAngle = true, bool bInvert = false);


	/**
	 * Compute normal at mesh vertex by weighted sum of one-ring triangle normals. Can optionally weight by area, angle, or both (averaged)
	 * @param bWeightByArea weight neighbor triangles by area
	 * @param bWeightByAngle weight neighbor triangles by angle
	 * @return the vertex normal at vertex VertIdx of Mesh.
	 */
	static GEOMETRYCORE_API FVector3d ComputeVertexNormal(const FDynamicMesh3& Mesh, int VertIdx, bool bWeightByArea = true, bool bWeightByAngle = true);

	/**
	 * Compute normal at mesh vertex by weighted sum of subset of one-ring triangle normals. Can optionally weight by area, angle, or both (averaged)
	 * @param TriangleFilterFunc Only one-ring triangles for which this function returns true will be included
	 * @param bWeightByArea weight neighbor triangles by area
	 * @param bWeightByAngle weight neighbor triangles by angle
	 * @return the vertex normal at vertex VertIdx of Mesh.
	 */
	static GEOMETRYCORE_API FVector3d ComputeVertexNormal(const FDynamicMesh3& Mesh, int32 VertIdx, TFunctionRef<bool(int32)> TriangleFilterFunc, bool bWeightByArea = true, bool bWeightByAngle = true);


	/**
	 * @return the computed overlay normal at an element of the overlay (ie based on normals of triangles connected to this element)
	 * @param bWeightByArea weight neighbor triangles by area
	 * @param bWeightByAngle weight neighbor triangles by angle
	 */
	static GEOMETRYCORE_API FVector3d ComputeOverlayNormal(const FDynamicMesh3& Mesh, const FDynamicMeshNormalOverlay* NormalOverlay, int ElemIdx, bool bWeightByArea = true, bool bWeightByAngle = true);

	/**
	 * Initialize the given NormalOverlay with per-vertex normals, ie single overlay element for each mesh vertex.
	 * @param bUseMeshVertexNormalsIfAvailable if true and the parent mesh has per-vertex normals, use them instead of calculating new ones
	 */
	static GEOMETRYCORE_API void InitializeOverlayToPerVertexNormals(FDynamicMeshNormalOverlay* NormalOverlay, bool bUseMeshVertexNormalsIfAvailable = true);


	/**
	 * Initialize the given NormalOverlay with per-face normals, ie separate overlay element for each vertex of each triangle.
	 */
	static GEOMETRYCORE_API void InitializeOverlayToPerTriangleNormals(FDynamicMeshNormalOverlay* NormalOverlay);


	static GEOMETRYCORE_API void InitializeOverlayTopologyFromOpeningAngle(const FDynamicMesh3* Mesh, FDynamicMeshNormalOverlay* NormalOverlay, 
		double AngleThresholdDeg);

	static GEOMETRYCORE_API void InitializeOverlayTopologyFromFaceGroups(const FDynamicMesh3* Mesh, FDynamicMeshNormalOverlay* NormalOverlay);


	/**
	 * Initialize the given Mesh with per-face normals, ie separate overlay element for each vertex of each triangle.
	 */
	static GEOMETRYCORE_API void InitializeMeshToPerTriangleNormals(FDynamicMesh3* Mesh);


	/**
	 * Initialize the given triangles of NormalOverlay with per-vertex normals, ie single overlay element for each mesh vertex.
	 * Only the triangles included in the region are considered when calculating per-vertex normals.
	 */
	static GEOMETRYCORE_API void InitializeOverlayRegionToPerVertexNormals(FDynamicMeshNormalOverlay* NormalOverlay, const TArray<int32>& Triangles);

	/**
	 * Compute overlay normals for the given mesh
	 * @param bInvert if true, normals are flipped
	 */
	static GEOMETRYCORE_API bool QuickRecomputeOverlayNormals(FDynamicMesh3& Mesh, bool bInvert = false, bool bWeightByArea = true, bool bWeightByAngle = true, bool bParallelCompute = true);

	/**
	 * Compute overlay normals for the given mesh, for the given set of triangles
	 */
	static GEOMETRYCORE_API bool RecomputeOverlayTriNormals(FDynamicMesh3& Mesh, const TArray<int32>& Triangles, bool bWeightByArea = true, bool bWeightByAngle = true);

	/**
	 * Compute overlay normals for the given mesh, for the given set of element IDs
	 */
	static GEOMETRYCORE_API bool RecomputeOverlayElementNormals(FDynamicMesh3& Mesh, const TArray<int32>& ElementIDs, bool bWeightByArea = true, bool bWeightByAngle = true);

	/**
	 * Retrieve the area and/or angle weights for each vertex of a triangle
	 * @param Mesh the mesh to query
	 * @param TriID the triangle index of the mesh to query
	 * @param TriArea the area of the triangle
	 * @param bWeightByArea if true, include weighting by the area of the triangle
	 * @param bWeightByAngle if true, include weighting by the interior angles of the triangle
	 */
	static GEOMETRYCORE_API FVector3d GetVertexWeightsOnTriangle(const FDynamicMesh3* Mesh, int TriID, double TriArea, bool bWeightByArea, bool bWeightByAngle);

protected:
	/** Compute per-vertex normals using area-weighted averaging of one-ring triangle normals */
	GEOMETRYCORE_API void Compute_FaceAvg_AreaWeighted();
	/** Compute per-vertex normals using a custom combination of area-weighted and angle-weighted averaging of one-ring triangle normals */
	GEOMETRYCORE_API void Compute_FaceAvg(bool bWeightByArea, bool bWeightByAngle);

	/** Compute per-triangle normals */
	GEOMETRYCORE_API void Compute_Triangle();

	/** Recompute the element Normals of the given attribute overlay using area-weighted averaging of one-ring triangle normals */
	GEOMETRYCORE_API void Compute_Overlay_FaceAvg_AreaWeighted(const FDynamicMeshNormalOverlay* NormalOverlay);
	/** Recompute the element Normals of the given attribute overlay using a custom combination of area-weighted and angle-weighted averaging of one-ring triangle normals */
	GEOMETRYCORE_API void Compute_Overlay_FaceAvg(const FDynamicMeshNormalOverlay* NormalOverlay, bool bWeightByArea, bool bWeightByAngle);

};


} // end namespace UE::Geometry
} // end namespace UE
