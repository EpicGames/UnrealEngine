// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
namespace Geometry
{


/**
 * TConstObjectSharedAccess provides a way for the owner of some object (eg a Mesh)
 * to share read-only access to that object with background threads.
 * 
 * This avoids the obvious alternative, using TSharedPtr, which tends to result in some
 * bad patterns (in particular calling code that /doesn't/ have a shared pointer to
 * the object, now has to make a copy, or also convert all related code to use a shared pointer).
 * 
 * Currently TConstObjectSharedAccess's template ObjectType must be default constructible. This
 * allows the background threads to still access an instance of ObjectType even if the owner thread/code
 * decides to revoke access. In that case, TConstObjectSharedAccess provides an "empty" ObjectType instead.
 * (the background processing code must handle this case).
 * 
 * TConstObjectSharedAccess can be constructed with either a raw pointer to ObjectType, or a TSharedPtr.
 * In the raw pointer case, the creator is responsible for destruction. 
 * 
 * The standard usage pattern is:
 * 
 * [Owner thread]
 *   T* ObjectPtr = (...);
 *   TSharedPtr<TConstObjectSharedAccess<T>> SharedAccess = MakeShared<TConstObjectSharedAccess<T>>(ObjectPtr);
 *   pass_to_background_thread(SharedAccess);
 * 
 * [Background thread]
 *   SharedAccess->AccessSharedObject( [](const ObjectType& Object) { do_my_stuff(Object); } );
 * 
 * [Owner thread (later)]
 *   SharedAccess->ReleaseSharedObject();
 * 
 * Note that a TSharedPtr does not strictly need to be used. However this allows the Owner
 * thread to destruct the Object, and/or terminate, and the background thread can still safely
 * access a shared ObjectType (now empty).
 * 
 */
template<typename ObjectType>
class TConstObjectSharedAccess
{
public:
	TConstObjectSharedAccess()
	{
	}

	/**
	 * Construct shared access to a raw pointer of ObjectType
	 */
	TConstObjectSharedAccess(const ObjectType* Object)
	{
		ObjectRawPtr = Object;
	}

	/**
	 * Construct shared access to a shared pointer of ObjectType
	 */
	TConstObjectSharedAccess(TSharedPtr<ObjectType> Object)
	{
		ObjectSharedPtr = Object;
	}

	~TConstObjectSharedAccess()
	{
		ensureMsgf(ObjectRawPtr == nullptr && ObjectSharedPtr.IsValid() == false,
			TEXT("TConstObjectSharedAccess::~TConstObjectSharedAccess() : Object was not released!"));
	}

	/**
	 * Release the shared object. Generally this is intended to only be called
	 * by the "owner" of the object.
	 */
	void ReleaseSharedObject()
	{
		ObjectLock.Lock();
		ObjectRawPtr = nullptr;
		ObjectSharedPtr = TSharedPtr<ObjectType>();
		ObjectLock.Unlock();
	}

	/**
	 * Call ProcessFunc(Object) on the shared Object. 
	 * @return false if the Object was empty, ie if ReleaseSharedObject() has been called
	 * @warning calling code must not store a reference to ObjectType!
	 */
	bool AccessSharedObject(TFunctionRef<void(const ObjectType&)> ProcessFunc)
	{
		bool bDone = false;
		ObjectLock.Lock();
		if (ObjectRawPtr != nullptr)
		{
			ProcessFunc(*ObjectRawPtr);
			bDone = true;
		}
		else if (ObjectSharedPtr.IsValid())
		{
			ProcessFunc(*ObjectSharedPtr);
			bDone = true;
		}
		ObjectLock.Unlock();

		if (bDone)
		{
			return true;
		}
		else 
		{
			ProcessFunc(EmptyObject);
			return false;
		}
	}


protected:
	// only one of ObjectRawPtr or ObjectSharedPtr should be initialized. ObjectRawPtr will be checked first.
	const ObjectType* ObjectRawPtr = nullptr;
	TSharedPtr<ObjectType> ObjectSharedPtr;

	// default-constructed value of ObjectType that will be returned if ObjectRawPtr and ObjectSharedPtr are both null/invalid
	ObjectType EmptyObject;

	// controls access to the shared object
	FCriticalSection ObjectLock;
};


typedef TConstObjectSharedAccess<FDynamicMesh3> FSharedConstDynamicMesh3;

}
}