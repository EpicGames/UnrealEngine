// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh3.h"
#include "DynamicMeshAttributeSet.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "VectorTypes.h"


namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * FDynamicMeshTangents is a helper object for accessing tangents stored in the AttributeSet of a FDynamicMesh3.
 */
class FDynamicMeshTangents
{
public:
	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshNormalOverlay* Normals = nullptr;
	const FDynamicMeshNormalOverlay* Tangents = nullptr;
	const FDynamicMeshNormalOverlay* Bitangents = nullptr;

	GEOMETRYCORE_API FDynamicMeshTangents(const FDynamicMesh3* MeshIn);

	/**
	 * Checks the mesh for valid tangents. When bCheckValues == true,
	 * inspects the tangents for invalid values (ex. zero, NaN).
	 * 
	 * @param bCheckValues inspect tangent values for zero/NaN
	 * @return true if the mesh has valid tangents
	 */
	GEOMETRYCORE_API bool HasValidTangents(bool bCheckValues=false) const;

	/**
	 * If tangents are available in the overlays, returns them. If only Normal is available, computes orthogonal basis. Falls back to unit axes if no overlays are available.
	 */
	GEOMETRYCORE_API void GetTangentFrame(int32 TriangleID, int32 TriVertexIndex, FVector3f& NormalOut, FVector3f& TangentOut, FVector3f& BitangentOut) const;
	/**
	 * If tangents are available in the overlays, returns them. Otherwise computes orthogonal basis to Normal argument.
	 */
	GEOMETRYCORE_API void GetTangentVectors(int32 TriangleID, int32 TriVertexIndex, const FVector3f& Normal, FVector3f& TangentOut, FVector3f& BitangentOut) const;
	/**
	 * If tangents are available in the overlays, returns them, otherwise falls back to unit X/Y axes.
	 */
	GEOMETRYCORE_API void GetTangentVectors(int32 TriangleID, int32 TriVertexIndex, FVector3f& TangentOut, FVector3f& BitangentOut) const;
};


/**
 * Options used by TMeshTangents for tangents computation
 */
struct FComputeTangentsOptions
{
	/** if true, per-face tangents are blended to create triange-vertex tangents */
	bool bAveraged = true;		
	/** if true, per-face tangents are weighted by opening angle at triangle-vertices when blending */
	bool bAngleWeighted = true;
	/** if true, the computation will be done in parallel (via ParallelFor) */
	bool bParallel = true;
};


/**
 * TMeshTangents is a utility class that can calculate and store various types of
 * tangent vectors for a FDynamicMesh.
 */
template<typename RealType>
class TMeshTangents
{
protected:
	/** Target Mesh */
	const FDynamicMesh3* Mesh;
	/** Set of computed tangents */
	TArray<TVector<RealType>> Tangents;
	/** Set of computed bitangents */
	TArray<TVector<RealType>> Bitangents;
	/** Indices of degenerate triangles. This may be useful to know externally, as those tangets are often meaningless */
	TArray<int32> AllDegenerateTris;

public:
	TMeshTangents()
	{
		Mesh = nullptr;
	}

	TMeshTangents(const FDynamicMesh3* Mesh)
	{
		SetMesh(Mesh);
	}

	void SetMesh(const FDynamicMesh3* MeshIn)
	{
		this->Mesh = MeshIn;
	}

	const TArray<TVector<RealType>>& GetTangents() const { return Tangents; }

	const TArray<TVector<RealType>>& GetBitangents() const { return Bitangents; }

	const TArray<int32>& GetDegenerateTris() const { return AllDegenerateTris; }

	// Get the target mesh
	const FDynamicMesh3* GetMesh() const
	{
		return Mesh; 
	}

	//
	// Per-Triangle-Vertex Tangents
	// Number of Tangents is NumTriangles * 3
	//

	/**
	 * Set internal buffer sizes suitable for calculating per-triangle tangents.
	 * This is intended to be used if you wish to calculate your own tangents and use SetPerTriangleTangent()
	 */
	void InitializeTriVertexTangents(bool bClearToZero)
	{
		SetTangentCount(Mesh->MaxTriangleID() * 3, bClearToZero);
	}

	/**
	 * Initialize Tangents from other Tangents set
	 */
	template<typename OtherRealType>
	void CopyTriVertexTangents(const TMeshTangents<OtherRealType>& OtherTangents);

	/**
	 * Initialize Tangents from the FDynamicMeshAttributeSet of SourceMesh 
	 * @return false if tangent attributes do not exist
	 */
	GEOMETRYCORE_API bool CopyTriVertexTangents(const FDynamicMesh3& SourceMesh);


	/**
	 * Calculate per-triangle tangent spaces based on the given per-triangle normal and UV overlays.
	 * In this mode there is no averaging of tangents across triangles. So if we have N triangles
	 * in the mesh, then 3*N tangents are generated. These tangents are optionally computed in parallel.
	 */
	GEOMETRYCORE_API void ComputeTriVertexTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options);

	/**
	 * Return tangent and bitangent at a vertex of triangle for per-triangle computed tangents
	 * @param TriangleID triangle index in mesh
	 * @param TriVertIdx vertex index in range 0,1,2
	 */
	void GetPerTriangleTangent(int32 TriangleID, int32 TriVertIdx, UE::Math::TVector<RealType>& TangentOut, UE::Math::TVector<RealType>& BitangentOut) const
	{
		int32 k = TriangleID * 3 + TriVertIdx;
		TangentOut = Tangents[k];
		BitangentOut = Bitangents[k];
	}


	/**
	 * Return tangent and bitangent at a vertex of triangle for per-triangle computed tangents
	 * @param TriangleID triangle index in mesh
	 * @param TriVertIdx vertex index in range 0,1,2
	 */
	template<typename VectorType, typename OtherRealType>
	void GetPerTriangleTangent(int32 TriangleID, int32 TriVertIdx, VectorType& TangentOut, VectorType& BitangentOut) const
	{
		int32 k = TriangleID * 3 + TriVertIdx;
		TangentOut.X = (OtherRealType)Tangents[k].X;
		TangentOut.Y = (OtherRealType)Tangents[k].Y;
		TangentOut.Z = (OtherRealType)Tangents[k].Z;
		BitangentOut.X = (OtherRealType)Bitangents[k].X;
		BitangentOut.Y = (OtherRealType)Bitangents[k].Y;
		BitangentOut.Z = (OtherRealType)Bitangents[k].Z;
	}


	/**
	 * @return Interpolated tangent and bitangent at barycentric coordinates inside a triangle, for per-triangle computed tangents
	 * @param TriangleID triangle index in mesh
	 * @param BaryCoords barycentric coordinates in triangle
	 * @param TangentOut interpolated tangent
	 * @param BitangentOut interpolated bitangent
	 */
	void GetInterpolatedTriangleTangent(int32 TriangleID, 
		const UE::Math::TVector<RealType>& BaryCoords,
		UE::Math::TVector<RealType>& TangentOut,
		UE::Math::TVector<RealType>& BitangentOut) const
	{
		int32 k = TriangleID * 3;
		if (k >= 0 && (k+2) < Tangents.Num())
		{
			TangentOut = Blend3(Tangents[k], Tangents[k+1], Tangents[k+2], BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
			Normalize(TangentOut);
			BitangentOut = Blend3(Bitangents[k], Bitangents[k+1], Bitangents[k+2], BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
			Normalize(BitangentOut);
		}
		else
		{
			TangentOut = TVector<RealType>::UnitX();
			BitangentOut = TVector<RealType>::UnitY();
		}
	}


	/**
	 * Set tangent and bitangent at a vertex of triangle for per-triangle computed tangents.
	 * @param TriangleID triangle index in mesh
	 * @param TriVertIdx vertex index in range 0,1,2
	 */
	void SetPerTriangleTangent(int TriangleID, int TriVertIdx, const TVector<RealType>& Tangent, const TVector<RealType>& Bitangent)
	{
		int k = TriangleID * 3 + TriVertIdx;
		Tangents[k] = Tangent;
		Bitangents[k] = Bitangent;
	}

	/**
	 * Get tangent and bitangent at a vertex of a triangle for per-triangle computed tangents
	 * @param TriangleID triangle index in mesh
	 * @param TriVertIdx vertex index in range 0,1,2
	 */
	template<typename OtherVectorType>
	inline void GetTriangleVertexTangentVectors(int32 TriangleID, int32 TriVertexIndex, OtherVectorType& TangentOut, OtherVectorType& BitangentOut) const
	{
		int k = TriangleID * 3 + TriVertexIndex;
		TangentOut = (OtherVectorType)Tangents[k];
		BitangentOut = (OtherVectorType)Bitangents[k];
	}


	//
	// Per-Triangle Tangents
	// Number of Tangents is NumTriangles
	//

	/** Initialize buffer sizes to one tangent/bitangent per mesh triangle */
	void InitializeTriangleTangents(bool bClearToZero)
	{
		SetTangentCount(Mesh->MaxTriangleID(), bClearToZero);
	}

	/**
	 * Compute per-triangle tangents for the given UV Overlay. Computation is done in parallel.
	 * @param bOrthogonalize if true, we orthogonalize the tangents by computing the Bitangent sign and regenerating it. Otherwise the tangents are left non-orthogonal.
	 */
	GEOMETRYCORE_API void ComputeTriangleTangents(const FDynamicMeshUVOverlay* UVOverlay, bool bOrthogonalize = true);


	/**
	 * Set Tangents on mesh overlays
	 * @param MeshToSet Mesh to copy overlays to; does not need to be the same as the Mesh member of this class
	 */
	GEOMETRYCORE_API bool CopyToOverlays(FDynamicMesh3& MeshToSet) const;



	/**
	 * Convenience function to compute tangents from the mesh attribute set's primary UVs and normals
	 * @return true if tangents were computed, false otherwise
	 */
	static bool ComputeDefaultOverlayTangents(FDynamicMesh3& Mesh)
	{
		const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();
		if (!Attributes)
		{
			return false;
		}
		const FDynamicMeshUVOverlay* UVOverlay = Attributes->PrimaryUV();
		const FDynamicMeshNormalOverlay* NormalOverlay = Attributes->PrimaryNormals();
		if (UVOverlay && NormalOverlay)
		{
			TMeshTangents Tangents;
			Tangents.SetMesh(&Mesh);
			Tangents.ComputeTriVertexTangents(NormalOverlay, UVOverlay, FComputeTangentsOptions());
			Mesh.Attributes()->EnableTangents();
			return Tangents.CopyToOverlays(Mesh);
		}
		return false;
	}

protected:
	/**
	 * Set the size of the Tangents array to Count, and optionally clear all values to (0,0,0)
	 */
	GEOMETRYCORE_API void SetTangentCount(int Count, bool bClearToZero);

	// calculate per-triangle tangents and then projected to overlay normals at each triangle-vertex
	void ComputeSeparatePerTriangleTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options);

	// Calculate MikkT-space-style tangents
	void ComputeMikkTStyleTangents(const FDynamicMeshNormalOverlay* NormalOverlay, const FDynamicMeshUVOverlay* UVOverlay, const FComputeTangentsOptions& Options);
};

typedef TMeshTangents<float> FMeshTangentsf;
typedef TMeshTangents<double> FMeshTangentsd;




template<typename RealType>
template<typename OtherRealType>
void TMeshTangents<RealType>::CopyTriVertexTangents(const TMeshTangents<OtherRealType>& OtherMeshTangents)
{
	const TArray<TVector<OtherRealType>>& OtherTangents = OtherMeshTangents.GetTangents();
	const TArray<TVector<OtherRealType>>& OtherBitangents = OtherMeshTangents.GetBitangents();

	int32 Num = OtherTangents.Num();
	check(Mesh->MaxTriangleID() * 3 == Num);
	InitializeTriVertexTangents(false);
	for (int32 k = 0; k < Num; ++k)
	{
		Tangents[k] = TVector<RealType>(OtherTangents[k]);
		Bitangents[k] = TVector<RealType>(OtherBitangents[k]);
	}
	AllDegenerateTris = OtherMeshTangents.GetDegenerateTris();
}


} // end namespace UE::Geometry
} // end namespace UE
