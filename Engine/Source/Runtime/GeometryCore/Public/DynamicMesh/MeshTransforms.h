// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshWeights

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "FrameTypes.h"
#include "GeometryBase.h"
#include "Math/MathFwd.h"
#include "TransformTypes.h"

namespace UE { namespace Geometry { class FDynamicMesh3; } }
template <typename FuncType> class TFunctionRef;


/**
 * Utility functions for applying transformations to meshes
 */
namespace MeshTransforms
{
	using namespace UE::Geometry;

	// enum to control which attributes are transformed
	enum class ETransformAttributes
	{
		Positions = 1 << 0,
		VertexNormals = 1 << 1,
		Normals = 1 << 2,
		Tangents = 1 << 3,
		SculptLayers = 1 << 4,
		All = Positions | VertexNormals | Normals | Tangents | SculptLayers
	};
	ENUM_CLASS_FLAGS(ETransformAttributes);

	/**
	 * Apply Translation to the Mesh.
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void Translate(FDynamicMesh3& Mesh, const FVector3d& Translation, ETransformAttributes TransformAttributes = ETransformAttributes::All);

	/**
	 * Apply Scale to Mesh, relative to given Origin. Correctly updates normals/tangents as needed (unless bOnlyPositions is true).
	 * @param bReverseOrientationIfNeeded		If negative scaling inverts the mesh, also invert the triangle orientations
	 * @param bOnlyPositions						If true, only apply scale positions and do not affect normals/tangents
	 */
	GEOMETRYCORE_API void Scale(FDynamicMesh3& Mesh, const FVector3d& Scale, const FVector3d& Origin, bool bReverseOrientationIfNeeded = false, bool bOnlyPositions = false);
	
	/**
	 * Apply Scale to Mesh, relative to given Origin. Correctly updates normals/tangents as needed (unless TransformAttributes flags are set not to update them).
	 * @param bReverseOrientationIfNeeded		If negative scaling inverts the mesh, also invert the triangle orientations
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void Scale(FDynamicMesh3& Mesh, const FVector3d& Scale, const FVector3d& Origin, bool bReverseOrientationIfNeeded, ETransformAttributes TransformAttributes);

	/**
	 * Apply Rotation to the Mesh, relative to the given RotationOrigin.
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void Rotate(FDynamicMesh3& Mesh, const FRotator& Rotation, const FVector3d& RotationOrigin, ETransformAttributes TransformAttributes = ETransformAttributes::All);

	/**
	 * Transform Mesh into local coordinates of Frame
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void WorldToFrameCoords(FDynamicMesh3& Mesh, const FFrame3d& Frame, ETransformAttributes TransformAttributes = ETransformAttributes::All);

	/**
	 * Transform Mesh out of local coordinates of Frame
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void FrameCoordsToWorld(FDynamicMesh3& Mesh, const FFrame3d& Frame, ETransformAttributes TransformAttributes = ETransformAttributes::All);


	/**
	 * Apply given Transform to a Mesh.
	 * @param bReverseOrientationIfNeeded		If the Transform inverts the mesh with negative scale, also invert the triangle orientations
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void ApplyTransform(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform, bool bReverseOrientationIfNeeded = false, ETransformAttributes TransformAttributes = ETransformAttributes::All);


	/**
	 * Apply inverse of given Transform to a Mesh.
	 * Modifies Vertex Positions and Normals, and any Per-Triangle Normal Overlays
	 * @param bReverseOrientationIfNeeded		If the Transform inverts the mesh with negative scale, also invert the triangle orientations
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void ApplyTransformInverse(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform, bool bReverseOrientationIfNeeded = false, ETransformAttributes TransformAttributes = ETransformAttributes::All);


	/**
	 * Apply given Transform to a Mesh.
	 * Modifies Vertex Positions and Normals, and any Per-Triangle Normal Overlays
	 */
	GEOMETRYCORE_API void ApplyTransform(FDynamicMesh3& Mesh, 
		TFunctionRef<FVector3d(const FVector3d&)> PositionTransform,
		TFunctionRef<FVector3f(const FVector3f&)> NormalTransform);

	/**
	 * Apply given Transform to a Mesh.
	 * @param TransformAttributes				Bitflag enum specifying which attributes to transform
	 */
	GEOMETRYCORE_API void ApplyTransform(FDynamicMesh3& Mesh,
		TFunctionRef<FVector3d(const FVector3d&)> PositionTransform,
		TFunctionRef<FVector3f(const FVector3f&)> NormalTransform,
		TFunctionRef<FVector3f(const FVector3f&)> TangentTransform,
		ETransformAttributes TransformAttributes = ETransformAttributes::All
	);

	/**
	 * If applying Transform would invert Mesh w/ a negative scale, then invert Mesh's triangle orientations.
	 * Note: Does not apply the transform.
	 */
	GEOMETRYCORE_API void ReverseOrientationIfNeeded(FDynamicMesh3& Mesh, const FTransformSRT3d& Transform);

};