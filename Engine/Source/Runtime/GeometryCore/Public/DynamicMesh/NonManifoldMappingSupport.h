// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

namespace UE
{
namespace Geometry
{

// forward
class FDynamicMesh3;
class FNonManifoldSourceVertexIdAttribute;


/*
* The FNonManifoldMappingSupport is a light-weight class that exists to interpret and manage additional DynamicMesh data stored 
* when the mesh was created from a potentially non-manifold source mesh.  
*
* For context the FDynamicMesh3 is less accepting of non-manifold mesh features than some source mesh data-structures
* and conversion to a DynamicMesh may require changes to the topology ( e.g. vertex splits)  and in such cases
* this class can be use to access the mapping from DynamicMesh vertices to the source mesh vertices that generated them.
* 
* Note:
* Since this class holds pointers to named attributes owned by the DynamicMesh, unintended behavior will result if said attributes
* are removed from the mesh while an associated NomManifoldSupport object is in use. 
*
* Also, any mapping information between the DynamicMesh and source data will be corrupted by topological changes 
* to the DynamicMesh (e.g. edge splits, collapses etc).
*  
*/
class FNonManifoldMappingSupport
{
public:

	GEOMETRYCORE_API FNonManifoldMappingSupport(const FDynamicMesh3& Mesh);


	/**
	* Update the support for a new DynamicMesh.
	*/ 
	GEOMETRYCORE_API void Reset(const FDynamicMesh3& Mesh);

	/*
	* Return true if attribute data indicates that the source data that was converted to this DynamicMesh contained non-manifold vertices.
	*/
	GEOMETRYCORE_API bool IsNonManifoldVertexInSource() const;

	/*
	* Return true if the provided DynamicMesh vertex id resulted from a non-manifold vertex in the source data.
	* @param vid - the id of a vertex in the DynamicMesh.
	* 
	* Note: the code assumes but does not check that vid is a valid vertex id
	*/
	bool IsNonManifoldVertexID(const int32 vid) const
	{
		return !( vid == GetOriginalNonManifoldVertexID(vid) ); 
	}

	/*
	* Return the vertex ID in the potentially non-manifold data used to generate this DynamicMesh associated with the provided vertex id.   
	* In the case that the source data was actually manifold the returned vertex id will be identical to the DynamicMesh vertex id.
	* @param vid - the id of a vertex in the DynamicMesh. 
	* 
	* Note: the code assumes but does not check that vid is a valid vertex vid.
	*/
	GEOMETRYCORE_API int32 GetOriginalNonManifoldVertexID(const int32 vid) const;


	// --- helper functions.
	
	/*
	* Attaches or replaces non-manifold vertex mapping data to the provided mesh.  
	* @param VertexToNonManifoldVertexIDMap - an array that maps each DynamicMesh vertex id to the associated non-manifold vertex id.
	* @return false on failure (no attribute will be attached to the DynamicMesh in this case) 
	* 
	* Note: Failure occurs if the DynamicMesh does not have attributes enabled or if the provided array is not long enough to provide a mapping value for each DynamicMesh vertex id.
	*/
	static GEOMETRYCORE_API bool AttachNonManifoldVertexMappingData(const TArray<int32>& VertexToNonManifoldVertexIDMap, FDynamicMesh3& MeshInOut);

	/*
	*  Removes vertex mapping data related to the non-manifold nature of the source data the produced this DynamicMesh.
	* 
	*  Note: this will invalidate any NonManifoldMappingSupport object associated with this DynamicMesh, 
	*  and subsequent use of such object will produce unexpected results.
	*/
	static GEOMETRYCORE_API void RemoveNonManifoldVertexMappingData(FDynamicMesh3& MeshInOut);



	/*
	*  Removes all  mapping data related to the non-manifold nature of the source data the produced this DynamicMesh.
	* 
	*  Note, this will invalidate any NonManifoldMappingSupport object associated with this DynamicMesh, 
	*  and subsequent use of such object will produce unexpected results.
	*/
	static void RemoveAllNonManifoldMappingData(FDynamicMesh3& MeshInOut) 
	{
		RemoveNonManifoldVertexMappingData(MeshInOut);
	}

	/*
	* Name used to identify vertex attribute data generated during conversion to a DynamicMesh in the case that the source was non-manifold.
	*/
	static GEOMETRYCORE_API FName NonManifoldMeshVIDsAttrName;


protected:

	const FNonManifoldSourceVertexIdAttribute*  NonManifoldSrcVIDsAttribute = nullptr;
	const FDynamicMesh3* DynamicMesh = nullptr;
};


} // namespace Geometry
} // namespace UE
