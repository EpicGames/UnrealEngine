// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MergeCoincidentEdges

#pragma once

#include "Math/UnrealMathSSE.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "SplitAttributeWelder.h"

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * FMergeCoincidentMeshEdges finds pairs of boundary edges of the mesh that are identical 
 * (ie have endpoint vertices at the same locations) and merges the pair into a single edge.
 * This is similar to welding vertices but safer because it prevents bowties from being formed.
 *
 * Currently if the two edges have the same "orientation" (ie from their respective triangles)
 * they cannot be merged. 
 *
 */
class FMergeCoincidentMeshEdges
{
public:
	/** default tolerance is float ZeroTolerance */
	static GEOMETRYCORE_API const double DEFAULT_TOLERANCE;  // = FMathf::ZeroTolerance;

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** Edges are coincident if both pairs of endpoint vertices are closer than this distance */
	double MergeVertexTolerance = DEFAULT_TOLERANCE;

	/** Only merge unambiguous pairs that have unique duplicate-edge matches */
	bool OnlyUniquePairs = false;

	/** 
	  * Edges are considered as potentially the same if their midpoints are within this distance.
	  * Due to floating-point roundoff this should be larger than MergeVertexTolerance.
	  * If zero, we set to MergeVertexTolerance*2
	  */
	double MergeSearchTolerance = 0;


	/** Number of mesh boundary edges before merging */
	int32 InitialNumBoundaryEdges = 0;
	/** Number of mesh boundary edges after merging */
	int32 FinalNumBoundaryEdges = 0;

	/** Enable / Disable attribute welding along merged mesh edges */
	bool bWeldAttrsOnMergedEdges = false;

	/** Used to weld attributes at the merged edges */
	FSplitAttributeWelder  SplitAttributeWelder;
	
	/** Set of EdgesIds to merge. If null, perform merge operation across entire mesh */
	TSet<int32>* EdgesToMerge = nullptr;

public:
	FMergeCoincidentMeshEdges(FDynamicMesh3* mesh) : Mesh(mesh)
	{
	}

	/**
	 * Run the merge operation and modify .Mesh
	 * @return true if the algorithm succeeds
	 */
	GEOMETRYCORE_API virtual bool Apply();


protected:
	double MergeVtxDistSqr;		// cached value

	// returns true if endpoint vertices are within tolerance. Note that we do not know the order of
	// the vertices here so we try both combinations.
	inline bool IsSameEdge(const FVector3d& a, const FVector3d& b, const FVector3d& c, const FVector3d& d) const
	{
		return (DistanceSquared(a,c) < MergeVtxDistSqr && DistanceSquared(b,d) < MergeVtxDistSqr) ||
			(DistanceSquared(a,d) < MergeVtxDistSqr && DistanceSquared(b,c) < MergeVtxDistSqr);
	}
};


} // end namespace UE::Geometry
} // end namespace UE
