// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Math/MathFwd.h"

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

template<typename RealType, int ElementSize, typename VectorType>
class TDynamicMeshVectorOverlay; 


/**
* FSplitAttributeWelder can be used to weld split-attributes that share the same vertex in the parent mesh (e.g. split normals)
*
*/
class FSplitAttributeWelder
{

	typedef TDynamicMeshVectorOverlay<float, 2, FVector2f> FDynamicMeshUVOverlay;
	typedef TDynamicMeshVectorOverlay<float, 3, FVector3f> FDynamicMeshNormalOverlay;
	typedef TDynamicMeshVectorOverlay<float, 4, FVector4f> FDynamicMeshColorOverlay;

public:

	FSplitAttributeWelder() {}


	/**
	* Merge threshold used to compare UV Overlay elements.
	* Applied as FVector2f::DistSquared(UVA , UVB) <= UVDisSqrdThreshold
	*/ 
	float UVDistSqrdThreshold = 0.f;

	/**
	* Merge threshold used to compare Color Overlay elements
	* Applied as (ColorA - ColorB).SizeSquared() <= ColorDistSqrdThreshold
	*/ 
	float ColorDistSqrdThreshold = 0.f;
	
	/**
	* Merge threshold used to compare Normal Overlay vectors. 
	* Applied as Abs(1 - VecA.dot.VecB)  <= NormalVecDotThreshold.
	*/
	float NormalVecDotThreshold = 0.f;


	/**
	* Merge threshold used to compare Tangent (and BiTangent) Overlay vectors. 
	* Applied as Abs(1 - VecA.dot.VecB)  <= TangentVecDotThreshold.
	*/
	float TangentVecDotThreshold = 0.f;

	/**
	* Weld split-elements at the ParentVID in each overlay that are "close enough" as defined by the appropriate threshold. 
	*/
	GEOMETRYCORE_API void WeldSplitElements(FDynamicMesh3& ParentMesh, const int32 ParentVID);
	

	/**
	*  Weld split-elements across the entire mesh. 
	*/
	GEOMETRYCORE_API void WeldSplitElements(FDynamicMesh3& ParentMesh);
	

	/**
	* Welds split UVs shared by the ParentVID vertex in the ParentMesh.  
	* 
	* @param ParentVID  defines a vertex in the parent mesh. 
	* @param UVOverlay contains potentially split UVs associated with the ParentVID. Assumed to be an overlay of the ParentMesh.
	* @param UVDistSqrdThreshold  provides a threshold in distance squared UV space between split UVs that should be welded. 
	*/
	static GEOMETRYCORE_API void WeldSplitUVs(const int32 ParentVID, FDynamicMeshUVOverlay& UVOverlay, float UVDistSqrdThreshold);
	

	/**
	* Welds split Normals / Tangents shared by the ParentVID vertex in the ParentMesh based on the angled between the vectors.
	* 
	* @param ParentVID  defines a vertex in the Parent Mesh 
	* @param NormalOverlay contains potentially split vectors associated with the ParentVID. Assumed to be an overlay of the ParentMesh.
	* @param DotThreshold  provides a threshold for the dot product between two vectors that should be welded.
	* @param bMergeZeroVectors will weld vectors that are too small to normalize.
	*/
	static GEOMETRYCORE_API void WeldSplitUnitVectors(const int32 ParentVID, FDynamicMeshNormalOverlay& NormalOverlay, float DotThreshold, bool bMergeZeroVectors = true);
	
	/**
	* Welds split colors shared by the ParentVID vertex in the ParentMesh.  
	* 
	* @param ParentVID  defines a vertex in the Parent Mesh 
	* @param ColorOverlay contains potentially split colors associated with the ParentVID. Assumed to be an overlay of the ParentMesh.
	* @param ColorDistSqrdThreshold  provides a threshold applied to the sum of the square differences between to colors.  Used to determine if a weld should be preformed.
	*/
	static GEOMETRYCORE_API void WeldSplitColors(const int32 ParentVID, FDynamicMeshColorOverlay& ColorOverlay, float ColorDistSqrdThreshold);

};

} // end Geometry
} // end UE
