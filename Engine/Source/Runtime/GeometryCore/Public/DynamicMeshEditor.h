// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshEditor

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "EdgeLoop.h"
#include "Util/SparseIndexCollectionTypes.h"
#include "DynamicMesh/MeshIndexMappings.h"
#include "MeshAdapter.h"

namespace UE
{
namespace Geometry
{

struct FDynamicSubmesh3;

/**
 * FDynamicMeshEditResult is used to return information about new mesh elements
 * created by mesh changes, primarily in FDynamicMeshEditor
 */
struct FDynamicMeshEditResult
{
	/** New vertices created by an edit */
	TArray<int> NewVertices;

	/** New triangles created by an edit. Note that this list may be empty if the operation created quads or polygons */
	TArray<int> NewTriangles;
	/** New quads created by an edit, where each quad is a pair of triangle IDs */
	TArray<FIndex2i> NewQuads;
	/** New polygons created by an edit, where each polygon is a list of triangle IDs */
	TArray<TArray<int>> NewPolygons;

	/** New triangle groups created by an edit */
	TArray<int> NewGroups;

	/** New normal overlay elements */
	TArray<TArray<int32>> NewNormalOverlayElements;

	/** New color overlay elements */
	TArray<int32> NewColorOverlayElements;

	/** clear this data structure */
	void Reset()
	{
		NewVertices.Reset();
		NewTriangles.Reset();
		NewQuads.Reset();
		NewPolygons.Reset();
		NewGroups.Reset();
		NewNormalOverlayElements.Reset();
		NewColorOverlayElements.Reset();
	}

	/** Flatten the triangle/quad/polygon lists into a single list of all triangles */
	GEOMETRYCORE_API void GetAllTriangles(TArray<int>& TrianglesOut) const;
};




/**
 * FDynamicMeshEditor implements low-level mesh editing operations. These operations
 * can be used to construct higher-level operations. For example an Extrude operation
 * could be implemented via DuplicateTriangles() and StitchLoopMinimal().
 */
class FDynamicMeshEditor
{
public:
	/** The mesh we will be editing */
	FDynamicMesh3* Mesh;

	FDynamicMeshEditor(FDynamicMesh3* MeshIn)
	{
		Mesh = MeshIn;
	}

	/**
	 * Remove any vertices that are not used by any triangles
	 */
	GEOMETRYCORE_API bool RemoveIsolatedVertices();

	//////////////////////////////////////////////////////////////////////////
	// Create and Remove Triangle Functions
	//////////////////////////////////////////////////////////////////////////

	/**
	 * Stitch together two loops of vertices with a quad-strip of triangles.
	 * Loops must be oriented (ordered) correctly for your use case.
	 * If loop edges are [a,b] and [c,d], then tris added are [b,a,d] and [a,c,d]
	 * @param Loop1 first loop of sequential vertices
	 * @param Loop2 second loop of sequential vertices
	 * @param ResultOut lists of newly created triangles/vertices/etc
	 * @return true if operation succeeded. If a failure occurs, any added triangles are removed via RemoveTriangles
	 */
	GEOMETRYCORE_API bool StitchVertexLoopsMinimal(const TArray<int>& VertexLoop1, const TArray<int>& VertexLoop2, FDynamicMeshEditResult& ResultOut);


	/**
	 * Given a sequence of edges in the form given by ConvertLoopToTriVidPairSequence and a loop of vertices, 
	 * stitch the two together. Edges are identified as (TriangleID, (0-2 subindex of vert1, 0-2 subindex of vert2)), 
	 * and should match the number of vertex ids in the corresponding vertex loop (each i corresponds to the edge 
	 * between vert i and i+1 in the vertex loop).
	 * This works much like StitchVertexLoopsMinimal, but it can be used after one of the loops undergoes
	 * vertex splits (to remove bowties) as long as the "loop" (which may now be broken) can still be
	 * identified by the composing triangles. In particular, this is used in extrude and inset where the
	 * base loop needs to remain as a sequence of Vids (since some of these may be free vertices, unattached
	 * to a triangle) but the offset/inset loop is guaranteed to be attached to triangles yet may have its
	 * Vids change due to bowtie splits.
	 * 
	 * If an entry of TriVidPairs1 resolves to verts [a,b] and the corresponding pair of verts in VertexLoop2
	 * are [c,d], then tris added are [b,a,d] and [a,c,d].
	 * @param TriVidPairs1 First loop, given as output of ConvertLoopToTriVidPairSequence()
	 * @param VertexLoop2 Second loop, given as vertex IDs
	 * @param ResultOut lists of newly created triangles/vertices/etc
	 * @return true if operation succeeded. If a failure occurs, any added triangles are removed via RemoveTriangles
	 */
	GEOMETRYCORE_API bool StitchVertexLoopToTriVidPairSequence(
		const TArray<TPair<int32, TPair<int8, int8>>>& TriVidPairs1,
		const TArray<int>& VertexLoop2, FDynamicMeshEditResult& ResultOut);

	/**
	 * Converts a loop to a sequence of edge identifiers that are both Vid and Eid independent. The identifiers are
	 * of the form (TriangleID, ([0,2] vert sub index, [0,2] vert sub index)), and they are used in some cases 
	 * (extrude, inset) where we want to perform vertex splits along a region boundary (to remove bowties) but 
	 * need to maintain a record of the original loop path. We don't use (TriangleID, Eid sub index) because that
	 * makes it harder to keep track of individual edge directionality.
	 * 
	 * @param VidLoop Vertex IDs of loop to convert.
	 * @param EdgeLoop Edge IDs of loop to convert, must match VidLoop.
	 * @param TriVertPairsOut Output
	 * @return false if not successful (usually due to a mismatch between VidLoop and EdgeLoop).
	 */
	static GEOMETRYCORE_API bool ConvertLoopToTriVidPairSequence(const FDynamicMesh3& Mesh,
		const TArray<int32>& VidLoop, const TArray<int32>& EdgeLoop,
		TArray<TPair<int32, TPair<int8, int8>>>& TriVertPairsOut);

	/**
	 * Stitch together two loops of vertices where vertices are only sparsely corresponded
	 * @param VertexIDs1 first array of sequential vertices
	 * @param MatchedIndices1 indices into the VertexIDs1 array of vertices that have a corresponding match in the VertexIDs2 array; Must be ordered
	 * @param VertexIDs2 second array of sequential vertices
	 * @param MatchedIndices2 indices into the VertexIDs2 array of vertices that have a corresponding match in the VertexIDs1 array; Must be ordered
	 * @param ResultOut lists of newly created triangles/vertices/etc
	 * @return true if operation succeeded.  If a failure occurs, any added triangles are removed via RemoveTriangles
	 */
	GEOMETRYCORE_API bool StitchSparselyCorrespondedVertexLoops(const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2, FDynamicMeshEditResult& ResultOut, bool bReverseOrientation = false);


	/**
	 * Weld together two loops of vertices. Loops must be oriented (ordered) correctly for your use case.
	 * @param Loop1 first loop of sequential vertices
	 * @param Loop2 second loop of sequential vertices. These vertices and their edges will not exist after the operation.
	 * @return true if operation succeeded, false if any errors ocurred
	 */
	GEOMETRYCORE_API bool WeldVertexLoops(const TArray<int32>& VertexLoop1, const TArray<int32>& VertexLoop2);


	/**
	 * Fill hole with a triangle fan given an existing (unconnected) center vertex and
	 * an ordered loop of boundary vertices on the hole border.
	 * @param CenterVertex Index of floating vertex in the center of the hole
	 * @param VertexLoop Indices of vertices on the boundary of the hole, in order
	 * @param ResultOut lists of newly created triangles
	 * @return true if operation succeeded.  If a failure occurs, any added triangles are removed via RemoveTriangles.
	 */
	GEOMETRYCORE_API bool AddTriangleFan_OrderedVertexLoop(int CenterVertex, const TArray<int>& VertexLoop, int GroupID, FDynamicMeshEditResult& ResultOut);


	/**
	 * Duplicate triangles of a mesh. This duplicates the current groups and also any attributes existing on the triangles.
	 * @param Triangles the triangles to duplicate
	 * @param IndexMaps returned mappings from old to new triangles/vertices/etc (you may initialize to optimize memory usage, etc)
	 * @param ResultOut lists of newly created triangles/vertices/etc
	 */
	GEOMETRYCORE_API void DuplicateTriangles(const TArray<int>& Triangles, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut);


	/**
	 * Pair of associated vertex and edge loops. Assumption is that loop sizes are the same and have a 1-1 correspondence.
	 * OuterVertices may include unreferenced vertices, and OuterEdges may include InvalidID edges in that case.
	 */
	struct FLoopPairSet
	{
		TArray<int32> OuterVertices;
		TArray<int32> OuterEdges;

		TArray<int32> InnerVertices;
		TArray<int32> InnerEdges;

		/** If true, some OuterVertices are not referenced by any triangles, and the pair of OuterEdges that would touch that vertex are InvalidID */
		bool bOuterIncludesIsolatedVertices;
	};

	/**
	 * Finds boundary loops of connected components of a set of triangles, and duplicates the vertices
	 * along the boundary, such that the triangles become disconnected.
	 * If the triangle set includes boundary vertices, they cannot be disconnected as they will become unreferenced.
	 * The function returns false if boundary vertices are found, unless bHandleBoundaryVertices = true.
	 * In that case, the boundary vertex is duplicated, and the original vertex is kept with the "Inner" loop.
	 * The new unreferenced vertex is left on the "Outer" loop, and the attached edges do not exist so are set as InvalidID.
	 * The FLoopPairSet.bOuterIncludesIsolatedVertices flag is set to true if boundary vertices are encountered.
	 * Note: This function can create a mesh with bowties if you pass in a subset of triangles that would have bowtie connectivity.
	 *		 (it is impossible to create the paired LoopSetOut with 1:1 vertex pairs without leaving in the bowties?)
	 * @param Triangles set of triangles
	 * @param LoopSetOut set of boundary loops. LoopA is original loop which remains with "outer" triangles, and LoopB is new boundary loop of triangle set
	 * @param bHandleBoundaryVertices if true, boundary vertices are handled as described above, otherwise function aborts and returns false if a boundary vertex is encountered
	 * @return true on success
	 */
	GEOMETRYCORE_API bool DisconnectTriangles(const TArray<int>& Triangles, TArray<FLoopPairSet>& LoopSetOut, bool bHandleBoundaryVertices);

	/**
	 * Variant of DisconnectTriangles that takes a precomputed set of BoundaryLoops of the triangles, and a TSet of Triangles.
	 * These are both computed internally by the version that only takes a TArray of Triangles.
	 * @warning If the BoundaryLoops are not correct for the provided TriangleSet, this version will likely crash.
	 * @param Triangles set of triangles
	 * @param BoundaryLoops precomputed set of boundary EdgeLoops of TriangleSet
	 * @param LoopSetOut returned set of boundary loops. LoopA is original loop which remains with "outer" triangles previously connected to TriangleSet, and LoopB is new boundary loop of TriangleSet
	 * @param bAllowBoundaryVertices if true, mesh boundary vertices are duplicated and left on the 'outer' loop, otherwise function aborts and returns false if a boundary vertex is encountered
	 */
	GEOMETRYCORE_API bool DisconnectTriangles(const TSet<int>& TriangleSet, const TArray<FEdgeLoop>& BoundaryLoops, TArray<FLoopPairSet>& LoopSetOut, bool bAllowBoundaryVertices);

	/**
	 * Disconnect triangles along the given Edges, so that all input edges become boundary edges.
	 * @param Edges IDs of edges to disconnect
	 * @param AddedVertexIDs Optional array of vertices added by this operation
	 */
	GEOMETRYCORE_API bool DisconnectTrianglesAlongEdges(const TSet<int32>& Edges, TArray<int32>* AddedVertexIDs = nullptr);



	/**
	* Disconnects triangles (without constructing boundary loops) such that the input Triangles are not connected to any other triangles in the mesh
	* @param Triangles set of triangles
	* @param bPreventBowties do some additional processing and vertex splitting as needed to prevent the creation of any new bowties
	*/
	GEOMETRYCORE_API void DisconnectTriangles(const TArray<int>& Triangles, bool bPreventBowties = true);


	/**
	 * Splits all bowties across the whole mesh
	 */
	GEOMETRYCORE_API void SplitBowties(FDynamicMeshEditResult& ResultOut);

	/**
	 * Splits any bowties specifically on the given vertex, and (if not null) updates (does not reset!) NewVertices with any added vertices
	 */
	void SplitBowties(int VertexID, TArray<int32>* NewVertices = nullptr)
	{
		TArray<int> Triangles, ContiguousGroupLengths;
		TArray<bool> GroupIsLoop;
		SplitBowtiesInternal(VertexID, Triangles, ContiguousGroupLengths, GroupIsLoop, NewVertices);
	}

private:
	// Internal version of split bowties that takes some arrays for reusable storage
	GEOMETRYCORE_API void SplitBowtiesInternal(int VertexID, TArray<int>& Triangles, TArray<int>& ContiguousGroupLengths, TArray<bool>& GroupIsLoop, TArray<int32>* NewVertices = nullptr);
public:

	/**
	 * Splits any bowties specifically on the given vertex, and updates (does not reset!) ResultOut with any added vertices
	 */
	void SplitBowties(int VertexID, FDynamicMeshEditResult& ResultOut)
	{
		SplitBowties(VertexID, &ResultOut.NewVertices);
	}

	/**
	 * Splits bowties attached to any of the given triangles, and updates (does not reset!) ResultOut with any added vertices
	 */
	GEOMETRYCORE_API void SplitBowtiesAtTriangles(const TArray<int32>& TriangleIDs, FDynamicMeshEditResult& ResultOut);

	/**
	 * In ReinsertSubmesh, a problem can arise where the mesh we are 
	 * inserting has duplicate triangles of the base mesh.
	 *
	 * This can lead to problematic behavior later. We can do various things,
	 * like delete and replace that existing triangle, or just use it instead
	 * of adding a new one. Or fail, or ignore it.
	 * 
	 * This enum/argument controls the behavior. 
	 * However, fundamentally this kind of problem should be handled upstream!!
	 * For example by not trying to remesh areas that contain nonmanifold geometry...
	 */
	enum class EDuplicateTriBehavior : uint8
	{
		EnsureContinue,         
		EnsureAbort, UseExisting, Replace
	};

	/**
	 * Update a Base Mesh from a Submesh; See FMeshRegionOperator::BackPropropagate for a usage example.
	 * 
	 * Assumes that Submesh has been modified, but boundary loop has been preserved, and that old submesh has already been removed from this mesh.
	 * Just appends new vertices and rewrites triangles.
	 *
	 * @param Submesh The mesh to insert back into its BaseMesh.  The original submesh triangles should have already been removed when this function is called
	 * @param SubToNewV Mapping from submesh to vertices in the updated base mesh
	 * @param NewTris If not null, will be filled with IDs of triangles added to the base mesh
	 * @param DuplicateBehavior Choice of what to do if inserting a triangle from the submesh would duplicate a triangle in the base mesh
	 * @return true if submesh successfully inserted, false if any triangles failed (which happens if triangle would result in non-manifold mesh)
	 */
	GEOMETRYCORE_API bool ReinsertSubmesh(const FDynamicSubmesh3& Submesh, FOptionallySparseIndexMap& SubToNewV, TArray<int>* NewTris = nullptr,
						 EDuplicateTriBehavior DuplicateBehavior = EDuplicateTriBehavior::EnsureAbort);

	/**
	 * Remove a list of triangles from the mesh, and optionally any vertices that are now orphaned
	 * @param Triangles the triangles to remove
	 * @param bRemoveIsolatedVerts if true, remove vertices that end up with no triangles
	 * @return true if all removes succeeded
	 */
	GEOMETRYCORE_API bool RemoveTriangles(const TArray<int>& Triangles, bool bRemoveIsolatedVerts);


	/**
	 * Remove any connected components with volume or  area below the given thresholds
	 * @param MinVolume Remove components with less volume than this
	 * @param MinArea Remove components with less area than this
	 * @return number of components removed
	 */
	GEOMETRYCORE_API int RemoveSmallComponents(double MinVolume, double MinArea = 0.0, int MinTriangleCount = 0);


	/**
	 * Remove a list of triangles from the mesh, and optionally any vertices that are now orphaned
	 * @param Triangles the triangles to remove
	 * @param bRemoveIsolatedVerts if true, remove vertices that end up with no triangles
	 * @param OnRemoveTriFunc called for each triangle to be removed
	 * @return true if all removes succeeded
	 */
	GEOMETRYCORE_API bool RemoveTriangles(const TArray<int>& Triangles, bool bRemoveIsolatedVerts, TFunctionRef<void(int)> OnRemoveTriFunc);


	//////////////////////////////////////////////////////////////////////////
	// Normal utility functions
	//////////////////////////////////////////////////////////////////////////

	/**
	 * Reverse the orientation of the given triangles, and optionally flip relevant normals
	 * @param Triangles the triangles to modify
	 * @param bInvertNormals if ture we call InvertTriangleNormals()
	 */
	GEOMETRYCORE_API void ReverseTriangleOrientations(const TArray<int>& Triangles, bool bInvertNormals);

	/**
	 * Flip the normals of the given triangles. This includes their vertex normals, if they
	 * exist, as well as any per-triangle attribute normals. @todo currently creates full-mesh bit arrays, could be more efficient on subsets
	 * @param Triangles the triangles to modify
	 */
	GEOMETRYCORE_API void InvertTriangleNormals(const TArray<int>& Triangles);


	/**
	 * Calculate and set the per-triangle normals of the two input quads.
	 * Average of the two face normals is used unless the quad is planar
	 * @param QuadTris pair of triangle IDs. If second ID is invalid, it is ignored
	 * @param bIsPlanar if the quad is known to be planar, operation is more efficient
	 * @return the normal vector that was set
	 */
	GEOMETRYCORE_API FVector3f ComputeAndSetQuadNormal(const FIndex2i& QuadTris, bool bIsPlanar = false);


	/**
	 * Create and set new shared per-triangle normals for a pair of triangles that share one edge (ie a quad)
	 * @param QuadTris pair of triangle IDs. If second ID is invalid, it is ignored
	 * @param Normal normal vector to set
	 */
	GEOMETRYCORE_API void SetQuadNormals(const FIndex2i& QuadTris, const FVector3f& Normal);

	/**
	 * Create and set new shared per-triangle normals for a list of triangles
	 * @param Triangles list of triangle IDs
	 * @param Normal normal vector to set
	 */
	GEOMETRYCORE_API void SetTriangleNormals(const TArray<int>& Triangles, const FVector3f& Normal);

	/**
	 * Create and set new shared per-triangle normals for a list of triangles. 
	 * Normal at each vertex is calculated based only on average of triangles in set
	 * @param Triangles list of triangle IDs
	 */
	GEOMETRYCORE_API void SetTriangleNormals(const TArray<int>& Triangles);

	/**
	 * For a 'tube' of triangles connecting loops of corresponded vertices, set smooth normals such that corresponding vertices have corresponding normals
	 */
	GEOMETRYCORE_API void SetTubeNormals(const TArray<int>& Triangles, const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2, bool bReverseNormals = false);


	//////////////////////////////////////////////////////////////////////////
	// UV utility functions
	//////////////////////////////////////////////////////////////////////////


	/**
	 * Project the two triangles of the quad onto a plane defined by the ProjectionFrame and use that to create/set new shared per-triangle UVs.
	 * UVs are translated so that their bbox min-corner is at origin, and scaled by given scale factor
	 * @param QuadTris pair of triangle IDs. If second ID is invalid, it is ignored
	 * @param ProjectFrame vertices are projected into XY axes of this frame
	 * @param UVScaleFactor UVs are scaled by this uniform scale factor
	 * @param UVTranslation UVs are translated after scaling
	 * @param UVLayerIndex which UV layer to operate on (must exist)
	 */
	GEOMETRYCORE_API void SetQuadUVsFromProjection(const FIndex2i& QuadTris, const FFrame3d& ProjectionFrame, float UVScaleFactor = 1.0f, const FVector2f& UVTranslation = FVector2f::Zero(), int UVLayerIndex = 0);

	/**
	* Project triangles onto a plane defined by the ProjectionFrame and use that to create/set new shared per-triangle UVs.
	* UVs can be translated so that their bbox min-corner is at origin, and scaled by given scale factor
	* This is an older function signature that forwards to the more specific one.
	*
	* @param Triangles TArray of triangle IDs
	* @param ProjectFrame vertices are projected into XY axes of this frame
	* @param UVScaleFactor UVs are scaled by this uniform scale factor
	* @param UVTranslation UVs are translated after scaling
	* @param bShiftToOrigin Whether to translate the UV coordinates to make their bounding box min corner be (0,0) before applying UVTranslation
	* @param UVLayerIndex which UV layer to operate on (must exist)
	*/
	GEOMETRYCORE_API void SetTriangleUVsFromProjection(const TArray<int32>& Triangles, const FFrame3d& ProjectionFrame, 
		float UVScaleFactor = 1.0f, const FVector2f& UVTranslation = FVector2f::Zero(), bool bShiftToOrigin = true, int32 UVLayerIndex = 0);

	/**
	* Project triangles onto a plane defined by the ProjectionFrame and use that to create/set new shared per-triangle UVs.
	*
	* @param Triangles TArray of triangle IDs
	* @param ProjectFrame Vertices are projected into XY axes of this frame
	* @param UVScaleFactor UVs are scaled by these factors
	* @param UVTranslation UVs are translated after scaling by these amounts
	* @param UVLayerIndex Which UV layer to operate on (must exist)
	* @param bShiftToOrigin Whether to translate the UV coordinates to make their bounding box min corner be (0,0) before applying UVTranslation
	* @param bNormalizeBeforeScaling Whether to place the UV coordinates into the range [0,1] before applying UVScaleFactor and UVTranslation
	*/
	GEOMETRYCORE_API void SetTriangleUVsFromProjection(const TArray<int>& Triangles, const FFrame3d& ProjectionFrame, 
		const FVector2f& UVScale = FVector2f::One(), const FVector2f& UVTranslation = FVector2f::Zero(), int UVLayerIndex = 0,
		bool bShiftToOrigin = true, bool bNormalizeBeforeScaling = false);

	/**
	 * For triangles connecting loops of corresponded vertices, set UVs in a cylindrical pattern so that the U coordinate starts at 0 for the first corresponded pair of vertices, and cycles around to 1
	 * Assumes Triangles array stores indices of triangles in progressively filling the tube, starting with VertexIDs*[0].  (This is used to set the UVs correctly at the seam joining the start & end of the loop)
	 */
	GEOMETRYCORE_API void SetGeneralTubeUVs(const TArray<int>& Triangles, const TArray<int>& VertexIDs1, const TArray<int>& MatchedIndices1, const TArray<int>& VertexIDs2, const TArray<int>& MatchedIndices2, const TArray<float>& UValues, const FVector3f& VDir, float UVScaleFactor = 1.0f, const FVector2f& UVTranslation = FVector2f::Zero(), int UVLayerIndex = 0);

	/**
	 * Rescale UVs for the whole mesh, for the given UV attribute layer
	 * @param UVScale Scale factor to multiply into UVs.  If in world space, this is in centimeters relative to the average UV scale
	 * @param bWorldSpace If true, UVs are rescaled relative to an absolute world scale.
	 * @param UVLayerIndex which UV layer to operate on (must exist)
	 * @param ToWorld Optionally transform vertices for world space scaling
	 */
	GEOMETRYCORE_API void RescaleAttributeUVs(float UVScale = 1.0f, bool bWorldSpace = false, int UVLayerIndex = 0, TOptional<FTransformSRT3d> ToWorld = TOptional<FTransformSRT3d>());


	//////////////////////////////////////////////////////////////////////////
	// mesh element copying / duplication
	//////////////////////////////////////////////////////////////////////////


	/**
	 * Find "new" vertex for input vertex under Index mapping, or create new if missing
	 * @param VertexID the source vertex we want a copy of
	 * @param IndexMaps source/destination mapping of already-duplicated vertices
	 * @param ResultOut newly-created vertices are stored here
	 * @return index of duplicate vertex
	 */	
	GEOMETRYCORE_API int FindOrCreateDuplicateVertex(int VertexID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut);

	/**
	 * Find "new" group for input group under Index mapping, or create new if missing
	 * @param TriangleID the source triangle whose group we want a copy of
	 * @param IndexMaps source/destination mapping of already-duplicated groups
	 * @param ResultOut newly-created groups are stored here
	 * @return index of duplicate group
	 */
	GEOMETRYCORE_API int FindOrCreateDuplicateGroup(int TriangleID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut);

	/**
	 * Find "new" UV for input UV element under Index mapping, or create new if missing
	 * @param ElementID the source UV we want a duplicate of
	 * @param UVLayerIndex which UV layer to consider
	 * @param IndexMaps source/destination mapping of already-duplicated UVs
	 * @return index of duplicate UV in given UV layer
	 */
	GEOMETRYCORE_API int FindOrCreateDuplicateUV(int ElementID, int UVLayerIndex, FMeshIndexMappings& IndexMaps);

	/**
	 * Find "new" normal for input normal element under Index mapping, or create new if missing
	 * @param ElementID the source normal we want a duplicate of
	 * @param NormalLayerIndex which normal layer to consider
	 * @param IndexMaps source/destination mapping of already-duplicated normals
	 * @param ResultOut any newly created element indices are stored in NewNormalOverlayElements here. Note that
	 *   NewNormalOverlayElements must have size > NormalLayerIndex.
	 * @return index of duplicate normal in given normal layer
	 */
	GEOMETRYCORE_API int FindOrCreateDuplicateNormal(int ElementID, int NormalLayerIndex, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult* ResultOut = nullptr);

	/**
	 * Find "new" color for input color element under Index mapping, or create new if missing
	 * @param ElementID the source color we want a duplicate of
	 * @param IndexMaps source/destination mapping of already-duplicated colors
	 * @param ResultOut any newly created element indices are stored in NewColorOverlayElements here. 
	 * @return index of duplicate color in given color layer
	 */
	GEOMETRYCORE_API int FindOrCreateDuplicateColor(int ElementID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult* ResultOut);
	
	/**
	 * Copy all attribute-layer values from one triangle to another, using the IndexMaps to track and re-use shared attribute values.
	 * @param FromTriangleID source triangle
	 * @param ToTriangleID destination triangle
	 * @param IndexMaps mappings passed to FindOrCreateDuplicateX functions to track already-created attributes
	 * @param ResultOut information about new attributes is stored here (@todo finish populating this, at time of writing only normal overlay elements get tracked)
	 */
	GEOMETRYCORE_API void CopyAttributes(int FromTriangleID, int ToTriangleID, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut);




	/**
	 * Append input mesh to our internal mesh
	 * @param AppendMesh mesh to append
	 * @param IndexMapsOut mesh element index mappings generated in this append operation
	 * @param PositionTransform optional transformation function applied to mesh vertex positions
	 * @param NormalTransform optional transformation function applied to mesh normals
	 */
	GEOMETRYCORE_API void AppendMesh(const FDynamicMesh3* AppendMesh, FMeshIndexMappings& IndexMapsOut, 
		TFunction<FVector3d(int, const FVector3d&)> PositionTransform = nullptr,
		TFunction<FVector3d(int, const FVector3d&)> NormalTransform = nullptr,
		bool bReverseOrientation = false);

	/**
	 * Append input mesh to our internal Mesh. If the internal Mesh has attributes enabled,
	 * per-triangle normals will be computed and set. No other attributes are initialized.
	 * @param AppendMesh mesh to append
	 * @param IndexMapsOut mesh element index mappings generated in this append operation
	 * @param PositionTransform optional transformation function applied to mesh vertex positions
	 */
	GEOMETRYCORE_API void AppendMesh(const TTriangleMeshAdapter<double>* AppendMesh, FMeshIndexMappings& IndexMapsOut,
		TFunction<FVector3d(int, const FVector3d&)> PositionTransform = nullptr);


	/**
	 * Append normals from one attribute overlay to another.
	 * Assumes that AppendMesh has already been appended to Mesh.
	 * Note that this function has no dependency on .Mesh, it could be static
	 * @param AppendMesh mesh that owns FromNormals attribute overlay
	 * @param FromNormals Normals overlay we want to append from (owned by AppendMesh)
	 * @param ToNormals Normals overlay we want to append to (owned by Mesh)
	 * @param VertexMap map from AppendMesh vertex IDs to vertex IDs applicable to ToNormals (ie of .Mesh)
	 * @param TriangleMap map from AppendMesh triangle IDs to triangle IDs applicable to ToNormals (ie of .Mesh)
	 * @param NormalTransform optional transformation function applied to mesh normals
	 * @param NormalMapOut Mapping from element IDs of FromNormals to new element IDs in ToNormals
	 */
	GEOMETRYCORE_API void AppendNormals(const FDynamicMesh3* AppendMesh,
		const FDynamicMeshNormalOverlay* FromNormals, FDynamicMeshNormalOverlay* ToNormals,
		const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
		TFunction<FVector3d(int, const FVector3d&)> NormalTransform,
		FIndexMapi& NormalMapOut);


	/**
	 * Append UVs from one attribute overlay to another.
	 * Assumes that AppendMesh has already been appended to Mesh.
	 * Note that this function has no dependency on .Mesh, it could be static
	 * @param AppendMesh mesh that owns FromUVs attribute overlay
	 * @param FromUVs UV overlay we want to append from (owned by AppendMesh)
	 * @param ToUVs UV overlay we want to append to (owned by Mesh)
	 * @param VertexMap map from AppendMesh vertex IDs to vertex IDs applicable to ToUVs (ie of .Mesh)
	 * @param TriangleMap map from AppendMesh triangle IDs to triangle IDs applicable to ToUVs (ie of .Mesh)
	 * @param UVMapOut Mapping from element IDs of FromUVs to new element IDs in ToUVs
	 */
	GEOMETRYCORE_API void AppendUVs(const FDynamicMesh3* AppendMesh,
		const FDynamicMeshUVOverlay* FromUVs, FDynamicMeshUVOverlay* ToUVs,
		const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
		FIndexMapi& UVMapOut);

	/**
	* Append Colors from one attribute overlay to another.
	* Assumes that AppendMesh has already been appended to Mesh.
	* Note that this function has no dependency on .Mesh, it could be static
	* @param AppendMesh mesh that owns FromOverlay attribute overlay
	* @param FromOverlay Color overlay we want to append from (owned by AppendMesh)
	* @param ToOverlay Color overlay we want to append to (owned by Mesh)
	* @param VertexMap map from AppendMesh vertex IDs to vertex IDs applicable to ToOverlay (ie of .Mesh)
	* @param TriangleMap map from AppendMesh triangle IDs to triangle IDs applicable to ToOverlay (ie of .Mesh)
	* @param ColorMapOut Mapping from element IDs of FromUVs to new element IDs in ToOverlay
	*/
	GEOMETRYCORE_API void AppendColors(const FDynamicMesh3* AppendMesh,
		const FDynamicMeshColorOverlay* FromOverlay, FDynamicMeshColorOverlay* ToOverlay,
		const FIndexMapi& VertexMap, const FIndexMapi& TriangleMap,
		FIndexMapi& ColorMapOut);



	/**
	 * Append overlay elements in both ROIs from one overlay to another
	 *
	 * @param FromMesh     Mesh that owns FromOverlay attribute overlay
	 * @param TriangleROI  Triangles containing the copied overlay elements
	 * @param VertexROI    Parent vertices of the copied overlay elements
	 * @param FromOverlay  Overlay we want to append from (owned by FromMesh)
	 * @param ToOverlay    Overlay we want to append into (owned by .Mesh)
	 *
	 * @note Required overloads should be declared in the .cpp file
	 */
	template <typename RealType, int ElementSize>
	GEOMETRYCORE_API void AppendElementSubset(
		const FDynamicMesh3* FromMesh,
		const TSet<int>& TriangleROI,
		const TSet<int>& VertexROI,
		const TDynamicMeshOverlay<RealType, ElementSize>* FromOverlay,
		TDynamicMeshOverlay<RealType, ElementSize>* ToOverlay);



	/**
	 * Append triangles of an existing mesh. This duplicates the current groups and also any attributes existing on the triangles.
	 * @param SourceMesh the mesh to copy from
	 * @param SourceTriangles the triangles to copy
	 * @param IndexMaps returned mappings from old to new triangles/vertices/etc (you may initialize to optimize memory usage, etc)
	 * @param ResultOut lists of newly created triangles/vertices/etc
	 * @param bComputeTriangleMap if true, computes the triangle map section of IndexMaps (which is not needed for the append to work, so is optional)
	 */
	GEOMETRYCORE_API void AppendTriangles(const FDynamicMesh3* SourceMesh, const TArrayView<const int>& SourceTriangles, FMeshIndexMappings& IndexMaps, FDynamicMeshEditResult& ResultOut, bool bComputeTriangleMap = true);

	/**
	 * NOTE: Please use the TArray<TUniquePtr<FDynamicMesh3>> version of this method, below, instead of this one, because storing FDynamicMesh3 in TArray directly is not safe.
	 * This method will not directly cause memory corruption, but any subsequent manipulation of the array will be unsafe.
	 * 
	 * Create multiple meshes out of the source mesh by splitting triangles out.
	 * Static because it creates multiple output meshes, so doesn't quite fit in the FDynamicMeshEditor model of operating on a single mesh
	 *
	 * @param SourceMesh The mesh to split
	 * @param SplitMeshes An array with one mesh per unique Mesh ID found in SourceMesh (optionally excluding the DeleteMeshID)
	 * @param TriIDToMeshID A function mapping Triangle IDs to Mesh IDs
	 * @param DeleteMeshID If non-negative, triangles with this Mesh ID will be removed rather than being placed in a split mesh
	 * @param MeshIDPerSplitMesh If non-null, will be filled with the Mesh ID for each of the SplitMeshes generated
	 * @param bSortByMeshID If true, SplitMeshes will be sorted in order of increasing Mesh ID
	 * @return true if needed split, false if there were not multiple mesh ids so no split was needed
	 */
	UE_DEPRECATED(5.8, "Use the version of SplitMesh that takes a TArray<TUniquePtr<FDynamicMesh3>> array instead")
	static GEOMETRYCORE_API bool SplitMesh(const FDynamicMesh3* SourceMesh, TArray<FDynamicMesh3>& SplitMeshes, TFunctionRef<int(int)> TriIDToMeshID, int DeleteMeshID = -1, TArray<int32>* MeshIDPerSplitMesh = nullptr, bool bSortByMeshID = false);

	/**
	 * Create multiple meshes out of the source mesh by splitting triangles out.
	 * Static because it creates multiple output meshes, so doesn't quite fit in the FDynamicMeshEditor model of operating on a single mesh
	 *
	 * @param SourceMesh The mesh to split
	 * @param SplitMeshes An array with one mesh per unique Mesh ID found in SourceMesh (optionally excluding the DeleteMeshID)
	 * @param bSplitIfSingle Whether to still split if the split would just create a single mesh with no deletion.
	 * @param TriIDToMeshID A function mapping Triangle IDs to Mesh IDs
	 * @param DeleteMeshID If non-negative, triangles with this Mesh ID will be removed rather than being placed in a split mesh
	 * @param MeshIDPerSplitMesh If non-null, will be filled with the Mesh ID for each of the SplitMeshes generated
	 * @param bSortByMeshID If true, SplitMeshes will be sorted in order of increasing Mesh ID
	 * @return true if needed split, false if there were not multiple mesh ids, and bSplitIfSingle was false. If false, the SplitMeshes array will not be updated.
	 */
	static GEOMETRYCORE_API bool SplitMesh(const FDynamicMesh3* SourceMesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes, bool bSplitIfSingle, TFunctionRef<int(int)> TriIDToMeshID, int DeleteMeshID = -1, TArray<int32>* MeshIDPerSplitMesh = nullptr, bool bSortByMeshID = false);

	/**
	 * Merge any seams in the given 2D attribute Overlay along the given mesh edge IDs
	 *
	 * @param EidsToRemoveAsSeams list of edges to remove seams from
	 * @return true on success
	 */
	static GEOMETRYCORE_API bool RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams, TDynamicMeshOverlay<float, 2>* Overlay);
	/**
	 * Merge any seams in the given (3D attribute) Overlay along the given mesh edge IDs
	 *
	 * @param EidsToRemoveAsSeams list of edges to remove seams from
	 * @return true on success
	 */
	static GEOMETRYCORE_API bool RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams, TDynamicMeshOverlay<float, 3>* Overlay);

	/**
	 * Cut existing 2D attribute overlay topology with a set of edges. This allows for creating partial seams/darts, interior cuts, etc.
	 *
	 * Avoids creating bowties in the overlay. In cases where an edge is not next to any present or future seams/borders, some
	 * adjacent edge will be picked to be made into a seam as well, since it's impossible to make the original
	 * into a seam otherwise.

	 * @param EidsToMakeIntoSeams list of edges to turn into seams
	 * @param AddedElementIDs if non-null, list of new elements created along the path will be stored here (not ordered)
	 * @return true on success
	 */
	static GEOMETRYCORE_API bool CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, TDynamicMeshOverlay<float, 2>* Overlay, TArray<int32>* AddedElementIDs = nullptr);
	/**
	 * Cut existing 3D attribute overlay topology with a set of edges. This allows for creating partial seams/darts, interior cuts, etc.
	 *
	 * Avoids creating bowties in the overlay. In cases where an edge is not next to any present or future seams/borders, some
	 * adjacent edge will be picked to be made into a seam as well, since it's impossible to make the original
	 * into a seam otherwise.

	 * @param EidsToMakeIntoSeams list of edges to turn into seams
	 * @param AddedElementIDs if non-null, list of new elements created along the path will be stored here (not ordered)
	 * @return true on success
	 */
	static GEOMETRYCORE_API bool CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, TDynamicMeshOverlay<float, 3>* Overlay, TArray<int32>* AddedElementIDs = nullptr);

};


} // end namespace UE::Geometry
} // end namespace UE
