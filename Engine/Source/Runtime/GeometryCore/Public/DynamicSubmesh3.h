// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp DSubmesh3

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Util/SparseIndexCollectionTypes.h"

namespace UE
{
namespace Geometry
{

struct FDynamicSubmesh3
{
protected:
	const FDynamicMesh3* BaseMesh;
	FDynamicMesh3 Submesh;

	// TODO: this is a fully generic mapping backed by TMaps both ways
	// we could instead back the reverse mapping by a TArray since it's dense for submeshes
	FMeshIndexMappings Mappings;

	TSet<int> BaseBorderE;        // set of internal border edge indices on base mesh. Does not include mesh boundary edges.
	TSet<int> BaseBoundaryE;      // set of mesh-boundary edges on base mesh that are in submesh
	TSet<int> BaseBorderV;        // set of border vertex indices on base mesh (ie verts of BaseBorderE - does not include mesh boundary vertices)

public:
	/**
	 * Whether to compute triangle maps (adds additional cost). True by default.
	 */
	bool bComputeTriMaps = true;

public:
	/** Default constructor */
	FDynamicSubmesh3() : BaseMesh(nullptr) {}

	/** Base mesh-only constructor; does not build submesh */
	FDynamicSubmesh3(const FDynamicMesh3* BaseMesh) : BaseMesh(BaseMesh)
	{
	}

	/** Constructor sets the base mesh and computes the submesh */
	FDynamicSubmesh3(const FDynamicMesh3* BaseMesh, const TArray<int>& Triangles, int WantComponents = (int)EMeshComponents::All, bool bAttributes = true) : BaseMesh(BaseMesh)
	{
		Compute(Triangles, WantComponents, bAttributes);
	}

	/** const accessor for base mesh */
	inline const FDynamicMesh3* GetBaseMesh() const
	{
		return BaseMesh;
	}

	/** accessor for submesh */
	inline FDynamicMesh3& GetSubmesh()
	{
		return Submesh;
	}

	/** const accessor for submesh */
	inline const FDynamicMesh3& GetSubmesh() const
	{
		return Submesh;
	}

	/** @return set of internal border edge indices on base mesh. Does not include mesh boundary edges. */
	inline const TSet<int> GetBaseBorderEdges() const
	{
		return BaseBorderE;
	}

	/** @return set of mesh-boundary edges on base mesh that are in submesh */
	inline const TSet<int> GetBaseBoundaryEdges() const
	{
		return BaseBoundaryE;
	}

	/** @return set of border vertex indices on base mesh (ie verts of BaseBorderE - does not include mesh boundary vertices) */
	inline const TSet<int> GetBaseBorderVertices() const
	{
		return BaseBorderV;
	}

	/** @return true if edge ID is in base border edge ID set */
	inline bool InBaseBorderEdges(int BaseEID) const
	{
		return BaseBorderE.Contains(BaseEID);
	}

	/** @return true if edge ID is in base boundary edge ID set */
	inline bool InBaseBoundaryEdges(int BaseEID) const
	{
		return BaseBoundaryE.Contains(BaseEID);
	}

	/** @return true if vertex ID is in base border vertex ID set */
	inline bool InBaseBorderVertices(int BaseVID) const
	{
		return BaseBorderV.Contains(BaseVID);
	}
	
	/** @return Vertex ID in submesh, mapped from base mesh */
	inline int MapVertexToSubmesh(int BaseVID) const
	{
		return Mappings.GetVertexMap().GetTo(BaseVID);
	}
	/** @return Vertex ID in base mesh, mapped from submesh */
	inline int MapVertexToBaseMesh(int SubVID) const
	{
		return Mappings.GetVertexMap().GetFrom(SubVID);
	}

	/** @return Pair of vertex IDs in submesh, mapped from base mesh */
	inline FIndex2i MapVerticesToSubmesh(FIndex2i VIDs) const
	{
		return FIndex2i(MapVertexToSubmesh(VIDs.A), MapVertexToSubmesh(VIDs.B));
	}
	/** @return Pair of vertex IDs in base mesh, mapped from submesh */
	inline FIndex2i MapVerticesToBaseMesh(FIndex2i VIDs) const
	{
		return FIndex2i(MapVertexToBaseMesh(VIDs.A), MapVertexToBaseMesh(VIDs.B));
	}
	/** @param Vertices Array of vertex IDs in base mesh that will all be changed to the corresponding IDs in the submesh */
	void MapVerticesToSubmesh(TArrayView<int>& Vertices) const
	{
		for (int i = 0; i < Vertices.Num(); ++i)
		{
			Vertices[i] = MapVertexToSubmesh(Vertices[i]);
		}
	}

	/** @return Edge ID in submesh, mapped from base mesh*/
	inline int MapEdgeToSubmesh(int BaseEID) const
	{
		check(BaseMesh);
		FIndex2i base_ev = BaseMesh->GetEdgeV(BaseEID);
		FIndex2i sub_ev = MapVerticesToSubmesh(base_ev);
		return Submesh.FindEdge(sub_ev.A, sub_ev.B);
	}
	/** @return Edge ID in base mesh, mapped from submesh*/
	inline int MapEdgeToBaseMesh(int SubEID) const
	{
		check(BaseMesh);
		FIndex2i sub_ev = Submesh.GetEdgeV(SubEID);
		FIndex2i base_ev = MapVerticesToBaseMesh(sub_ev);
		return BaseMesh->FindEdge(base_ev.A, base_ev.B);
	}
	/** @param Edges Array of edge IDs in base mesh that will all be changed to the corresponding IDs in the submesh */
	void MapEdgesToSubmesh(TArrayView<int>& Edges) const
	{
		for (int i = 0; i < Edges.Num(); ++i)
		{
			Edges[i] = MapEdgeToSubmesh(Edges[i]);
		}
	}
	

	/** @return Triangle ID in the submesh, mapped from base mesh */
	inline int MapTriangleToSubmesh(int BaseTID) const
	{
		checkSlow(bComputeTriMaps);
		return Mappings.GetTriangleMap().GetTo(BaseTID);
	}
	/** @return Triangle ID in the base mesh, mapped from submesh */
	inline int MapTriangleToBaseMesh(int SubTID) const
	{
		checkSlow(bComputeTriMaps);
		return Mappings.GetTriangleMap().GetFrom(SubTID);
	}
	/** @param Triangles Array of triangle IDs in the base mesh that will all be changed to the corresponding IDs in the submesh */
	void MapTrianglesToSubmesh(TArrayView<int>& Triangles) const
	{
		checkSlow(bComputeTriMaps);
		for (int i = 0; i < Triangles.Num(); ++i)
		{
			Triangles[i] = MapTriangleToSubmesh(Triangles[i]);
		}
	}

	/** @return Group ID in the submesh, mapped from base mesh */
	inline int MapGroupToSubmesh(int BaseGID) const
	{
		checkSlow(bComputeTriMaps);
		return Mappings.GetGroupMap().GetTo(BaseGID);
	}
	/** @return Group ID in the base mesh, mapped from submesh */
	inline int MapGroupToBaseMesh(int SubGID) const
	{
		checkSlow(bComputeTriMaps);
		return Mappings.GetGroupMap().GetFrom(SubGID);
	}
	/** @param GroupIDs Array of group IDs in the base mesh that will all be changed to the corresponding IDs in the submesh */
	void MapGroupsToSubmesh(TArrayView<int32> GroupIDs) const
	{
		checkSlow(bComputeTriMaps);
		for (int i = 0; i < GroupIDs.Num(); ++i)
		{
			GroupIDs[i] = MapGroupToSubmesh(GroupIDs[i]);
		}
	}

	/** @return Normal ID in the submesh, mapped from the base mesh */
	int MapNormalToSubmesh(int NormalLayer, int BaseNID) const
	{
		return Mappings.GetNormalMap(NormalLayer).GetTo(BaseNID);
	}

	/** @return Normal ID in the base mesh, mapped from the submesh */
	int MapNormalToBaseMesh(int NormalLayer, int SubNID) const
	{
		return Mappings.GetNormalMap(NormalLayer).GetFrom(SubNID);
	}

	/** @return UV ID in the submesh, mapped from the base mesh */
	int MapUVToSubmesh(int UVLayer, int BaseUVID) const
	{
		return Mappings.GetUVMap(UVLayer).GetTo(BaseUVID);
	}

	/** @return UV ID in the base mesh, mapped from the submesh */
	int MapUVToBaseMesh(int UVLayer, int SubUVID) const
	{
		return Mappings.GetUVMap(UVLayer).GetFrom(SubUVID);
	}

	/** @return Vertex Color ID in the submesh, mapped from the base mesh */
	int MapColorToSubmesh(int BaseCID) const
	{
		return Mappings.GetColorMap().GetTo(BaseCID);
	}

	/** @return Vertex Color ID in the base mesh, mapped from the submesh */
	int MapColorToBaseMesh(int SubCID) const
	{
		return Mappings.GetColorMap().GetFrom(SubCID);
	}

	/**
	 * Computes the Submesh object, index mappings corresponding sub to base mesh, and boundary between sub and base mesh
	 * @param SubTriangles ArrayView of triangle IDs to include in the submesh
	 * @param WantComponents Bit flag of desired mesh components; will only enable if flag is set AND the components are enabled in the BaseMesh
	 * @param bAttributes Flag indicating if attributes are needed for submesh; will only enable attributes if flag is set AND attributes are enabled in the BaseMesh
	 */
	void Compute(const TArrayView<const int>& SubTriangles, int WantComponents = (int)EMeshComponents::All, bool bAttributes = true)
	{
		check(BaseMesh);

		// construct submesh matching basemesh & desired flags
		Submesh = FDynamicMesh3((EMeshComponents)(BaseMesh->GetComponentsFlags() & WantComponents));
		if (bAttributes && BaseMesh->HasAttributes())
		{
			Submesh.EnableAttributes();
			Submesh.Attributes()->EnableMatchingAttributes(*BaseMesh->Attributes());
		}

		FDynamicMeshEditor Editor(&Submesh);
		FDynamicMeshEditResult ResultOut;
		Editor.AppendTriangles(BaseMesh, SubTriangles, Mappings, ResultOut, bComputeTriMaps);

		ComputeBoundaryInfo(SubTriangles);
	}

	/** Computes submesh object, setting a new BaseMesh first */
	void Compute(FDynamicMesh3* Base, const TArrayView<const int>& Triangles, int WantComponents = (int)EMeshComponents::All, bool bAttributes = true)
	{
		BaseMesh = Base;
		Compute(Triangles, WantComponents, bAttributes);
	}

protected:

	/**
	 * Compute boundary vertices and edges between the SubTriangles and the rest of the mesh
	 * Called by Compute after the Submesh is computed.
	 */
	void ComputeBoundaryInfo(const TArrayView<const int>& SubTriangles)
	{
		check(BaseMesh);

		// set of base-mesh triangles that are in submesh
		FIndexFlagSet SubTris(BaseMesh->MaxTriangleID(), SubTriangles.Num());
		for (int TID : SubTriangles)
		{
			SubTris.Add(TID);
		}

		BaseBorderV.Reset();
		BaseBorderE.Reset();
		BaseBoundaryE.Reset();

		// Iterate through edges in submesh roi on base mesh. If
		// one of the tris of the edge is not in submesh roi, then this
		// is a boundary edge.
		//
		// (edge iteration via triangle iteration processes each internal edge twice...)
		for (int TID : SubTriangles)
		{
			FIndex3i tedges = BaseMesh->GetTriEdges(TID);
			for (int j = 0; j < 3; ++j)
			{
				int eid = tedges[j];
				FIndex2i tris = BaseMesh->GetEdgeT(eid);
				if (tris.B == FDynamicMesh3::InvalidID) // this is a boundary edge
				{
					BaseBoundaryE.Add(eid);

				}
				else if (SubTris[tris.A] != SubTris[tris.B]) // this is a border edge
				{
					BaseBorderE.Add(eid);
					FIndex2i ve = BaseMesh->GetEdgeV(eid);
					BaseBorderV.Add(ve.A);
					BaseBorderV.Add(ve.B);
				} 
			}
		}

	}
};



} // end namespace UE::Geometry
} // end namespace UE
