// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp EdgeLoop

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{

/**
 * Sequential lists of vertices/edges in a mesh that form a closed loop
 */
class FEdgeLoop
{
public:
	/** The Mesh that contains this EdgeLoop */
	const FDynamicMesh3* Mesh;

	/** Ordered list of vertices forming the EdgeLoop */
	TArray<int> Vertices;
	/** Ordered list of edges forming the EdgeLoop */
	TArray<int> Edges;

	/** List of bowtie vertices. This list is only valid if bBowtiesCalculated = true. */
	TArray<int> BowtieVertices;

	/** If true, then BowtieVertices list is valid */
	bool bBowtiesCalculated = false;

	FEdgeLoop()
	{
		Mesh = nullptr;
	}

	FEdgeLoop(const FDynamicMesh3* mesh)
	{
		Mesh = mesh;
	}

	/**
	 * Initialize EdgeLoop with the given vertices and edges
	 */
	FEdgeLoop(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges)
	{
		Mesh = mesh;
		Vertices = vertices;
		Edges = edges;
	}


	/**
	 * Initialize the loop data
	 */
	GEOMETRYCORE_API void Initialize(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges, const TArray<int>* BowtieVerticesIn = nullptr);

	/**
	 * Construct an FEdgeLoop from a list of edges of the mesh
	 * @param EdgesIn list of sequential connected edges
	 */
	GEOMETRYCORE_API void InitializeFromEdges(const TArray<int>& EdgesIn);

	/**
	 * Construct an FEdgeLoop from a list of edges of the mesh
	 * @param MeshIn the mesh the edges exist on
	 * @param EdgesIn list of sequential connected edges
	 */
	void InitializeFromEdges(const FDynamicMesh3* MeshIn, const TArray<int>& EdgesIn)
	{
		Mesh = MeshIn;
		InitializeFromEdges(EdgesIn);
	}



	/**
	 * Construct EdgeLoop from list of vertices of mesh
	 * @param VerticesIn list of vertices that are sequentially connected by edges
	 * @param bAutoOrient if true, and any of the edges are boundary edges, we will re-orient the loop to be consistent with boundary edges
	 * @return false if we found any parts of vertices that are not connected by an edge
	 */
	GEOMETRYCORE_API bool InitializeFromVertices(const TArray<int>& VerticesIn, bool bAutoOrient = true);

	/**
	 * Construct EdgeLoop from list of vertices of mesh
	 * @param MeshIn Mesh that contains the loop
	 * @param VerticesIn list of vertices that are sequentially connected by edges
	 * @param bAutoOrient if true, and any of the edges are boundary edges, we will re-orient the loop to be consistent with boundary edges
	 * @return false if we found any parts of vertices that are not connected by an edge
	 */
	bool InitializeFromVertices(const FDynamicMesh3* MeshIn, const TArray<int>& VerticesIn, bool bAutoOrient = true)
	{
		Mesh = MeshIn;
		return InitializeFromVertices(VerticesIn, bAutoOrient);
	}




	/** Set the bowtie vertices */
	void SetBowtieVertices(const TArray<int>& Bowties)
	{
		BowtieVertices = Bowties;
		bBowtiesCalculated = true;
	}


	/**
	 * Populate the BowtieVertices member
	 */
	GEOMETRYCORE_API void CalculateBowtieVertices();

	/**
	 * @return number of vertices in the loop
	 */
	int GetVertexCount() const
	{
		return Vertices.Num();
	}

	/**
	 * @return number of edges in the loop
	 */
	int GetEdgeCount() const
	{
		return Edges.Num();
	}

	/**
	 * @return vertex position in loop at the given LoopIndex
	 */
	inline FVector3d GetVertex(int LoopIndex) const
	{
		return Mesh->GetVertex(Vertices[LoopIndex]);
	}

	/**
	 * @return vertex position in loop at the vertex previous to LoopIndex
	 */
	inline FVector3d GetPrevVertex(int32 LoopIndex) const
	{
		return Mesh->GetVertex(Vertices[ (LoopIndex == 0) ? (Vertices.Num()-1) : (LoopIndex-1) ]);
	}

	/**
	 * @return vertex position in loop at the vertex after LoopIndex
	 */
	inline FVector3d GetNextVertex(int32 LoopIndex) const
	{
		return Mesh->GetVertex(Vertices[ (LoopIndex + 1) % Vertices.Num() ]);
	}



	/**
	 * @return bounding box of the vertices of the EdgeLoop
	 */
	GEOMETRYCORE_API FAxisAlignedBox3d GetBounds() const;

	/**
	 * Add the vertices of the loop to the Vertices array
	 */
	template<typename VecType>
	void GetVertices(TArray<VecType>& VerticesOut) const
	{
		int NumV = Vertices.Num();
		for (int i = 0; i < NumV; ++i)
		{
			VecType Pos = Mesh->GetVertex(Vertices[i]);
			VerticesOut.Add(Pos);
		}
	}


	/**
	 * If any edges if the loop are on a mesh boundary, we can check that the loop is 
	 * oriented such that it corresponds with the boundary edges, and if not, reverse it.
	 * @return true if the loop was reversed
	 */
	GEOMETRYCORE_API bool SetCorrectOrientation();


	/**
	 * Reverse order of vertices and edges in loop
	 */
	void Reverse()
	{
		Algo::Reverse(Vertices);
		Algo::Reverse(Edges);
	}


	/**
	 * @return true if all edges of this loop are internal edges, ie not on the mesh boundary
	 */
	GEOMETRYCORE_API bool IsInternalLoop() const;


	/**
	 * @param TestMesh use this mesh instead of the internal Mesh pointer
	 * @return true if all edges of this loop are boundary edges
	 */
	GEOMETRYCORE_API bool IsBoundaryLoop(const FDynamicMesh3* TestMesh = nullptr) const;


	/**
	 * @return index of VertexID in the Vertices list, or -1 if not found
	 */
	GEOMETRYCORE_API int FindVertexIndex(int VertexID) const;


	/**
	 * @return index of vertex in the Vertices list that is closest to QueryPoint
	 */
	GEOMETRYCORE_API int FindNearestVertexIndex(const FVector3d& QueryPoint) const;



	/**
	 * Exhaustively check that verts and edges of this EdgeLoop are consistent. This is quite expensive.
	 * @return true if loop is consistent/valid
	 */
	GEOMETRYCORE_API bool CheckValidity(EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check) const;




	/**
	 * Utility function to convert a vertex loop to an edge loop
	 * @param Mesh mesh to operate on
	 * @param VertexLoop ordered list of vertices
	 * @param OutEdgeLoop computed list of sequential connected vertices
	 */
	static GEOMETRYCORE_API void VertexLoopToEdgeLoop(const FDynamicMesh3* Mesh, const TArray<int>& VertexLoop, TArray<int>& OutEdgeLoop);

};


//
// Utility functions for converting/processing edge loops
//

/**
 * Convert the input Vertex/Edge loop into an edge loop representation that is
 * independent of Vertex/Edge IDs. See description of FMeshTriOrderedEdgeID for
 * more information on how this works. But essentially the resulting TriOrderedEdgesLoopOut
 * is stable across some topological changes, in particular if the vertices of
 * the loop change but the triangle topology remains the same (eg if a bowtie vertex
 * is disconnected, or if the mesh is disconnected along the edge loop), the encoded
 * loop remains valid, while an explicit vertex or edge loop would not.
 * 
 * @param SelectEdgeTriangleFunc This function is used to select which triangle to use for encoding at each edge
 */
GEOMETRYCORE_API bool ConvertLoopToTriOrderedEdgeLoop(const FDynamicMesh3& Mesh,
	const TArray<int32>& VertexLoop, const TArray<int32>& EdgeLoop,
	TFunctionRef<int(int EdgeID, int TriangleA, int TriangleB)> SelectEdgeTriangleFunc,
	TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut );

/**
 * Variant of ConvertLoopToTriOrderedEdgeLoop that always encodes using EdgeTriangles.A, ie suitable for encoding open border loops
 */
GEOMETRYCORE_API bool ConvertLoopToTriOrderedEdgeLoop(const FDynamicMesh3& Mesh,
	const TArray<int32>& VertexLoop, const TArray<int32>& EdgeLoop,
	TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut );

/**
 * Recover the current Vertex and Edge loops given an encoded loop of FMeshTriOrderedEdgeID.
 * See FMeshTriOrderedEdgeID for details on this encoding, this function basically reverses
 * ConvertLoopToTriOrderedEdgeLoop() for the current mesh topology
 */
GEOMETRYCORE_API bool ConvertTriOrderedEdgeLoopToLoop(const FDynamicMesh3& Mesh,
	const TArray<FMeshTriOrderedEdgeID>& TriOrderedEdgesLoopOut,
	TArray<int32>& VertexLoop, TArray<int32>* EdgeLoop = nullptr);



} // end namespace UE::Geometry
} // end namespace UE
