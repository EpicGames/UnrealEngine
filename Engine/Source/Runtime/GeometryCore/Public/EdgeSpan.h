// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp EdgeSpan

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Util/IndexUtil.h"
#include "Polyline3.h"

namespace UE
{
namespace Geometry
{

/**
 * Sequential lists of vertices/edges in a mesh that is *not* closed.
 * If the list is closed it should be an FEdgeLoop
 */
class FEdgeSpan
{
public:
	/** The Mesh that contains this EdgeSpan */
	const FDynamicMesh3* Mesh;

	/** Ordered list of vertices forming the EdgeSpan */
	TArray<int> Vertices;
	/** Ordered list of edges forming the EdgeSpan */
	TArray<int> Edges;

	/** List of bowtie vertices. This list is only valid if bBowtiesCalculated = true. */
	TArray<int> BowtieVertices;

	/** If true, then BowtieVertices list is valid */
	bool bBowtiesCalculated = false;

	FEdgeSpan()
	{
		Mesh = nullptr;
	}

	FEdgeSpan(const FDynamicMesh3* mesh)
	{
		Mesh = mesh;
	}

	/**
	 * Initialize EdgeSpan with the given vertices and edges
	 */
	FEdgeSpan(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges)
	{
		Mesh = mesh;
		Vertices = vertices;
		Edges = edges;
	}


	/**
	 * Initialize the loop data
	 */
	GEOMETRYCORE_API void Initialize(const FDynamicMesh3* mesh, const TArray<int>& vertices, const TArray<int> & edges, const TArray<int>* BowtieVerticesIn = nullptr);


	/**
	 * Construct an FEdgeSpan from a list of edges of the mesh
	 * @param EdgesIn list of sequential connected edges
	 */
	GEOMETRYCORE_API void InitializeFromEdges(const TArray<int>& EdgesIn);


	/**
	 * Construct an FEdgeSpan from a list of edges of the mesh
	 * @param MeshIn the mesh the edges exist on
	 * @param EdgesIn list of sequential connected edges
	 */
	void InitializeFromEdges(const FDynamicMesh3* MeshIn, const TArray<int>& EdgesIn)
	{
		Mesh = MeshIn;
		InitializeFromEdges(EdgesIn);
	}


	/**
	 * Construct EdgeSpan from list of vertices of mesh
	 * @param MeshIn Mesh that contains the span
	 * @param VerticesIn list of vertices that are sequentially connected by edges
	 * @param bAutoOrient if true, and any of the edges are boundary edges, we will re-orient the span to be consistent with boundary edges
	 * @return false if we found any parts of VerticesIn that are not connected by an edge
	 */
	bool InitializeFromVertices(const FDynamicMesh3* MeshIn, const TArray<int>& VerticesIn, bool bAutoOrient = true)
	{
		Mesh = MeshIn;
		return InitializeFromVertices(VerticesIn, bAutoOrient);
	}


	/**
	 * Construct EdgeSpan from list of vertices of mesh
	 * @param VerticesIn list of vertices that are sequentially connected by edges
	 * @param bAutoOrient if true, and any of the edges are boundary edges, we will re-orient the span to be consistent with boundary edges
	 * @return false if we found any parts of VerticesIn that are not connected by an edge
	 */
	GEOMETRYCORE_API bool InitializeFromVertices(const TArray<int>& VerticesIn, bool bAutoOrient = true);


	/** Set the bowtie vertices */
	void SetBowtieVertices(const TArray<int>& Bowties)
	{
		BowtieVertices = Bowties;
		bBowtiesCalculated = true;
	}


	/**
	 * Populate the BowtieVertices member
	 */
	GEOMETRYCORE_API void CalculateBowtieVertices();


	/**
	 * @return number of vertices in the span
	 */
	int GetVertexCount() const
	{
		return Vertices.Num();
	}

	/**
	 * @return number of edges in the span
	 */
	int GetEdgeCount() const
	{
		return Edges.Num();
	}

	/**
	 * @return vertex position in span at the given SpanIndex
	 */
	FVector3d GetVertex(int SpanIndex) const
	{
		return Mesh->GetVertex(Vertices[SpanIndex]);
	}

	/**
	 * @return bounding box of the vertices of the EdgeSpan
	 */
	GEOMETRYCORE_API FAxisAlignedBox3d GetBounds() const;


	/**
	 * Extract Polyline from Mesh based on vertex list
	 */
	GEOMETRYCORE_API void GetPolyline(FPolyline3d& PolylineOut) const;

	/**
	 * If any edges if the span are on a mesh boundary, we can check that the span is
	 * oriented such that it corresponds with the boundary edges, and if not, reverse it.
	 * @return true if the span was reversed
	 */
	GEOMETRYCORE_API bool SetCorrectOrientation();


	/**
	 * Reverse order of vertices and edges in span
	 */
	void Reverse()
	{
		Algo::Reverse(Vertices);
		Algo::Reverse(Edges);
	}


	/**
	 * @return true if all edges of this span are internal edges, ie not on the mesh boundary
	 */
	GEOMETRYCORE_API bool IsInternalspan() const;

	/**
	 * @param TestMesh use this mesh instead of the internal Mesh pointer
	 * @return true if all edges of this span are boundary edges
	 */
	GEOMETRYCORE_API bool IsBoundaryspan(const FDynamicMesh3* TestMesh = nullptr) const;

	/**
	 * @return index of VertexID in the Vertices list, or -1 if not found
	 */
	GEOMETRYCORE_API int FindVertexIndex(int VertexID) const;

	/**
	 * @return index of vertex in the Vertices list that is closest to QueryPoint
	 */
	GEOMETRYCORE_API int FindNearestVertexIndex(const FVector3d& QueryPoint) const;

	   
	/**
	 * Exhaustively check that verts and edges of this EdgeSpan are consistent. This is quite expensive.
	 * @return true if span is consistent/valid
	 */
	GEOMETRYCORE_API bool CheckValidity(EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check) const;



	/**
	 * Utility function to convert a vertex span to an edge span
	 * @param Mesh mesh to operate on
	 * @param Vertexspan ordered list of vertices
	 * @param OutEdgeSpan computed list of sequential connected vertices
	 */
	static GEOMETRYCORE_API void VertexSpanToEdgeSpan(const FDynamicMesh3* Mesh, const TArray<int>& VertexSpan, TArray<int>& OutEdgeSpan);

	/**
	 * Split a single EdgeSpan into multiple subspans based on an edge angle threshold. Subspans are copied out, the original span is not modified.
	 * @param AngleThresholdDeg max angle between adjacant edges to mark a vertex as a "corner" (in degrees). Angle is measured between edge vectors extending from their
	 *							common vertex, so a value of 180 means all vertices would be labelled as corners.
	 * @param MinSpanSize minimum number of edges necessary for a subspan, regardless of angles between edges. Subspans that would be too small are merged to adjacent subspans.
	 * @param OutSpan output subspans
	 */
	GEOMETRYCORE_API void GetSubspansByAngle(double AngleThresholdDeg, int32 MinSpanSize, TArray<FEdgeSpan>& OutSpans) const;

};


} // end namespace UE::Geometry
} // end namespace UE
