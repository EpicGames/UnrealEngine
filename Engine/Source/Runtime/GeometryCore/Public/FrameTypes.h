// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "VectorTypes.h"
#include "VectorUtil.h"
#include "Quaternion.h"
#include "TransformTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TFrame3 is an object that represents an oriented 3D coordinate frame, ie orthogonal X/Y/Z axes at a point in space.
 * One can think of this Frame as a local coordinate space measured along these axes.
 * Functions are provided to map geometric objects to/from the Frame coordinate space.
 * 
 * Internally the representation is the same as an FTransform, except a Frame has no Scale.
 */
template<typename RealType>
struct TFrame3
{
	/**
	 * Origin of the frame
	 */
	TVector<RealType> Origin;

	/**
	 * Rotation of the frame. Think of this as the rotation of the unit X/Y/Z axes to the 3D frame axes.
	 */
	TQuaternion<RealType> Rotation;

	/**
	 * Construct a frame positioned at (0,0,0) aligned to the unit axes
	 */
	TFrame3()
	{
		Origin = TVector<RealType>::Zero();
		Rotation = TQuaternion<RealType>::Identity();
	}

	/**
	 * Construct a frame at the given Origin aligned to the unit axes
	 */
	explicit TFrame3(const TVector<RealType>& OriginIn)
	{
		Origin = OriginIn;
		Rotation = TQuaternion<RealType>::Identity();
	}

	/**
	 * Construct a Frame from the given Origin and Rotation
	 */
	TFrame3(const TVector<RealType>& OriginIn, const TQuaternion<RealType> RotationIn)
	{
		Origin = OriginIn;
		Rotation = RotationIn;
	}

	/**
	 * Construct a frame with the Z axis aligned to a target axis
	 * @param OriginIn origin of frame
	 * @param SetZ target Z axis
	 */
	TFrame3(const TVector<RealType>& OriginIn, const TVector<RealType>& SetZ)
	{
		Origin = OriginIn;
		Rotation.SetFromTo(TVector<RealType>::UnitZ(), SetZ);
	}

	/**
	 * Construct Frame from X/Y/Z axis vectors. Vectors must be mutually orthogonal.
	 * @param OriginIn origin of frame
	 * @param X desired X axis of frame
	 * @param Y desired Y axis of frame
	 * @param Z desired Z axis of frame
	 */
	TFrame3(const TVector<RealType>& OriginIn, const TVector<RealType>& X, const TVector<RealType>& Y, const TVector<RealType>& Z)
	{
		Origin = OriginIn;
		Rotation = TQuaternion<RealType>( TMatrix3<RealType>(X, Y, Z, false) );
	}

	/** Construct a Frame from an FTransform */
	explicit TFrame3(const FTransform& Transform)
	{
		Origin = TVector<RealType>(Transform.GetTranslation());
		Rotation = TQuaternion<RealType>(Transform.GetRotation());
	}

	/** Construct a Frame from an FPlane */
	explicit TFrame3(const FPlane& Plane)
	{
		FVector Normal(Plane.X, Plane.Y, Plane.Z);
		Origin = (RealType)Plane.W * TVector<RealType>(Normal);
		Rotation.SetFromTo(TVector<RealType>::UnitZ(), (TVector<RealType>)Normal);
	}


	/** Construct a Frame from an FVector and FQuat */
	explicit TFrame3(const FVector& OriginIn, const FQuat& RotationIn)
	{
		Origin = TVector<RealType>(OriginIn);
		Rotation = TQuaternion<RealType>(RotationIn);
	}

	/** Convert between TFrame of different types */
	template<typename RealType2>
	explicit TFrame3(const TFrame3<RealType2>& OtherFrame)
	{
		Origin = static_cast<TVector<RealType>>(OtherFrame.Origin);
		Rotation = static_cast<TQuaternion<RealType>>(OtherFrame.Rotation);
	}

	
	/**
	 * @param AxisIndex index of axis of frame, either 0, 1, or 2
	 * @return axis vector
	 */
	TVector<RealType> GetAxis(int AxisIndex) const
	{
		switch (AxisIndex)
		{
		case 0:
			return Rotation.AxisX();
		case 1:
			return Rotation.AxisY();
		case 2:
			return Rotation.AxisZ();
		default:
			checkNoEntry();
			return TVector<RealType>::Zero(); // compiler demands a return value
		}
	}

	/**
	 * @return X/Y/Z axes of frame. This is more efficient than calculating each axis separately.
	 */
	void GetAxes(TVector<RealType>& X, TVector<RealType>& Y, TVector<RealType>& Z) const
	{
		Rotation.GetAxes(X, Y, Z);
	}

	/** @return X axis of frame (axis 0) */
	TVector<RealType> X() const
	{
		return Rotation.AxisX();
	}

	/** @return Y axis of frame (axis 1) */
	TVector<RealType> Y() const
	{
		return Rotation.AxisY();
	}

	/** @return Z axis of frame (axis 2) */
	TVector<RealType> Z() const
	{
		return Rotation.AxisZ();
	}

	/** @return conversion of this Frame to FTransform */
	FTransform ToFTransform() const
	{
		return FTransform((FQuat)Rotation, (FVector)Origin);
	}

	/** @return conversion of this Frame to an inverse FTransform */
	FTransform ToInverseFTransform() const
	{
		TQuaternion<RealType> InverseRotation = Rotation.Inverse();
		return FTransform((FQuat)InverseRotation, (FVector)(InverseRotation * (-Origin)));
	}

	/** @return conversion of this Frame to FPlane */
	FPlane ToFPlane() const
	{
		return FPlane((FVector)Origin, (FVector)Z());
	}

	/** @return conversion of this Frame to TTransform */
	TTransformSRT3<RealType> ToTransform() const
	{
		return TTransformSRT3<RealType>(Rotation, Origin);
	}

	/** @return conversion of this Frame to an inverse TTransform */
	TTransformSRT3<RealType> ToInverseTransform() const
	{
		TQuaternion<RealType> InverseRotation = Rotation.Inverse();
		return TTransformSRT3<RealType>(InverseRotation, InverseRotation * (-Origin));
	}

	/** @return point at distances along frame axes */
	TVector<RealType> PointAt(RealType X, RealType Y, RealType Z) const
	{
		return Rotation * TVector<RealType>(X,Y,Z) + Origin;
	}

	/** @return point at distances along frame axes */
	TVector<RealType> PointAt(const TVector<RealType>& Point) const
	{
		return Rotation * TVector<RealType>(Point.X, Point.Y, Point.Z) + Origin;
	}
	
	/** @return input Point transformed into local coordinate system of Frame */
	TVector<RealType> ToFramePoint(const TVector<RealType>& Point) const
	{
		return Rotation.InverseMultiply((Point-Origin));
	}
	/** @return input Point transformed from local coordinate system of Frame into "World" coordinate system */
	TVector<RealType> FromFramePoint(const TVector<RealType>& Point) const
	{
		return Rotation * Point + Origin;
	}


	/** @return input Vector transformed into local coordinate system of Frame */
	TVector<RealType> ToFrameVector(const TVector<RealType>& Vector) const
	{
		return Rotation.InverseMultiply(Vector);
	}
	/** @return input Vector transformed from local coordinate system of Frame into "World" coordinate system */
	TVector<RealType> FromFrameVector(const TVector<RealType>& Vector) const
	{
		return Rotation * Vector;
	}


	/** @return input Quaternion transformed into local coordinate system of Frame */
	TQuaternion<RealType> ToFrame(const TQuaternion<RealType>& Quat) const
	{
		return Rotation.Inverse() * Quat;
	}
	/** @return input Quaternion transformed from local coordinate system of Frame into "World" coordinate system */
	TQuaternion<RealType> FromFrame(const TQuaternion<RealType>& Quat) const
	{
		return Rotation * Quat;
	}


	/** @return input Ray transformed into local coordinate system of Frame */
	TRay<RealType> ToFrame(const TRay<RealType>& Ray) const
	{
		return TRay<RealType>(ToFramePoint(Ray.Origin), ToFrameVector(Ray.Direction));
	}
	/** @return input Ray transformed from local coordinate system of Frame into "World" coordinate system */
	TRay<RealType> FromFrame(const TRay<RealType>& Ray) const
	{
		return TRay<RealType>(FromFramePoint(Ray.Origin), FromFrameVector(Ray.Direction));
	}


	/** @return input Frame transformed into local coordinate system of this Frame */
	TFrame3<RealType> ToFrame(const TFrame3<RealType>& Frame) const
	{
		return TFrame3<RealType>(ToFramePoint(Frame.Origin), ToFrame(Frame.Rotation));
	}
	/** @return input Frame transformed from local coordinate system of this Frame into "World" coordinate system */
	TFrame3<RealType> FromFrame(const TFrame3<RealType>& Frame) const
	{
		return TFrame3<RealType>(FromFramePoint(Frame.Origin), FromFrame(Frame.Rotation));
	}




	/**
	 * Project 3D point into plane and convert to UV coordinates in that plane
	 * @param Pos 3D position
	 * @param PlaneNormalAxis which plane to project onto, identified by perpendicular normal. Default is 2, ie normal is Z, plane is (X,Y)
	 * @return 2D coordinates in UV plane, relative to origin
	 */
	TVector2<RealType> ToPlaneUV(const TVector<RealType>& Pos, int PlaneNormalAxis = 2) const
	{
		int Axis0 = 0, Axis1 = 1;
		if (PlaneNormalAxis == 0)
		{
			Axis0 = 2;
		}
		else if (PlaneNormalAxis == 1)
		{
			Axis1 = 2;
		}
		TVector<RealType> LocalPos = Pos - Origin;
		RealType U = LocalPos.Dot(GetAxis(Axis0));
		RealType V = LocalPos.Dot(GetAxis(Axis1));
		return TVector2<RealType>(U, V);
	}



	/**
	 * Map a point from local UV plane coordinates to the corresponding 3D point in one of the planes of the frame
	 * @param PosUV local UV plane coordinates
	 * @param PlaneNormalAxis which plane to map to, identified by perpendicular normal. Default is 2, ie normal is Z, plane is (X,Y)
	 * @return 3D coordinates in frame's plane (including Origin translation)
	 */
	TVector<RealType> FromPlaneUV(const TVector2<RealType>& PosUV, int PlaneNormalAxis = 2) const
	{
		TVector<RealType> PlanePos(PosUV[0], PosUV[1], 0);
		if (PlaneNormalAxis == 0)
		{
			PlanePos[0] = 0; PlanePos[2] = PosUV[0];
		}
		else if (PlaneNormalAxis == 1)
		{
			PlanePos[1] = 0; PlanePos[2] = PosUV[1];
		}
		return Rotation*PlanePos + Origin;
	}



	/**
	 * Project a point onto one of the planes of the frame
	 * @param Pos 3D position
	 * @param PlaneNormalAxis which plane to project onto, identified by perpendicular normal. Default is 2, ie normal is Z, plane is (X,Y)
	 * @return 3D coordinate in the plane
	 */
	TVector<RealType> ToPlane(const TVector<RealType>& Pos, int PlaneNormalAxis = 2) const
	{
		TVector<RealType> Normal = GetAxis(PlaneNormalAxis);
		TVector<RealType> LocalVec = Pos - Origin;
		RealType SignedDist = LocalVec.Dot(Normal);
		return Pos - SignedDist * Normal;
	}




	/**
	 * Rotate this frame by given quaternion
	 */
	void Rotate(const TQuaternion<RealType>& Quat)
	{
		TQuaternion<RealType> NewRotation = Quat * Rotation;
		if (NewRotation.Normalize() > 0 )
		{
			Rotation = NewRotation;
		}
		else
		{
			checkSlow(false);
		}
	}


	/**
	 * transform this frame by the given transform. Note: Ignores scale, as TFrame3 does not support scaling
	 */
	void Transform(const FTransform& XForm)
	{
		Origin = TVector<RealType>(XForm.TransformPosition((FVector)Origin));
		Rotate(TQuaternion<RealType>(XForm.GetRotation()));
	}


	/**
	 * transform this frame by the given transform. Note: Ignores scale, as TFrame3 does not support scaling
	 */
	void Transform(const TTransformSRT3<RealType>& XForm)
	{
		Origin = XForm.TransformPosition(Origin);
		Rotate(XForm.GetRotation());
	}


	/**
	 * Align an axis of this frame with a target direction
	 * @param AxisIndex which axis to align
	 * @param ToDirection target direction
	 */
	void AlignAxis(int AxisIndex, const TVector<RealType>& ToDirection)
	{
		TQuaternion<RealType> RelRotation(GetAxis(AxisIndex), ToDirection);
		Rotate(RelRotation);
	}


	/**
	 * Compute rotation around vector that best-aligns axis of frame with target direction
	 * @param AxisIndex which axis to try to align
	 * @param ToDirection target direction
	 * @param AroundVector rotation is constrained to be around this vector (ie this direction in frame stays constant)
	 */
	void ConstrainedAlignAxis(int AxisIndex, const TVector<RealType>& ToDirection, const TVector<RealType>& AroundVector)
	{
		//@todo PlaneAngleSigned does acos() and then SetAxisAngleD() does cos/sin...can we optimize this?
		TVector<RealType> AxisVec = GetAxis(AxisIndex);
		RealType AngleDeg = VectorUtil::PlaneAngleSignedD(AxisVec, ToDirection, AroundVector);
		TQuaternion<RealType> RelRotation;
		RelRotation.SetAxisAngleD(AroundVector, AngleDeg);
		Rotate(RelRotation);
	}


	/**
	 * Compute rotation around NormalAxis that best-aligns one of the other two frame axes with either given UpAxis or FallbackAxis
	 * (FallbackAxis is required if Dot(NormalAxis,UpAxis) > UpDotTolerance, ie if the Normal and Up directions are too closely aligned.
	 * Basically this divides direction-sphere into three regions - polar caps with size defined by UpDotTolerance, and
	 * a wide equator band covering the rest. When crossing between these regions the alignment has a discontinuity.
	 * It is impossible to avoid this discontinuity because it is impossible to comb a sphere.
	 * @param PerpAxis1 Index of first axis orthogonal to NormalAxis
	 * @param PerpAxis2 Index of second axis orthogonal to NormalAxis
	 * @param NormalAxis Axis of frame to rotate around
	 * @param UpAxis Target axis in equator region, defaults to UnitZ
	 * @param FallbackAxis Target axis in polar region, defaults to UnitX
	 * @param UpDotTolerance defaults to cos(45), ie flip between regions happens roughly half way to poles
	 */
	void ConstrainedAlignPerpAxes(int PerpAxis1 = 0, int PerpAxis2 = 1, int NormalAxis = 2, 
		const TVector<RealType>& UpAxis = TVector<RealType>::UnitZ(),
		const TVector<RealType>& FallbackAxis = TVector<RealType>::UnitX(),
		RealType UpDotTolerance = (RealType)0.707)
	{
		check(PerpAxis1 != PerpAxis2 && PerpAxis1 != NormalAxis && PerpAxis2 != NormalAxis);
		const TVector<RealType> NormalVec = GetAxis(NormalAxis);

		// decide if we should use Fallback (polar-cap) axis or main (equator-region) axis
		const TVector<RealType>& TargetAxis =
			(TMathUtil<RealType>::Abs(NormalVec.Dot(UpAxis)) > UpDotTolerance) ?
			FallbackAxis : UpAxis;

		// figure out which PerpAxis is closer to target, and align that one to +/- TargetAxis (whichever is smaller rotation)
		TVector2<RealType> Dots(GetAxis(PerpAxis1).Dot(TargetAxis), GetAxis(PerpAxis2).Dot(TargetAxis));
		int UseAxis = (TMathUtil<RealType>::Abs(Dots.X) > TMathUtil<RealType>::Abs(Dots.Y)) ? 0 : 1;
		RealType UseSign = Dots[UseAxis] < 0 ? -1 : 1;
		ConstrainedAlignAxis(UseAxis, UseSign*TargetAxis, NormalVec);
	}


	/**
	 * Compute intersection of ray with plane defined by frame origin and axis as normal
	 * @param RayOrigin origin of ray
	 * @param RayDirection direction of ray
	 * @param PlaneNormalAxis which axis of frame to use as plane normal
	 * @param HitPointOut intersection point, or invalid point if ray does not hit plane or is parallel to plane
	 * @return true if ray intersects plane and HitPointOut is valid
	 */
	bool RayPlaneIntersection(const TVector<RealType>& RayOrigin, const TVector<RealType>& RayDirection, int PlaneNormalAxis, UE::Math::TVector<RealType>& HitPointOut) const
	{
		TVector<RealType> Normal = GetAxis(PlaneNormalAxis);
		RealType PlaneD = -Origin.Dot(Normal);
		RealType NormalDot = RayDirection.Dot(Normal);
		if (VectorUtil::EpsilonEqual(NormalDot, (RealType)0, TMathUtil<RealType>::ZeroTolerance))
		{
			HitPointOut = TVector<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max());
			return false;
		}
		RealType t = -( RayOrigin.Dot(Normal) + PlaneD) / NormalDot;
		if (t < 0)
		{
			HitPointOut = TVector<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max());
			return false;
		}
		HitPointOut = RayOrigin + t * RayDirection;
		return true;
	}

};

template <typename T>
TFrame3<T> Lerp(const TFrame3<T>& A, const TFrame3<T>& B, T Alpha)
{
	return TFrame3<T>(
		UE::Geometry::Lerp(A.Origin, B.Origin, Alpha),
		TQuaternion<T>(A.Rotation, B.Rotation, Alpha));
}



typedef TFrame3<float> FFrame3f;
typedef TFrame3<double> FFrame3d;

} // end namespace UE::Geometry
} // end namespace UE