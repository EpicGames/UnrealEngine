// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp SphereGenerators

#pragma once

#include "MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

#include "GridBoxMeshGenerator.h"

namespace UE
{
namespace Geometry
{


/**
 *  Generate a sphere by pushing a boxes vertices onto a sphere -- often more useful
 */
class /*GEOMETRYCORE_API*/ FBoxSphereGenerator : public FGridBoxMeshGenerator
{
public:

	/** Sphere radius */
	float Radius = 1;

	/** Options for methods to project cube vertices to the sphere */
	enum class ESphereProjectionMethod
	{
		NormalizedVector,
		CubeMapping             // produces more even distribution of quads
								// see http://catlikecoding.com/unity/tutorials/cube-sphere/
								// or http://mathproofs.blogspot.ca/2005/07/mapping-cube-to-sphere.html
	};
	ESphereProjectionMethod SphereProjectionMethod = ESphereProjectionMethod::CubeMapping;

	FMeshShapeGenerator& Generate() override
	{
		FGridBoxMeshGenerator::Generate();

		FVector3d Center = Box.Center();
		FVector3d AxX = Box.AxisX(), AxY = Box.AxisY(), AxZ = Box.AxisZ();
		for (int32 VertIdx = 0; VertIdx < Vertices.Num(); VertIdx++)
		{
			FVector3d V = Vertices[VertIdx] - Center;
			if (SphereProjectionMethod == ESphereProjectionMethod::CubeMapping) {
				double x = V.Dot(AxX) / Box.Extents.X;
				double y = V.Dot(AxY) / Box.Extents.Y;
				double z = V.Dot(AxZ) / Box.Extents.Z;
				double x2 = x * x, y2 = y * y, z2 = z * z;
				double sx = x * FMathd::Sqrt(1.0 - y2*0.5 - z2*0.5 + y2*z2/3.0);
				double sy = y * FMathd::Sqrt(1.0 - x2*0.5 - z2*0.5 + x2*z2/3.0);
				double sz = z * FMathd::Sqrt(1.0 - x2*0.5 - y2*0.5 + x2*y2/3.0);
				V = sx*AxX + sy*AxY + sz*AxZ;
			}
			Normalize(V);
			Vertices[VertIdx] = V;
		}

		for (int32 NormalIdx = 0; NormalIdx < Normals.Num(); NormalIdx++)
		{
			Normals[NormalIdx] = (FVector3f)Vertices[NormalParentVertex[NormalIdx]];
		}

		for (int32 VertIdx = 0; VertIdx < Vertices.Num(); VertIdx++)
		{
			Vertices[VertIdx] = Center + Radius*Vertices[VertIdx];
		}

		return *this;
	}

};



} // end namespace UE::Geometry
} // end namespace UE