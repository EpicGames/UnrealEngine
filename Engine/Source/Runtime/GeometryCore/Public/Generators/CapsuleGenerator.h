// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{


/**
 * Generate a Capsule mesh, with UVs wrapped cylindrically.
 * This is basically a "stretched" standard sphere triangulation, where we have
 * a set of quad strips "around" the sphere with a disc-shaped cap at each pole.
 * For the Capsule we duplicate the equatorial ring, creating two separate hemispherical
 * caps which are joined with a single quad strip.
 *
 * The Capsule line segment is oriented along +Z, with the start point at (0,0,0) and
 * the end point at (0,0,SegmentLength). So the lower hemispherical cap is below the origin,
 * ie the bottom pole is at (0,0,-Radius) and the top pole is at (0,0,SegmentLength+Radius).
 */
class /*GEOMETRYCORE_API*/ FCapsuleGenerator : public FMeshShapeGenerator
{
public:
	/** Radius of capsule */
	double Radius = 1.0;

	/** Length of capsule line segment, so total height is SegmentLength + 2*Radius */
	double SegmentLength = 1.0;

	/** Number of vertices along the 90-degree arc from the pole to edge of spherical cap. */
	int NumHemisphereArcSteps = 5;

	/** Number of vertices along each circle */
	int NumCircleSteps = 3;

	/** Number of subdivisions lengthwise along the cylindrical section */
	int NumSegmentSteps = 0;

	/** If true, each quad gets a separate polygroup, otherwise the entire mesh is a single polygroup */
	bool bPolygroupPerQuad = false;

private:
	static FVector3d SphericalToCartesian(double r, double theta, double phi)
	{
		double Sphi = sin(phi);
		double Cphi = cos(phi);
		double Ctheta = cos(theta);
		double Stheta = sin(theta);

		return FVector3d(r * Ctheta * Sphi, r * Stheta * Sphi, r * Cphi);
	}

	void GenerateVertices()
	{
		auto SetVertex = [this](int32 VtxIdx,
								FVector3d Pos, FVector3f Normal )
		{
			Vertices[VtxIdx] = Pos;
			Normals[VtxIdx] = Normal;
			NormalParentVertex[VtxIdx] = VtxIdx;
		};
		{
			const double Dphi = FMathd::HalfPi / double(NumHemisphereArcSteps - 1);
			const double Dtheta = FMathd::TwoPi / double(NumCircleSteps);

			int32 VtxIdx = 0;
			double Phi, Theta;
			int32 p, t;

			FVector3d Offset(0, 0, SegmentLength);

			// add points for first arc section
			for (p = 1, Phi = Dphi; p < NumHemisphereArcSteps; ++p, Phi += Dphi) // NB: this skips the poles.
			{
				for (t = 0, Theta = 0; t < NumCircleSteps; ++t, ++VtxIdx, Theta += Dtheta)
				{
					FVector3d Normal = SphericalToCartesian(1., Theta, Phi);
					SetVertex(VtxIdx, Normal * Radius + Offset, FVector3f(Normal));
				}
			}

			// add intermediate loops along the cylindrical section
			double SegStepSize = 1.0 / (double)(NumSegmentSteps + 1.0);
			double SegAlong = SegStepSize;
			for (int32 SegStep = 0; SegStep < NumSegmentSteps; ++SegStep, SegAlong += SegStepSize)
			{
				for (t = 0, Theta = 0; t < NumCircleSteps; ++t, ++VtxIdx, Theta += Dtheta)
				{
					FVector3d Normal(FMath::Cos(Theta), FMath::Sin(Theta), 0.0);
					SetVertex(VtxIdx, Normal * Radius + Offset * (1 - SegAlong), FVector3f(Normal));
				}
			}

			// add points for second arc section
			for (p = 1, Phi = FMathd::HalfPi; p < NumHemisphereArcSteps; ++p, Phi += Dphi) // NB: this skips the poles.
			{
				for (t = 0, Theta = 0; t < NumCircleSteps; ++t, ++VtxIdx, Theta += Dtheta)
				{
					FVector3d Normal = SphericalToCartesian(1., Theta, Phi);
					SetVertex(VtxIdx, Normal * Radius, FVector3f(Normal));
				}
			}
			// add a single point at the North Pole
			SetVertex(VtxIdx++, FVector3d::UnitZ() * Radius + Offset, FVector3f::UnitZ());
			// add a single point at the South Pole
			SetVertex(VtxIdx++, -FVector3d::UnitZ() * Radius, -FVector3f::UnitZ());
		}
	}

	void GenerateUVVertices()
	{
		// generate the UVs
		int32 NumPhi = (2 * NumHemisphereArcSteps + NumSegmentSteps);
		const float DUVtheta = -1.0f / float(NumCircleSteps);
		int32 UVIdx = 0;

		// helper to add UVs for a given range in Phi
		auto AddUVSpan = [this, &UVIdx, DUVtheta](int32 StepStart, int32 NumSteps, float PhiStart, float PhiStepSize)
		{
			float UVPhi = PhiStart;
			int32 PIdx = StepStart;
			for (; PIdx < StepStart + NumSteps; ++PIdx, UVPhi += PhiStepSize)
			{
				float UVTheta = 1;
				for (int32 t = 0; t < NumCircleSteps; ++t, ++UVIdx, UVTheta += DUVtheta)
				{
					UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
					UVParentVertex[UVIdx] = PIdx * NumCircleSteps + t;
				}
				UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
				UVParentVertex[UVIdx] = PIdx * NumCircleSteps; // Wrap around
				++UVIdx;
			}
			return PIdx;
		};

		const float PhiSpan = static_cast<float>(2 * Radius + SegmentLength);
		const float HemisphereStepSize = static_cast<float>(Radius) / (PhiSpan * static_cast<float>(NumHemisphereArcSteps - 1));
		// add the first hemisphere cap (except the pole)
		int32 PIdx = AddUVSpan(1, NumHemisphereArcSteps - 1, HemisphereStepSize, HemisphereStepSize);

		// add the cylindrical section
		const float SegmentStepSize = static_cast<float>(SegmentLength) / (PhiSpan * static_cast<float>(NumSegmentSteps + 1));
		PIdx = AddUVSpan(PIdx, NumSegmentSteps, static_cast<float>(Radius) / PhiSpan + SegmentStepSize, SegmentStepSize);

		// add the closing hemisphere end-cap (except the pole)
		AddUVSpan(PIdx, NumHemisphereArcSteps - 1, static_cast<float>(Radius + SegmentLength) / PhiSpan, HemisphereStepSize);

		int32 NorthPoleVtxIdx = (NumPhi - 2 + NumSegmentSteps) * NumCircleSteps;
		float UVTheta;
		int32 t;
		for (t = 0, UVTheta = 1 + DUVtheta; t < NumCircleSteps; ++t, ++UVIdx, UVTheta += DUVtheta)
		{
			UVs[UVIdx] = FVector2f(UVTheta, 0.0);
			UVParentVertex[UVIdx] = NorthPoleVtxIdx;
		}
		int32 SouthPoleVtxIdx = NorthPoleVtxIdx + 1;
		for (t = 0, UVTheta = 1 + DUVtheta; t < NumCircleSteps; ++t, ++UVIdx, UVTheta += DUVtheta)
		{
			UVs[UVIdx] = FVector2f(UVTheta, 1.0);
			UVParentVertex[UVIdx] = SouthPoleVtxIdx;
		}
	}

	using CornerIndices =  FVector3i;
	void OutputTriangle(int TriIdx, int PolyIdx,  CornerIndices Corners, CornerIndices UVCorners)
	{
		SetTriangle(TriIdx, Corners.X, Corners.Y, Corners.Z);
		SetTrianglePolygon(TriIdx, PolyIdx);
		SetTriangleUVs(TriIdx, UVCorners.X, UVCorners.Y, UVCorners.Z);
		SetTriangleNormals(TriIdx, Corners.X, Corners.Y, Corners.Z);
	}

	void OutputEquatorialTriangles()
	{
		int32 NumPhi = (2 * NumHemisphereArcSteps + NumSegmentSteps);
		int32 TriIdx = 0, PolyIdx = 0;

		// Generate equatorial triangles
		int32 Corners[4] =   { 0, 1,     NumCircleSteps + 1, NumCircleSteps};
		int32 UVCorners[4] = { 0, 1, NumCircleSteps + 2, NumCircleSteps + 1};
		for (int32 p = 1; p < NumPhi - 2; ++p)
		{
			for (int32 t = 0; t < NumCircleSteps - 1; ++t)
			{
				// convert each quad into 2 triangles.
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[0],   Corners[1],   Corners[2]},
							   {UVCorners[0], UVCorners[1], UVCorners[2]});
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[2],   Corners[3],   Corners[0]},
							   {UVCorners[2], UVCorners[3], UVCorners[0]});
				for (int32& i : Corners) ++i; 
				for (int32& i : UVCorners) ++i;
				if (bPolygroupPerQuad)
				{
					PolyIdx++;
				}
			}
			OutputTriangle(TriIdx++, PolyIdx,
						   {Corners[0], Corners[1] - NumCircleSteps, Corners[2] - NumCircleSteps},
						   {UVCorners[0]         , UVCorners[1],            UVCorners[2] });
			OutputTriangle(TriIdx++, PolyIdx,
						   {Corners[2] - NumCircleSteps, Corners[3],   Corners[0]},
						   {UVCorners[2],          UVCorners[3],            UVCorners[0]});
			for (int32& i : Corners) ++i;
			for (int32& i : UVCorners) i += 2;
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}

	void OutputPolarTriangles()
	{
		int32 NumPhi = (2 * NumHemisphereArcSteps + NumSegmentSteps);
		const int32 NumEquatorialVtx = (NumPhi - 2) * NumCircleSteps;
		const int32 NumEquatorialUVVtx = (NumPhi - 2) * (NumCircleSteps + 1);
		const int32 NorthPoleVtxIdx = NumEquatorialVtx;
		const int32 SouthPoleVtxIdx = NumEquatorialVtx + 1;
		int32 PolyIdx = (NumCircleSteps  * (NumPhi - 3));
		int32 TriIdx = PolyIdx * 2;
		if (bPolygroupPerQuad == false)
		{
			PolyIdx = 0;
		}

		// Triangles that connect to north pole
		for (int32 t = 0; t < NumCircleSteps; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t, NorthPoleVtxIdx,       (t + 1) % NumCircleSteps},
						   {t, NumEquatorialUVVtx + t, t + 1});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}

		// Triangles that connect to South pole
		const int32 Offset   = NumEquatorialVtx - NumCircleSteps;
		const int32 OffsetUV = NumEquatorialUVVtx - (NumCircleSteps + 1);
		for (int32 t = 0; t < NumCircleSteps; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t + Offset,   ((t + 1) % NumCircleSteps) + Offset, SouthPoleVtxIdx},
						   {t + OffsetUV, t + 1 + OffsetUV             , NumEquatorialUVVtx + NumCircleSteps + t});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}
public:
	/** Generate the mesh */
	FMeshShapeGenerator& Generate() override
	{
		// enforce sane values for vertex counts
		NumHemisphereArcSteps = FMath::Max(NumHemisphereArcSteps, 2);
		int32 NumPhi = (2 * NumHemisphereArcSteps);
		NumCircleSteps = FMath::Max(NumCircleSteps, 3);
		const int32 NumVertices = (NumPhi - 2 + NumSegmentSteps) * NumCircleSteps + 2;
		const int32 NumUVs = (NumPhi - 2 + NumSegmentSteps) * (NumCircleSteps + 1) + (2 * NumCircleSteps);
		const int32 NumTris = (NumPhi - 2 + NumSegmentSteps) * NumCircleSteps * 2;
		SetBufferSizes(NumVertices, NumTris, NumUVs, NumVertices);

		GenerateVertices();
		GenerateUVVertices();
		OutputEquatorialTriangles();
		OutputPolarTriangles();
		return *this;
	}
};


} // end namespace UE::Geometry
} // end namespace UE
