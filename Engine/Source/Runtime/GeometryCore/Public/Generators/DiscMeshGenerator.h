// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "MeshShapeGenerator.h"

namespace UE
{
namespace Geometry
{

/**
 * Generate planar disc (a circle polygon)
 */
class FDiscMeshGenerator : public FMeshShapeGenerator
{
public:
	/** Radius */
	float Radius;

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	FVector3f Normal;

	/** Number of vertices around circumference */
	int AngleSamples;

	/** Number of vertices along radial spokes */
	int RadialSamples;

	/** Start of angle range spanned by disc, in degrees */
	float StartAngle;

	/** End of angle range spanned by disc, in degrees */
	float EndAngle;

	/** If true, output mesh has a single polygroup, otherwise each quad/tri gets a separate group */
	bool bSinglePolygroup = false;

	/** How to map 2D indices to 3D. Default is (0,1) = (x,y,0). */
	FIndex2i IndicesMap;

public:
	GEOMETRYCORE_API FDiscMeshGenerator();

	/** Generate the disc */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;


	/** Create vertex at position under IndicesMap, shifted to Origin*/
	inline FVector3d MakeVertex(double x, double y)
	{
		FVector3d v(0, 0, 0);
		v[IndicesMap.A] = x;
		v[IndicesMap.B] = y;
		return v;
	}
};

/**
* Generate planar disc with a hole
*/
class FPuncturedDiscMeshGenerator : public FDiscMeshGenerator
{
public:
	/** Hole Radius */
	float HoleRadius;

public:
	GEOMETRYCORE_API FPuncturedDiscMeshGenerator();

	/** Generate the disc */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;
};

} // end namespace UE::Geometry
} // end namespace UE
