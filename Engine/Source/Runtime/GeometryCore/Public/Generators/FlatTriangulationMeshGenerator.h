// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IndexTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "MeshShapeGenerator.h"

namespace UE
{
namespace Geometry
{

/**
 * Generate planar triangulation of a Polygon.
 */
class FFlatTriangulationMeshGenerator : public FMeshShapeGenerator
{
public:
	/** Vertices of 2D triangulation. */
	TArray<FVector2d> Vertices2D;

	/** Source triangle indices */
	TArray<FIndex3i> Triangles2D;

	/** Optional polygroup IDs per triangle -- ignored if empty */
	TArray<int> Triangles2DPolygroups;

	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	FVector3f Normal;

	/** How to map 2D indices to 3D. Default is (0,1) = (x,y,0). */
	FIndex2i IndicesMap;

public:
	GEOMETRYCORE_API FFlatTriangulationMeshGenerator();
	GEOMETRYCORE_API ~FFlatTriangulationMeshGenerator();

	/** Generate the triangulation */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;


	/** Create vertex at position under IndicesMap, shifted to Origin*/
	inline FVector3d MakeVertex(double x, double y)
	{
		FVector3d v(0, 0, 0);
		v[IndicesMap.A] = x;
		v[IndicesMap.B] = y;
		return v;
	}
};


} // end namespace UE::Geometry
} // end namespace UE
