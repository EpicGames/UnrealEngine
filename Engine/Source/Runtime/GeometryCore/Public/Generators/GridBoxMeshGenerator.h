// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{

/**
 *  Generate a mesh of a box that has "gridded" faces, i.e. grid of triangulated quads, 
 *  with EdgeVertices setting the number of verts along each edge in each dimension
 */
class /*GEOMETRYCORE_API*/ FGridBoxMeshGenerator : public FMeshShapeGenerator
{
public:
	/** 3D box */
	FOrientedBox3d Box;

	FIndex3i EdgeVertices { 8,8,8 };

	/** If true (default), UVs are scaled so that there is no stretching. If false, UVs are scaled to fill unit square */
	bool bScaleUVByAspectRatio = true;

	/** If true, each quad of box gets a separate polygroup */
	bool bPolygroupPerQuad = false;

public:

	/** Generate the mesh */
	virtual FMeshShapeGenerator& Generate() override
	{
		FIndex3i N = EdgeVertices;
		N.A = FMath::Max(2, N.A);
		N.B = FMath::Max(2, N.B);
		N.C = FMath::Max(2, N.C);

		FVector3d Nscale(1.0 / (N.A-1), 1.0 / (N.B-1), 1.0 / (N.C-1));

		FIndex3i NInternal = N;
		NInternal.A -= 2; NInternal.B -= 2; NInternal.C -= 2;

		FIndex3i NTri = N;
		NTri.A--; NTri.B--; NTri.C--;
		int NumUVsAndNormals = 2 * (N.A * N.B + N.B * N.C + N.C * N.A);
		int NumVertices = 8 + (NInternal.A + NInternal.B + NInternal.C) * 4 + (NInternal.A*NInternal.B + NInternal.B*NInternal.C + NInternal.C*NInternal.A) * 2;
		int NumTriangles = 4 * (NTri.A * NTri.B + NTri.B * NTri.C + NTri.C * NTri.A);
		SetBufferSizes(NumVertices, NumTriangles, NumUVsAndNormals, NumUVsAndNormals);

		int FaceDimOrder[3]{ 1, 2, 0 }; // ordering from IndexUtil.cpp
		int D[2][3]{ { 1,2,0 }, { 2,0,1 } }; // helper mapping to go from major dimension to the two canonical sub-dimensions in order; is just D[i][j] -> (j+i+1)%3 (TODO: replace with the modulus version?)

		// allocate a big mapping from faces to vertices, to make life easier when creating triangles later
		FIndex3i VerticesPerFace(N.B * N.C, N.A * N.C, N.A * N.B);
		TArray<TArray<int>> FaceVertIndices;
		FaceVertIndices.SetNum(6);
		for (int Dim = 0; Dim < 3; Dim++)
		{
			int FaceIdxForDim = FaceDimOrder[Dim] * 2;
			int VertexNum = VerticesPerFace[Dim];
			FaceVertIndices[FaceIdxForDim].SetNum(VertexNum);
			FaceVertIndices[FaceIdxForDim+1].SetNum(VertexNum);
		}

		auto ToFaceV = [&N, &D](int Dim, int D0, int D1)
		{
			return D0 + N[D[0][Dim]] * D1;
		};

		// create the corners and distribute them into the face mapping
		for (int i = 0; i < 8; ++i) 
		{
			Vertices[i] = Box.GetCorner(i);
			FIndex3i CornerSides = FOrientedBox3d::GetCornerSide(i);
			for (int Dim = 0; Dim < 3; Dim++)
			{
				int FaceIdx = FaceDimOrder[Dim]*2 + CornerSides[Dim];
				int D0Ind = CornerSides[D[0][Dim]] * (N[D[0][Dim]] - 1);
				int D1Ind = CornerSides[D[1][Dim]] * (N[D[1][Dim]] - 1);
				int CornerVInd = ToFaceV(Dim, D0Ind, D1Ind);
				FaceVertIndices[FaceIdx][CornerVInd] = i;
			}
		}

		// create the internal (non-corner) edge vertices and distribute them into the face mapping
		int CurrentVertIndex = 8;
		for (int Dim = 0; Dim < 3; Dim++)
		{
			int EdgeLen = N[Dim];
			if (EdgeLen <= 2)
			{
				continue; // no internal edge vertices on this dimension
			}
			int MajorFaceInd = FaceDimOrder[Dim] * 2;
			int FaceInds[2]{ -1,-1 };
			int DSides[2]{ 0,0 };
			for (DSides[0] = 0; DSides[0] < 2; DSides[0]++)
			{
				FaceInds[0] = FaceDimOrder[D[0][Dim]] * 2 + DSides[0];
				for (DSides[1] = 0; DSides[1] < 2; DSides[1]++)
				{
					FaceInds[1] = FaceDimOrder[D[1][Dim]] * 2 + DSides[1];
					int MajorCornerInd = ToFaceV(Dim, DSides[0] * (N[D[0][Dim]] - 1), DSides[1] * (N[D[1][Dim]] - 1));
					FVector3d Corners[2]
					{
						Vertices[FaceVertIndices[MajorFaceInd  ][MajorCornerInd]],
						Vertices[FaceVertIndices[MajorFaceInd+1][MajorCornerInd]]
					};
					
					for (int EdgeVert = 1; EdgeVert + 1 < EdgeLen; EdgeVert++)
					{
						Vertices[CurrentVertIndex] = Lerp(Corners[0], Corners[1], EdgeVert * Nscale[Dim]);
						for (int WhichFace = 0; WhichFace < 2; WhichFace++) // each edge is shared by two faces (w/ major axes of subdim 0 and subdim 1 respectively)
						{
							int FaceInd = FaceInds[WhichFace];

							int FaceDim = D[WhichFace][Dim];
							int SubDims[2];
							SubDims[1 - WhichFace] = EdgeVert;
							SubDims[WhichFace] = NTri[D[WhichFace][FaceDim]] * DSides[1-WhichFace];
							int FaceV = ToFaceV(FaceDim, SubDims[0], SubDims[1]); //-V614
							FaceVertIndices[FaceInds[WhichFace]][FaceV] = CurrentVertIndex;
						}
						CurrentVertIndex++;
					}
				}
			}
		}

		// create the internal (non-corner, non-edge) face vertices and distribute them into the face mapping
		for (int Dim = 0; Dim < 3; Dim++)
		{
			int FaceIdxBase = FaceDimOrder[Dim]*2;
			int FaceInternalVNum = NInternal[D[0][Dim]] * NInternal[D[1][Dim]];
			if (FaceInternalVNum <= 0)
			{
				continue;
			}

			for (int Side = 0; Side < 2; Side++)
			{
				int MajorFaceInd = FaceIdxBase + Side;
				for (int D0 = 1; D0 + 1 < N[D[0][Dim]]; D0++)
				{
					int BotInd = ToFaceV(Dim, D0, 0);
					int TopInd = ToFaceV(Dim, D0, N[D[1][Dim]] - 1);

					FVector3d Edges[2]
					{
						Vertices[FaceVertIndices[MajorFaceInd][BotInd]],
						Vertices[FaceVertIndices[MajorFaceInd][TopInd]]
					};
					for (int D1 = 1; D1 + 1 < N[D[1][Dim]]; D1++)
					{
						Vertices[CurrentVertIndex] = Lerp(Edges[0], Edges[1], D1 * Nscale[D[1][Dim]]);
						FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0, D1)] = CurrentVertIndex;
						CurrentVertIndex++;
					}
				}
			}
		}

		double MaxDimension = MaxAbsElement(2.0*Box.Extents);
		float UVScale = (bScaleUVByAspectRatio) ? (1.0f / (float)MaxDimension) : 1.0f;

		// create the face triangles and UVs+normals
		int CurrentTriIdx = 0;
		int CurrentUVIdx = 0;
		int CurrentQuadIdx = 0;
		for (int Dim = 0; Dim < 3; Dim++)
		{
			int FaceIdxBase = FaceDimOrder[Dim]*2;

			// UV-specific minor axes + flips; manually set to match default UnrealEngine cube texture arrangement
			int Minor1Flip[3] = { -1, 1, 1 };
			int Minor2Flip[3] = { -1, -1, 1 };

			// UV scales for D0, D1
			double FaceWidth = FMathd::Abs(Box.Extents[D[0][Dim]]) * 2.0;
			double FaceHeight = FMathd::Abs(Box.Extents[D[1][Dim]]) * 2.0;
			double WidthUVScale = FaceWidth * UVScale;
			double HeightUVScale = FaceHeight * UVScale;

			
			for (int Side = 0; Side < 2; Side++)
			{
				int SideOpp = 1 - Side;
				int SideSign = Side * 2 - 1;

				FVector3f Normal(0, 0, 0);
				Normal[Dim] = float(2 * Side - 1);
				Normal = (FVector3f)Box.Frame.FromFrameVector((FVector3d)Normal);

				int MajorFaceInd = FaceIdxBase + Side;

				int FaceUVStartInd = CurrentUVIdx;
				// set all the UVs and normals
				FVector2f UV;
				int UVXDim = Dim == 1 ? 1 : 0;	// which dim (of D0,D1) follows the horizontal UV coordinate
				int UVYDim = 1 - UVXDim;		// which dim (of D0,D1) follows the vertical UV coordinate
				for (int D0 = 0; D0 < N[D[0][Dim]]; D0++)
				{
					for (int D1 = 0; D1 < N[D[1][Dim]]; D1++)
					{
						// put the grid coordinates (centered at 0,0) into the UVs
						UV[UVXDim] = float (D0 * Nscale[D[0][Dim]] - .5);
						UV[UVYDim] = float (D1 * Nscale[D[1][Dim]] - .5);
						// invert axes to match the desired UV patterns & so the opp faces are not backwards
						UV.X *= float(SideSign * Minor1Flip[Dim]);
						UV.Y *= float(Minor2Flip[Dim]);
						// recenter and scale up
						UV[UVXDim] = float ( (UV[UVXDim] + .5f) * WidthUVScale);
						UV[UVYDim] = float ( (UV[UVYDim] + .5f) * HeightUVScale);
						UVs[CurrentUVIdx] = UV;
						Normals[CurrentUVIdx] = Normal;
						UVParentVertex[CurrentUVIdx] = FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0, D1)];
						NormalParentVertex[CurrentUVIdx] = FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0, D1)];
						CurrentUVIdx++;
					}
				}

				// set all the triangles
				for (int D0 = 0; D0 + 1 < N[D[0][Dim]]; D0++)
				{
					for (int D1 = 0; D1 + 1 < N[D[1][Dim]]; D1++)
					{
						SetTriangle(CurrentTriIdx,
								FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0, D1)],
								FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0+SideOpp, D1+Side)],
								FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0+1, D1+1)]
							);

						SetTriangleUVs(CurrentTriIdx,
							FaceUVStartInd + D1 + (D0) * N[D[1][Dim]],
							FaceUVStartInd + D1+Side + (D0+SideOpp) * N[D[1][Dim]],
							FaceUVStartInd + D1+1 + (D0+1) * N[D[1][Dim]]
						);
						SetTriangleNormals(CurrentTriIdx,
							FaceUVStartInd + D1 + (D0) * N[D[1][Dim]],
							FaceUVStartInd + D1+Side + (D0+SideOpp) * N[D[1][Dim]],
							FaceUVStartInd + D1+1 + (D0+1) * N[D[1][Dim]]
						);
						SetTrianglePolygon(CurrentTriIdx, (bPolygroupPerQuad) ? CurrentQuadIdx : MajorFaceInd);
						CurrentTriIdx++;

						SetTriangle(CurrentTriIdx,
							FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0, D1)],
							FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0+1, D1+1)],
							FaceVertIndices[MajorFaceInd][ToFaceV(Dim, D0+Side, D1+SideOpp)]
						);
						SetTriangleUVs(CurrentTriIdx,
							FaceUVStartInd + D1 + (D0) * N[D[1][Dim]],
							FaceUVStartInd + D1+1 + (D0+1) * N[D[1][Dim]],
							FaceUVStartInd + D1+SideOpp + (D0+Side) * N[D[1][Dim]]
						);
						SetTriangleNormals(CurrentTriIdx,
							FaceUVStartInd + D1 + (D0) * N[D[1][Dim]],
							FaceUVStartInd + D1+1 + (D0+1) * N[D[1][Dim]],
							FaceUVStartInd + D1+SideOpp + (D0+Side) * N[D[1][Dim]]
						);
						SetTrianglePolygon(CurrentTriIdx, (bPolygroupPerQuad) ? CurrentQuadIdx : MajorFaceInd);
						CurrentTriIdx++;
						CurrentQuadIdx++;
					}
				}
			}
		}


		return *this;
	}

};



} // end namespace UE::Geometry
} // end namespace UE
