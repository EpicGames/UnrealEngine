// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "TransformTypes.h"

namespace UE
{
	namespace Geometry
	{
		using namespace UE::Math;

		/**
		 * call EmitLineFunc for the 12 line segments defined by the given box parameters
		 * @param HalfDimensions X/Y/Z dimensions of box
		 * @param Center center of box
		 * @param AxisX X axis of box
		 * @param AxisY Y axis of box
		 * @param AxisZ Z axis of box
		 * @param Transform transform applied to points of box
		 */
		template<typename RealType>
		void GenerateBoxSegments(
			const TVector<RealType>& HalfDimensions, 
			const TVector<RealType>& Center, 
			const TVector<RealType>& AxisX, 
			const TVector<RealType>& AxisY, 
			const TVector<RealType>& AxisZ,
			const TTransformSRT3<RealType>& Transform,
			TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc);

		/**
		 * call EmitLineFunc for the line segments that make up the Circle defined by the given parameters and sampled with NumSteps vertices
		 * @param NumSteps number of vertices used to approximate circle
		 * @param Radius radius of circle
		 * @param Center center of circle
		 * @param AxisX first axis of circle
		 * @param AxisY second axis of circle
		 * @param Transform transform applied to each point
		 */
		template<typename RealType>
		void GenerateCircleSegments(
			int32 NumSteps, 
			RealType Radius, 
			const TVector<RealType>& Center,
			const TVector<RealType>& AxisX,
			const TVector<RealType>& AxisY,
			const TTransformSRT3<RealType>& Transform,
			TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc);

		/**
		 * call EmitLineFunc for the line segments that make up the Circular Arc defined by the given parameters and sampled with NumSteps vertices
		 * @param NumSteps number of vertices used to approximate arc
		 * @param Radius radius of arc
		 * @param StartAngle arc starts at this angle
		 * @param EndAngle arc ends at this angle
		 * @param AxisX first axis of arc
		 * @param AxisY second axis of arc
		 * @param Transform transform applied to each point
		 */
		template<typename RealType>
		void GenerateArcSegments(
			int32 NumSteps, 
			RealType Radius,
			RealType StartAngle, 
			RealType EndAngle,
			const TVector<RealType>& Center,
			const TVector<RealType>& AxisX,
			const TVector<RealType>& AxisY,
			const TTransformSRT3<RealType>& Transform,
			TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc);
	}
}





template<typename RealType>
void UE::Geometry::GenerateBoxSegments(
	const TVector<RealType>& HalfDimensions,
	const TVector<RealType>& Center,
	const TVector<RealType>& AxisX,
	const TVector<RealType>& AxisY,
	const TVector<RealType>& AxisZ,
	const TTransformSRT3<RealType>& Transform,
	TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc)
{
	// B is box max/min, P and Q are used to store the start and endpoints of the segments we create
	TVector<RealType>	B[2], P, Q;
	B[0] = Center + HalfDimensions; // max
	B[1] = Center - HalfDimensions; // min

	// Iterate across the four corners of top/side/front and create segments along the Z axis
	for (int32 i = 0; i < 2; i++)
	{
		for (int32 j = 0; j < 2; j++)
		{
			P.X = B[i].X; Q.X = B[i].X;
			P.Y = B[j].Y; Q.Y = B[j].Y;
			P.Z = B[0].Z; Q.Z = B[1].Z;
			EmitLineFunc(Transform.TransformPosition(P), Transform.TransformPosition(Q));

			P.Y = B[i].Y; Q.Y = B[i].Y;
			P.Z = B[j].Z; Q.Z = B[j].Z;
			P.X = B[0].X; Q.X = B[1].X;
			EmitLineFunc(Transform.TransformPosition(P), Transform.TransformPosition(Q));

			P.Z = B[i].Z; Q.Z = B[i].Z;
			P.X = B[j].X; Q.X = B[j].X;
			P.Y = B[0].Y; Q.Y = B[1].Y;
			EmitLineFunc(Transform.TransformPosition(P), Transform.TransformPosition(Q));
		}
	}
}



template<typename RealType>
void UE::Geometry::GenerateCircleSegments(
	int32 NumSteps, RealType Radius,
	const TVector<RealType>& Center,
	const TVector<RealType>& AxisX,
	const TVector<RealType>& AxisY,
	const TTransformSRT3<RealType>& Transform,
	TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc)
{
	TVector<RealType> PrevPos = TVector<RealType>::Zero();
	for (int32 i = 0; i <= NumSteps; ++i)
	{
		RealType t = (RealType)i / (RealType)NumSteps;
		RealType Angle = (RealType)2.0 * TMathUtil<RealType>::Pi * t;
		RealType PlaneX = Radius * TMathUtil<RealType>::Cos(Angle);
		RealType PlaneY = Radius * TMathUtil<RealType>::Sin(Angle);
		TVector<RealType> CurPos = Transform.TransformPosition(Center + PlaneX * AxisX + PlaneY * AxisY);
		if (i > 0)
		{
			EmitLineFunc(PrevPos, CurPos);
		}
		PrevPos = CurPos;
	}
}



template<typename RealType>
void UE::Geometry::GenerateArcSegments(
	int32 NumSteps,
	RealType Radius,
	RealType StartAngle,
	RealType EndAngle,
	const TVector<RealType>& Center,
	const TVector<RealType>& AxisX,
	const TVector<RealType>& AxisY,
	const TTransformSRT3<RealType>& Transform,
	TFunctionRef<void(const TVector<RealType>& A, const TVector<RealType>& B)> EmitLineFunc)
{
	TVector<RealType> PrevPos = TVector<RealType>::Zero();
	for (int32 i = 0; i <= NumSteps; ++i)
	{
		RealType t = (RealType)i / (RealType)NumSteps;
		RealType Angle = ((RealType)1 - t) * StartAngle + (t)*EndAngle;
		RealType PlaneX = Radius * TMathUtil<RealType>::Cos(Angle);
		RealType PlaneY = Radius * TMathUtil<RealType>::Sin(Angle);
		TVector<RealType> CurPos = Transform.TransformPosition(Center + PlaneX * AxisX + PlaneY * AxisY);
		if (i > 0)
		{
			EmitLineFunc(PrevPos, CurPos);
		}
		PrevPos = CurPos;
	}
}