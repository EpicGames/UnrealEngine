// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MarchingCubesPro

#pragma once

#include "Async/ParallelFor.h"
#include "BoxTypes.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "IndexTypes.h"
#include "IntBoxTypes.h"
#include "IntVectorTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "MathUtil.h"
#include "MeshShapeGenerator.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Spatial/BlockedDenseGrid3.h"
#include "Spatial/DenseGrid3.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Util/IndexUtil.h"
#include "VectorTypes.h"

#include <atomic>

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

enum class /*GEOMETRYCORE_API*/ ERootfindingModes
{
	SingleLerp,
	LerpSteps,
	Bisection
};

class /*GEOMETRYCORE_API*/ FMarchingCubes : public FMeshShapeGenerator
{
public:
	/**
	*  this is the function we will evaluate
	*/
	TFunction<double(TVector<double>)> Implicit;

	/**
	*  mesh surface will be at this isovalue. Normally 0 unless you want
	*  offset surface or field is not a distance-field.
	*/
	double IsoValue = 0;

	/** bounding-box we will mesh inside of. We use the min-corner and
	 *  the width/height/depth, but do not clamp vertices to stay within max-corner,
	 *  we may spill one cell over
	 */
	TAxisAlignedBox3<double> Bounds;

	/**
	 *  Length of edges of cubes that are marching.
	 *  currently, # of cells along axis = (int)(bounds_dimension / CellSize) + 1
	 */
	double CubeSize = 0.1;

	/**
	 *  Use multi-threading? Generally a good idea unless problem is very small or
	 *  you are multi-threading at a higher level (which may be more efficient)
	 */
	bool bParallelCompute = true;

	/**
	 * If true, code will assume that Implicit() is expensive enough that it is worth it to cache
	 * evaluations when possible. For something simple like evaluation of an SDF defined by a discrete
	 * grid, this is generally not worth the overhead.
	 */
	bool bEnableValueCaching = true;

	/**
	 * Max number of cells on any dimension; if exceeded, CubeSize will be automatically increased to fix
	 */
	int SafetyMaxDimension = 4096;

	/**
	 *  Which rootfinding method will be used to converge on surface along edges
	 */
	ERootfindingModes RootMode = ERootfindingModes::SingleLerp;

	/**
	 *  number of iterations of rootfinding method (ignored for SingleLerp)
	 */
	int RootModeSteps = 5;


	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []() { return false; };

	/*
	 * Outputs
	 */

	// cube indices range from [Origin,CellDimensions)   
	FVector3i CellDimensions;


	FMarchingCubes()
	{
		Bounds = TAxisAlignedBox3<double>(TVector<double>::Zero(), 8);
		CubeSize = 0.25;
	}

	virtual ~FMarchingCubes() = default;

	bool Validate()
	{
		return CubeSize > 0 && FMath::IsFinite(CubeSize) && !Bounds.IsEmpty() && FMath::IsFinite(Bounds.MaxDim());
	}

	/**
	*  Run MC algorithm and generate Output mesh
	*/
	FMeshShapeGenerator& Generate() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_MCMesh_Generate);

		if (!ensure(Validate()))
		{
			return *this;
		}

		SetDimensions();
		GridBounds = FAxisAlignedBox3i(FVector3i::Zero(), CellDimensions - FVector3i(1,1,1)); // grid bounds are inclusive

		if (bEnableValueCaching)
		{
			BlockedCornerValuesGrid.Reset(CellDimensions.X + 1, CellDimensions.Y + 1, CellDimensions.Z + 1, FMathf::MaxReal);
		}
		InitHashTables();
		ResetMesh();

		if (bParallelCompute) 
		{
			generate_parallel();
		} 
		else 
		{
			generate_basic();
		}

		// finalize mesh
		BuildMesh();

		return *this;
	}


	FMeshShapeGenerator& GenerateContinuation(TArrayView<const FVector3d> Seeds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_MCMesh_GenerateContinuation);

		if (!ensure(Validate()))
		{
			return *this;
		}

		SetDimensions();
		GridBounds = FAxisAlignedBox3i(FVector3i::Zero(), CellDimensions - FVector3i(1,1,1)); // grid bounds are inclusive

		InitHashTables();
		ResetMesh();

		if (LastGridBounds != GridBounds)
		{
			if (bEnableValueCaching)
			{
				BlockedCornerValuesGrid.Reset(CellDimensions.X + 1, CellDimensions.Y + 1, CellDimensions.Z + 1, FMathf::MaxReal);
			}
			if (bParallelCompute)
			{
				BlockedDoneCells.Reset(CellDimensions.X, CellDimensions.Y, CellDimensions.Z, 0);
			}
		}
		else
		{
			if (bEnableValueCaching)
			{
				BlockedCornerValuesGrid.Resize(CellDimensions.X + 1, CellDimensions.Y + 1, CellDimensions.Z + 1);
			}
			if (bParallelCompute)
			{
				BlockedDoneCells.Resize(CellDimensions.X, CellDimensions.Y, CellDimensions.Z);
			}
		}

		if (bParallelCompute) 
		{
			generate_continuation_parallel(Seeds);
		} 
		else 
		{
			generate_continuation(Seeds);
		}

		// finalize mesh
		BuildMesh();

		LastGridBounds = GridBounds;

		return *this;
	}


protected:


	FAxisAlignedBox3i GridBounds;
	FAxisAlignedBox3i LastGridBounds;


	// we pass Cells around, this makes code cleaner
	struct FGridCell
	{
		// TODO we do not actually need to store i, we just need the min-corner!
		FVector3i i[8];    // indices of corners of cell
		double f[8];      // field values at corners
	};

	void SetDimensions()
	{
		int NX = (int)(Bounds.Width() / CubeSize) + 1;
		int NY = (int)(Bounds.Height() / CubeSize) + 1;
		int NZ = (int)(Bounds.Depth() / CubeSize) + 1;
		int MaxDim = FMath::Max3(NX, NY, NZ);
		if (!ensure(MaxDim <= SafetyMaxDimension))
		{
			CubeSize = Bounds.MaxDim() / double(SafetyMaxDimension - 1);
			NX = (int)(Bounds.Width() / CubeSize) + 1;
			NY = (int)(Bounds.Height() / CubeSize) + 1;
			NZ = (int)(Bounds.Depth() / CubeSize) + 1;
		}
		CellDimensions = FVector3i(NX, NY, NZ);
	}

	void corner_pos(const FVector3i& IJK, TVector<double>& Pos)
	{
		Pos.X = Bounds.Min.X + CubeSize * IJK.X;
		Pos.Y = Bounds.Min.Y + CubeSize * IJK.Y;
		Pos.Z = Bounds.Min.Z + CubeSize * IJK.Z;
	}
	TVector<double> corner_pos(const FVector3i& IJK)
	{
		return TVector<double>(Bounds.Min.X + CubeSize * IJK.X,
			Bounds.Min.Y + CubeSize * IJK.Y,
			Bounds.Min.Z + CubeSize * IJK.Z);
	}
	FVector3i cell_index(const TVector<double>& Pos)
	{
		return FVector3i(
			(int)((Pos.X - Bounds.Min.X) / CubeSize),
			(int)((Pos.Y - Bounds.Min.Y) / CubeSize),
			(int)((Pos.Z - Bounds.Min.Z) / CubeSize));
	}



	//
	// corner and edge hash functions, these pack the coordinate
	// integers into 16-bits, so max of 65536 in any dimension.
	//


	int64 corner_hash(const FVector3i& Idx)
	{
		return ((int64)Idx.X&0xFFFF) | (((int64)Idx.Y&0xFFFF) << 16) | (((int64)Idx.Z&0xFFFF) << 32);
	}
	int64 corner_hash(int X, int Y, int Z)
	{
		return ((int64)X & 0xFFFF) | (((int64)Y & 0xFFFF) << 16) | (((int64)Z & 0xFFFF) << 32);
	}

	const int64 EDGE_X = int64(1) << 60;
	const int64 EDGE_Y = int64(1) << 61;
	const int64 EDGE_Z = int64(1) << 62;

	int64 edge_hash(const FVector3i& Idx1, const FVector3i& Idx2)
	{
		if ( Idx1.X != Idx2.X )
		{
			int xlo = FMath::Min(Idx1.X, Idx2.X);
			return corner_hash(xlo, Idx1.Y, Idx1.Z) | EDGE_X;
		}
		else if ( Idx1.Y != Idx2.Y )
		{
			int ylo = FMath::Min(Idx1.Y, Idx2.Y);
			return corner_hash(Idx1.X, ylo, Idx1.Z) | EDGE_Y;
		}
		else
		{
			int zlo = FMath::Min(Idx1.Z, Idx2.Z);
			return corner_hash(Idx1.X, Idx1.Y, zlo) | EDGE_Z;
		}
	}



	//
	// Hash table for edge vertices
	//

	const int64 NumEdgeVertexSections = 64;
	TArray<TMap<int64, int>> EdgeVertexSections;
	TArray<FCriticalSection> EdgeVertexSectionLocks;
	
	int FindVertexID(int64 hash)
	{
		int32 SectionIndex = (int32)(hash % (NumEdgeVertexSections - 1));
		FScopeLock Lock(&EdgeVertexSectionLocks[SectionIndex]);
		int* Found = EdgeVertexSections[SectionIndex].Find(hash);
		return (Found != nullptr) ? *Found : IndexConstants::InvalidID;
	}

	int AppendOrFindVertexID(int64 hash, TVector<double> Pos)
	{
		int32 SectionIndex = (int32)(hash % (NumEdgeVertexSections - 1));
		FScopeLock Lock(&EdgeVertexSectionLocks[SectionIndex]);
		int* FoundVID = EdgeVertexSections[SectionIndex].Find(hash);
		if (FoundVID != nullptr)
		{
			return *FoundVID;
		}
		int NewVID = append_vertex(Pos, hash);
		EdgeVertexSections[SectionIndex].Add(hash, NewVID);
		return NewVID;
	}


	int edge_vertex_id(const FVector3i& Idx1, const FVector3i& Idx2, double F1, double F2)
	{
		int64 hash = edge_hash(Idx1, Idx2);

		int foundvid = FindVertexID(hash);
		if (foundvid != IndexConstants::InvalidID)
		{
			return foundvid;
		}

		// ok this is a bit messy. We do not want to lock the entire hash table 
		// while we do find_iso. However it is possible that during this time we
		// are unlocked we have re-entered with the same edge. So when we
		// re-acquire the lock we need to check again that we have not already
		// computed this edge, otherwise we will end up with duplicate vertices!

		TVector<double> pa = TVector<double>::Zero(), pb = TVector<double>::Zero();
		corner_pos(Idx1, pa);
		corner_pos(Idx2, pb);
		TVector<double> Pos = TVector<double>::Zero();
		find_iso(pa, pb, F1, F2, Pos);

		return AppendOrFindVertexID(hash, Pos);
	}








	//
	// store corner values in pre-allocated grid that has
	// FMathf::MaxReal as sentinel. 
	// (note this is float grid, not double...)
	//

	FBlockedDenseGrid3f BlockedCornerValuesGrid;

	double corner_value_grid_parallel(const FVector3i& Idx)
	{
		// note: it's possible to have a race here, where multiple threads might both
		// GetValue, see that the value is invalid, and compute and set it. Since Implicit(V)
		// is (intended to be) determinstic, they will compute the same value, so this doesn't cause an error, 
		// it just wastes a bit of computation time. Since it is common for multiple corners to be
		// in the same grid-block, and locking is on the block level, it is (or was in some testing)
		// better to not lock the entire block while Implicit(V) computed, at the cost of
		// some wasted evals in some cases.

		float CurrentValue = BlockedCornerValuesGrid.GetValueThreadSafe(Idx.X, Idx.Y, Idx.Z);
		if (CurrentValue != FMathf::MaxReal)
		{
			return (double)CurrentValue;
		}

		TVector<double> V = corner_pos(Idx);
		CurrentValue = (float)Implicit(V);

		BlockedCornerValuesGrid.SetValueThreadSafe(Idx.X, Idx.Y, Idx.Z, CurrentValue);

		return (double)CurrentValue;
	}
	double corner_value_grid(const FVector3i& Idx)
	{
		if (bParallelCompute)
		{
			return corner_value_grid_parallel(Idx);
		}

		float CurrentValue = BlockedCornerValuesGrid.GetValue(Idx.X, Idx.Y, Idx.Z);
		if (CurrentValue != FMathf::MaxReal)
		{
			return (double)CurrentValue;
		}

		TVector<double> V = corner_pos(Idx);
		CurrentValue = (float)Implicit(V);

		BlockedCornerValuesGrid.SetValue(Idx.X, Idx.Y, Idx.Z, CurrentValue);

		return (double)CurrentValue;
	}

	void initialize_cell_values_grid(FGridCell& Cell, bool Shift)
	{
		if (Shift)
		{
			Cell.f[1] = corner_value_grid(Cell.i[1]);
			Cell.f[2] = corner_value_grid(Cell.i[2]);
			Cell.f[5] = corner_value_grid(Cell.i[5]);
			Cell.f[6] = corner_value_grid(Cell.i[6]);
		}
		else
		{
			for (int i = 0; i < 8; ++i)
			{
				Cell.f[i] = corner_value_grid(Cell.i[i]);
			}
		}
	}



	//
	// explicitly compute corner values as necessary
	//
	//

	double corner_value_nohash(const FVector3i& Idx) 
	{
		TVector<double> V = corner_pos(Idx);
		return Implicit(V);
	}
	void initialize_cell_values_nohash(FGridCell& Cell, bool Shift)
	{
		if (Shift)
		{
			Cell.f[1] = corner_value_nohash(Cell.i[1]);
			Cell.f[2] = corner_value_nohash(Cell.i[2]);
			Cell.f[5] = corner_value_nohash(Cell.i[5]);
			Cell.f[6] = corner_value_nohash(Cell.i[6]);
		}
		else
		{
			for (int i = 0; i < 8; ++i)
			{
				Cell.f[i] = corner_value_nohash(Cell.i[i]);
			}
		}
	}



	/**
	*  compute 3D corner-positions and field values for cell at index
	*/
	void initialize_cell(FGridCell& Cell, const FVector3i& Idx)
	{
		Cell.i[0] = FVector3i(Idx.X + 0, Idx.Y + 0, Idx.Z + 0);
		Cell.i[1] = FVector3i(Idx.X + 1, Idx.Y + 0, Idx.Z + 0);
		Cell.i[2] = FVector3i(Idx.X + 1, Idx.Y + 0, Idx.Z + 1);
		Cell.i[3] = FVector3i(Idx.X + 0, Idx.Y + 0, Idx.Z + 1);
		Cell.i[4] = FVector3i(Idx.X + 0, Idx.Y + 1, Idx.Z + 0);
		Cell.i[5] = FVector3i(Idx.X + 1, Idx.Y + 1, Idx.Z + 0);
		Cell.i[6] = FVector3i(Idx.X + 1, Idx.Y + 1, Idx.Z + 1);
		Cell.i[7] = FVector3i(Idx.X + 0, Idx.Y + 1, Idx.Z + 1);

		if (bEnableValueCaching)
		{
			initialize_cell_values_grid(Cell, false);
		}
		else
		{
			initialize_cell_values_nohash(Cell, false);
		}
	}


	// assume we just want to slide cell at XIdx-1 to cell at XIdx, while keeping
	// yi and ZIdx constant. Then only x-coords change, and we have already 
	// computed half the values
	void shift_cell_x(FGridCell& Cell, int XIdx)
	{
		Cell.f[0] = Cell.f[1];
		Cell.f[3] = Cell.f[2];
		Cell.f[4] = Cell.f[5];
		Cell.f[7] = Cell.f[6];

		Cell.i[0].X = XIdx; Cell.i[1].X = XIdx+1; Cell.i[2].X = XIdx+1; Cell.i[3].X = XIdx;
		Cell.i[4].X = XIdx; Cell.i[5].X = XIdx+1; Cell.i[6].X = XIdx+1; Cell.i[7].X = XIdx;

		if (bEnableValueCaching)
		{
			initialize_cell_values_grid(Cell, true);
		}
		else
		{
			initialize_cell_values_nohash(Cell, true);
		}
	}


	void InitHashTables()
	{
		EdgeVertexSections.Reset();
		EdgeVertexSections.SetNum((int32)NumEdgeVertexSections);
		EdgeVertexSectionLocks.Reset();
		EdgeVertexSectionLocks.SetNum((int32)NumEdgeVertexSections);
	}


	bool parallel_mesh_access = false;


	/**
	*  processing z-slabs of cells in parallel
	*/
	void generate_parallel()
	{
		parallel_mesh_access = true;

		// [TODO] maybe shouldn't alway use Z axis here?
		ParallelFor(CellDimensions.Z, [this](int32 ZIdx)
		{
			FGridCell Cell;
			int vertTArray[12];
			for (int yi = 0; yi < CellDimensions.Y; ++yi)
			{
				if (CancelF())
				{
					return;
				}
				// compute full cell at x=0, then slide along x row, which saves half of value computes
				FVector3i Idx(0, yi, ZIdx);
				initialize_cell(Cell, Idx);
				polygonize_cell(Cell, vertTArray);
				for (int XIdx = 1; XIdx < CellDimensions.X; ++XIdx)
				{
					shift_cell_x(Cell, XIdx);
					polygonize_cell(Cell, vertTArray);
				}
			}
		});


		parallel_mesh_access = false;
	}




	/**
	*  fully sequential version, no threading
	*/
	void generate_basic()
	{
		FGridCell Cell;
		int vertTArray[12];

		for (int ZIdx = 0; ZIdx < CellDimensions.Z; ++ZIdx)
		{
			for (int yi = 0; yi < CellDimensions.Y; ++yi)
			{
				if (CancelF())
				{
					return;
				}
				// compute full Cell at x=0, then slide along x row, which saves half of value computes
				FVector3i Idx(0, yi, ZIdx);
				initialize_cell(Cell, Idx);
				polygonize_cell(Cell, vertTArray);
				for (int XIdx = 1; XIdx < CellDimensions.X; ++XIdx)
				{
					shift_cell_x(Cell, XIdx);
					polygonize_cell(Cell, vertTArray);
				}

			}
		}
	}




	/**
	*  fully sequential version, no threading
	*/
	void generate_continuation(TArrayView<const FVector3d> Seeds)
	{
		FGridCell Cell;
		int vertTArray[12];

		BlockedDoneCells.Reset(CellDimensions.X, CellDimensions.Y, CellDimensions.Z, 0);

		TArray<FVector3i> stack;

		for (FVector3d seed : Seeds)
		{
			FVector3i seed_idx = cell_index(seed);
			if (!BlockedDoneCells.IsValidIndex(seed_idx) || BlockedDoneCells.GetValue(seed_idx.X, seed_idx.Y, seed_idx.Z) == 1)
			{
				continue;
			}
			stack.Add(seed_idx);
			BlockedDoneCells.SetValue(seed_idx.X, seed_idx.Y, seed_idx.Z, 1);

			while ( stack.Num() > 0 )
			{
				FVector3i Idx = stack[stack.Num()-1]; 
				stack.RemoveAt(stack.Num()-1);
				if (CancelF())
				{
					return;
				}

				initialize_cell(Cell, Idx);
				if ( polygonize_cell(Cell, vertTArray) )
				{     // found crossing
					for ( FVector3i o : IndexUtil::GridOffsets6 )
					{
						FVector3i nbr_idx = Idx + o;
						if (GridBounds.Contains(nbr_idx) && BlockedDoneCells.GetValue(nbr_idx.X, nbr_idx.Y, nbr_idx.Z) == 0)
						{
							stack.Add(nbr_idx);
							BlockedDoneCells.SetValue(nbr_idx.X, nbr_idx.Y, nbr_idx.Z, 1);
						}
					}
				}
			}
		}
	}




	/**
	*  parallel seed evaluation
	*/
	void generate_continuation_parallel(TArrayView<const FVector3d> Seeds)
	{
		// Parallel marching cubes based on continuation (ie surface-following / front propagation) 
		// can have quite poor multithreaded performance depending on the ordering of the region-growing.
		// For example processing each seed point in parallel can result in one thread that
		// takes significantly longer than others, if the seed point distribution is such
		// that a large part of the surface is only reachable from one seed (or gets
		// "cut off" by a thin area, etc). So we want to basically do front-marching in
		// parallel passes. However this can result in a large number of very short passes
		// if the front ends up with many small regions, etc. So, in the implementation below,
		// each "seed cell" is allowed to process up to N neighbour cells before terminating, at 
		// which point any cells remaining on the active front are added as seed cells for the next pass.
		// This seems to provide good utilization, however more profiling may be needed.
		// (In particular, if the active cell list is large, some blocks of the ParallelFor
		//  may still end up doing much more work than others)


		// set this flag so that append vertex/triangle operations will lock the mesh
		parallel_mesh_access = true;

		// maximum number of cells to process in each ParallelFor iteration
		static constexpr int MaxNeighboursPerActiveCell = 100;

		// list of active cells on the MC front to process in the next pass
		TArray<FVector3i> ActiveCells;

		// initially push list of seed-point cells onto the ActiveCells list
		for (FVector3d Seed : Seeds)
		{
			FVector3i seed_idx = cell_index(Seed);
			if ( BlockedDoneCells.IsValidIndex(seed_idx) && set_cell_if_not_done(seed_idx) )
			{
				ActiveCells.Add(seed_idx);
			}
		}

		// new active cells will be accumulated in each parallel-pass
		TArray<FVector3i> NewActiveCells;
		FCriticalSection NewActiveCellsLock;

		while (ActiveCells.Num() > 0)
		{
			// process all active cells
			ParallelFor(ActiveCells.Num(), [&](int32 Idx)
			{
				FVector3i InitialCellIndex = ActiveCells[Idx];
				if (CancelF())
				{
					return;
				}

				FGridCell TempCell;
				int TempArray[12];

				// we will process up to MaxNeighboursPerActiveCell new cells in each ParallelFor iteration
				TArray<FVector3i, TInlineAllocator<64>> LocalStack;
				LocalStack.Add(InitialCellIndex);
				int32 CellsProcessed = 0;

				while (LocalStack.Num() > 0 && CellsProcessed++ < MaxNeighboursPerActiveCell)
				{
					FVector3i CellIndex = LocalStack.Pop(EAllowShrinking::No);

					initialize_cell(TempCell, CellIndex);
					if (polygonize_cell(TempCell, TempArray))
					{
						// found crossing
						for (FVector3i GridOffset : IndexUtil::GridOffsets6)
						{
							FVector3i NbrCellIndex = CellIndex + GridOffset;
							if (GridBounds.Contains(NbrCellIndex))
							{
								if (set_cell_if_not_done(NbrCellIndex) == true)
								{ 
									LocalStack.Add(NbrCellIndex);
								}
							}
						}
					}
				}

				// if stack is not empty, ie hit MaxNeighboursPerActiveCell, add remaining cells to next-pass Active list
				if (LocalStack.Num() > 0)
				{
					NewActiveCellsLock.Lock();
					NewActiveCells.Append(LocalStack);
					NewActiveCellsLock.Unlock();
				}

			});

			ActiveCells.Reset();
			if (NewActiveCells.Num() > 0)
			{
				Swap(ActiveCells, NewActiveCells);
			}
		}

		parallel_mesh_access = false;
	}


	FBlockedDenseGrid3i BlockedDoneCells;

	bool set_cell_if_not_done(const FVector3i& Idx)
	{
		bool was_set = false;
		{
			BlockedDoneCells.ProcessValueThreadSafe(Idx.X, Idx.Y, Idx.Z, [&](int& CellValue) 
			{
				if (CellValue == 0)
				{
					CellValue = 1;
					was_set = true;
				}
			});
		}
		return was_set;
	}





	/**
	*  find edge crossings and generate triangles for this cell
	*/
	bool polygonize_cell(FGridCell& Cell, int VertIndexArray[])
	{
		// construct bits of index into edge table, where bit for each
		// corner is 1 if that value is < isovalue.
		// This tell us which edges have sign-crossings, and the int value
		// of the bitmap is an index into the edge and triangle tables
		int cubeindex = 0, Shift = 1;
		for (int i = 0; i < 8; ++i)
		{
			if (Cell.f[i] < IsoValue)
			{
				cubeindex |= Shift;
			}
			Shift <<= 1;
		}

		// no crossings!
		if (EdgeTable[cubeindex] == 0)
		{
			return false;
		}

		// check each bit of value in edge table. If it is 1, we
		// have a crossing on that edge. Look up the indices of this
		// edge and find the intersection point along it
		Shift = 1;
		TVector<double> pa = TVector<double>::Zero(), pb = TVector<double>::Zero();
		for (int i = 0; i <= 11; i++)
		{
			if ((EdgeTable[cubeindex] & Shift) != 0)
			{
				int a = EdgeIndices[i][0], b = EdgeIndices[i][1];
				VertIndexArray[i] = edge_vertex_id(Cell.i[a], Cell.i[b], Cell.f[a], Cell.f[b]);
			}
			Shift <<= 1;
		}

		int64 CellHash = corner_hash(Cell.i[0]);

		// now iterate through the set of triangles in TriTable for this cube,
		// and emit triangles using the vertices we found.
		int tri_count = 0;
		for (uint64 tris = TriTable[cubeindex]; tris != 0; tris >>= 12)
		{
			int ta = int(tris & 0xf);
			int tb = int((tris >> 4) & 0xf);
			int tc = int((tris >> 8) & 0xf);
			int a = VertIndexArray[ta], b = VertIndexArray[tb], c = VertIndexArray[tc];

			// if a corner is within tolerance of isovalue, then some triangles
			// will be degenerate, and we can skip them w/o resulting in cracks (right?)
			// !! this should never happen anymore...artifact of old hashtable impl
			if (!ensure(a != b && a != c && b != c))
			{
				continue;
			}

			append_triangle(a, b, c, CellHash);
			tri_count++;
		}

		return (tri_count > 0);
	}


	struct FIndexedVertex
	{
		int32 Index;
		FVector3d Position;
	};
	std::atomic<int32> VertexCounter;

	int64 NumVertexSections = 64;
	TArray<FCriticalSection> VertexSectionLocks;
	TArray<TArray<FIndexedVertex>> VertexSectionLists;
	int GetVertexSectionIndex(int64 hash)
	{
		return (int32)(hash % (NumVertexSections - 1));
	}

	/**
	*  add vertex to mesh, with locking if we are computing in parallel
	*/
	int append_vertex(TVector<double> V, int64 CellHash)
	{
		int SectionIndex = GetVertexSectionIndex(CellHash);
		int32 NewIndex = VertexCounter++;

		if (parallel_mesh_access)
		{
			FScopeLock Lock(&VertexSectionLocks[SectionIndex]);
			VertexSectionLists[SectionIndex].Add(FIndexedVertex{ NewIndex, V });
		}
		else
		{
			VertexSectionLists[SectionIndex].Add(FIndexedVertex{ NewIndex, V });
		}

		return NewIndex;
	}


	int64 NumTriangleSections = 64;
	TArray<FCriticalSection> TriangleSectionLocks;
	TArray<TArray<FIndex3i>> TriangleSectionLists;
	int GetTriangleSectionIndex(int64 hash)
	{
		return (int32)(hash % (NumTriangleSections - 1));
	}

	/**
	*  add triangle to mesh, with locking if we are computing in parallel
	*/
	void append_triangle(int A, int B, int C, int64 CellHash)
	{
		int SectionIndex = GetTriangleSectionIndex(CellHash);
		if (parallel_mesh_access)
		{
			FScopeLock Lock(&TriangleSectionLocks[SectionIndex]);
			TriangleSectionLists[SectionIndex].Add(FIndex3i(A, B, C));
		}
		else
		{
			TriangleSectionLists[SectionIndex].Add(FIndex3i(A, B, C));
		}
	}


	/**
	 * Reset internal mesh-assembly data structures
	 */
	void ResetMesh()
	{
		VertexSectionLocks.SetNum((int32)NumVertexSections);
		VertexSectionLists.Reset();
		VertexSectionLists.SetNum((int32)NumVertexSections);
		VertexCounter = 0;

		TriangleSectionLocks.SetNum((int32)NumTriangleSections);
		TriangleSectionLists.Reset();
		TriangleSectionLists.SetNum((int32)NumTriangleSections);
	}

	/**
	 * Populate FMeshShapeGenerator data structures from accumulated
	 * vertex/triangle sets
	 */
	void BuildMesh()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_MCMesh_BuildMesh);

		int32 NumVertices = VertexCounter;
		TArray<FVector3d> VertexBuffer;
		VertexBuffer.SetNum(NumVertices);
		for (const TArray<FIndexedVertex>& VertexList : VertexSectionLists)
		{
			for (FIndexedVertex Vtx : VertexList)
			{
				VertexBuffer[Vtx.Index] = Vtx.Position;
			}
		}
		for (int32 k = 0; k < NumVertices; ++k)
		{
			int32 vid = AppendVertex(VertexBuffer[k]);
		}

		for (const TArray<FIndex3i>& TriangleList : TriangleSectionLists)
		{
			for (FIndex3i Tri : TriangleList)
			{
				AppendTriangle(Tri.A, Tri.B, Tri.C);
			}
		}
	}



	/**
	*  root-find the intersection along edge from f(P1)=ValP1 to f(P2)=ValP2
	*/
	void find_iso(const TVector<double>& P1, const TVector<double>& P2, double ValP1, double ValP2, TVector<double>& PIso)
	{
		// Ok, this is a bit hacky but seems to work? If both isovalues
		// are the same, we just return the midpoint. If one is nearly zero, we can
		// but assume that's where the surface is. *However* if we return that point exactly,
		// we can get nonmanifold vertices, because multiple fans may connect there. 
		// Since FDynamicMesh3 disallows that, it results in holes. So we pull 
		// slightly towards the other point along this edge. This means we will get
		// repeated nearly-coincident vertices, but the mesh will be manifold.
		const double dt = 0.999999;
		if (FMath::Abs(ValP1 - ValP2) < 0.00001)
		{
			PIso = (P1 + P2) * 0.5;
			return;
		}
		if (FMath::Abs(IsoValue - ValP1) < 0.00001)
		{
			PIso = dt * P1 + (1.0 - dt) * P2;
			return;
		}
		if (FMath::Abs(IsoValue - ValP2) < 0.00001)
		{
			PIso = (dt) * P2 + (1.0 - dt) * P1;
			return;
		}

		// Note: if we don't maintain min/max order here, then numerical error means
		//   that hashing on point x/y/z doesn't work
		TVector<double> a = P1, b = P2;
		double fa = ValP1, fb = ValP2;
		if (ValP2 < ValP1)
		{
			a = P2; b = P1;
			fb = ValP1; fa = ValP2;
		}

		// converge on root
		if (RootMode == ERootfindingModes::Bisection)
		{
			for (int k = 0; k < RootModeSteps; ++k)
			{
				PIso.X = (a.X + b.X) * 0.5; PIso.Y = (a.Y + b.Y) * 0.5; PIso.Z = (a.Z + b.Z) * 0.5;
				double mid_f = Implicit(PIso);
				if (mid_f < IsoValue)
				{
					a = PIso; fa = mid_f;
				}
				else
				{
					b = PIso; fb = mid_f;
				}
			}
			PIso = Lerp(a, b, 0.5);

		}
		else
		{
			double mu = 0;
			if (RootMode == ERootfindingModes::LerpSteps)
			{
				for (int k = 0; k < RootModeSteps; ++k)
				{
					mu = FMathd::Clamp((IsoValue - fa) / (fb - fa), 0.0, 1.0);
					PIso.X = a.X + mu * (b.X - a.X);
					PIso.Y = a.Y + mu * (b.Y - a.Y);
					PIso.Z = a.Z + mu * (b.Z - a.Z);
					double mid_f = Implicit(PIso);
					if (mid_f < IsoValue)
					{
						a = PIso; fa = mid_f;
					}
					else
					{
						b = PIso; fb = mid_f;
					}
				}
			}

			// final lerp
			mu = FMathd::Clamp((IsoValue - fa) / (fb - fa), 0.0, 1.0);
			PIso.X = a.X + mu * (b.X - a.X);
			PIso.Y = a.Y + mu * (b.Y - a.Y);
			PIso.Z = a.Z + mu * (b.Z - a.Z);
		}
	}




	/*
	* Below here are standard marching-cubes tables. 
	*/

	static GEOMETRYCORE_API const uint8 EdgeIndices[12][2];
	static GEOMETRYCORE_API const uint16 EdgeTable[256];
	static GEOMETRYCORE_API const uint64 TriTable[256];
};


} // end namespace UE::Geometry
} // end namespace UE
