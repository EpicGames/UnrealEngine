// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "IndexTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * Base class for triangle mesh generators (eg like to generate sphere, cylinder, etc)
 * Subclasses must implement ::Generate() 
 */
class /*GEOMETRYCORE_API*/ FMeshShapeGenerator
{
public:
	/** Array of vertex positions */
	TArray<FVector3d> Vertices;
	
	/** Array of UV positions. These are completely independent of vertex list (ie not per-vertex unless that's what generator produces) */
	TArray<FVector2f> UVs;
	/** Parent vertex index for each UV. Same length as UVs array. */
	TArray<int> UVParentVertex;

	/** Array of Normals. These are completely independent of vertex list (ie not per-vertex unless that's what generator produces) */
	TArray<FVector3f> Normals;
	/** Parent vertex index for each Normal. Same length as Normals array. */
	TArray<int> NormalParentVertex;

	/** Array of triangle corner positions, stored as tuples of indices into Vertices array */
	TArray<FIndex3i> Triangles;
	/** Array of triangle corner UVs, stored as tuples of indices into UVs array. Same length as Triangles array. */
	TArray<FIndex3i> TriangleUVs;
	/** Array of triangle corner Normals, stored as tuples of indices into Normals array. Same length as Triangles array. */
	TArray<FIndex3i> TriangleNormals;
	/** Array of per-triangle integer polygon IDs. Same length as Triangles array. */
	TArray<int> TrianglePolygonIDs;


public:
	/** If true, reverse orientation of created mesh */
	bool bReverseOrientation = false;

	virtual ~FMeshShapeGenerator() = default;


	/**
	 * Subclasses implement this to generate mesh
	 */
	virtual FMeshShapeGenerator& Generate() = 0;

	bool HasAttributes() const
	{
		return UVs.Num() > 0 || Normals.Num() > 0;
	}

	// Reset UV and normal-related on a generator, leaving vertices and triangles alone
	// @param bResetPolygonIDs Whether to also reset polygon IDs
	void ResetAttributes(bool bResetPolygonIDs = false)
	{
		UVs.Reset();
		UVParentVertex.Reset();
		Normals.Reset();
		NormalParentVertex.Reset();
		TriangleUVs.Reset();
		TriangleNormals.Reset();
		if (bResetPolygonIDs)
		{
			TrianglePolygonIDs.Reset();
		}
	}

	/** clear arrays so that Generate() can be run again */
	void Reset()
	{
		Vertices.Reset();
		UVs.Reset();
		UVParentVertex.Reset();
		Normals.Reset();
		NormalParentVertex.Reset();
		Triangles.Reset();
		TriangleUVs.Reset();
		TriangleNormals.Reset();
		TrianglePolygonIDs.Reset();
	}


	/**
	 * Set the various internal buffers to the correct sizes for the given element counts
	 */
	void SetBufferSizes(int NumVertices, int NumTriangles, int NumUVs, int NumNormals)
	{
		if (NumVertices > 0)
		{
			Vertices.SetNum(NumVertices);
		}
		if (NumTriangles > 0)
		{
			Triangles.SetNum(NumTriangles);
			TriangleUVs.SetNum(NumTriangles);
			TriangleNormals.SetNum(NumTriangles);
			TrianglePolygonIDs.SetNum(NumTriangles);
		}
		if (NumUVs > 0)
		{
			UVs.SetNum(NumUVs);
			UVParentVertex.SetNum(NumUVs);
		}
		if (NumNormals > 0)
		{
			Normals.SetNum(NumNormals);
			NormalParentVertex.SetNum(NumNormals);
		}
	}
	/**
	 * Extends the various internal buffers to the correct sizes for the given additional element counts
	 */
	void ExtendBufferSizes(int AddVertices, int AddTriangles, int AddUVs, int AddNormals)
	{
		if (AddVertices > 0)
		{
			int NumVertices = Vertices.Num() + AddVertices;
			Vertices.SetNum(NumVertices);
		}
		if (AddTriangles > 0)
		{
			int NumTriangles = Triangles.Num() + AddTriangles;
			Triangles.SetNum(NumTriangles);
			TriangleUVs.SetNum(NumTriangles);
			TriangleNormals.SetNum(NumTriangles);
			TrianglePolygonIDs.SetNum(NumTriangles);
		}
		if (AddUVs > 0)
		{
			int NumUVs = UVs.Num() + AddUVs;
			UVs.SetNum(NumUVs);
			UVParentVertex.SetNum(NumUVs);
		}
		if (AddNormals > 0)
		{
			int NumNormals = Normals.Num() + AddNormals;
			Normals.SetNum(NumNormals);
			NormalParentVertex.SetNum(NumNormals);
		}
	}


	/** Set vertex at Index to given Position */
	inline void SetVertex(int Index, const FVector3d& Position)
	{
		Vertices[Index] = Position;
	}
	/** Append a new vertex at the given Position */
	inline int AppendVertex(const FVector3d& Position)
	{
		Vertices.Add(Position);
		return Vertices.Num()-1;
	}

	/** Set UV at Index to given value with given ParentVertex */
	inline void SetUV(int Index, const FVector2f& UV, int ParentVertex)
	{
		UVs[Index] = UV;
		UVParentVertex[Index] = ParentVertex;
	}
	inline int AppendUV(const FVector2f& UV, int ParentVertex)
	{
		UVs.Add(UV);
		UVParentVertex.Add(ParentVertex);
		return UVs.Num() - 1;
	}

	/** Set Normal at Index to given value with given ParentVertex */
	inline void SetNormal(int Index, const FVector3f& Normal, int ParentVertex)
	{
		Normals[Index] = Normal;
		NormalParentVertex[Index] = ParentVertex;
	}
	inline int AppendNormal(const FVector3f& Normal, int ParentVertex)
	{
		Normals.Add(Normal);
		NormalParentVertex.Add(ParentVertex);
		return Normals.Num() - 1;
	}


	inline void SetTriangle(int Index, const FIndex3i& Tri)
	{
		Triangles[Index] = bReverseOrientation ? FIndex3i(Tri.C, Tri.B, Tri.A) : Tri;
	}
	inline void SetTriangle(int Index, int A, int B, int C)
	{
		Triangles[Index] = bReverseOrientation ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}
	inline void SetTriangle(int Index, int A, int B, int C, bool bClockwiseOverride)
	{
		Triangles[Index] = (bReverseOrientation != bClockwiseOverride) ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}

	
	inline int AppendTriangle(int A, int B, int C)
	{
		Triangles.Add( bReverseOrientation ? FIndex3i(C, B, A) : FIndex3i(A, B, C) );
		TriangleUVs.SetNum(Triangles.Num());
		TriangleNormals.SetNum(Triangles.Num());
		TrianglePolygonIDs.SetNum(Triangles.Num());
		return Triangles.Num() - 1;
	}


	inline void SetTriangleUVs(int Index, const FIndex3i& Tri)
	{
		TriangleUVs[Index] = bReverseOrientation ? FIndex3i(Tri.C, Tri.B, Tri.A) : Tri;
	}
	inline void SetTriangleUVs(int Index, int A, int B, int C)
	{
		TriangleUVs[Index] = bReverseOrientation ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}
	inline void SetTriangleUVs(int Index, int A, int B, int C, bool bClockwiseOverride)
	{
		TriangleUVs[Index] = (bReverseOrientation != bClockwiseOverride) ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}


	inline void SetTriangleNormals(int Index, const FIndex3i& Tri)
	{
		TriangleNormals[Index] = bReverseOrientation ? FIndex3i(Tri.C, Tri.B, Tri.A) : Tri;
	}
	inline void SetTriangleNormals(int Index, int A, int B, int C)
	{
		TriangleNormals[Index] = bReverseOrientation ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}
	inline void SetTriangleNormals(int Index, int A, int B, int C, bool bClockwiseOverride)
	{
		TriangleNormals[Index] = (bReverseOrientation != bClockwiseOverride) ? FIndex3i(C, B, A) : FIndex3i(A, B, C);
	}

	inline void SetTrianglePolygon(int Index, int PolygonID)
	{
		TrianglePolygonIDs[Index] = PolygonID;
	}

	/**
	 * Set triangle and UVs and normals with matching indices
	 * Convenience function for shapes with no uv or normal seams
	 */
	inline void SetTriangleWithMatchedUVNormal(int Index, int A, int B, int C)
	{
		SetTriangle(Index, A, B, C);
		SetTriangleNormals(Index, A, B, C);
		SetTriangleUVs(Index, A, B, C);
	}


	static FVector3d BilinearInterp(const FVector3d &v00, const FVector3d &v10, const FVector3d &v11, const FVector3d &v01, double tx, double ty)
	{
		FVector3d a = Lerp(v00, v01, ty);
		FVector3d b = Lerp(v10, v11, ty);
		return Lerp(a, b, tx);
	}

	static FVector2d BilinearInterp(const FVector2d &v00, const FVector2d &v10, const FVector2d &v11, const FVector2d &v01, double tx, double ty)
	{
		FVector2d a = Lerp(v00, v01, ty);
		FVector2d b = Lerp(v10, v11, ty);
		return Lerp(a, b, tx);
	}

	static FVector2f BilinearInterp(const FVector2f &v00, const FVector2f &v10, const FVector2f &v11, const FVector2f &v01, float tx, float ty)
	{
		FVector2f a = Lerp(v00, v01, ty);
		FVector2f b = Lerp(v10, v11, ty);
		return Lerp(a, b, tx);
	}


	static FVector3i LinearInterp(const FVector3i &a, const FVector3i &b, double t)
	{
		FVector3d c = Lerp((FVector3d)a, (FVector3d)b, t);
		return FVector3i((int)round(c.X), (int)round(c.Y), (int)round(c.Z));
	}

};

} // end namespace UE::Geometry
} // end namespace UE
