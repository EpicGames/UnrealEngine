// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{

/**
 * Generate an oriented Box mesh with the smallest number of triangles possible (6 vertices, 12 triangles)
 */
class /*GEOMETRYCORE_API*/ FMinimalBoxMeshGenerator : public FMeshShapeGenerator
{
public:
	/** 3D box */
	FOrientedBox3d Box;

	/** If true (default), UVs are scaled so that there is no stretching. If false, UVs are scaled to fill unit square */
	bool bScaleUVByAspectRatio = true;

public:

	/** Generate the mesh */
	virtual FMeshShapeGenerator& Generate() override
	{
		// box has 8 corners, 6 quad-faces, each face has 2 triangles and 4 attribute-corners
		SetBufferSizes(8, 12, 24, 24);


		for (int i = 0; i < 8; ++i) 
		{
			Vertices[i] = Box.GetCorner(i);
		}

		double MaxDimension = MaxAbsElement(2.0*Box.Extents);
		float UVScale = (bScaleUVByAspectRatio) ? (1.0f / (float)MaxDimension) : 1.0f;

		int TriIndex = 0;
		int AttribIndex = 0;
		for (int fi = 0; fi < 6; ++fi) 
		{		
			int NormalAxisIdx = IndexUtil::BoxFaceNormals[fi];
			FVector3f FaceNormal = (FVector3f)((double)FMath::Sign(NormalAxisIdx) * Box.GetAxis(FMath::Abs(NormalAxisIdx) - 1));

			double FaceWidth = Distance(Vertices[IndexUtil::BoxFaces[fi][0]], Vertices[IndexUtil::BoxFaces[fi][1]]);
			double FaceHeight = Distance(Vertices[IndexUtil::BoxFaces[fi][1]], Vertices[IndexUtil::BoxFaces[fi][2]]);
			double WidthUVScale = FaceWidth * UVScale;
			double HeightUVScale = FaceHeight * UVScale;

			int ElementIndices[4];

			for (int j = 0; j < 4; ++j)
			{
				Normals[AttribIndex] = FaceNormal;
				FVector2i CornerUV = IndexUtil::BoxFacesUV[j];
				UVs[AttribIndex] = FVector2f(float(WidthUVScale*CornerUV.X), float(HeightUVScale*CornerUV.Y));
				ElementIndices[j] = AttribIndex;
				AttribIndex++;
			}


			SetTriangle(TriIndex,
				IndexUtil::BoxFaces[fi][0], IndexUtil::BoxFaces[fi][1], IndexUtil::BoxFaces[fi][2]);
			SetTrianglePolygon(TriIndex, fi);
			SetTriangleUVs(TriIndex, ElementIndices[0], ElementIndices[1], ElementIndices[2]);
			SetTriangleNormals(TriIndex, ElementIndices[0], ElementIndices[1], ElementIndices[2]);
			TriIndex++;

			SetTriangle(TriIndex,
				IndexUtil::BoxFaces[fi][0], IndexUtil::BoxFaces[fi][2], IndexUtil::BoxFaces[fi][3]);
			SetTrianglePolygon(TriIndex, fi);
			SetTriangleUVs(TriIndex, ElementIndices[0], ElementIndices[2], ElementIndices[3]);
			SetTriangleNormals(TriIndex, ElementIndices[0], ElementIndices[2], ElementIndices[3]);
			TriIndex++;


		}

		return *this;
	}

};


} // end namespace UE::Geometry
} // end namespace UE
