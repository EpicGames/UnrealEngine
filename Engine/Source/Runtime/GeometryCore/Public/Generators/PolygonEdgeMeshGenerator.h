// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "FrameTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "MeshShapeGenerator.h"

namespace UE
{
namespace Geometry
{

/** Mesh generator that generates a quad for each edge of a closed polygon */
class FPolygonEdgeMeshGenerator : public FMeshShapeGenerator
{

private:

	// Polygon to thicken and triangulate. Assumed to be closed (i.e. last edge is (LastVertex, FirstVertex)). If Polygon has 
	// self-intersections or degenerate edges, result is undefined.
	TArray<FFrame3d> Polygon;

	// True if Polygon represents a closed path
	bool bClosed;

	// For each polygon vertex, a scale factor for the patch width at that vertex. It's a variable offset in order to keep the overall 
	// width constant going around acute corners. 
	TArray<double> OffsetScaleFactors;

	// Width of quads to generate.
	double Width = 1.0;

	// Normal vector of all vertices will be set to this value. Default is +Z axis.
	FVector3d Normal = FVector3d::UnitZ();

public:

	double UVWidth = 1.0;
	double UVHeight = 1.0;
	bool bScaleUVByAspectRatio = true;

	/** If true, output mesh has a single polygroup, otherwise each quad gets a separate group */
	bool bSinglePolyGroup = false;

	/** Use arc segments instead of straight lines in corners */
	bool bRoundedCorners = false;

	/** Radius of the corner arcs; this is only available if Rounded Corners is enabled */
	double CornerRadius = 0.0;

	/** Limit corner radius to prevent overlapping */
	bool bLimitCornerRadius = false;

	/** Number of radial subdivisions for rounded corners */
	int NumArcVertices;

	GEOMETRYCORE_API FPolygonEdgeMeshGenerator(const TArray<FFrame3d>& InPolygon,
		bool bInClosed,
		const TArray<double>& InOffsetScaleFactors,
		double InWidth = 1.0,
		FVector3d InNormal = FVector3d::UnitZ(),
		bool bInRoundedCorners = false,
		double InCornerRadius = 0.0,
		bool bInLimitCornerRadius = false,
		int InNumArcVertices = 3);

	GEOMETRYCORE_API ~FPolygonEdgeMeshGenerator();

	// Generate triangulation
	// TODO: Enable more subdivisions along the width and length dimensions if requested
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() final;

private:

	void CurvePath(const TArray<FVector3d>& InPath,
		const TArray<bool>& InteriorAngleFlag,
		const TArray<double>& MaxCornerRadii,
		const TArray<double>& OtherSideMaxCornerRadii,
		const FFrame3d& PolygonFrame,
		TArray<FVector3d>& OutPath) const;

};


} // end namespace UE::Geometry
} // end namespace UE
