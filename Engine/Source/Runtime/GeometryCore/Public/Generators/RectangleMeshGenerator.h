// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "MeshShapeGenerator.h"
#include "Misc/EnumClassFlags.h"

namespace UE
{
namespace Geometry
{

/**
 * Generate planar rectangular mesh with variable number of subdivisions along width and height.
 * By default, center of rectangle is centered at (0,0,0) origin
 */
class FRectangleMeshGenerator : public FMeshShapeGenerator
{
public:
	/** Rectangle will be translated so that center is at this point */
	FVector3d Origin;
	/** Normal vector of all vertices will be set to this value. Default is +Z axis. */
	FVector3f Normal;

	/** Width of rectangle */
	double Width;
	/** Height of rectangle */
	double Height;

	/** Number of vertices along Width axis */
	int WidthVertexCount;
	/** Number of vertices along Height axis */
	int HeightVertexCount;

	/** If true (default), UVs are scaled so that there is no stretching. If false, UVs are scaled to fill unit square */
	bool bScaleUVByAspectRatio = true;

	/** If true, output mesh has a single polygroup, otherwise each quad gets a separate group */
	bool bSinglePolyGroup = false;

	/** Specifies how 2D indices are mapped to 3D points. Default is (0,1) = (x,y,0). */ 
	FIndex2i IndicesMap;

public:
	GEOMETRYCORE_API FRectangleMeshGenerator();

	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;

	/** Create vertex at position under IndicesMap, shifted to Origin*/
	virtual FVector3d MakeVertex(int CornerIndex, double x, double y)
	{
		FVector3d v(0, 0, 0);
		v[IndicesMap.A] = x;
		v[IndicesMap.B] = y;
		return Origin + v;
	}

};

enum class ERoundedRectangleCorner : uint8
{
	None = 0,
	BottomLeft = 1,
	BottomRight = 2,
	TopLeft = 4,
	TopRight = 8,
	All = 15
};
ENUM_CLASS_FLAGS(ERoundedRectangleCorner);

/**
* Adds rounded corners to the rectangle mesh
*/
class FRoundedRectangleMeshGenerator : public FRectangleMeshGenerator
{
public:
	/** Radius of rounded corners */
	double Radius;

	/** Number of samples to put per rounded corner */
	int AngleSamples;


	ERoundedRectangleCorner SharpCorners;

	static inline bool SideInCorners(int SideX, int SideY, ERoundedRectangleCorner Corners)
	{
		int SideInCorners = int(Corners)&((SideY ? 4 : 1) * (SideX ? 2 : 1));
		return SideInCorners != 0;
	}

	static inline int NumSharpCorners(ERoundedRectangleCorner SharpFlags)
	{
		int NumSharp = 0;
		int FlagsValue = (int)SharpFlags;
		while (FlagsValue)
		{
			FlagsValue &= (FlagsValue - 1);
			NumSharp++;
		}
		return NumSharp;
	}

public:
	GEOMETRYCORE_API FRoundedRectangleMeshGenerator();

	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;
};


} // end namespace UE::Geometry
} // end namespace UE
