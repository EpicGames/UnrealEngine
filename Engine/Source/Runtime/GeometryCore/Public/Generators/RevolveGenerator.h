// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Generators/SweepGenerator.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "MeshShapeGenerator.h"

namespace UE
{
namespace Geometry
{


/**
 * FBaseRevolveGenerator is a base class for simple surface-of-revolution generators.
 * Internally these are implemented via FSweepGenerator, but the Revolve generators provide
 * simpler-to-use interfaces and handle some other aspects like endcapping/etc.
 * 
 * Revolve generators currently depend on the ability to use DynamicMesh operations to (eg) triangulate endcaps/etc,
 * and so they are not currently FMeshShapeGenerators (this may be resolved in future)
 */
class FBaseRevolveGenerator
{
public:

	enum class ECapFillMode
	{
		None,
		EarClipping,
	};

	/** Number of discrete steps used to sample the revolution path */
	int32 Steps = 16;

	/** Degrees of the revolution path (eg circle for a regular revolution). Maybe clamped in some cases. */
	float RevolveDegrees = 360.0f;

	/** offset for revolution degrees, eg to shift partial revolution around the revolve circle */
	float DegreeOffset = 0.0f;

	/** Reverse the direction of the revolution, for non-closed revolutions */
	bool bReverseDirection = false;

	/**  */
	bool bProfileAtMidpoint = false;

	/** Whether or not to fill the open ends of a partial revolve, if possible */
	bool bFillPartialRevolveEndcaps = true;

	// Generated UV coordinates will be multiplied by these values.
	FVector2d UVScale = FVector2d(1,1);

	// These values will be added to the generated UV coordinates after applying UVScale.
	FVector2d UVOffset = FVector2d(0, 0);

	// When true, the generator attempts to scale UV's in a way that preserves scaling across different mesh
	// results, aiming for 1.0 in UV space to be equal to UnitUVInWorldCoordinates in world space. This is 
	// generally speaking unrealistic because UV's are going to be variably stretched no matter what, but 
	// in practice it means adjusting the V scale relative to the profile curve length and U scale relative
	// to a very crude measurement of movement across sweep frames.
	bool bUVScaleRelativeWorld = false;

	// Only relevant if bUVScaleRelativeWorld is true (see that description)
	float UnitUVInWorldCoordinates = 100;

	/** QuadSplitMethod controls how quads are split into triangles (see enum comments) */
	EProfileSweepQuadSplit QuadSplitMethod = EProfileSweepQuadSplit::Uniform;

	// When QuadSplitMode is ShortestDiagonal, biases one of the diagonals so that symmetric
	// quads are split uniformly. The tolerance is a proportion allowable difference.
	double DiagonalTolerance = 0.01;

	/** PolygonGroupingMode controls how polygroups are assigned on the output mesh */
	EProfileSweepPolygonGrouping PolygonGroupingMode = EProfileSweepPolygonGrouping::PerFace;


};



/**
 * Revolve a planar polyline (in the XY plane) around the +Z axis along a circular path
 * (+X, +Y) in the 2D polyline is mapped to (+X, 0, +Z)
 */
class FRevolvePlanarPathGenerator : public FBaseRevolveGenerator
{
public:
	/** Vertices of the planar polyline/path that will be revolved */
	TArray<FVector2d> PathVertices;

	/** If true, open tops/bottoms of the revolved shape are capped */
	bool bCapped = true;

	/** Generate the mesh */
	GEOMETRYCORE_API FDynamicMesh3 GenerateMesh();

};

/**
* Revolve a planar polygon (in the XY plane) around the +Z axis along a circular path
* (+X, +Y) in the 2D polygon is mapped to (+X, 0, +Z)
*/
class FRevolvePlanarPolygonGenerator : public FBaseRevolveGenerator
{
public:
	/** Vertices of the planar polygon that will be revolved */
	TArray<FVector2d> PolygonVertices;

	/** If true, vertices that lie on the +Z axis will be welded */
	bool bWeldVertsOnAxis = false;

	/** Generate the mesh */
	GEOMETRYCORE_API FDynamicMesh3 GenerateMesh();
};


/**
* Revolve a planar polygon (in the XY plane) around the +Z axis along a spiral path.
* (+X, +Y) in the 2D polygon is mapped to (+X, 0, +Z)
*/
class FSpiralRevolvePlanarPolygonGenerator : public FBaseRevolveGenerator
{
public:
	/** Vertices of the planar polygon that will be revolved */
	TArray<FVector2d> PolygonVertices;

	/** If true, vertices that lie on the +Z axis will be welded */
	bool bWeldVertsOnAxis = false;

	/** Vertical rise of the spiral path per full 360-degree revolution */
	double RisePerFullRevolution = 50;

	/** Generate the mesh */
	GEOMETRYCORE_API FDynamicMesh3 GenerateMesh();
};





} // end namespace UE::Geometry
}

