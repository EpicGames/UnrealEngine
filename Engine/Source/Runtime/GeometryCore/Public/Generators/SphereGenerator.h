// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshShapeGenerator.h"
#include "OrientedBoxTypes.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{

/**
 * Generate a sphere mesh, with UVs wrapped cylindrically
 */
class /*GEOMETRYCORE_API*/ FSphereGenerator : public FMeshShapeGenerator
{
public:
	/** Radius */
	double Radius = 1;

	int NumPhi = 3; // number of vertices along vertical extent from north pole to south pole
	int NumTheta = 3; // number of vertices around circles

	/** If true, each quad of box gets a separate polygroup */
	bool bPolygroupPerQuad = false;

private:
	static FVector3d SphericalToCartesian(double r, double theta, double phi)
	{
		double Sphi = sin(phi);
		double Cphi = cos(phi);
		double Ctheta = cos(theta);
		double Stheta = sin(theta);

		return FVector3d(r * Ctheta * Sphi, r * Stheta * Sphi, r * Cphi);
	}

	void GenerateVertices()
	{
		auto SetVertex = [this](int32 VtxIdx,
								FVector3d Pos, FVector3f Normal )
		{
			Vertices[VtxIdx] = Pos;
			Normals[VtxIdx] = Normal;
			NormalParentVertex[VtxIdx] = VtxIdx;
		};
		{
			const double Dphi = FMathd::Pi / double(NumPhi - 1);
			const double Dtheta = FMathd::TwoPi / double(NumTheta);

			int32 p,t;
			double Phi, Theta;
			int32 VtxIdx = 0;

			// Add points between the poles
			for (p = 1, Phi = Dphi; p < NumPhi - 1; ++p, Phi += Dphi) // NB: this skips the poles.
			{
				for (t = 0, Theta = 0; t < NumTheta; ++t, ++VtxIdx, Theta += Dtheta)
				{
					FVector3d Normal = SphericalToCartesian(1., Theta, Phi);
					SetVertex(VtxIdx, Normal * Radius, FVector3f(Normal));
				}
			}
			// add a single point at the North Pole
			SetVertex(VtxIdx++, FVector3d::UnitZ() * Radius, FVector3f::UnitZ());
			// add a single point at the South Pole
			SetVertex(VtxIdx++, -FVector3d::UnitZ() * Radius, -FVector3f::UnitZ());
		}
	}

	void GenerateUVVertices()
	{
		// generate the UV's
		const float DUVphi = 1.f / float(NumPhi - 1);
		const float DUVtheta = -1.f / float(NumTheta);

		int32 UVIdx = 0;
		int32 p,t;
		float UVPhi, UVTheta;
		for ( p = 1, UVPhi = DUVphi; p < NumPhi - 1; ++p, UVPhi += DUVphi)
		{
			for (t = 0, UVTheta = 1; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
			{
				UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
				UVParentVertex[UVIdx] = (p - 1) * NumTheta + t;
			}
			UVs[UVIdx] = FVector2f(UVTheta, UVPhi);
			UVParentVertex[UVIdx] = (p - 1) * NumTheta; // Wrap around
			++UVIdx;
		}
		int32 NorthPoleVtxIdx = (NumPhi - 2) * NumTheta;
		for (t = 0, UVTheta = 1 + DUVtheta; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
		{
			UVs[UVIdx] = FVector2f(UVTheta, 0.0);
			UVParentVertex[UVIdx] = NorthPoleVtxIdx;
		}
		int32 SouthPoleVtxIdx = NorthPoleVtxIdx + 1;
		for (t = 0, UVTheta = 1 + DUVtheta; t < NumTheta; ++t, ++UVIdx, UVTheta += DUVtheta)
		{
			UVs[UVIdx] = FVector2f(UVTheta, 1.0);
			UVParentVertex[UVIdx] = SouthPoleVtxIdx;
		}
	}

	using CornerIndices =  FVector3i;
	void OutputTriangle(int TriIdx, int PolyIdx,  CornerIndices Corners, CornerIndices UVCorners)
	{
		SetTriangle(TriIdx, Corners.X, Corners.Y, Corners.Z);
		SetTrianglePolygon(TriIdx, PolyIdx);
		SetTriangleUVs(TriIdx, UVCorners.X, UVCorners.Y, UVCorners.Z);
		SetTriangleNormals(TriIdx, Corners.X, Corners.Y, Corners.Z);
	}

	void OutputEquatorialTriangles()
	{
		int32 TriIdx = 0, PolyIdx = 0;

		// Generate equatorial triangles
		int32 Corners[4] =   { 0, 1,     NumTheta + 1, NumTheta};
		int32 UVCorners[4] = { 0, 1, NumTheta + 2, NumTheta + 1};
		for (int32 p = 1; p < NumPhi - 2; ++p)
		{
			for (int32 t = 0; t < NumTheta - 1; ++t)
			{
				// convert each quad into 2 triangles.
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[0],   Corners[1],   Corners[2]},
							   {UVCorners[0], UVCorners[1], UVCorners[2]});
				OutputTriangle(TriIdx++, PolyIdx,
							   {Corners[2],   Corners[3],   Corners[0]},
							   {UVCorners[2], UVCorners[3], UVCorners[0]});
				for (auto& i : Corners) ++i; 
				for (auto& i : UVCorners) ++i;
				if (bPolygroupPerQuad)
				{
					PolyIdx++;
				}
			}
			OutputTriangle(TriIdx++, PolyIdx,
						   {Corners[0], Corners[1] - NumTheta, Corners[2] - NumTheta},
						   {UVCorners[0]         , UVCorners[1],            UVCorners[2] });
			OutputTriangle(TriIdx++, PolyIdx,
						   {Corners[2] - NumTheta, Corners[3],   Corners[0]},
						   {UVCorners[2],          UVCorners[3],            UVCorners[0]});
			for (auto& i : Corners) ++i; 
			for (auto& i : UVCorners) i += 2;
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}

	void OutputPolarTriangles()
	{
		const int32 NumEquatorialVtx = (NumPhi - 2) * NumTheta;
		const int32 NumEquatorialUVVtx = (NumPhi - 2) * (NumTheta + 1);
		const int32 NorthPoleVtxIdx = NumEquatorialVtx;
		const int32 SouthPoleVtxIdx = NumEquatorialVtx + 1;
		int32 PolyIdx = NumTheta  * (NumPhi - 3);
		int32 TriIdx = PolyIdx * 2;
		if (bPolygroupPerQuad == false)
		{
			PolyIdx = 0;
		}

		// Triangles that connect to north pole
		for (int32 t = 0; t < NumTheta; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t, NorthPoleVtxIdx,       (t + 1) % NumTheta},
						   {t, NumEquatorialUVVtx + t, t + 1});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}

		// Triangles that connect to South pole
		const int32 Offset   = NumEquatorialVtx - NumTheta;
		const int32 OffsetUV = NumEquatorialUVVtx - (NumTheta + 1);
		for (int32 t = 0; t < NumTheta; ++t)
		{
			OutputTriangle(TriIdx++, PolyIdx,
						   {t + Offset,   ((t + 1) % NumTheta) + Offset, SouthPoleVtxIdx},
						   {t + OffsetUV, t + 1 + OffsetUV             , NumEquatorialUVVtx + NumTheta + t});
			if (bPolygroupPerQuad)
			{
				PolyIdx++;
			}
		}
	}
public:
	/** Generate the mesh */
	FMeshShapeGenerator& Generate() override
	{
		// enforce sane values for vertex counts
		NumPhi = FMath::Max(NumPhi, 3);
		NumTheta = FMath::Max(NumTheta, 3);
		const int32 NumVertices = (NumPhi - 2) * NumTheta + 2;
		const int32 NumUVs = (NumPhi - 2) * (NumTheta + 1) + (2 * NumTheta);
		const int32 NumTris = (NumPhi - 2) * NumTheta * 2;
		SetBufferSizes(NumVertices, NumTris, NumUVs, NumVertices);

		GenerateVertices();
		GenerateUVVertices();
		OutputEquatorialTriangles();
		OutputPolarTriangles();
		return *this;
	}


};


} // end namespace UE::Geometry
} // end namespace UE