// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Generators/MeshShapeGenerator.h"
#include "IndexTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"


namespace UE
{
namespace Geometry
{


/**
 * Base Stair mesh generator class.
 */
class FStairGenerator : public FMeshShapeGenerator
{
public:
	/** If true (default), UVs are scaled so that there is no stretching. If false, UVs are scaled to fill unit square */
	bool bScaleUVByAspectRatio = true;

	/** If true, each quad of box gets a separate polygroup */
	bool bPolygroupPerQuad = false;

	/** The width of each step. */
	float StepWidth = 150.0f;

	/** The height of each step. */
	float StepHeight = 20.0f;

	/** The number of steps in this staircase. */
	int NumSteps = 8;

	enum class ESide
	{
		Right,	// +Y
		Left,	// -Y
		Front,	// -X
		Top,	// +Z
		Back,	// +X
		Bottom	// -Z
	};

public:
	GEOMETRYCORE_API virtual ~FStairGenerator() override;

	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;

private:
	/** Generate solid style stairs */
	GEOMETRYCORE_API FMeshShapeGenerator& GenerateSolidStairs();

	/** Generate floating style stairs */
	GEOMETRYCORE_API FMeshShapeGenerator& GenerateFloatingStairs();

	/**
	 * Helper to identify the side based on the face index.
	 *
	 * This method is only valid after the face description array has been
	 * initialized.
	 */
	GEOMETRYCORE_API ESide FaceToSide(int FaceId);

protected:
	enum class EStairStyle
	{
		Solid,		// Each step is supported by the floor.
		Floating	// Each step only supported by previous step.
	};
	/** The style of the stairs */
	EStairStyle StairStyle = EStairStyle::Solid;

protected:
	TArray<TArray<int>> VertexIds;
	TArray<FIndex2i> VertexIdsToColumnRow;
	TArray<int> FaceDesc;
	TArray<int> NormalDesc;
	TArray<int> UVDesc;

	/** Mesh counts. */
	int NumQuadsPerSide = 0;
	int NumQuads = 0;
	int NumVertsPerSide = 0;
	int NumVerts = 0;
	int NumAttrs = 0;
	
	/** Vertex column indices. */
	int RightSideColumnId = 0;
	int LeftSideColumnId = 0;

	/** Face descriptor indices. */
	int RightStartFaceId = 0;
	int LeftStartFaceId = 0;
	int FrontStartFaceId = 0;
	int TopStartFaceId = 0;
	int BackStartFaceId = 0;
	int BottomStartFaceId = 0;
	int LastFaceId = 0;

protected:
	/** Reset state data on the generator */
	GEOMETRYCORE_API virtual void ResetData();

	/**
	 * Returns a vertex position.
	 *
	 * The method is provided the Right or Left side and the
	 * corresponding vertex column/row index to compute. Column
	 * and row indices refer to the matrix-like ordered vertex layout.
	 * See diagram in GenerateSolidStairs() / GenerateFloatingStairs().
	 *
	 * This generator only generates vertices for the Right & Left
	 * Sides of the stairs.
	 *
	 * Mesh count protected variables are the only transient variables
	 * guaranteed to be valid at the time GenerateVertex is invoked.
	 *
	 * @param Side The Right or Left side of the stairs.
	 * @param VertexColumn The column index into the stair vertex layout.
	 * @param VertexRow The row index into the stair vertex layout.
	 */
	virtual FVector3d GenerateVertex(ESide Side, int VertexColumn, int VertexRow) = 0;

	/**
	 * Returns a vertex normal vector.
	 * 
	 * All normals for a given side are shared except for Front & Top.
	 * Border vertex normals per side are not shared.
	 *
	 * @param Side The side of the stairs to compute the normal.
	 * @param VertexId The vertex index to compute the normal.
	 */
	virtual FVector3f GenerateNormal(ESide Side, int VertexId) = 0;
	
	/**
	 * Returns a UV vector.
	 * 
	 * The Step parameter provides the Side-relative face. This
	 * indicates which face for a given side is computing its
	 * UV.
	 *
	 * All UVs for a given side are shared except for Front & Top.
	 * Border edges along each side are UV island edges.
	 *
	 * @param Side The side of the stairs to compute the UV.
	 * @param Step The Side-relative step face ID.
	 * @param VertexId The vertex index to compute the UV.
	 * @param UVScale The UV scale
	 */
	virtual FVector2f GenerateUV(ESide Side, int Step, int VertexId, float UVScale) = 0;

	/**
	 * Returns the max dimension of the staircase for the purposes
	 * of computing the world UV scale.
	 */
	virtual float GetMaxDimension() = 0;
};

/**
 * Generate an oriented Linear Stair mesh.
 */
class FLinearStairGenerator : public FStairGenerator
{
public:
	/** The depth of each step. */
	float StepDepth = 30.0f;

public:
	FLinearStairGenerator()
	{
		StairStyle = EStairStyle::Solid;
	}
	virtual ~FLinearStairGenerator() override = default;

protected:
	typedef FStairGenerator Super;

protected:
	GEOMETRYCORE_API virtual FVector3d GenerateVertex(ESide Side, int VertexColumn, int VertexRow) override;
	GEOMETRYCORE_API virtual FVector3f GenerateNormal(ESide Side, int VertexId) override;
	GEOMETRYCORE_API virtual FVector2f GenerateUV(ESide Side, int Step, int VertexId, float UVScale) override;
	GEOMETRYCORE_API virtual float GetMaxDimension() override;
};

/**
 * Generate an oriented Floating Stair mesh.
 */
class FFloatingStairGenerator : public FLinearStairGenerator
{
public:
	FFloatingStairGenerator()
	{
		StairStyle = EStairStyle::Floating;
	}
	virtual ~FFloatingStairGenerator() override = default;

protected:
	typedef FLinearStairGenerator Super;

protected:
	GEOMETRYCORE_API virtual FVector3d GenerateVertex(ESide Side, int VertexColumn, int VertexRow) override;
	GEOMETRYCORE_API virtual FVector2f GenerateUV(ESide Side, int Step, int VertexId, float UVScale) override;
};

/**
 * Generate an oriented Curved Stair mesh.
 */
class FCurvedStairGenerator : public FStairGenerator
{
public:
	/** Inner radius of the curved staircase */
	float InnerRadius = 150.0f;

	/** Curve angle of the staircase (in degrees) */
	float CurveAngle = 90.0f;

public:
	FCurvedStairGenerator()
	{
		StairStyle = EStairStyle::Solid;
	}
	virtual ~FCurvedStairGenerator() override = default;

protected:
	typedef FStairGenerator Super;

	/** Precompute/cached data */
	bool bIsClockwise = true;
	float CurveRadians = 0.0f;
	float CurveRadiansPerStep = 0.0f;
	float OuterRadius = 0.0f;
	float RadiusRatio = 1.0f;
	FVector3f BackNormal = FVector3f::Zero();

protected:
	GEOMETRYCORE_API virtual void ResetData() override;
	GEOMETRYCORE_API virtual FVector3d GenerateVertex(ESide Side, int VertexColumn, int VertexRow) override;
	GEOMETRYCORE_API virtual FVector3f GenerateNormal(ESide Side, int VertexId) override;
	GEOMETRYCORE_API virtual FVector2f GenerateUV(ESide Side, int Step, int VertexId, float UVScale) override;
	GEOMETRYCORE_API virtual float GetMaxDimension() override;
};


/**
 * Generate an oriented Curved Stair mesh.
 */
class FSpiralStairGenerator : public FCurvedStairGenerator
{
public:
	FSpiralStairGenerator()
	{
		StairStyle = EStairStyle::Floating;
	}
	virtual ~FSpiralStairGenerator() override = default;

protected:
	typedef FCurvedStairGenerator Super;

protected:
	GEOMETRYCORE_API virtual FVector3d GenerateVertex(ESide Side, int VertexColumn, int VertexRow) override;
	GEOMETRYCORE_API virtual FVector2f GenerateUV(ESide Side, int Step, int VertexId, float UVScale) override;
};

} // end namespace UE::Geometry
} // end namespace UE
