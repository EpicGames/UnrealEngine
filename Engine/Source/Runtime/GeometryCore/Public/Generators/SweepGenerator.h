// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp CylinderGenerator

#pragma once


#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "FrameTypes.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "MeshShapeGenerator.h"
#include "Polygon2.h"
#include "Util/ProgressCancel.h"
#include "VectorTypes.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

/**
 * ECapType indicates the type of cap to use on a sweep
 */
enum class ECapType
{
	None = 0,
	FlatTriangulation = 1,
	FlatMidpointFan = 2
	// TODO: Cone, other caps ...
};

class FSweepGeneratorBase : public FMeshShapeGenerator
{
public:
	virtual ~FSweepGeneratorBase() = default;

	/** If true, each quad gets a separate polygroup */
	bool bPolygroupPerQuad = false;

	/** If true, the last point of the profile curve is considered to be connected to the first. */
	bool bProfileCurveIsClosed = true;

protected:
	int32 CapVertStart[2], CapNormalStart[2], CapUVStart[2], CapTriangleStart[2], CapPolygonStart[2];

	/**
	 * Shared logic for creating vertex buffers and triangulations across all sweep primitives
	 * Note: Does not set vertex positions or normals; a separate call must do that.
	 */
	GEOMETRYCORE_API void ConstructMeshTopology(const FPolygon2d& CrossSection,
							   const TArrayView<const int32>& UVSections,
							   const TArrayView<const int32>& NormalSections,
							   const TArrayView<const int32>& SharpNormalsAlongLength,
							   bool bEvenlySpaceUVs,
							   const TArrayView<const FVector3d>& Path, // can be empty unless bEvenlySpaceUVs is true
							   int32 NumCrossSections,
							   bool bLoop,
							   const ECapType Caps[2],
							   FVector2f SectionsUVScale, 
							   FVector2f CapUVScale, 
							   FVector2f CapUVOffset,
							   const TArrayView<const float>& CustomCrossSectionTexCoord = {}, // if specified and valid, we use custom UVs instead of automatically generating them
							   const TArrayView<const float>& CustomPathTexCoord = {});
};

/**
* Generate a cylinder with optional end caps
*/
class FVerticalCylinderGeneratorBase : public FSweepGeneratorBase
{
public:
	int AngleSamples = 16;
	bool bCapped = false;
	bool bUVScaleMatchSidesAndCaps = true;
	ECapType CapType = ECapType::FlatMidpointFan;

	GEOMETRYCORE_API static float ComputeSegLengths(const TArrayView<float>& Radii, const TArrayView<float>& Heights, TArray<float>& AlongPercents);;

	GEOMETRYCORE_API bool GenerateVerticalCircleSweep(const TArrayView<float>& Radii, const TArrayView<float>& Heights, const TArrayView<int>& SharpNormalsAlongLength);
};

/**
 * Generate a cylinder with optional end caps
 */
class FCylinderGenerator : public FVerticalCylinderGeneratorBase
{
public:
	float Radius[2] = {1.0f, 1.0f};
	float Height = 1.0f;
	int LengthSamples = 0;

public:
	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;
};

/**
* Generate a 3D arrow
*/
class FArrowGenerator : public FVerticalCylinderGeneratorBase
{
public:
	float StickRadius = 0.5f;
	float StickLength = 1.0f;
	float HeadBaseRadius = 1.0f;
	float HeadTipRadius = 0.01f;
	float HeadLength = 0.5f;

	int AdditionalLengthSamples[3]{ 0,0,0 }; // additional length-wise samples on the three segments (along stick, along arrow base, along arrow cone)

	void DistributeAdditionalLengthSamples(int TargetSamples)
	{
		TArray<float> AlongPercents;
		TArray<float> Radii{ StickRadius, StickRadius, HeadBaseRadius, HeadTipRadius };
		TArray<float> Heights{ 0, StickLength, StickLength, StickLength + HeadLength };
		float LenAlong = ComputeSegLengths(Radii, Heights, AlongPercents);
		for (int Idx = 0; Idx < 3; Idx++)
		{
			AdditionalLengthSamples[Idx] = (int)(.5f+AlongPercents[Idx + 1] * float(TargetSamples));
		}
	}

public:
	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;
};



/**
 * Sweep a 2D Profile Polygon along a 3D Path.
 * 
 * TODO: 
 *  - a custom variant for toruses specifically (would be faster)
 */
class FGeneralizedCylinderGenerator : public FSweepGeneratorBase
{
public:
	FPolygon2d CrossSection;
	TArray<FVector3d> Path;

	FFrame3d InitialFrame;
	// If PathFrames.Num == Path.Num, then PathFrames[k] is used for each step instead of the propagated InitialFrame.
	TArray<FFrame3d> PathFrames;
	// If PathScales.Num == Path.Num, then PathScales[k] is applied to the CrossSection at each step (this is combined with StartScale/EndScale, but ignored if bLoop=true)
	TArray<FVector2d> PathScales;

	bool bCapped = false;
	bool bLoop = false;
	ECapType CapType = ECapType::FlatTriangulation;

	// 2D uniform scale of the CrossSection, interpolated along the Path (via arc length) from StartScale to EndScale
	double StartScale = 1.0;
	double EndScale = 1.0;

	// Maximum factor by which mitering can expand the cross section at sharp turns, to give the appearance of a consistent cross section width. Only used if > 1.0.
	// Note: If PathFrames are specified, then bAlignFramesToSampledTangents must be true for mitering to be performed.
	// (Similar to the MiterLimit concept in SVG / CSS)
	double MiterLimit = 1.0;

	// Whether to align the frames to the path tangents. Only relevant if PathFrames is specified.
	bool bAlignFramesToSampledTangents = false;

	// Configure settings to support mitering -- i.e., scaling the cross sections as needed to maintain consistent cross section size through sharp corners, up to the specified scale limit
	void EnableMitering(double InMiterLimit = 10)
	{
		MiterLimit = InMiterLimit;
		bAlignFramesToSampledTangents = true;
	}

	// Set the MiterLimit based on the maximum turn angle at which a correct miter should be applied. For turns sharper than this angle, the cross section will appear to shrink at the turn.
	// @param MiterAngleLimitInDeg Maximum turn angle for correct mitering; should be >= 0 and < 180
	void SetMiterLimitByAngle(double MiterAngleLimitInDeg)
	{
		MiterAngleLimitInDeg = FMath::Clamp(MiterAngleLimitInDeg, 0, 180 - FMathd::ZeroTolerance);
		MiterLimit = 1.0/FMath::Sin(MiterAngleLimitInDeg * FMathd::DegToRad * .5);
	}

	// When true, the generator attempts to scale UV's in a way that preserves scaling across different mesh
	// results, aiming for 1.0 in UV space to be equal to UnitUVInWorldCoordinates in world space. This in
	// practice means adjusting the U scale relative to the CrossSection curve length and V scale relative
	// to the distance between vertices on the Path.
	bool bUVScaleRelativeWorld = false;

	// Only relevant if bUVScaleRelativeWorld is true (see that description)
	float UnitUVInWorldCoordinates = 100;

    // Optional custom UV values:
	
    // -If FSweepGeneratorBase::bProfileCurveIsClosed == true and CrossSectionTexCoord.Num() >= CrossSection.VertexCount() + 1 
	// then the first CrossSection.VertexCount() + 1 values will be used as U coordinates.
    //
    // -If FSweepGeneratorBase::bProfileCurveIsClosed == true and CrossSectionTexCoord.Num() == CrossSection.VertexCount() 
	// then the CrossSectionTexCoord[0] will be used as the value for the last element.
    //
    // -If FSweepGeneratorBase::bProfileCurveIsClosed == false and CrossSectionTexCoord.Num() >= CrossSection.VertexCount() 
	// then the first CrossSection.VertexCount() values will be used as U coordinates.
    // 
    // -Otherwise, the U coordinates will be automatically generated.
    TArray<float> CrossSectionTexCoord;
    
    // -If bLoop == true and PathTexCoord.Num() >= Path.Num() + 1 then the first Path.Num() + 1 values will be used as
    //  V coordinates.
    //
    // -If bLoop == true and PathTexCoord.Num() == Path.Num() then the PathTexCoord[0] will be used as the value for the 
    //  last element.
    //
    // -If bLoop == false and PathTexCoord.Num() >= Path.Num() then the first Path.Num() values will be used as V 
    // coordinates.
	//
    // -Otherwise, the V coordinates will be automatically generated.
    TArray<float> PathTexCoord;

public:
	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;
};

enum class EProfileSweepPolygonGrouping : uint8
{
	/** One polygroup for entire output mesh */
	Single,
	/** One polygroup per mesh quad/triangle */
	PerFace,

	/* One polygroup per strip that represents a step along the sweep curve. */
	PerSweepSegment,
	/* One polygroup per strip coming from each individual edge of the profile curve. */
	PerProfileSegment
};

enum class EProfileSweepQuadSplit : uint8
{
	/** Always split the quad in the same way relative sweep direction and profile direction. */
	Uniform,
	/** Split the quad to connect the shortest diagonal. */
	ShortestDiagonal
};

/**
 * Much like FGeneralizedCylinderGenerator, but allows an arbitrary profile curve to be swept, and gives
 * control over the frames of the sweep curve. A mesh will be properly oriented if the profile curve is
 * oriented counterclockwise when facing down the direction in which it is being swept.
 *
 * Because it supports open profile curves, as well as welded points (for welding points on an axis of rotation), 
 * it cannot actually use the utility function from FSweepGeneratorBase, and so it doesn't inherit from 
 * that class.
 */
class FProfileSweepGenerator : public FMeshShapeGenerator
{
public:

	// Curve that will be swept along the curve, given in coordinates of the frames used in the sweep curve.
	TArray<FVector3d> ProfileCurve;

	// Curve along which to sweep the profile curve.
	TArray<FFrame3d> SweepCurve;

	// (Optional) Curve along which to scale the profile curve, corresponding to each frame in SweepCurve.
	TArray<FVector3d> SweepScaleCurve;

	// Indices into ProfileCurve that should not be swept along the curve, instead being instantiated
	// just once. This is useful for welding vertices on an axis of rotation if the sweep curve denotes
	// a revolution.
	TSet<int32> WeldedVertices;

	// Generated UV coordinates will be multiplied by these values.
	FVector2d UVScale = FVector2d(1,1);

	// These values will be added to the generated UV coordinates after applying UVScale.
	FVector2d UVOffset = FVector2d(0, 0);

	// When true, the generator attempts to scale UV's in a way that preserves scaling across different mesh
	// results, aiming for 1.0 in UV space to be equal to UnitUVInWorldCoordinates in world space. This is 
	// generally speaking unrealistic because UV's are going to be variably stretched no matter what, but 
	// in practice it means adjusting the V scale relative to the profile curve length and U scale relative
	// to a very crude measurement of movement across sweep frames.
	bool bUVScaleRelativeWorld = false;

	// Only relevant if bUVScaleRelativeWorld is true (see that description)
	float UnitUVInWorldCoordinates = 100;

	// If true, the last point of the sweep curve is considered to be connected to the first.
	bool bSweepCurveIsClosed = false;

	// If true, the last point of the profile curve is considered to be connected to the first.
	bool bProfileCurveIsClosed = false;

	// If true, each triangle will have its own normals at each vertex, rather than sharing averaged
	// ones with nearby triangles.
	bool bSharpNormals = true;

	// If true, welded-to-welded connections in the profile curve (which can't result in triangles)
	// do not affect the UV layout.
	bool bUVsSkipFullyWeldedEdges = true;

	EProfileSweepQuadSplit QuadSplitMethod = EProfileSweepQuadSplit::ShortestDiagonal;

	// When QuadSplitMode is ShortestDiagonal, biases one of the diagonals so that symmetric
	// quads are split uniformly. The tolerance is a proportion allowable difference.
	double DiagonalTolerance = 0.01;

	EProfileSweepPolygonGrouping PolygonGroupingMode = EProfileSweepPolygonGrouping::PerFace;

	// If not null, this pointer is intermittently used to check whether the current operation should stop early
	FProgressCancel* Progress = nullptr;

	// TODO: We could allow the user to dissallow bowtie vertex creation, which currently could 
	// happen depending on which vertices are welded.

public:

	/** Generate the mesh */
	GEOMETRYCORE_API virtual FMeshShapeGenerator& Generate() override;

	/** If the sweep curve is not closed, this will store the vertex ids of the first and last instances
	 * of the profile curve. Note that even if the profile curve is closed, depending on the welding,
	 * these could be part of a single boundary (ie, a square revolved 90 degrees around a welded side
	 * actually has one open boundary rather than two, since they are joined), but the user likely
	 * wants to be given them separately for ease in making end caps.
	 */
	TArray<int32> EndProfiles[2];

	// TODO: We could output other boundaries too, but that's probably only worth doing once we find
	// a case where we would actually use them.
protected:

	GEOMETRYCORE_API void InitializeUvBuffer(const TArray<int32>& VertPositionOffsets, 
		int32& NumUvRowsOut, int32& NumUvColumnsOut);
	GEOMETRYCORE_API void AdjustNormalsForTriangle(int32 TriIndex, int32 FirstIndex, int32 SecondIndex, int32 ThirdIndex,
		TArray<FVector3d>& WeightedNormals);
	GEOMETRYCORE_API void AdjustNormalsForTriangle(int32 TriIndex, int32 FirstIndex, int32 SecondIndex, int32 ThirdIndex,
		TArray<FVector3d>& WeightedNormals, const FVector3d& AbNormalized);
};


} // end namespace UE::Geometry
} // end namespace UE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "BoxTypes.h"
#include "IndexTypes.h"
#include "MatrixTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/PlatformCrt.h"
#include "CoreMinimal.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Curve/CurveUtil.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"
#endif
