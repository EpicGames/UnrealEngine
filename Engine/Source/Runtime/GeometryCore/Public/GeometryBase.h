// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"   // required for GEOMETRYCORE_API macro
#include "Logging/LogMacros.h"

//
// The following are convenience macros that can be used to predeclare types in the UE::Geometry::
// namespace as one-liners.
//

/** Predeclare the type TypeName in the UE::Geometry:: namespace */
#define PREDECLARE_GEOMETRY(TypeName) namespace UE { namespace Geometry { TypeName; }}

/** Predeclare the class ClassName in the UE::Geometry:: namespace, and then add a using UE::Geometry::ClassName declaration. */
#define PREDECLARE_USE_GEOMETRY_CLASS(ClassName) namespace UE { namespace Geometry { class ClassName; }} using UE::Geometry::ClassName;

/** Predeclare struct class StructName in the UE::Geometry:: namespace, and then add a using UE::Geometry::StructName declaration. */
#define PREDECLARE_USE_GEOMETRY_STRUCT(StructName) namespace UE { namespace Geometry { struct StructName; }} using UE::Geometry::StructName;


// The above macros will not work if no other header that defines the UE::Geometry:: namespace has been #included,
// this declaraction resolves that problem.
namespace UE
{
	namespace Geometry
	{
		struct FForceGeometryNamespaceToExist
		{
			int a;
		};
	}
}

GEOMETRYCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogGeometry, Log, All);