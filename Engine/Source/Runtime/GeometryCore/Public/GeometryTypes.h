// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/AssertionMacros.h"

namespace UE
{
namespace Geometry
{

/**
 * EMeshResult is returned by various mesh/graph operations to either indicate success,
 * or communicate which type of error ocurred (some errors are recoverable, and some not).
 */
enum class EMeshResult
{
	Ok = 0,
	Failed_NotAVertex = 1,
	Failed_NotATriangle = 2,
	Failed_NotAnEdge = 3,

	Failed_BrokenTopology = 10,
	Failed_HitValenceLimit = 11,

	Failed_IsBoundaryEdge = 20,
	Failed_FlippedEdgeExists = 21,
	Failed_IsBowtieVertex = 22,
	Failed_InvalidNeighbourhood = 23,       // these are all failures for CollapseEdge
	Failed_FoundDuplicateTriangle = 24,
	Failed_CollapseTetrahedron = 25,
	Failed_CollapseTriangle = 26,
	Failed_NotABoundaryEdge = 27,
	Failed_SameOrientation = 28,
	Failed_CollapseQuad = 29,

	Failed_WouldCreateBowtie = 30,
	Failed_VertexAlreadyExists = 31,
	Failed_CannotAllocateVertex = 32,
	Failed_VertexStillReferenced = 33,

	Failed_WouldCreateNonmanifoldEdge = 50,
	Failed_TriangleAlreadyExists = 51,
	Failed_CannotAllocateTriangle = 52,

	Failed_UnrecoverableError = 1000,
	Failed_Unsupported = 1001
};



/**
 * EOperationValidationResult is meant to be returned by Validate() functions of 
 * Operation classes (eg like ExtrudeMesh, etc) to indicate whether the operation
 * can be successfully applied.
 */
enum class EOperationValidationResult
{
	Ok = 0,
	Failed_UnknownReason = 1,

	Failed_InvalidTopology = 2
};


/**
 * EValidityCheckFailMode is passed to CheckValidity() functions of various classes 
 * to specify how validity checks should fail.
 */
enum class EValidityCheckFailMode 
{ 
	/** Function returns false if a failure is encountered */
	ReturnOnly = 0,
	/** Function check()'s if a failure is encountered */
	Check = 1,
	/** Function ensure()'s if a failure is encountered */
	Ensure = 2
};





/**
 * TIndexMap stores mappings between indices, which are assumed to be an integer type.
 * Both forward and backward mapping are stored
 *
 * @todo make either mapping optional
 * @todo optionally support using flat arrays instead of TMaps
 * @todo constructors that pick between flat array and TMap modes depending on potential size/sparsity of mapped range
 * @todo identity and shift modes that don't actually store anything
 */
template<typename IntType>
struct TIndexMap
{
protected:
	TMap<IntType, IntType> ForwardMap;
	TMap<IntType, IntType> ReverseMap;
	bool bWantForward;
	bool bWantReverse;

public:

	TIndexMap()
	{
		bWantForward = bWantReverse = true;
	}

	void Reset()
	{
		ForwardMap.Reset();
		ReverseMap.Reset();
	}

	/** @return the value used to indicate an ID is not present in the mapping */
	constexpr IntType UnmappedID() const { return (IntType)-1; }

	TMap<IntType, IntType>& GetForwardMap() { return ForwardMap; }
	const TMap<IntType, IntType>& GetForwardMap() const { return ForwardMap; }

	TMap<IntType, IntType>& GetReverseMap() { return ReverseMap; }
	const TMap<IntType, IntType>& GetReverseMap() const { return ReverseMap; }

	/** add mapping from one index to another */
	inline void Add(IntType FromID, IntType ToID)
	{
		checkSlow(FromID >= 0 && ToID >= 0);
		ForwardMap.Add(FromID, ToID);
		ReverseMap.Add(ToID, FromID);
	}

	/** @return true if we can map forward from this value */
	inline bool ContainsFrom(IntType FromID) const
	{
		checkSlow(FromID >= 0);
		check(bWantForward);
		return ForwardMap.Contains(FromID);
	}

	/** @return true if we can reverse-map from this value */
	inline bool ContainsTo(IntType ToID) const
	{
		checkSlow(ToID >= 0);
		check(bWantReverse);
		return ReverseMap.Contains(ToID);
	}


	/** @return forward-map of input value */
	inline IntType GetTo(IntType FromID) const
	{
		checkSlow(FromID >= 0);
		check(bWantForward);
		const IntType* FoundVal = ForwardMap.Find(FromID);
		return (FoundVal == nullptr) ? UnmappedID() : *FoundVal;
	}

	/** @return reverse-map of input value */
	inline IntType GetFrom(IntType ToID) const
	{
		checkSlow(ToID >= 0);
		check(bWantReverse);
		const IntType* FoundVal = ReverseMap.Find(ToID);
		return (FoundVal == nullptr) ? UnmappedID() : *FoundVal;
	}

	/** @return forward-map of input value or null if not found */
	inline const IntType* FindTo(IntType FromID) const
	{
		checkSlow(FromID >= 0);
		check(bWantForward);
		return ForwardMap.Find(FromID);
	}

	/** @return reverse-map of input value or null if not found */
	inline const IntType* FindFrom(IntType ToID) const
	{
		checkSlow(ToID >= 0);
		check(bWantReverse);
		return ReverseMap.Find(ToID);
	}


	void Reserve(int NumElements)
	{
		checkSlow(NumElements >= 0);
		if (bWantForward)
		{
			ForwardMap.Reserve(NumElements);
		}
		if (bWantReverse)
		{
			ReverseMap.Reserve(NumElements);
		}
	}
	
	SIZE_T GetAllocatedSize() const
	{
		return ForwardMap.GetAllocatedSize() + ReverseMap.GetAllocatedSize();
	}
};

typedef TIndexMap<int> FIndexMapi;


} // end namespace UE::Geometry
} // end namespace UE