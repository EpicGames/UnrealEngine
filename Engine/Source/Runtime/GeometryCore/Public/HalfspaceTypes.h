// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/*
 * 3D Halfspace stored as parameters to Plane Equation (Normal, Normal.Dot(PointOnPlane))
 * The Normal points "into" the halfspace, ie X is inside if (Normal.Dot(X) - Constant) >= 0
 */
template<typename T>
struct THalfspace3
{
public:
	/** Normal vector of 3D plane that defines Halfspace */
	TVector<T> Normal = TVector<T>::UnitY();
	/** Distance along Normal that defines position of Halfspace */
	T Constant = (T)0;

	THalfspace3() = default;

	THalfspace3(const TVector<T>& PlaneNormalIn, T ConstantIn)
		: Normal(PlaneNormalIn), Constant(ConstantIn) {}

	THalfspace3(T NormalX, T NormalY, T NormalZ, T ConstantIn)
		: Normal(NormalX,NormalY,NormalZ), Constant(ConstantIn) {}


	/** Construct a Halfspace from the plane Normal and a Point lying on the plane */
	THalfspace3(const TVector<T>& PlaneNormalIn, const TVector<T>& PlanePoint)
		: Normal(PlaneNormalIn), Constant(Normal.Dot(PlanePoint)) {}

	/** Construct a Halfspace from three points */
	THalfspace3(const TVector<T>& P0, const TVector<T>& P1, const TVector<T>& P2)
	{
		Normal = VectorUtil::Normal(P0, P1, P2);
		Constant = Normal.Dot(P0);
	}


	/** @return true if Halfspace contains given Point */
	bool Contains(const TVector<T>& Point) const
	{
		return (Normal.Dot(Point) - Constant) >= 0;
	}

};


typedef THalfspace3<float> FHalfspace3f;
typedef THalfspace3<double> FHalfspace3d;


} // end namespace UE::Geometry
} // end namespace UE