// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "MathUtil.h"

namespace UE
{
namespace Geometry
{

enum class EBCSplineType
{
	BSpline,
	CatmullRom,
	MitchellNetravali
};

/**
 * @param SplineType The BC-spline filter type (predefined B/C parameters)
 * @param bIsRadial When true, use Euclidean distance to compute filter weights: Weight = Filter(sqrt(X*X + Y*Y)).
 *                  Otherwise use linear 1D distance in X/Y separately: WX = Filter(X), WY = Filter(Y), Weight = WX*WY 
 * 
 * TODO The Unreal naming convention uses the T prefix for template functions. This would require deprecation.
 * TODO Consider removing the Abs from GetWeight since distances are non-negative. This would require deprecation.
 * TODO Consider using constexpr polynomial coefficients to make this class smaller
 */
template <EBCSplineType SplineType, bool bIsRadial>
class FBCSplineFilter
{
private:
	float InvRadius = 1.0f;

	float Coeff1[3] = {0.0f, 0.0f, 0.0f};
	float Coeff2[4] = {0.0f, 0.0f, 0.0f, 0.0f};

public:
	FBCSplineFilter(const float RadiusIn);

	/** @return The filter weight given a 2D distance vector
	 *  @note   Works on general 2D point sets i.e., does not assume Cartesian/pixel grids
	 */
	float GetWeight(const FVector2d& Dist) const;

	/** @return true if the given 2D distance vector is in the region where the filter is defined and false otherwise */
	bool IsInFilterRegion(const FVector2d& Dist) const;
};

template<EBCSplineType SplineType, bool bIsRadial>
FBCSplineFilter<SplineType, bIsRadial>::FBCSplineFilter(const float RadiusIn)
	: InvRadius(1.f/RadiusIn)
{
	auto ComputeCoeffs = [this](const float& B, const float& C)
	{
		// x < 1:        1/6 * ((12 - 9B - 6C)x^3 + (-18 + 12B + 6C)x^2 + (6 - 2B))
		Coeff1[0] = (12.0f - 9.0f * B - 6.0f * C) / 6.0f;
		Coeff1[1] = (-18.0f + 12.0f * B + 6.0f * C) / 6.0f;
		Coeff1[2] = (6.0f - 2.0f * B) / 6.0f;

		// 1 <= x < 2:   1/6 * ((-B - 6C)x^3 + (6B + 30C)x^2 + (-12B - 48C)x + (8B + 24C))
		Coeff2[0] = (-B - 6.0f * C) / 6.0f;
		Coeff2[1] = (6.0f * B + 30.0f * C) / 6.0f;
		Coeff2[2] = (-12.0f * B - 48.0f * C) / 6.0f;
		Coeff2[3] = (8.0f * B + 24.0f * C) / 6.0f;
	};
	
	if constexpr(SplineType == EBCSplineType::BSpline)
	{
		const float B = 1.0f;
		const float C = 0.0f;
		ComputeCoeffs(B, C);
	}
	else if constexpr(SplineType == EBCSplineType::CatmullRom)
	{
		const float B = 0.0f;
		const float C = 0.5f;
		ComputeCoeffs(B, C);
	}
	else if constexpr(SplineType == EBCSplineType::MitchellNetravali)
	{
		const float B = 1.0f / 3.0f;
		const float C = B;
		ComputeCoeffs(B, C);
	}
}

template<EBCSplineType SplineType, bool bIsRadial>
float FBCSplineFilter<SplineType, bIsRadial>::GetWeight(const FVector2d& Dist) const
{
	auto ComputeWeight = [this](float X) -> float
	{
		// Scale the filter kernel f(X) in the X-direction by making the substitution f(X/Radius)
		const float AbsX = FMathf::Abs(X * InvRadius);
		const float AbsX2 = AbsX * AbsX;
		const float AbsX3 = AbsX * AbsX2;

		float Weight = 0.0f;
		if(AbsX < 1.0f)
		{
			Weight = Coeff1[0] * AbsX3 + Coeff1[1] * AbsX2 + Coeff1[2];
		}
		else if(AbsX < 2.0f)
		{
			Weight = Coeff2[0] * AbsX3 + Coeff2[1] * AbsX2 + Coeff2[2] * AbsX + Coeff2[3];
		}
		return Weight;
	};

	if constexpr(bIsRadial)
	{
		return ComputeWeight((float)Dist.Length());
	}
	else
	{
		const float WeightX = ComputeWeight((float)Dist.X);
		const float WeightY = ComputeWeight((float)Dist.Y);
		return WeightX * WeightY;
	}
}

template<EBCSplineType SplineType, bool bIsRadial>
bool FBCSplineFilter<SplineType, bIsRadial>::IsInFilterRegion(const FVector2d& Dist) const
{
	if constexpr(bIsRadial)
	{
		return (FMathf::Abs((float)Dist.Length() * InvRadius) < 2.0f);
	}
	return (FMathf::Abs((float)Dist.X * InvRadius) < 2.0f) && (FMathf::Abs((float)Dist.Y * InvRadius) < 2.0f);
}

using FBSplineFilter = FBCSplineFilter<EBCSplineType::BSpline, /*bIsRadial*/ false>;
using FCatmullRomFilter = FBCSplineFilter<EBCSplineType::CatmullRom, /*bIsRadial*/ false>;
using FMitchellNetravaliFilter = FBCSplineFilter<EBCSplineType::MitchellNetravali, /*bIsRadial*/ false>;
	
} // end namespace UE::Geometry
} // end namespace UE