// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Vector2D.h"

namespace UE
{
namespace Geometry
{

class FBoxFilter
{
private:
	float Radius = 0.5f;
	
public:
	FBoxFilter(const float RadiusIn)
		: Radius(RadiusIn)
	{		
	}

	/** @return The filter weight given a 2D distance vector
	 *  @note   Works on general 2D point sets i.e., does not assume Cartesian/pixel grids
	 */
	float GetWeight(const FVector2d& Dist) const
	{
		// Returns 1 if Dist is within the region [-Radius, Radius)x[-Radius, Radius) and 0 otherwise.
		return -Radius <= Dist.X && Dist.X < Radius && -Radius <= Dist.Y && Dist.Y < Radius;
	}

	/** @return true if the given 2D distance vector is in the region where the filter is defined and false otherwise */
	bool IsInFilterRegion(const FVector2d& Dist) const
	{
		return -Radius <= Dist.X && Dist.X < Radius && -Radius <= Dist.Y && Dist.Y < Radius;
	}
};

} // end namespace UE::Geometry
} // end namespace UE