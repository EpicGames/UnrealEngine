// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureDefines.h"

namespace UE {
namespace Geometry {

class FDisplacementMap
{
private:
	static constexpr size_t MAX_NUM_MIP_LEVELS { 13 }; // maximum additional miplevels, total (incl. finest is +1)
	static constexpr int32  MAX_RESOLUTION     { 1 << MAX_NUM_MIP_LEVELS }; // 8192 

	ETextureSourceFormat	SourceFormat;

	int32		BytesPerPixel;
	int32		SizeX;
	int32		SizeY;
	uint32		NumLevels;

	float		Magnitude;
	float		Center;

	TextureAddress	AddressX;
	TextureAddress	AddressY;

public:
	virtual ~FDisplacementMap() = default;

	GEOMETRYCORE_API FDisplacementMap();
	
	FDisplacementMap(const FDisplacementMap&) = delete;
	FDisplacementMap(FDisplacementMap&&) = default;
	
	FDisplacementMap& operator=(const FDisplacementMap&) = delete;
	FDisplacementMap& operator=(FDisplacementMap&&) = default;
	
	/**
	 * Claims ownership of raw input source-data. 
	 * 
	 * The size of this data must match SizeX * SizeY * GetBytesPerPixel(Format)
	 * SizeX and SizeY must be <= GetMaxResolution()
	 * 
	 */
	GEOMETRYCORE_API			FDisplacementMap( 
		TArray64<uint8>&& InSourceData, 
		ETextureSourceFormat InSourceFormat,
		const int32 InSizeX,
		const int32 InSizeY,
		float InMagnitude, 
		float InCenter, 
		TextureAddress InAddressX, 
		TextureAddress InAddressY);

	/**
	 * Reference input data through view. 
	 * 
	 * The size of view data must match SizeX * SizeY * GetBytesPerPixel(Format)
	 * SizeX and SizeY must be <= GetMaxResolution()
	 * 
	 */
	GEOMETRYCORE_API			FDisplacementMap( 
		TArrayView64<const uint8> InSourceDataView,
		ETextureSourceFormat InSourceFormat,
		const int32 InSizeX,
		const int32 InSizeY,
		float InMagnitude, 
		float InCenter, 
		TextureAddress InAddressX, 
		TextureAddress InAddressY);
	
	// Bilinear filtered
	GEOMETRYCORE_API float		Sample( FVector2f UV ) const;

	// Bounds over UV-rectangle (approximate, but conservative)
	GEOMETRYCORE_API FVector2f	Sample( FVector2f MinUV, FVector2f MaxUV ) const;

	// Bounds over UV-rectangle (hierarchical traversal, increased refinements lead to tighter bounds, 
	// but significantly more expensive for large numbers of refinements)
	GEOMETRYCORE_API FVector2f	SampleHierarchical( FVector2f MinUV, FVector2f MaxUV, uint32 MaxRefinements ) const;

	// hierarchical sample warping for perfect importance sampling according to probability density represented by image (not required to be normalized).
	// [ Clarberg, et al., "Wavelet importance sampling: efficiently evaluating products of complex functions" ]
	GEOMETRYCORE_API FVector2f   WarpSample(const FVector2f& UV) const;

	/**
	 * Filtered with elliptic weighted averaging
	 * 
	 * \param Axis0 major axis
	 * \param Axis1 minor axis
	 */
	[[nodiscard]] GEOMETRYCORE_API float		SampleEWA( FVector2f UV, FVector2f Axis0, FVector2f Axis1 ) const;

	[[nodiscard]] float		Sample( int32 x, int32 y ) const;
	[[nodiscard]] FVector2f	Sample( int32 x, int32 y, uint32 Level ) const;

	[[nodiscard]] float		Load( int32 x, int32 y ) const;

	[[nodiscard]] FVector2f	Load( int32 x, int32 y, uint32 Level ) const;

	[[nodiscard]] float     LoadFiltered( int32 x, int32 y, uint32 Level ) const;

	[[nodiscard]] int32 GetSizeX() const
	{
		return SizeX;
	}

	[[nodiscard]] int32 GetSizeY() const
	{
		return SizeY;
	}
	
	[[nodiscard]] static bool IsSupportedSourceFormat(const ETextureSourceFormat SourceFormat)
	{
		return GetBytesPerPixel(SourceFormat) != 0;
	}

	/// returns, iff SourceFormat is not supported
	[[nodiscard]] GEOMETRYCORE_API static int32 GetBytesPerPixel(const ETextureSourceFormat SourceFormat);

	[[nodiscard]] static int32 GetMaxResolution()
	{
		return MAX_RESOLUTION;
	}

private:

	void Init(); // Build mipmaps

	// create 1x1 black default map
	void InitDefault(); 

	// per-level EWA
	[[nodiscard]] float EWA( uint32 Level, FVector2f UV, FVector2f Axis0, FVector2f Axis1 ) const;

	// map coordinates back to [0, SizeX) x [0, SizeY)
	void		Address( int32& x, int32& y ) const;

	TArray64<uint8>           Storage;    // optional storage for deserialized objects
	TArrayView64<const uint8> SourceData;
	
	TArray<FVector2f>         MipData        [ MAX_NUM_MIP_LEVELS ];
	TArray<float>             MipDataFiltered[ MAX_NUM_MIP_LEVELS ];

	
};

FORCEINLINE float FDisplacementMap::Sample( int32 x, int32 y ) const
{
	Address( x, y );

	float Displacement = Load( x, y );
	Displacement -= Center;
	Displacement *= Magnitude;

	return Displacement;
}

FORCEINLINE FVector2f FDisplacementMap::Sample( int32 x, int32 y, uint32 Level ) const
{
	Address( x, y );

	x >>= Level;
	y >>= Level;

	FVector2f Displacement = Load( x, y, Level );
	Displacement -= FVector2f( Center );
	Displacement *= Magnitude;

	return Displacement;
}

FORCEINLINE float FDisplacementMap::Load( int32 x, int32 y ) const
{
	checkSlow(x < SizeX && x >= 0 && y < SizeY && y >= 0);
	const uint8* PixelPtr = &SourceData[ int32( x + (int32)y * SizeX ) * BytesPerPixel ];

	if( SourceFormat == TSF_BGRA8 )
	{
		return float( PixelPtr[2] ) / 255.0f;
	}
	else if( SourceFormat == TSF_RGBA16 )
	{
		checkSlow( BytesPerPixel == sizeof(uint16) * 4 );
		return float( *(uint16*)PixelPtr ) / 65535.0f;
	}
	else if( SourceFormat == TSF_RGBA16F || SourceFormat == TSF_R16F )
	{
		FFloat16 HalfValue = *(FFloat16*)PixelPtr;
		return HalfValue;
	}
	else if( SourceFormat == TSF_G8 )
	{
		return float( PixelPtr[0] ) / 255.0f;
	}
	else if( SourceFormat == TSF_G16 )
	{
		return float( *(uint16*)PixelPtr ) / 65535.0f;
	}
	else if( SourceFormat == TSF_RGBA32F || SourceFormat == TSF_R32F )
	{
		return *(float*)PixelPtr;
	}
	else
	{
		checkf( 0, TEXT("Displacement map format not supported") );
		return 0.0f;
	}
}

FORCEINLINE FVector2f FDisplacementMap::Load( int32 x, int32 y, uint32 Level ) const
{
	checkSlow( Level > 0 );

	uint32 MipSizeX = ( ( SizeX - 1 ) >> Level ) + 1;
	uint32 MipSizeY = ( ( SizeY - 1 ) >> Level ) + 1;

	return MipData[ Level - 1 ][ x + y * MipSizeX ];
}


FORCEINLINE float FDisplacementMap::LoadFiltered( int32 x, int32 y, uint32 Level ) const
{
	checkSlow( Level >= 0 );

	if( Level == 0 )
	{
		return Load( x, y );
	}
	
	uint32 MipSizeX = ( ( SizeX - 1 ) >> Level ) + 1;
	uint32 MipSizeY = ( ( SizeY - 1 ) >> Level ) + 1;

	return MipDataFiltered[ Level - 1 ][ x + y * MipSizeX ];
}

FORCEINLINE void FDisplacementMap::Address( int32& x, int32& y ) const
{
	if (AddressX == TA_Clamp)
	{
		x = FMath::Clamp(x, 0, SizeX - 1);
	}
	else
	{
		x  = x % SizeX;
		x += x < 0 ? SizeX : 0;
	}

	if (AddressY == TA_Clamp)
	{
		y = FMath::Clamp(y, 0, SizeY - 1);
	}
	else
	{
		y  = y % SizeY;
		y += y < 0 ? SizeY : 0;
	}
}

} // namespace Geometry
} // namespace UE
