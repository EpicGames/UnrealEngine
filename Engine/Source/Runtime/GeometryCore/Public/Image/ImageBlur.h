// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Solvers/Tridiagonal.h"
#include "Spatial/SampledScalarField2.h"

#include "Containers/Array.h"
#include "Async/ParallelFor.h"

namespace UE {
namespace Geometry {

/*
 * Solve diffusion equation u_t = Laplacian(u), u(0) = u_0 until time Sigma^2/2, with Neumann boundary conditions.
 * This corresponds to Gaussian blur with variance Sigma.
 *
 * Applies additive operator splitting and solves one implicit timestep. It converges independently of filtersize
 * but becomes approximate for large Sigma.
 */
template <typename RealType>
void HeatEquationImplicitAOS(TSampledScalarField2<RealType, RealType>& Field, const RealType SigmaX, const RealType SigmaY)
{
	const int32 Width  = Field.Width();
	const int32 Height = Field.Height();

	const RealType TauX = RealType(0.5) * SigmaX * SigmaX; // Timestep
	const RealType TauY = RealType(0.5) * SigmaY * SigmaY; // Timestep

	struct FTaskContext
	{
		TArray<RealType> Rhs;
		TArray<RealType> Solution;
	};
	
	TArray<FTaskContext> TaskContexts;

	// columns
	{
		TArray<RealType> A, B, C;
		A.Init(-TauY, Height);
		B.Init(1. + 2. * TauY, Height);
		C.Init(-TauY, Height);

		C[0] *= 2.;
		A[Height-1] *= 2.; 
	
		TTridiagonalSolver<RealType> Solver(A, B, C);
		
		ParallelForWithTaskContext(TEXT("HeatEquationImplicitAOS.Columns"), TaskContexts, Width, 32, [&](FTaskContext& TaskContext, int32 X)
		{
			TaskContext.Rhs.SetNum(Height);
			TaskContext.Solution.SetNum(Height);

			for (int32 Y=0; Y<Height; ++Y)
			{
				TaskContext.Rhs[Y] = Field.GridValues.At(X, Y);
			}

			Solver.Solve(TaskContext.Rhs, TaskContext.Solution);
			
			for (int32 Y=0; Y<Height; ++Y)
			{
				Field.GridValues.At(X, Y) = TaskContext.Solution[Y];
			}
		});
	}

	// blur along rows
	{
		TArray<RealType> A, B, C;
		TArray<RealType> RowRhs, RowX;
		
		A.Init(-TauX, Width);
		B.Init(1. + 2. * TauX, Width);
		C.Init(-TauX, Width);

		C[0] *= 2.;
		A[Width-1] *= 2.; 
	
		TTridiagonalSolver<RealType> Solver(A, B, C);

		ParallelForWithTaskContext(TEXT("HeatEquationImplicitAOS.Rows"), TaskContexts, Height, 32, [&](FTaskContext& TaskContext, int32 Y)
		{
			TaskContext.Rhs.SetNum(Width);
			TaskContext.Solution.SetNum(Width);
		
			for (int32 X=0; X<Width; ++X)
			{
				TaskContext.Rhs[X] = Field.GridValues.At(X, Y);
			}

			Solver.Solve(TaskContext.Rhs, TaskContext.Solution);
			
			for (int32 X=0; X<Width; ++X)
			{
				Field.GridValues.At(X, Y) = TaskContext.Solution[X];
			}
		});
	}
}

} // end namespace Geometry
} // end namespace UE