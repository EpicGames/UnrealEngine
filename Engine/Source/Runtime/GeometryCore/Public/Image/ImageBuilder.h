// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "Image/ImageDimensions.h"
#include "Spatial/DenseGrid2.h"

namespace UE
{
namespace Geometry
{

enum class EImageTilingMethod
{
	Clamp,
	Wrap
};

/**
 * TImageBuilder is used to create and populate a 2D image with a templated "pixel" type.
 */
template<typename PixelType>
class TImageBuilder
{
protected:
	FImageDimensions Dimensions;
	TDenseGrid2<PixelType> Image;

public:

	void SetDimensions(FImageDimensions DimensionsIn)
	{
		Dimensions = DimensionsIn;
		Image.Resize(Dimensions.GetWidth(), Dimensions.GetHeight(), EAllowShrinking::Yes);
	}

	const FImageDimensions& GetDimensions() const
	{
		return Dimensions;
	}

	/**
	 * Clear all Pixels in the current Mip to the clear/default color for the texture build type
	 */
	void Clear(const PixelType& ClearValue)
	{
		Image.Assign(ClearValue);
	}

	/**
	 * @return true if the given X/Y coordinates are inside the image pixel bounds, ie can be used to index a pixel
	 */
	bool ContainsPixel(int32 X, int32 Y) const
	{
		return X >= 0 && Y >= 0 && X < Dimensions.GetWidth() && Y < Dimensions.GetHeight();
	}

	/**
	 * @return true if the given X/Y coordinates are inside the image pixel bounds, ie can be used to index a pixel
	 */
	bool ContainsPixel(const FVector2i& ImageCoords) const
	{
		return ImageCoords.X >= 0 && ImageCoords.Y >= 0 &&
			ImageCoords.X < Dimensions.GetWidth() && ImageCoords.Y < Dimensions.GetHeight();
	}

	/**
	 * Get the Pixel at the given X/Y coordinates
	 */
	PixelType& GetPixel(int32 X, int32 Y)
	{
		int64 LinearIndex = Dimensions.GetIndex(FVector2i(X, Y));
		return Image[LinearIndex];
	}

	/**
	 * Get the Pixel at the given X/Y coordinates
	 */
	PixelType& GetPixel(const FVector2i& ImageCoords)
	{
		int64 LinearIndex = Dimensions.GetIndex(ImageCoords);
		return Image[LinearIndex];
	}

	/**
	 * Get the Pixel at the given linear index
	 */
	PixelType& GetPixel(int64 LinearIndex)
	{
		return Image[LinearIndex];
	}

	/**
	 * Get the Pixel at the given X/Y coordinates
	 */
	const PixelType& GetPixel(int32 X, int32 Y) const
	{
		int64 LinearIndex = Dimensions.GetIndex(FVector2i(X,Y));
		return Image[LinearIndex];
	}

	/**
	 * Get the Pixel at the given X/Y coordinates
	 */
	const PixelType& GetPixel(const FVector2i& ImageCoords) const
	{
		int64 LinearIndex = Dimensions.GetIndex(ImageCoords);
		return Image[LinearIndex];
	}

	/**
	 * Get the Pixel at the given linear index
	 */
	const PixelType& GetPixel(int64 LinearIndex) const
	{
		return Image[LinearIndex];
	}

	TArrayView64<PixelType> GetImageBuffer()
	{
		TArray64<PixelType>& Buffer = Image.GridValues();
		return TArrayView64<PixelType>(Buffer.GetData(), Buffer.Num());
	}

	TConstArrayView64<PixelType> GetImageBuffer() const
	{
		const TArray64<PixelType>& Buffer = Image.GridValues();
		return TConstArrayView64<PixelType>(Buffer.GetData(), Buffer.Num());
	}

	/**
	 * Set the Pixel at the given X/Y coordinates to the given PixelType
	 */
	void SetPixel(int32 X, int32 Y, const PixelType& NewValue)
	{
		int64 LinearIndex = Dimensions.GetIndex(FVector2i(X,Y));
		Image[LinearIndex] = NewValue;
	}

	/**
	 * Set the Pixel at the given X/Y coordinates to the given PixelType
	 */
	void SetPixel(const FVector2i& ImageCoords, const PixelType& NewValue)
	{
		int64 LinearIndex = Dimensions.GetIndex(ImageCoords);
		Image[LinearIndex] = NewValue;
	}


	/**
	 * Set the Pixel at the given linear index to the given PixelType
	 */
	void SetPixel(int64 LinearIndex, const PixelType& NewValue)
	{
		Image[LinearIndex] = NewValue;
	}


	/**
	 * Copy Pixel value from one linear index to another
	 */
	void CopyPixel(int64 FromLinearIndex, int64 ToLinearIndex)
	{
		Image[ToLinearIndex] = Image[FromLinearIndex];
	}


	/**
	 * Convert to a different data type of same Dimensions using ConvertFunc
	 */
	template<typename OtherType>
	void Convert(TFunctionRef<OtherType(const PixelType&)> ConvertFunc,
				 TImageBuilder<OtherType>& ConvertedImageOut) const
	{
		ConvertedImageOut.SetDimensions(Dimensions);
		int64 NumPixels = Dimensions.Num();
		for (int64 k = 0; k < NumPixels; ++k)
		{
			ConvertedImageOut.SetPixel(k, ConvertFunc(Image[k]));
		}
	}


	/**
	 * Sample the image value at floating-point pixel coords with Bilinear interpolation
	 * The pixel coords are expected to be in the [0,Width]x[0,Height] image pixel rectangle.
	 */
	template<typename ScalarType, EImageTilingMethod TilingMethod=EImageTilingMethod::Clamp>
	PixelType BilinearSample(const FVector2d& PixelCoords, const PixelType& InvalidValue) const
	{
		const FVector2i Max = FVector2i(Dimensions.GetWidth(), Dimensions.GetHeight());
		if (PixelCoords.X < 0.0 || PixelCoords.X > Max.X || PixelCoords.Y < 0.0 || PixelCoords.Y > Max.Y)
		{
			return InvalidValue;
		}

		// Compute bounding pixel corners
		const FVector2d XY0d(FMathd::Floor(PixelCoords.X - 0.5), FMathd::Floor(PixelCoords.Y - 0.5));
		const FVector2d XY1d(FMathd::Floor(PixelCoords.X + 0.5), FMathd::Floor(PixelCoords.Y + 0.5));
		FVector2i XY0 = FVector2i(XY0d);
		FVector2i XY1 = FVector2i(XY1d);

		// Compute interpolation factors in [0,1] range.
		// This must be done before wrapped tiling to ensure the [0,1] range.
		// Rather than transforming the two pixel corners (XY0,XY1), we shift
		// our PixelCoords by half a pixel to account for pixel centers.
		const FVector2d A = PixelCoords - FVector2d(0.5) - XY0d;
		const FVector2d OneMinusA = FVector2d::One() - A;

		if constexpr (TilingMethod == EImageTilingMethod::Wrap)
		{
			XY0.X = XY0.X < 0 ? XY0.X + Max.X : XY0.X;
			XY0.Y = XY0.Y < 0 ? XY0.Y + Max.Y : XY0.Y;
			XY1.X = XY1.X >= Max.X ? 0 : XY1.X;
			XY1.Y = XY1.Y >= Max.Y ? 0 : XY1.Y;
		}
		else // TilingMethod == EImageTilingMethod::Clamp
		{
			XY0.X = FMath::Max(0, XY0.X);
			XY0.Y = FMath::Max(0, XY0.Y);
			XY1.X = FMath::Min(Max.X - 1, XY1.X);
			XY1.Y = FMath::Min(Max.Y - 1, XY1.Y);
		}

		PixelType V00 = GetPixel(XY0);
		PixelType V10 = GetPixel(FVector2i(XY1.X, XY0.Y));
		PixelType V01 = GetPixel(FVector2i(XY0.X, XY1.Y));
		PixelType V11 = GetPixel(XY1);

		return V00 * static_cast<ScalarType>(OneMinusA.X * OneMinusA.Y) +
			   V01 * static_cast<ScalarType>(OneMinusA.X * A.Y) +
			   V10 * static_cast<ScalarType>(A.X * OneMinusA.Y) +
			   V11 * static_cast<ScalarType>(A.X * A.Y);
	}


	/**
	 * Sample the image value at floating-point UV coords with Bilinear interpolation.
	 */
	template<typename ScalarType, EImageTilingMethod TilingMethod=EImageTilingMethod::Clamp>
	PixelType BilinearSampleUV(const FVector2d& UVCoords, const PixelType& InvalidValue) const
	{
		const FVector2d UV = GetTiledUV<TilingMethod>(UVCoords);
		
		FVector2d PixelCoords(
			UV.X * (double)Dimensions.GetWidth(),
			UV.Y * (double)Dimensions.GetHeight());

		return BilinearSample<ScalarType, TilingMethod>(PixelCoords, InvalidValue);
	}

	/**
	 * Sample the image value at floating-point UV coords with Nearest sampling.
	 */
	template<EImageTilingMethod TilingMethod=EImageTilingMethod::Clamp>
	PixelType NearestSampleUV(const FVector2d& UVCoords) const
	{
		const FVector2d UV = GetTiledUV<TilingMethod>(UVCoords);
		const int X = (int)(UV.X * (double)Dimensions.GetWidth());
		const int Y = (int)(UV.Y * (double)Dimensions.GetHeight());
		return GetPixel(X, Y);
	}


	/**
	 * @return true if all pixels have the same value
	 */
	bool IsConstantValue() const
	{
		int64 Num = Dimensions.Num();
		if (Num >= 1)
		{
			PixelType InitialValue = Image[0];
			for (int64 k = 1; k < Num; ++k)
			{
				if (Image[k] != InitialValue)
				{
					return false;
				}
			}
		}
		return true;
	}


	/**
	 * Very basic downsampling technqiue that just averages NxN pixel blocks. Multi-threaded.
	 * @param SubSteps each NxN pixel block of this size is averaged into 1 pixel in the output image
	 * @param Zero value to use with template PixelType must be provided, pixel values will be added to this value
	 * @param AverageFunc Called with Sum(Pixels) and PixelCount, return value is used as new pixel value (eg should be average)
	 */
	TImageBuilder<PixelType> FastDownsample(
		int32 SubSteps, 
		const PixelType& ZeroValue, 
		TFunctionRef<PixelType(PixelType, int)> AverageFunc) const
	{
		TImageBuilder<PixelType> DownsampleImage;
		int32 Width = Dimensions.GetWidth();
		int32 Height = Dimensions.GetHeight();

		// can only fast-downsample by even multiple of image size
		if (ensure((Width % SubSteps == 0) && (Height % SubSteps == 0)) == false)
		{
			DownsampleImage = *this;
			return DownsampleImage;
		}

		int32 SubWidth = Width / SubSteps;
		int32 SubHeight = Height / SubSteps;

		FImageDimensions SubDimensions(SubWidth, SubHeight);
		DownsampleImage.SetDimensions(SubDimensions);

		ParallelFor(SubHeight, [&](int32 yi)
		{
			int32 baseyi = yi * SubSteps;
			for (int32 xi = 0; xi < SubWidth; ++xi)
			{
				int32 basexi = xi * SubSteps;

				PixelType AccumBasePixels = ZeroValue;
				for (int32 dy = 0; dy < SubSteps; ++dy)
				{
					for (int32 dx = 0; dx < SubSteps; ++dx)
					{
						AccumBasePixels += GetPixel(basexi + dx, baseyi + dy);
					}
				}
				PixelType SubPixel = AverageFunc(AccumBasePixels, SubSteps * SubSteps);
				DownsampleImage.SetPixel(xi, yi, SubPixel);
			}
		});

		return DownsampleImage;
	}

protected:
	template<EImageTilingMethod TilingMethod>
	static FVector2d GetTiledUV(const FVector2d& UVCoords)
	{
		FVector2d UV(UVCoords);
		if constexpr (TilingMethod == EImageTilingMethod::Wrap)
		{
			UV = UV - FVector2d(FMathd::Floor(UV.X), FMathd::Floor(UV.Y));
		}
		else // TilingMethod == EImageTilingMethod::Clamp
		{
			UV.X = FMathd::Clamp(UV.X, 0.0, 1.0);
			UV.Y = FMathd::Clamp(UV.Y, 0.0, 1.0);
		}
		return UV;
	}
	
};


/**
 * FImageAdapter is a wrapper around different types of TImageBuilder that provides a
 * standard interface, which allows functions that don't need to know about the specific
 * image type to operate on "any" image. For example to allow code that works on a 4-channel
 * image to operate on a single-channel image (in which case the Adapter will expand/collapse
 * the channels automatically).
 */
class FImageAdapter
{
public:

	enum class EImageType
	{
		Float1,
		Float3,
		Float4
	};

	FImageAdapter(TImageBuilder<float>* Image)
	{
		ImageType = EImageType::Float1;
		Image1f = Image;
	}

	FImageAdapter(TImageBuilder<FVector3f>* Image)
	{
		ImageType = EImageType::Float3;
		Image3f = Image;
	}

	FImageAdapter(TImageBuilder<FVector4f>* Image)
	{
		ImageType = EImageType::Float4;
		Image4f = Image;
	}

	void SetDimensions(FImageDimensions Dimensions)
	{
		switch (ImageType)
		{
			case EImageType::Float1:
				Image1f->SetDimensions(Dimensions); break;
			case EImageType::Float3:
				Image3f->SetDimensions(Dimensions); break;
			case EImageType::Float4:
				Image4f->SetDimensions(Dimensions); break;
		}
	}

	FImageDimensions GetDimensions() const
	{
		switch (ImageType)
		{
			case EImageType::Float1:
				return Image1f->GetDimensions();
			case EImageType::Float3:
				return Image3f->GetDimensions();
			case EImageType::Float4:
				return Image4f->GetDimensions();
		}
		return FImageDimensions();
	}

	void SetPixel(const FVector2i& PixelCoords, const FVector4f& FloatPixel)
	{
		switch (ImageType)
		{
			case EImageType::Float1:
				Image1f->SetPixel(PixelCoords, FloatPixel.X);
				break;
			case EImageType::Float3:
				Image3f->SetPixel(PixelCoords, FVector3f(FloatPixel.X, FloatPixel.Y, FloatPixel.Z));
				break;
			case EImageType::Float4:
				Image4f->SetPixel(PixelCoords, FloatPixel);
				break;
		}
	}

	void SetPixel(const FVector2i& PixelCoords, const FLinearColor& FloatPixel)
	{
		switch (ImageType)
		{
		case EImageType::Float1:
			Image1f->SetPixel(PixelCoords, FloatPixel.R);
			break;
		case EImageType::Float3:
			Image3f->SetPixel(PixelCoords, FVector3f(FloatPixel.R, FloatPixel.G, FloatPixel.B));
			break;
		case EImageType::Float4:
			Image4f->SetPixel(PixelCoords, FVector4f(FloatPixel.R, FloatPixel.G, FloatPixel.B, FloatPixel.A));
			break;
		}
	}

	FVector4f GetPixel(int64 LinearIndex) const
	{
		switch (ImageType)
		{
			case EImageType::Float1:
			{
				float Value = Image1f->GetPixel(LinearIndex);
				return FVector4f(Value, Value, Value, 1.0f);
			}
			case EImageType::Float3:
			{
				FVector3f Value = Image3f->GetPixel(LinearIndex);
				return FVector4f(Value.X, Value.Y, Value.Z, 1.0f);
			}
			case EImageType::Float4:
				return Image4f->GetPixel(LinearIndex);
		}
		return FVector4f::One();
	}

	FVector4f GetPixel(const FVector2i& PixelCoords) const
	{
		switch (ImageType)
		{
			case EImageType::Float1:
			{
				float Value = Image1f->GetPixel(PixelCoords);
				return FVector4f(Value, Value, Value, 1.0f);
			}
			case EImageType::Float3:
			{
				FVector3f Value = Image3f->GetPixel(PixelCoords);
				return FVector4f(Value.X, Value.Y, Value.Z, 1.0f);
			}
			case EImageType::Float4:
				return Image4f->GetPixel(PixelCoords);
		}
		return FVector4f::One();
	}


protected:
	EImageType ImageType;

	TImageBuilder<float>* Image1f = nullptr;
	TImageBuilder<FVector3f>* Image3f = nullptr;
	TImageBuilder<FVector4f>* Image4f = nullptr;
};



} // end namespace UE::Geometry
} // end namespace UE