// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * FImageDimensions provides various functions for working with size/indices/coordinates of a
 * 2D image, as well as standard UV spaces
 */
class FImageDimensions
{
protected:
	int32 Width = 0;
	int32 Height = 0;

public:
	FImageDimensions(int32 WidthIn = 0, int32 HeightIn = 0)
	{
		SetDimensions(WidthIn, HeightIn);
	}

	/** Set the dimensions of this image. */
	void SetDimensions(int32 WidthIn, int32 HeightIn)
	{
		check(WidthIn >= 0 && HeightIn >= 0);
		Width = WidthIn;
		Height = HeightIn;
	}

	/** @return width of image */
	int32 GetWidth() const { return Width; }
	/** @return height of image */
	int32 GetHeight() const { return Height; }
	/** @return number of elements in image */
	int64 Num() const { return (int64)Width * (int64)Height; }

	/** @return true if image is square */
	bool IsSquare() const { return Width == Height; }

	/** @return true if coordinates are valid, ie in-bounds of image dimensions */
	bool IsValidCoords(const FVector2i& Coords) const
	{
		return (Coords.X >= 0 && Coords.X < Width && Coords.Y >= 0 && Coords.Y < Height);
	}

	/** Clamp input coordinates to valid range of image coordinates */
	void Clamp(int32& X, int32& Y) const
	{
		X = FMath::Clamp(X, 0, Width - 1);
		Y = FMath::Clamp(Y, 0, Height - 1);
	}

	/** Clamp input coordinates to valid range of image coordinates */
	void Clamp(FVector2i& Coords) const
	{
		Coords.X = FMath::Clamp(Coords.X, 0, Width - 1);
		Coords.Y = FMath::Clamp(Coords.Y, 0, Height - 1);
	}

	/** @return linear index into image from 2D coordinates */
	int64 GetIndex(int32 X, int32 Y) const
	{
		return (int64) Y * Width + X;
	}

	/** @return linear index into image from 2D coordinates */
	int64 GetIndex(const FVector2i& Coords) const
	{
		checkSlow(IsValidCoords(Coords));
		return ((int64)Coords.Y * (int64)Width) + (int64)Coords.X;
	}

	/** @return linear index into image from 2D coordinates, optionally flipped around X and Y axes */
	int64 GetIndexMirrored(const FVector2i& Coords, bool bFlipX, bool bFlipY) const
	{
		checkSlow(IsValidCoords(Coords));
		int64 UseX = (bFlipX) ? (Width - 1 - Coords.X) : Coords.X;
		int64 UseY = (bFlipY) ? (Height - 1 - Coords.Y) : Coords.Y;
		return (UseY * (int64)Width) + UseX;
	}

	/** @return 2D image coordinates from linear index */
	FVector2i GetCoords(int64 LinearIndex) const
	{
		checkSlow(LinearIndex >= 0 && LinearIndex < Num());
		return FVector2i((int32)(LinearIndex % (int64)Width), (int32)(LinearIndex / (int64)Width));
	}

	/** @return Real-valued dimensions of a pixel/texel in the image, relative to default UV space [0..1]^2 */
	FVector2d GetTexelSize() const
	{
		return FVector2d(1.0 / (double)Width, 1.0 / (double)Height);
	}

	/** @return Real-valued position of given texel center in default UV-space [0..1]^2 */
	FVector2d GetTexelUV(const FVector2i& Coords) const
	{
		return FVector2d(
			((double)Coords.X + 0.5) / (double)Width,
			((double)Coords.Y + 0.5) / (double)Height);
	}

	/** @return Real-valued position of given texel center in default UV-space [0..1]^2 */
	FVector2d GetTexelUV(int64 LinearIndex) const
	{
		return GetTexelUV(GetCoords(LinearIndex));
	}

	/** @return The distance, measured in texels, between points P and Q which should be given in the default UV-space [0..1]^2 */
	FVector2d GetTexelDistance(const FVector2d& P, const FVector2d Q) const
	{
		FVector2d TexelDistance = P - Q;
		TexelDistance.X *= Width;
		TexelDistance.Y *= Height;
		return TexelDistance;
	}

	/** @return integer XY coordinates for real-valued XY coordinates (ie texel that contains value, if texel origin is in bottom-left */
	FVector2i PixelToCoords(const FVector2d& PixelPosition) const
	{
		int32 X = FMath::Clamp((int32)PixelPosition.X, 0, Width - 1);
		int32 Y = FMath::Clamp((int32)PixelPosition.Y, 0, Height - 1);
		return FVector2i(X, Y);
	}

	/** @return integer XY coordinates for UV coordinates, assuming default UV space [0..1]^2 */
	FVector2i UVToCoords(const FVector2d& UVPosition) const
	{
		double PixelX = (UVPosition.X * (double)Width /*- 0.5*/);
		double PixelY = (UVPosition.Y * (double)Height /*- 0.5*/);
		return PixelToCoords(FVector2d(PixelX, PixelY));
	}

	bool operator==(const FImageDimensions& Other) const
	{
		return Width == Other.Width && Height == Other.Height;
	}

	bool operator!=(const FImageDimensions& Other) const
	{
		return !(*this == Other);
	}
};


} // end namespace UE::Geometry
} // end namespace UE


