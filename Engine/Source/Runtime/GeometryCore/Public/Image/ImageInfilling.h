// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntVectorTypes.h"
#include "Templates/Tuple.h"
#include "Util/IndexUtil.h"
#include "VectorTypes.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageBuilder.h"
#include "Util/IndexPriorityQueue.h"

namespace UE
{
namespace Geometry
{


/**
 * TMarchingPixelInfill implements a very basic infilling strategy where the missing pixel with the most "known"
 * neighbours is set to the average value of those neighbours, and this is iterated until all missing pixels are known.
 * A Priority Queue is used to track the active front. 
 * 
 * The infill sequence is tracked, which allows it to be "replayed" on multiple images. This is useful 
 * in cases where we have separate images with the same holes, which arises when baking textures from world sampling.
 */
template<typename PixelType>
class TMarchingPixelInfill
{
public:
	// Encoding of infill operations, so that they can be applied to multiple images
	// Values are [ NumPixels, Pixel1Count, Pixel1X, Pixel1Y, Pixel1Nbr1X, Pixel1Nbr1Y, ...
	TArray<int32> InfillSequence;


	/**
	 * Fill the values of MissingPixels in Image by propagating from known values.
	 * @param MissingValue this value is used to indicate 'missing' in the image. Input pixels do not have to be set to this value, they are set internally to avoid map lookups 
	 * @param NormalizeFunc Implements an "average" operation for the template PixelType, normally (SumOfNbrValues / Count)
	 */
	void ComputeInfill(
		TImageBuilder<PixelType>& Image,
		const TArray<FVector2i>& MissingPixels,
		PixelType MissingValue,
		TFunctionRef<PixelType(PixelType SumOfNbrValues, int NbrCount)> NormalizeFunc)
	{
		InfillSequence.Reset();
		InfillSequence.Add(0);

		// clear all missing values
		for (int32 k = 0; k < MissingPixels.Num(); ++k)
		{
			Image.SetPixel(MissingPixels[k], MissingValue);
		}

		// march inward from hole boundary and compute fill values. This is probably not the
		// right way to go and a smooth interpolant (eg mean-value) would be preferable
		FIndexPriorityQueue ActiveSetQueue(MissingPixels.Num());
		TMap<FVector2i, int32> PixelToIndexMap;
		for (int32 k = 0; k < MissingPixels.Num(); ++k)
		{
			int32 Count = 0;
			FVector2i CenterCoords = MissingPixels[k];
			for (int32 j = 0; j < 8; ++j)
			{
				FVector2i NbrCoords = CenterCoords + IndexUtil::GridOffsets8[j];
				if ( Image.ContainsPixel(NbrCoords) && Image.GetPixel(NbrCoords) != MissingValue)
				{
					Count++;
				}
			}
			ActiveSetQueue.Insert(k, float(8 - Count));
			PixelToIndexMap.Add(CenterCoords, k);
		}


		while (ActiveSetQueue.GetCount() > 0)
		{
			int32 Index = ActiveSetQueue.Dequeue();
			FVector2i CenterCoords = MissingPixels[Index];

			int32 CountIdx = InfillSequence.Num();
			InfillSequence.Add(1);
			InfillSequence.Add(CenterCoords.X);
			InfillSequence.Add(CenterCoords.Y);
			InfillSequence[0]++;

			// accumulate neighbour values in 8-neighbourhood
			PixelType ValidNbrAvg = PixelType::Zero();
			int32 ValidCount = 0;
			for (int32 j = 0; j < 8; ++j)
			{
				FVector2i NbrCoords = CenterCoords + IndexUtil::GridOffsets8[j];
				if (Image.ContainsPixel(NbrCoords))
				{
					PixelType PixelColor = Image.GetPixel(NbrCoords);
					if (PixelColor != MissingValue)
					{
						ValidNbrAvg += PixelColor;
						++ValidCount;
						InfillSequence.Add(NbrCoords.X);
						InfillSequence.Add(NbrCoords.Y);
						InfillSequence[CountIdx]++;
					}
				}
			}

			if (ValidCount > 0)
			{
				// update center pixel
				PixelType InfillValue = NormalizeFunc(ValidNbrAvg, ValidCount);
				Image.SetPixel(CenterCoords, InfillValue);

				// Update queue for nbrs. We set one pixel so we subtract one from
				// all those neighbours
				for (int32 j = 0; j < 8; ++j)
				{
					FVector2i NbrCoords = CenterCoords + IndexUtil::GridOffsets8[j];
					if (Image.ContainsPixel(NbrCoords) && Image.GetPixel(NbrCoords) == MissingValue)
					{
						const int32* FoundIndex = PixelToIndexMap.Find(NbrCoords);
						if (FoundIndex != nullptr && ActiveSetQueue.Contains(*FoundIndex))
						{
							float CurPriority = ActiveSetQueue.GetPriority(*FoundIndex);
							ActiveSetQueue.Update(*FoundIndex, CurPriority - 1.0f);
						}
					}
				}
			}
		}
	}


	/**
	 * Fill the missing values in Image by replaying the infill sequence computed by ComputeInfill()
	 * @param NormalizeFunc Implements an "average" operation for the template OtherPixelType, normally (SumOfNbrValues / NbrCount)
	 */
	template<typename OtherPixelType>
	void ApplyInfill(
		TImageBuilder<OtherPixelType>& Image,
		TFunctionRef<OtherPixelType(OtherPixelType SumOfNbrValues, int NbrCount)> NormalizeFunc ) const
	{
		int32 InfillPixels = InfillSequence[0];
		int32 i = 1;
		for (int32 pi = 0; pi < InfillPixels; ++pi)
		{
			int32 PixelCount = InfillSequence[i++];
			FVector2i CenterCoords;
			CenterCoords.X = InfillSequence[i++];
			CenterCoords.Y = InfillSequence[i++];
			if (PixelCount > 1)
			{
				OtherPixelType NbrSum = OtherPixelType::Zero();
				int32 NbrCount = 0;
				for (int32 j = 1; j < PixelCount; ++j)
				{
					FVector2i NbrCoords;
					NbrCoords.X = InfillSequence[i++];
					NbrCoords.Y = InfillSequence[i++];
					NbrSum += Image.GetPixel(NbrCoords);
					NbrCount++;
				}
				OtherPixelType InfillValue = NormalizeFunc(NbrSum, NbrCount);
				Image.SetPixel(CenterCoords, InfillValue);
			}
		}
	};


	/**
	 * Fill the missing values in Image by replaying the infill sequence computed by ComputeInfill()
	 * @param NormalizeFunc Implements an "average" operation for the template PixelType, normally (SumOfNbrValues / NbrCount)
	 */
	void ApplyInfill(
		TImageBuilder<PixelType>& Image,
		TFunctionRef<PixelType(PixelType SumOfNbrValues, int NbrCount)> NormalizeFunc) const
	{
		ApplyInfill<PixelType>(Image, NormalizeFunc);
	}

};





} // end namespace UE::Geometry
} // end namespace UE