// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Tuple.h"
#include "VectorTypes.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageTile.h"
#include "Spatial/MeshAABBTree3.h"
#include "Sampling/GridSampler.h"
#include "MathUtil.h"

namespace UE
{
namespace Geometry
{

/**
 * ImageOccupancyMap calculates and stores coverage information for a 2D image/texture,
 * for example coverage derived from UV islands of a mesh, 2D polygons, etc.
 * 
 * An optional set of gutter texels can be calculated, and correspondence between gutter
 * texels and the nearest interior texel is stored.
 *
 * In addition, a 2D coordinate (eg UV) and integer ID (eg Triangle ID) of each texel can be calculated/stored. 
 * This is not just a cache. For 'border' texels where the texel center is technically outside the mesh/polygon,
 * but the texel rectangle may still overlap the shape, the nearest UV/Triangle is stored.
 * This simplifies computing samples around the borders such that the shape is covered under linear interpolatione/etc.
 * 
 * TODO Some of the names in this class are misleading because the name mentions "Texel" but should actually say "Sample"
 * this is because the class initially supported only one sample per texel but later multi-sampling was added
 */
class FImageOccupancyMap
{
public:
	/** Image Dimensions */
	FImageDimensions Dimensions;
	FImageTile Tile;

	/** Width of the gutter. This is actually multiplied by the diagonal length of a texel, so
	    the gutter is generally larger than this number of pixels */
	int32 GutterSize = 4;

	// texel types
	static constexpr int8 EmptyTexel = 0;
	static constexpr int8 InteriorTexel = 1;
	static constexpr int8 BorderTexel = 2;
	static constexpr int8 GutterTexel = 3;

	/**
	 * Classification of each sample in Tile.
	 * Size = Tile.GetWidth() x Tile.GetHeight() x SamplesPerPixel.
	 */
	TArray64<int8> TexelType;

	/**
	 * Count of interior samples for each texel in Tile.
	 * Size = Tile.GetWidth() x Tile.GetHeight().
	 */
	TArray64<int32> TexelInteriorSamples;

	/**
	 * UV for each sample in Tile.
	 * Size = Tile.GetWidth() x Tile.GetHeight() x SamplesPerPixel.
	 */
	TArray64<FVector2f> TexelQueryUV;

	/**
	 * Integer/Triangle ID for each sample in Tile.
	 * Size = Tile.GetWidth() x Tile.GetHeight() x SamplesPerPixel.
	 */
	TArray64<int32> TexelQueryTriangle;

	/**
	 * UV Chart ID for each texel in Tile. Only set if UVSpaceMeshTriCharts is provided
	 * Size = Tile.GetWidth() x Tile.GetHeight().
	 */
	TArray64<int32> TexelQueryUVChart;

	/**
	 * Set of Border Texels, Pair is <LinearIndexOfBorderTexel, LinearIndexOfNearestInteriorTexel>, so
	 * Border can be filled by directly copying from source to target. 
	 */
	TArray64<TTuple<int64, int64>> BorderTexels;
	
	/**
	 * Set of Gutter Texels. Pair is <LinearIndexOfGutterTexel, LinearIndexOfNearestInteriorTexel>, so
	 * Gutter can be filled by directly copying from source to target.
	 */
	TArray64<TTuple<int64, int64>> GutterTexels;

	using FGridSampler = TGridSampler<double>;
	FGridSampler PixelSampler = FGridSampler(1);

	bool bParallel = true;


	void Initialize(FImageDimensions DimensionsIn, int32 SamplesPerPixel = 1)
	{
		check(DimensionsIn.IsSquare()); // are we sure it works otherwise?
		Dimensions = DimensionsIn;
		Tile = FImageTile(FVector2i(0,0), FVector2i(Dimensions.GetWidth(), Dimensions.GetHeight()));
		InitializePixelSampler(SamplesPerPixel);
	}

	void Initialize(FImageDimensions DimensionsIn, const FImageTile& TileIn, int32 SamplesPerPixel = 1)
	{
		check(DimensionsIn.IsSquare());
		Dimensions = DimensionsIn;
		Tile = TileIn;
		InitializePixelSampler(SamplesPerPixel);
	}

	/**
	 * @return true if texel sample at this sample linear index is an Interior texel
	 */
	bool IsInterior(int64 LinearIndex) const
	{
		return TexelType[LinearIndex] == InteriorTexel;
	}

	/** @return number of interior samples for the texel at this image linear index. */
	int32 TexelNumSamples(int64 LinearIndex) const
	{
		return TexelInteriorSamples[LinearIndex];
	}

	/** Set texel type at the given X/Y */
	//void SetTexelType(int64 X, int64 Y, int8 Type)
	//{
	//	TexelType[Dimensions.GetIndex(X, Y)] = Type;
	//}

	template<typename MeshType, typename GetTriangleIDFuncType>
	bool ClassifySamplesFromUVSpaceMesh(
		const MeshType& UVSpaceMesh,
		GetTriangleIDFuncType GetTriangleIDFunc = [](int32 TriangleID) { return TriangleID; },
		const TArray<int32>* UVSpaceMeshTriCharts = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FImageOccupancyMap::ClassifySamples_NoSpatial);
		
		// make flat mesh
		TMeshAABBTree3<MeshType> UVSpaceMeshSpatial(&UVSpaceMesh, true);
		return ClassifySamplesFromUVSpaceMesh(UVSpaceMesh, UVSpaceMeshSpatial, GetTriangleIDFunc, UVSpaceMeshTriCharts);
	}

	template<typename MeshType, typename GetTriangleIDFuncType>
	bool ClassifySamplesFromUVSpaceMesh(
		const MeshType& UVSpaceMesh,
		const TMeshAABBTree3<MeshType>& UVSpaceMeshSpatial,
		GetTriangleIDFuncType GetTriangleIDFunc = [](int32 TriangleID) { return TriangleID; },
		const TArray<int32>* UVSpaceMeshTriCharts = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FImageOccupancyMap::ClassifySamples);

		const int64 LinearImageSize = Tile.Num();
		TexelInteriorSamples.Init(0, LinearImageSize);
		TexelQueryUVChart.Init(IndexConstants::InvalidID, LinearImageSize);
		
		const int64 LinearSampleSize = Tile.Num() * PixelSampler.Num();
		TexelType.Init(EmptyTexel, LinearSampleSize);
		TexelQueryUV.Init(FVector2f::Zero(), LinearSampleSize);
		TexelQueryTriangle.Init(IndexConstants::InvalidID, LinearSampleSize);

		const FVector2d TexelSize = Dimensions.GetTexelSize();
		const double TexelDiag = TexelSize.Length();

		IMeshSpatial::FQueryOptions QueryOptions;
		QueryOptions.MaxDistance = (double)GutterSize * TexelDiag;

		const int32 NumUVSpaceMeshTriCharts = UVSpaceMeshTriCharts ? UVSpaceMeshTriCharts->Num() : 0;

		// Classify texel samples
		ParallelFor(Tile.GetHeight(),
					[this, &UVSpaceMesh, &GetTriangleIDFunc, &UVSpaceMeshSpatial, TexelDiag, &QueryOptions, &TexelSize, UVSpaceMeshTriCharts, NumUVSpaceMeshTriCharts]
		(int32 ImgY)
		{
			for (int32 ImgX = 0; ImgX < Tile.GetWidth(); ++ImgX)
			{
				const FVector2i SourceCoords = Tile.GetSourceCoords(ImgX, ImgY);
				const int64 SourceTexelLinearIdx = Dimensions.GetIndex(SourceCoords.X, SourceCoords.Y);
				const int64 TexelLinearIdx = Tile.GetIndex(ImgX, ImgY);
				const FVector2d SourceTexelCenterUV = Dimensions.GetTexelUV(SourceTexelLinearIdx);

				for (int32 Sample = 0; Sample < PixelSampler.Num(); ++Sample)
				{
					const int64 SampleLinearIdx = TexelLinearIdx * PixelSampler.Num() + Sample;
					const FVector2d SampleUV = PixelSampler.Sample(Sample);
					const FVector2d UVPoint = SourceTexelCenterUV - 0.5 * TexelSize + SampleUV * TexelSize;
					const FVector3d UVPoint3d(UVPoint.X, UVPoint.Y, 0);

					double NearDistSqr;
					const int32 NearestTriID = UVSpaceMeshSpatial.FindNearestTriangle(UVPoint3d, NearDistSqr, QueryOptions);
					if (NearestTriID >= 0)
					{
						FVector3d A, B, C;
						UVSpaceMesh.GetTriVertices(NearestTriID, A, B, C);
						const FTriangle2d UVTriangle(GetXY(A), GetXY(B), GetXY(C));

						const int32 TriId = GetTriangleIDFunc(NearestTriID);
						if (UVTriangle.IsInsideOrOn(UVPoint))
						{
							TexelType[SampleLinearIdx] = InteriorTexel;
							TexelQueryUV[SampleLinearIdx] = (FVector2f)UVPoint;
							TexelQueryTriangle[SampleLinearIdx] = TriId;
							++TexelInteriorSamples[TexelLinearIdx];
						}
						else if (NearDistSqr < TexelDiag * TexelDiag)
						{
							const FDistPoint3Triangle3d DistQuery = TMeshQueries<MeshType>::TriangleDistance(UVSpaceMesh, NearestTriID, UVPoint3d);
							FVector2d NearestUV = GetXY(DistQuery.ClosestTrianglePoint);

							// nudge point into triangle to improve numerical behavior of things like barycentric coord calculation
							// TODO Hmm this nudging could invalidate TriId? Maybe users should do it explicitly and then deal with/ignore this problem themselves
							NearestUV += (10.0 * FMathf::ZeroTolerance) * Normalized(NearestUV - UVPoint);

							// TODO Use BorderTexel here, and we should probably enable those
							TexelType[SampleLinearIdx] = InteriorTexel;
							TexelQueryUV[SampleLinearIdx] = (FVector2f)NearestUV;
							TexelQueryTriangle[SampleLinearIdx] = TriId;
							++TexelInteriorSamples[TexelLinearIdx];
						}
						else
						{
							TexelType[SampleLinearIdx] = GutterTexel;
							const FDistPoint3Triangle3d DistQuery = TMeshQueries<MeshType>::TriangleDistance(UVSpaceMesh, NearestTriID, UVPoint3d);
							const FVector2d NearestUV = GetXY(DistQuery.ClosestTrianglePoint);

							TexelQueryUV[SampleLinearIdx] = (FVector2f)NearestUV;
							TexelQueryTriangle[SampleLinearIdx] = TriId;
						}

						if (TriId < NumUVSpaceMeshTriCharts && TexelQueryUVChart[TexelLinearIdx] == IndexConstants::InvalidID)
						{
							TexelQueryUVChart[TexelLinearIdx] = (*UVSpaceMeshTriCharts)[TriId];
						}
					}
				} // end Sample loop
			} // end scanline loop
		}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

		return true;
	}

	bool ComputeGutterTexelsFromGutterSamples()
	{
		TAtomic<int64> TotalGutterCounter = 0;
		TArray< TArray64<TTuple<int64, int64>> > GutterTexelsPerScanline;
		GutterTexelsPerScanline.SetNum(Tile.GetHeight());

		ParallelFor(Tile.GetHeight(), [this, &TotalGutterCounter, &GutterTexelsPerScanline]
		(int32 ImgY)
		{
			for (int32 ImgX = 0; ImgX < Tile.GetWidth(); ++ImgX)
			{
				const FVector2i SourceCoords = Tile.GetSourceCoords(ImgX, ImgY);
				const int64 SourceTexelLinearIdx = Dimensions.GetIndex(SourceCoords.X, SourceCoords.Y);
				const int64 TexelLinearIdx = Tile.GetIndex(ImgX, ImgY);
				const TTuple<int64, int64> InvalidGutterNearestTexel(-1, -1);

				// With multiple samples, a texel may contain multiple gutter samples.
				// Since we copy nearest interior texel over top our gutter texels, we
				// should only consider a texel a gutter texel iff all samples in a
				// texel are gutter texels.
				TTuple<int64, int64> GutterNearestTexel = InvalidGutterNearestTexel;
				for (int32 Sample = 0; Sample < PixelSampler.Num(); ++Sample)
				{
					const int64 SampleLinearIdx = TexelLinearIdx * PixelSampler.Num() + Sample;

					if (TexelType[SampleLinearIdx] == InteriorTexel)
					{
						GutterNearestTexel = InvalidGutterNearestTexel;
						break;
					}

					if (TexelType[SampleLinearIdx] == GutterTexel)
					{
						const FVector2d NearestUV = (FVector2d)TexelQueryUV[SampleLinearIdx];
						const FVector2i NearestCoords = Dimensions.UVToCoords(NearestUV);

						// To avoid artifacts with mipmapping the gutter texels store the nearest
						// interior texel and a post pass copies those nearest interior texel values
						// to each gutter pixel.
						//
						// TODO There can be interior texels closer to the gutter the texel than the one we find here.
						// Search :NearestInteriorGutterTexel for a related problem in the MeshMapBaker.
						//
						const int64 NearestLinearIdx = Dimensions.GetIndex(NearestCoords);
						GutterNearestTexel = TTuple<int64, int64>(SourceTexelLinearIdx, NearestLinearIdx);
					}
				} // end Sample loop

				if (GutterNearestTexel != InvalidGutterNearestTexel)
				{
					GutterTexelsPerScanline[ImgY].Add(GutterNearestTexel);
				}
			} // end scanline loop

			TotalGutterCounter += GutterTexelsPerScanline[ImgY].Num();

		}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

		// Now build GutterTexels
		GutterTexels.Reserve(TotalGutterCounter);
		for (const TArray64<TTuple<int64, int64>>& LineGutterTexels : GutterTexelsPerScanline)
		{
			GutterTexels.Append(LineGutterTexels);
		}

		return true;
	}

	/**
	 * Computes the image occupancy map from a UV space mesh. This function classifies all the samples in the image,
	 * and computes the GutterTexels mapping, if you dont want the mapping you can call ClassifySamplesFromUVSpaceMesh
	 * and compute it yourself (eg for texture baking we use filter kernel coverage to compute the GutterTexels mapping)
	 * 
	 * @param UVSpaceMesh UV space mesh to compute the occupancy map from.
	 * @param GetTriangleIDFunc Lambda to remap a texel's nearest triangle ID
	 * @param UVSpaceMeshTriCharts Optional UVSpaceMesh triangle ID to UV chart map
	 */
	template<typename MeshType, typename GetTriangleIDFuncType>
	bool ComputeFromUVSpaceMesh(
		const MeshType& UVSpaceMesh, 
		GetTriangleIDFuncType GetTriangleIDFunc = [](int32 TriangleID) { return TriangleID; },
		const TArray<int32>* UVSpaceMeshTriCharts = nullptr)
	{
		ClassifySamplesFromUVSpaceMesh(UVSpaceMesh, GetTriangleIDFunc, UVSpaceMeshTriCharts);

		ComputeGutterTexelsFromGutterSamples();

		return true;
	}




	template<typename TexelValueType>
	void ParallelProcessingPass(
		TFunctionRef<TexelValueType(int64 LinearIdx)> BeginTexel,
		TFunctionRef<void(int64 LinearIdx, float Weight, TexelValueType&)> AccumulateTexel,
		TFunctionRef<void(int64 LinearIdx, float Weight, TexelValueType&)> CompleteTexel,
		TFunctionRef<void(int64 LinearIdx, TexelValueType&)> WriteTexel,
		TFunctionRef<float(const FVector2i& TexelOffset)> WeightFunction,
		int32 FilterWidth,
		TArray<TexelValueType>& PassBuffer
	) const
	{
		int64 N = Dimensions.Num();
		checkSlow(N <= TMathUtilConstants<int32>::MaxReal); // TArray< , FDefaultAllocator>::SetNum(int32 ), so max Dimension ~ 65k x 65k
		int32 N32 =  int32(N);
		PassBuffer.SetNum( N32 );
		
		ParallelFor(Dimensions.GetHeight(), 
			[this, &BeginTexel, & AccumulateTexel, &CompleteTexel, &WeightFunction, FilterWidth, &PassBuffer]
		(int32 ImgY)
		{
			for (int32 ImgX = 0; ImgX < Dimensions.GetWidth(); ++ImgX)
			{
				int32 LinearIdx = (int32)Dimensions.GetIndex(ImgX, ImgY);
				if (TexelType[LinearIdx] != EmptyTexel)
				{
					TexelValueType AccumValue = BeginTexel(LinearIdx);
					float WeightSum = 0;

					FVector2i Coords(ImgX, ImgY);
					FVector2i MaxNbr = Coords + FVector2i(FilterWidth, FilterWidth);
					Dimensions.Clamp(MaxNbr);
					FVector2i MinNbr = Coords - FVector2i(FilterWidth, FilterWidth);
					Dimensions.Clamp(MinNbr);

					for (int32 Y = MinNbr.Y; Y <= MaxNbr.Y; ++Y)
					{
						for (int32 X = MinNbr.X; X <= MaxNbr.X; ++X)
						{
							FVector2i NbrCoords(X, Y);
							if (Dimensions.IsValidCoords(NbrCoords))
							{
								FVector2i Offset = NbrCoords - Coords;
								int64 LinearNbrIndex = Dimensions.GetIndex(NbrCoords);
								if (TexelType[LinearNbrIndex] != EmptyTexel)
								{
									float NbrWeight = WeightFunction(Offset);
									AccumulateTexel(LinearNbrIndex, NbrWeight, AccumValue);
									WeightSum += NbrWeight;
								}
							}
						}
					}

					CompleteTexel(LinearIdx, WeightSum, AccumValue);
					PassBuffer[LinearIdx] = AccumValue;
				}
			}
		});

		// write results
		for (int32 k = 0; k < N32; ++k)
		{
			if (TexelType[k] != EmptyTexel)
			{
				WriteTexel(k, PassBuffer[k]);
			}
		}
	}

protected:
	/**
	 * Initialize the pixel sampler.
	 * @param SamplesPerPixelIn The desired number of samples per pixel
	 */
	void InitializePixelSampler(const int32 SamplesPerPixelIn)
	{
		const float GridDimensionFloat = SamplesPerPixelIn > 0 ? static_cast<float>(SamplesPerPixelIn) : 1.0f;
		const int32 GridDimension = FMath::FloorToInt32(FMathf::Clamp(FMathf::Sqrt(GridDimensionFloat), 1.0f, GridDimensionFloat));
		PixelSampler = FGridSampler(GridDimension);
	}

};


} // end namespace UE::Geometry
} // end namespace UE