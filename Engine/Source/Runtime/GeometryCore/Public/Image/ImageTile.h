// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntVectorTypes.h"
#include "Image/ImageDimensions.h"

namespace UE
{
namespace Geometry
{

class FImageTile
{
private:
	FVector2i Start = FVector2i::Zero();
	FVector2i End = FVector2i::Zero();
	int Width = 0;
	int Height = 0;

public:
	FImageTile() = default;
	FImageTile(const FVector2i& InStart, const FVector2i& InEnd)
		: Start(InStart)
		, End(InEnd)
		, Width(InEnd.X - InStart.X)
		, Height(InEnd.Y - InStart.Y)
	{
	}

	FVector2i GetStart() const
	{
		return Start;
	}

	FVector2i GetEnd() const
	{
		return End;
	}

	int32 GetWidth() const
	{
		return Width;
	}

	int32 GetHeight() const
	{
		return Height;
	}

	int64 Num() const
	{
		return Width * Height;
	}

	/** @return the linear index for the given local XY coordinates into this tile. */
	int64 GetIndex(const int32 X, const int32 Y) const
	{
		return Y * Width + X;
	}

	/** @return the linear index for the given local coordinates into this tile. */
	int64 GetIndex(const FVector2i& LocalCoords) const
	{
		return GetIndex(LocalCoords.X, LocalCoords.Y);
	}

	/** @return the coordinates into this tile given the linear index into this tile. */
	FVector2i GetCoords(const int64 LinearIdx) const
	{
		return FVector2i((int32)(LinearIdx % Width), (int32)(LinearIdx / Width));
	}

	/** @return true if the given source coordinates are contained by this tile. */
	bool Contains(const int32 X, const int32 Y) const
	{
		return (X >= Start.X && X < End.X && Y >= Start.Y && Y < End.Y);
	}

	/** @return the linear index for the given source coordinates. */
	int64 GetIndexFromSourceCoords(const int32 X, const int32 Y) const
	{
		checkSlow(Contains(X, Y));
		return GetIndex(X - Start.X, Y - Start.Y); 
	}

	/** @return the linear index for the given source coordinates. */
	int64 GetIndexFromSourceCoords(const FVector2i& SourceCoords) const
	{
		return GetIndexFromSourceCoords(SourceCoords.X, SourceCoords.Y); 
	}

	/** @return the source image's coordinates given linear index into this tile. */
	FVector2i GetSourceCoords(const int64 LinearIdx) const
	{
		return FVector2i(Start.X + int32(LinearIdx % Width), Start.Y + int32(LinearIdx / Width));
	}

	/** @return the source image's coordinates given local XY coordinates into this tile. */
	FVector2i GetSourceCoords(const int32 X, const int32 Y) const
	{
		return FVector2i(Start.X + X, Start.Y + Y);
	}

	/** @return the source image's coordinates given local coordinates into this tile. */
	FVector2i GetSourceCoords(const FVector2i& LocalCoords) const
	{
		return GetSourceCoords(LocalCoords.X, LocalCoords.Y);
	}
};

class FImageTiling
{
private:
	FImageDimensions Dimensions;
	int32 TileWidth = 32;
	int32 TileHeight = 32;

public:
	FImageTiling(const FImageDimensions DimensionsIn, const int32 TileWidthIn, const int32 TileHeightIn)
	{
		SetTiling(DimensionsIn, TileWidthIn, TileHeightIn);
	}

	void SetTiling(const FImageDimensions DimensionsIn, const int32 TileWidthIn, const int32 TileHeightIn)
	{
		checkSlow(TileWidthIn >= 1 && TileHeightIn >= 1);
		Dimensions = DimensionsIn;
		TileWidth = FMath::Clamp(TileWidthIn, 1, FMath::Max(1, Dimensions.GetWidth()));
		TileHeight = FMath::Clamp(TileHeightIn, 1, FMath::Max(1, Dimensions.GetHeight()));
	}

	int32 NumTilesX() const
	{
		return Dimensions.GetWidth() / TileWidth + (Dimensions.GetWidth() % TileWidth == 0 ? 0 : 1);
	}

	int32 NumTilesY() const
	{
		return Dimensions.GetHeight() / TileHeight + (Dimensions.GetHeight() % TileHeight == 0 ? 0 : 1);
	}

	int32 Num() const
	{
		return NumTilesX() * NumTilesY();
	}

	/** @return the tile data corresponding to the linear index [0, NumTiles()-1] w/ optional padding. */
	FImageTile GetTile(const int32 LinearTileIndex, const int32 Padding = 0) const
	{
		const int32 TilePadding = Padding < 0 ? 0 : Padding;
		const FVector2i TileIndex(LinearTileIndex % NumTilesX(), LinearTileIndex / NumTilesX());
		
		FVector2i TileStart, TileEnd;
		TileStart.X = FMath::Clamp(TileIndex.X * TileWidth - TilePadding, 0, Dimensions.GetWidth());
		TileStart.Y = FMath::Clamp(TileIndex.Y * TileHeight - TilePadding, 0, Dimensions.GetHeight());
		TileEnd.X = FMath::Clamp((TileIndex.X + 1) * TileWidth + TilePadding, 0, Dimensions.GetWidth());
		TileEnd.Y = FMath::Clamp((TileIndex.Y + 1) * TileHeight + TilePadding, 0, Dimensions.GetHeight());
		return FImageTile(TileStart, TileEnd);
	}
};
	
} // end namespace UE::Geometry
} // end namespace UE