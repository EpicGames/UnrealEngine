// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad include. Some headers are in Engine

#include "VectorTypes.h"
#include "FrameTypes.h"
#include "SceneView.h"
#include "Image/ImageBuilder.h"
#include "Image/ImageDimensions.h"


namespace UE
{
namespace Geometry
{

struct FSpatialPhotoParams
{
	/** Coordinate system of the view camera - X() is forward, Z() is up */
	FFrame3d Frame;

	/**
	 * Near-plane distance of the camera. Image pixels lie on this plane.
	 *
	 * Note: For best depth precision, the near plane should be placed as close as possible to the scene being captured.
	 * For more on depth precision, see: https://www.reedbeta.com/blog/depth-precision-visualized/
	 */	
	double NearPlaneDist = 1.0;

	/** Horizontal Field-of-View of the camera in degrees (full FOV, so generally calculations will use half this value) */
	double HorzFOVDegrees = 90.0;

	/** Pixel dimensions of the photo image */
	FImageDimensions Dimensions;

	bool operator==(const FSpatialPhotoParams& Other) const
	{
		return Frame.Origin == Other.Frame.Origin &&
			static_cast<const FVector4d>(Frame.Rotation) == static_cast<const FVector4d>(Other.Frame.Rotation) &&
			NearPlaneDist == Other.NearPlaneDist &&
			HorzFOVDegrees == Other.HorzFOVDegrees &&
			Dimensions == Other.Dimensions;
	}
};

/**
 * TSpatialPhoto represents a 2D image located in 3D space, ie the image plus camera parameters, 
 * which is essentially a "Photograph" of some 3D scene (hence the name)
 */
template<typename PixelType>
struct TSpatialPhoto
{
	/** Coordinate system of the view camera - X() is forward, Z() is up */
	FFrame3d Frame;

	/** Near-plane distance for the camera, image pixels lie on this plane */
	double NearPlaneDist = 1.0;
	/** Horizontal Field-of-View of the camera in degrees (full FOV, so generally calculations will use half this value) */
	double HorzFOVDegrees = 90.0;
	/** Pixel dimensions of the photo image */
	FImageDimensions Dimensions;
	/** Pixels of the image */
	TImageBuilder<PixelType> Image;
};
typedef TSpatialPhoto<FVector4f> FSpatialPhoto4f;
typedef TSpatialPhoto<FVector3f> FSpatialPhoto3f;
typedef TSpatialPhoto<float> FSpatialPhoto1f;



/**
 * TSpatialPhotoSet is a set of TSpatialPhotos. 
 * The ComputeSample() function can be used to determine the value "seen"
 * by the photo set at a given 3D position/normal, if possible.
 */
template<typename PixelType, typename RealType>
class TSpatialPhotoSet
{
public:
	
	/** Add a photo to the photo set via move operation */
	void Add(TSpatialPhoto<PixelType>&& Photo)
	{
		TSharedPtr<TSpatialPhoto<PixelType>, ESPMode::ThreadSafe> NewPhoto = MakeShared<TSpatialPhoto<PixelType>, ESPMode::ThreadSafe>(MoveTemp(Photo));
		Photos.Add(NewPhoto);
	}

	/** @return the number of photos in the photo set */
	int32 Num() const { return Photos.Num(); }

	/** @return removes all photos from the photo set */
	void Empty() { return Photos.Empty(); }

	/** @return the photo at the given index */
	const TSpatialPhoto<PixelType>& Get(int32 Index) const { return *Photos[Index]; }

	/**
	 * Estimate a pixel value at the given 3D Position/Normal using the PhotoSet. 
	 * This is effectively a reprojection process, that tries to find the "best"
	 * pixel value in the photo set that projects onto the given Position/Normal.
	 * 
	 * A position may be visible from multiple photos, in this case the dot-product
	 * between the view vector and normal is used to decide which photo pixel to use.
	 * 
	 * VisibilityFunction is used to determine if a 3D point is visible from the given photo point.
	 * Generally the caller would implement some kind of raycast to do this.
	 * 
	 * @returns the best valid sample if found, or DefaultValue if no suitable sample is available
	 */
	PixelType ComputeSample(
		const FVector3d& Position, 
		const FVector3d& Normal, 
		TFunctionRef<bool(const FVector3d&,const FVector3d&)> VisibilityFunction,
		const PixelType& DefaultValue
	) const;

	PixelType ComputeSample(
		const int& PhotoIndex,
		const FVector2d& PhotoCoords,
		const PixelType& DefaultValue
	) const;
	
	PixelType ComputeSampleNearest(
		const int& PhotoIndex,
		const FVector2d& PhotoCoords,
		const PixelType& DefaultValue
	) const;

	bool ComputeSampleLocation(
		const FVector3d& Position, 
		const FVector3d& Normal, 
		TFunctionRef<bool(const FVector3d&,const FVector3d&)> VisibilityFunction,
		int& PhotoIndex,
		FVector2d& PhotoCoords
	) const;

protected:
	TArray<TSharedPtr<TSpatialPhoto<PixelType>, ESPMode::ThreadSafe>> Photos;
};
typedef TSpatialPhotoSet<FVector4f, float> FSpatialPhotoSet4f;
typedef TSpatialPhotoSet<FVector3f, float> FSpatialPhotoSet3f;
typedef TSpatialPhotoSet<float, float> FSpatialPhotoSet1f;

template<typename PixelType, typename RealType>
bool TSpatialPhotoSet<PixelType, RealType>::ComputeSampleLocation(
	const FVector3d& Position, 
	const FVector3d& Normal, 
	TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
	int& PhotoIndex,
	FVector2d& PhotoCoords) const
{
	double DotTolerance = -0.1;		// dot should be negative for normal pointing towards photo

	PhotoIndex = IndexConstants::InvalidID;
	PhotoCoords = FVector2d(0., 0.);

	double MinDot = 1.0;

	int32 NumPhotos = Num();
	for (int32 Index = 0; Index < NumPhotos; ++Index)
	{
		const TSpatialPhoto<PixelType>& Photo = *Photos[Index];
		check(Photo.Dimensions.IsSquare());

		FVector3d ViewDirection = Photo.Frame.X();
		double ViewDot = ViewDirection.Dot(Normal);
		if (ViewDot > DotTolerance || ViewDot > MinDot)
		{
			continue;
		}

		FFrame3d ViewPlane = Photo.Frame;
		ViewPlane.Origin += Photo.NearPlaneDist * ViewDirection;

		double ViewPlaneWidthWorld = Photo.NearPlaneDist * FMathd::Tan(Photo.HorzFOVDegrees * 0.5 * FMathd::DegToRad);
		double ViewPlaneHeightWorld = ViewPlaneWidthWorld;

		FVector3d RayOrigin = Photo.Frame.Origin;
		FVector3d RayDir = Normalized(Position - RayOrigin);
		FVector3d HitPoint;
		bool bHit = ViewPlane.RayPlaneIntersection(RayOrigin, RayDir, 0, HitPoint);
		if (bHit)
		{
			bool bVisible = VisibilityFunction(Position, HitPoint);
			if ( bVisible )
			{
				double PlaneX = (HitPoint - ViewPlane.Origin).Dot(ViewPlane.Y());
				double PlaneY = (HitPoint - ViewPlane.Origin).Dot(ViewPlane.Z());

				//FVector2d PlanePos = ViewPlane.ToPlaneUV(HitPoint, 0);
				double u = PlaneX / ViewPlaneWidthWorld;
				double v = -(PlaneY / ViewPlaneHeightWorld);
				if (FMathd::Abs(u) < 1 && FMathd::Abs(v) < 1)
				{
					PhotoCoords.X = (u/2.0 + 0.5) * (double)Photo.Dimensions.GetWidth();
					PhotoCoords.Y = (v/2.0 + 0.5) * (double)Photo.Dimensions.GetHeight();
					PhotoIndex = Index;
					MinDot = ViewDot;
				}
			}
		}
	}

	return PhotoIndex != IndexConstants::InvalidID;
}

template<typename PixelType, typename RealType>
PixelType TSpatialPhotoSet<PixelType, RealType>::ComputeSample(
	const FVector3d& Position, 
	const FVector3d& Normal, 
	TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
	const PixelType& DefaultValue ) const
{
	PixelType Result = DefaultValue;
	
	int PhotoIndex;
	FVector2d PhotoCoords;
	if (ComputeSampleLocation(Position, Normal, VisibilityFunction, PhotoIndex, PhotoCoords))
	{
		// TODO This bilinear sampling causes artefacts when it blends pixels from different depths (hence unrelated
		// values), we could fix this with some kind of rejection strategy. Search :BilinearSamplingDepthArtefacts 
		const TSpatialPhoto<PixelType>& Photo = *Photos[PhotoIndex];
		Result = Photo.Image.template BilinearSample<RealType>(PhotoCoords, DefaultValue);
	}
	
	return Result;
}

template<typename PixelType, typename RealType>
PixelType TSpatialPhotoSet<PixelType, RealType>::ComputeSample(
	const int& PhotoIndex, 
	const FVector2d& PhotoCoords, 
	const PixelType& DefaultValue) const
{
	// TODO See the task tagged :BilinearSamplingDepthArtefacts
	const TSpatialPhoto<PixelType>& Photo = *Photos[PhotoIndex];
	return Photo.Image.template BilinearSample<RealType>(PhotoCoords, DefaultValue);
}


template<typename PixelType, typename RealType>
PixelType TSpatialPhotoSet<PixelType, RealType>::ComputeSampleNearest(
	const int& PhotoIndex, 
	const FVector2d& PhotoCoords, 
	const PixelType& DefaultValue) const
{
	const TSpatialPhoto<PixelType>& Photo = *Photos[PhotoIndex];

	FVector2d UVCoords;
	UVCoords.X = PhotoCoords.X / Photo.Image.GetDimensions().GetWidth();
	UVCoords.Y = PhotoCoords.Y / Photo.Image.GetDimensions().GetHeight();
	if (UVCoords.X < 0. || UVCoords.X > 1. || UVCoords.Y < 0. || UVCoords.Y > 1.)
	{
		return DefaultValue;
	}

	return Photo.Image.NearestSampleUV(UVCoords);
}



} // end namespace UE::Geometry
} // end namespace UE