// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp / gsShapeModels MeshVoxelBlendOp

#pragma once

#include "Implicit/CachingMeshSDF.h"
#include "Implicit/GridInterpolant.h"
#include "Implicit/ImplicitFunctions.h"

#include "Generators/MarchingCubes.h"

#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template<typename TriangleMeshType>
class TImplicitBlend 
{
public:

	virtual ~TImplicitBlend() = default;

	///
	/// Inputs
	///

	TArray<const TriangleMeshType*> Sources;
	TArray<FAxisAlignedBox3d> SourceBounds;

	// exponent used in blend; controls shape (larger number == sharper blend)
	double BlendPower = 2.0;

	// blend distance
	double BlendFalloff = 5.0;

	// size of the cells used when sampling the distance field
	double GridCellSize = 1.0;

	// size of the cells used when meshing the output (marching cubes' cube size)
	double MeshCellSize = 1.0;

	// if true, perform a smooth subtraction instead of a smooth union
	bool bSubtract = false;

	// Set cell sizes to hit the target voxel counts along the max dimension of the bounds
	void SetCellSizesAndFalloff(FAxisAlignedBox3d Bounds, double BlendFalloffIn, int TargetInputVoxelCount, int TargetOutputVoxelCount)
	{
		BlendFalloff = BlendFalloffIn;
		GridCellSize = (Bounds.MaxDim() + BlendFalloff * 2.0) / double(TargetInputVoxelCount);
		MeshCellSize = (Bounds.MaxDim() + BlendFalloff * 2.0) / double(TargetOutputVoxelCount);
	}

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []()
	{
		return false;
	};
	
protected:

	FMarchingCubes MarchingCubes;

	/// Intermediate

	FAxisAlignedBox3d CombinedBounds;
	TArray<TMeshAABBTree3<TriangleMeshType>> ComputedSpatials;
	TArray<TCachingMeshSDF<TriangleMeshType>> ComputedSDFs;
	TArray<double> SDFMaxDistances;

public:

	bool Validate()
	{
		bool bHasSourcesWithBounds = Sources.Num() > 0 && Sources.Num() == SourceBounds.Num();
		for (int SourceIdx = 0; SourceIdx < Sources.Num(); SourceIdx++)
		{
			if (Sources[SourceIdx] == nullptr)
			{
				return false;
			}
		}

		bool bValidParams = BlendPower > 0 && BlendFalloff > 0 && GridCellSize > 0 && MeshCellSize > 0 && FMath::IsFinite(GridCellSize) && FMath::IsFinite(MeshCellSize);
		return bHasSourcesWithBounds && bValidParams;
	}

	/**
	 * @param bReuseComputed If true, will attempt to reuse previously-computed AABB trees and SDFs where possible
	 */
	const FMeshShapeGenerator& Generate(bool bReuseComputed = false)
	{
		MarchingCubes.Reset();
		if (!ensure(Validate()))
		{
			// give up and return and empty result on invalid parameters
			return MarchingCubes;
		}

		ComputeBounds();
		GenerateBlendAnalytic(bReuseComputed);
		return MarchingCubes;
	}

protected:

	void ComputeBounds()
	{
		CombinedBounds = FAxisAlignedBox3d::Empty();
		for (FAxisAlignedBox3d Box : SourceBounds)
		{
			CombinedBounds.Contain(Box);
		}
	}

	void ComputeSpatials(bool bReuseComputed)
	{
		ComputedSpatials.SetNum(Sources.Num());
		ParallelFor(Sources.Num(), [this, bReuseComputed](int SourceIdx)
			{
				if (!bReuseComputed || ComputedSpatials[SourceIdx].GetMesh() != Sources[SourceIdx] || !ComputedSpatials[SourceIdx].IsValid(false))
				{
					ComputedSpatials[SourceIdx].SetMesh(Sources[SourceIdx], true);
				}
			}
		);
	}

	void ComputeLazySDFs(bool bReuseComputed)
	{
		ComputeSpatials(bReuseComputed);

		if (!bReuseComputed || ComputedSDFs.Num() != Sources.Num())
		{
			ComputedSDFs.Reserve(Sources.Num());
			SDFMaxDistances.Reserve(Sources.Num());
			for (int i = 0; i < Sources.Num(); i++)
			{
				ComputedSDFs.Emplace(Sources[i], GridCellSize, &ComputedSpatials[i], false); // not auto-building here, to share code w/ need-rebuild path below
				SDFMaxDistances.Add(0.0);
			}
		}

		double NeedDistance = BlendFalloff;
		ParallelFor(Sources.Num(), [this, bReuseComputed, NeedDistance](int SourceIdx)
			{
				// previously computed sdf has signs computed out to (at least) the required distance, no need to recompute
				if (bReuseComputed && NeedDistance <= SDFMaxDistances[SourceIdx])
				{
					return;
				}

				// TODO: if we do have a previously computed sdf, and want to reuse computed, but need more distance
				//       we could expand the sdf here instead of throwing it out and fully recomputing.

				float UseMaxOffset = (float)BlendFalloff;

				ComputedSDFs[SourceIdx].MaxOffsetDistance = UseMaxOffset;
				ComputedSDFs[SourceIdx].CellSize = GridCellSize;

				ComputedSDFs[SourceIdx].CancelF = CancelF;
				ComputedSDFs[SourceIdx].Initialize();

				SDFMaxDistances[SourceIdx] = UseMaxOffset;
			});
	}

	void GenerateBlendAnalytic(bool bReuseComputed)
	{
		if (CancelF())
		{
			return;
		}
		
		ComputeSpatials(bReuseComputed);

		if (CancelF())
		{
			return;
		}

		ComputeLazySDFs(bReuseComputed);

		if (CancelF())
		{
			return;
		}

		TSkeletalRicciNaryBlend3<TDistanceFieldToSkeletalField<TTriLinearGridInterpolant<TCachingMeshSDF<TriangleMeshType>>, double>, double> Blend;
		Blend.BlendPower = BlendPower;
		Blend.bSubtract = bSubtract;
		Blend.Children.Reserve(Sources.Num());
		TArray<TTriLinearGridInterpolant<TCachingMeshSDF<TriangleMeshType>>> Interpolants; Interpolants.Reserve(Sources.Num());
		TArray<TDistanceFieldToSkeletalField<TTriLinearGridInterpolant<TCachingMeshSDF<TriangleMeshType>>, double>> SkeletalFields; SkeletalFields.Reserve(Sources.Num());
		for (int SourceIdx = 0; SourceIdx < Sources.Num(); SourceIdx++)
		{
			Interpolants.Add(ComputedSDFs[SourceIdx].MakeInterpolant());
			SkeletalFields.Emplace(&Interpolants[SourceIdx], BlendFalloff);
			Blend.Children.Add(&SkeletalFields[SourceIdx]);
		}

		MarchingCubes.CancelF = CancelF;

		MarchingCubes.CubeSize = MeshCellSize;

		MarchingCubes.IsoValue = TDistanceFieldToSkeletalField<TBoundedImplicitFunction3<double>, double>::ZeroIsocontour;
		MarchingCubes.Bounds = CombinedBounds;
		MarchingCubes.Bounds.Expand(BlendFalloff);
		MarchingCubes.RootMode = ERootfindingModes::LerpSteps;
		MarchingCubes.RootModeSteps = 3;

		if (CancelF())
		{
			return;
		}

		TArray<FVector3d> Seeds;
		for (const TriangleMeshType* Source : Sources)
		{
			for (int VID = 0; VID < Source->MaxVertexID(); VID++)
			{
				if (!Source->IsVertex(VID))
				{
					continue;
				}

				FVector3d Seed = Source->GetVertex(VID);
				// Only add vertices that are inside the spatial bounds (only vertices that are not on any triangles will be outside)
				if (MarchingCubes.Bounds.Contains(Seed))
				{
					Seeds.Add(Seed);
				}
			}
		}

		MarchingCubes.Implicit = [&Blend](const FVector3d& Pt) {return Blend.Value(Pt);}; //-V1047 - This lambda is cleared before routine exit

		MarchingCubes.GenerateContinuation(Seeds);

		if (Seeds.Num() > 0 && MarchingCubes.Triangles.Num() == 0)
		{
			// fall back to full generation; seeds failed!
			MarchingCubes.Generate();
		}

		// TODO: refactor FMarchingCubes to not retain the implicit function, or refactor this function so the implicit function isn't invalid after returning,
		/// ..... then remove this line
		MarchingCubes.Implicit = nullptr;
	}
};


} // end namespace UE::Geometry
} // end namespace UE
