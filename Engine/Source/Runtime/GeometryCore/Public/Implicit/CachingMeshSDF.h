// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp CachingMeshSDF

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/DenseGrid3.h"
#include "Async/ParallelFor.h"

#include "Implicit/GridInterpolant.h"

namespace UE
{
namespace Geometry
{


/**
 * This is variant of TSweepingMeshSDF that does lazy evaluation of actual Distances,
 * using mesh spatial data structure. This is much faster if we are doing continuation-method
 * marching cubes as only values on surface will be computed!
 * 
 * Compute discretely-sampled (I.e. gridded) signed distance field for a mesh
 * only within a narrow band of the surface
 * (within MaxDistQueryDist = MaxOffsetDistance + (2 * CellSize * FMathf::Sqrt2))
 * 
 * 
 * This code is based on the implementation found at https://github.com/christopherbatty/SDFGen
 */
template <class TriangleMeshType>
class TCachingMeshSDF
{
public:

	// INPUTS

	const TriangleMeshType* Mesh;
	const TMeshAABBTree3<TriangleMeshType>* Spatial;
	float CellSize;

	// Bounds of Grid will be expanded this much in positive and negative directions.
	// Useful for if you want field to extend outwards.
	FVector3d ExpandBounds = FVector3d::Zero();

	// max distance away from surface that we might need to evaluate
	float MaxOffsetDistance = 0;

	// Most of this parallelizes very well, makes a huge speed difference
	bool bUseParallel = true;

	// should we try to compute signs? if not, Grid remains unsigned
	bool bComputeSigns = true;

	// If the number of cells in any dimension may exceed this, CellSize will be automatically increased to keep cell count reasonable
	int ApproxMaxCellsPerDimension = 4096;

	// What counts as "inside" the mesh. Crossing count does not use triangle
	// Orientation, so inverted faces are fine, but overlapping shells or self intersections
	// will be filled using even/odd rules (as seen along X axis...)
	// Winding count is basically mesh winding number, handles overlap shells and
	// self-intersections, but inverted shells are 'subtracted', and inverted faces are a disaster.
	// Both modes handle internal cavities, neither handles open sheets.
	enum EInsideModes
	{
		CrossingCount = 0,
		WindingCount = 1
	};
	EInsideModes InsideMode = EInsideModes::WindingCount;

	// Implementation computes the triangle closest to each Grid cell, can
	// return this Grid if desired (only reason not to is avoid hanging onto memory)
	bool bWantClosestTriGrid = false;

	// Grid of per-cell crossing or parity counts
	bool bWantIntersectionsGrid = false;

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []() { return false; };


	// OUTPUTS

	FVector3f GridOrigin;
	FDenseGrid3f Grid;
	FDenseGrid3i ClosestTriGrid;
	FDenseGrid3i IntersectionsGrid;

	TCachingMeshSDF(float CellSizeIn = 10) : Mesh(nullptr), Spatial(nullptr), CellSize(CellSizeIn)
	{
	}

	TCachingMeshSDF(const TriangleMeshType* MeshIn, float CellSizeIn, const TMeshAABBTree3<TriangleMeshType>* SpatialIn, bool bAutoBuild) :
		Mesh(MeshIn), Spatial(SpatialIn), CellSize(CellSizeIn)
	{
		if (bAutoBuild)
		{
			Initialize();
		}
	}

protected:

	// set by Initialize

	float UpperBoundDistance;
	double MaxDistQueryDist;

public:

	bool Validate()
	{
		FAxisAlignedBox3d Bounds = Spatial->GetBoundingBox();

		if (Bounds.IsEmpty() || !FMath::IsFinite(Bounds.MaxDim()))
		{
			return false;
		}
		if (CellSize <= 0 || !FMath::IsFinite(CellSize))
		{
			return false;
		}
		if (ApproxMaxCellsPerDimension < 5)
		{
			return false;
		}
		return true;
	}

	void Initialize()
	{
		if (!ensure(Validate()))
		{
			return;
		}

		// figure out Origin & dimensions
		FAxisAlignedBox3d Bounds = Spatial->GetBoundingBox();
		
		double MaxDim = MaxElement(Bounds.Max - Bounds.Min + ExpandBounds * 2.0) + 4.0 * MaxOffsetDistance;
		if (!ensureMsgf(MaxDim / CellSize <= ApproxMaxCellsPerDimension - 4, TEXT("SDF resolution clamped to avoid excessive memory use")))
		{
			CellSize = float (MaxDim / (ApproxMaxCellsPerDimension - 4));
			if (!ensure(CellSize > 0 && FMath::IsFinite(CellSize)))
			{
				return;
			}
		}

		float fBufferWidth = (float)FMath::Max(4 * CellSize, 2 * MaxOffsetDistance + 2 * CellSize);
		GridOrigin = (FVector3f)Bounds.Min - fBufferWidth * FVector3f::One() - (FVector3f)ExpandBounds;
		FVector3f max = (FVector3f)Bounds.Max + fBufferWidth * FVector3f::One() + (FVector3f)ExpandBounds;
		int NI = (int)((max.X - GridOrigin.X) / CellSize) + 1;
		int NJ = (int)((max.Y - GridOrigin.Y) / CellSize) + 1;
		int NK = (int)((max.Z - GridOrigin.Z) / CellSize) + 1;

		UpperBoundDistance = ((float)(NI + NJ + NK) * CellSize);
		Grid = FDenseGrid3f(NI, NJ, NK, UpperBoundDistance);

		MaxDistQueryDist = MaxOffsetDistance + (2 * CellSize * FMathf::Sqrt2);

		// closest triangle id for each Grid cell
		if (bWantClosestTriGrid)
		{
			ClosestTriGrid.Resize(NI, NJ, NK);
			ClosestTriGrid.Assign(-1);
		}

		if (bComputeSigns == true)
		{
			// IntersectionCount(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
			FDenseGrid3i& IntersectionCount = IntersectionsGrid;
			IntersectionCount.Resize(NI, NJ, NK);
			IntersectionCount.Assign(0);

			compute_intersections(GridOrigin, CellSize, NI, NJ, NK, IntersectionCount);
			if (CancelF())
			{
				CleanupUnwanted();
				return;
			}

			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, Grid, IntersectionCount);
			if (CancelF())
			{
				CleanupUnwanted();
				return;
			}

			CleanupUnwanted();
		}
	}

	void CleanupUnwanted()
	{
		if (!bWantIntersectionsGrid)
		{
			IntersectionsGrid.Resize(0, 0, 0);
		}
	}


	float GetValue(FVector3i Idx)
	{
		float f = Grid[Idx];
		if (f == UpperBoundDistance || f == -UpperBoundDistance)
		{
			FVector3d p = (FVector3d)cell_center(Idx);

			float sign = FMath::Sign(f);

			double dsqr;
			int near_tid = Spatial->FindNearestTriangle(p, dsqr, MaxDistQueryDist);
			//int near_tid = Spatial->FindNearestTriangle(p, dsqr);
			if (near_tid == IndexConstants::InvalidID)
			{
				f += 0.0001f;
			}
			else
			{
				f = sign * (float)FMathd::Sqrt(dsqr);
			}

			Grid[Idx] = f;
			if (bWantClosestTriGrid)
			{
				ClosestTriGrid[Idx] = near_tid;
			}
		}
		return f;
	}




	TTriLinearGridInterpolant<TCachingMeshSDF> MakeInterpolant()
	{
		return TTriLinearGridInterpolant<TCachingMeshSDF>(this, (FVector3d)GridOrigin, CellSize, Dimensions());
	}



	FVector3i Dimensions()
	{
		return Grid.GetDimensions();
	}

	/**
	 * SDF Grid available after calling Compute()
	 */
	const FDenseGrid3f& GetGrid() const
	{
		return Grid;
	}

	/**
	 * Origin of the SDF Grid, in same coordinates as mesh
	 */
	const FVector3f& GetGridOrigin() const
	{
		return GridOrigin;
	}


	const FDenseGrid3i& GetClosestTriGrid() const
	{
		checkf(bWantClosestTriGrid == true, TEXT("Set bWantClosestTriGrid=true to return this value"));
		return ClosestTriGrid;
	}
	
	const FDenseGrid3i& GetIntersectionsGrid() const
	{
		checkf(bWantIntersectionsGrid == true, TEXT("Set bWantIntersectionsGrid=true to return this value"));
		return IntersectionsGrid;
	}

	float At(int I, int J, int K) const
	{
		return Grid.At(I, J, K);
	}

	constexpr const float operator[](FVector3i Idx) const
	{
		return Grid[Idx];
	}

	FVector3f CellCenter(int I, int J, int K)
	{
		return cell_center(FVector3i(I, J, K));
	}

private:
	FVector3f cell_center(const FVector3i& IJK)
	{
		return FVector3f((float)IJK.X * CellSize + GridOrigin[0],
			(float)IJK.Y * CellSize + GridOrigin[1],
			(float)IJK.Z * CellSize + GridOrigin[2]);
	}

	// fill the intersection Grid w/ number of intersections in each cell
	void compute_intersections(FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, FDenseGrid3i& IntersectionCount)
	{
		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		double invdx = 1.0 / DX;

		bool cancelled = false;

		// this is what we will do for each triangle. There are no Grid-reads, only Grid-writes, 
		// since we use atomic_increment, it is always thread-safe
		ParallelFor(Mesh->MaxTriangleID(),
			[this, &Origin, &DX, &NI, &NJ, &NK,
			&IntersectionCount, &ox, &oy, &oz, &invdx, &cancelled](int32 TID)
		{
			if (TID % 100 == 0 && CancelF() == true)
			{
				cancelled = true;
			}
			if (cancelled || !Mesh->IsTriangle(TID))
			{
				return;
			}

			FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
			Mesh->GetTriVertices(TID, xp, xq, xr);


			bool neg_x = false;
			if (InsideMode == EInsideModes::WindingCount)
			{
				FVector3d n = VectorUtil::NormalDirection(xp, xq, xr);
				neg_x = n.X > 0;
			}

			// real IJK coordinates of xp/xq/xr
			double fip = (xp[0] - ox) * invdx, fjp = (xp[1] - oy) * invdx, fkp = (xp[2] - oz) * invdx;
			double fiq = (xq[0] - ox) * invdx, fjq = (xq[1] - oy) * invdx, fkq = (xq[2] - oz) * invdx;
			double fir = (xr[0] - ox) * invdx, fjr = (xr[1] - oy) * invdx, fkr = (xr[2] - oz) * invdx;

			// recompute J/K integer bounds of triangle w/o exact band
			int32 j0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fjp, fjq, fjr)), 0, NJ - 1);
			int32 j1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fjp, fjq, fjr)), 0, NJ - 1);
			int32 k0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fkp, fkq, fkr)), 0, NK - 1);
			int32 k1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fkp, fkq, fkr)), 0, NK - 1);

			// and do intersection counts
			for (int K = k0; K <= k1; ++K)
			{
				for (int J = j0; J <= j1; ++J)
				{
					double A, B, C;
					if (PointInTriangle2d(J, K, fjp, fkp, fjq, fkq, fjr, fkr, A, B, C))
					{
						double fi = A * fip + B * fiq + C * fir; // intersection I coordinate
						int i_interval = FMath::CeilToInt32(fi); // intersection is in (i_interval-1,i_interval]
						if (i_interval < 0)
						{
							DenseGrid::AtomicIncDec(IntersectionCount, 0, J, K, neg_x);
						}
						else if (i_interval < NI)
						{
							DenseGrid::AtomicIncDec(IntersectionCount, i_interval, J, K, neg_x);
						}
						else
						{
							// we ignore intersections that are beyond the +x side of the Grid
						}
					}
				}
			}
		}, !bUseParallel);
	}





	// iterate through each x-row of Grid and set unsigned distances to be negative
	// inside the mesh, based on the IntersectionCount
	void compute_signs(int NI, int NJ, int NK, FDenseGrid3f& Distances, FDenseGrid3i& IntersectionCount)
	{
		if (bUseParallel)
		{
			// can process each x-row in parallel
			ParallelFor(NJ*NK, [this, &NI, &NJ, &NK, &Distances, &IntersectionCount](int32 vi)
			{
				if (CancelF())
				{
					return;
				}

				int32 J = vi % NJ, K = vi / NJ;
				int32 total_count = 0;
				for (int I = 0; I < NI; ++I) {
					total_count += IntersectionCount.At(I, J, K);
					if (
						(InsideMode == EInsideModes::WindingCount && total_count > 0) || 
						(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
						)
					{
						Distances.At(I, J, K) = -Distances.At(I, J, K); // we are inside the mesh
					}
				}
			}, !bUseParallel);

		}
		else
		{
			for (int K = 0; K < NK; ++K)
			{
				if (CancelF())
				{
					return;
				}

				for (int J = 0; J < NJ; ++J)
				{
					int total_count = 0;
					for (int I = 0; I < NI; ++I)
					{
						total_count += IntersectionCount.At(I, J, K);
						if (
							(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
							(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
							)
						{
							Distances.At(I, J, K) = -Distances.At(I, J, K); // we are inside the mesh
						}
					}
				}
			}
		}
	}



public:

	// calculate twice signed area of triangle (0,0)-(X1,Y1)-(X2,Y2)
	// return an SOS-determined sign (-1, +1, or 0 only if it's a truly degenerate triangle)
	static int Orientation(double X1, double Y1, double X2, double Y2, double& TwiceSignedArea)
	{
		TwiceSignedArea = Y1 * X2 - X1 * Y2;
		if (TwiceSignedArea > 0) return 1;
		else if (TwiceSignedArea < 0) return -1;
		else if (Y2 > Y1) return 1;
		else if (Y2 < Y1) return -1;
		else if (X1 > X2) return 1;
		else if (X1 < X2) return -1;
		else return 0; // only true when X1==X2 and Y1==Y2
	}


	// robust test of (X0,Y0) in the triangle (X1,Y1)-(X2,Y2)-(X3,Y3)
	// if true is returned, the barycentric coordinates are set in A,B,C.
	static bool PointInTriangle2d(double X0, double Y0,
		double X1, double Y1, double X2, double Y2, double X3, double Y3,
		double& A, double& B, double& C)
	{
		A = B = C = 0;
		X1 -= X0; X2 -= X0; X3 -= X0;
		Y1 -= Y0; Y2 -= Y0; Y3 -= Y0;
		int signa = Orientation(X2, Y2, X3, Y3, A);
		if (signa == 0) return false;
		int signb = Orientation(X3, Y3, X1, Y1, B);
		if (signb != signa) return false;
		int signc = Orientation(X1, Y1, X2, Y2, C);
		if (signc != signa) return false;
		double sum = A + B + C;
		// if the SOS signs match and are nonzero, there's no way all of A, B, and C are zero.
		// TODO: is this mathematically impossible? can we just remove the check?
		checkf(sum != 0, TEXT("TCachingMeshSDF::PointInTriangle2d: impossible config?"));
		A /= sum;
		B /= sum;
		C /= sum;
		return true;
	}

};


} // end namespace UE::Geometry
} // end namespace UE