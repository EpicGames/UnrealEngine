// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp's CachingMeshSDFImplicit

#pragma once

#include "BoxTypes.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "IntVectorTypes.h"

#include <type_traits>

namespace UE
{
namespace Geometry
{


/**
 * Tri-linear interpolant for a 3D dense Grid. Supports Grid translation
 * via GridOrigin, but does not support scaling or rotation. If you need those,
 * you can wrap this in something that does the xform.
 *
 * GridType must have a GetValue() that returns a value to interpolate at a given FVector3i location -- (w/ locations ranging from [0,0,0] to Dimensions (exclusive))
 */
template <class GridType, typename RealType = double, bool bScalarCellSize = true>
class TTriLinearGridInterpolant /*: public BoundedImplicitFunction3d (TODO: consider add ImplicitFunction3d interface concept once more implicit functions are available*/
{

public:

	// Type to use for CellSize
	using CellSizeType = std::conditional_t<bScalarCellSize, RealType, TVector<RealType>>;

	GridType* Grid;
	TVector<RealType> GridOrigin;
	CellSizeType CellSize;
	FVector3i Dimensions;

	// value to return if query point is outside Grid (in an SDF
	// outside is usually positive). Need to do math with this value,
	// and cast this value to/from float; use TMathUtil<RealType>::SafeLargeValue to avoid overflow
	RealType Outside = TMathUtil<RealType>::SafeLargeValue;

	TTriLinearGridInterpolant(GridType* Grid, TVector<RealType> GridOrigin, CellSizeType CellSize, FVector3i Dimensions) : Grid(Grid), GridOrigin(GridOrigin), CellSize(CellSize), Dimensions(Dimensions)
	{
	}

	TAxisAlignedBox3<RealType> Bounds() const
	{
		return TAxisAlignedBox3<RealType>(
			{ GridOrigin.X, GridOrigin.Y, GridOrigin.Z },
			{ GridOrigin.X + GetDim(CellSize, 0) * (Dimensions.X - 1),
			  GridOrigin.Y + GetDim(CellSize, 1) * (Dimensions.Y - 1),
			  GridOrigin.Z + GetDim(CellSize, 2) * (Dimensions.Z - 1)});
	}

	FVector3i Cell(const TVector<RealType>& Pt) const
	{
		// compute integer coordinates
		FVector3i CellCoords;
		CellCoords.X = (int)((Pt.X - GridOrigin.X) / GetDim(CellSize, 0));
		CellCoords.Y = (int)((Pt.Y - GridOrigin.Y) / GetDim(CellSize, 1));
		CellCoords.Z = (int)((Pt.Z - GridOrigin.Z) / GetDim(CellSize, 2));

		return CellCoords;
	}

	template<bool bClamped = false>
	RealType Value(const TVector<RealType>& Pt) const
	{
		TVector<RealType> gridPt(
			((Pt.X - GridOrigin.X) / GetDim(CellSize, 0)),
			((Pt.Y - GridOrigin.Y) / GetDim(CellSize, 1)),
			((Pt.Z - GridOrigin.Z) / GetDim(CellSize, 2)));

		if constexpr (bClamped)
		{
			gridPt.X = FMath::Clamp(gridPt.X, 0, Dimensions.X - (1 + FMathd::Epsilon));
			gridPt.Y = FMath::Clamp(gridPt.Y, 0, Dimensions.Y - (1 + FMathd::Epsilon));
			gridPt.Z = FMath::Clamp(gridPt.Z, 0, Dimensions.Z - (1 + FMathd::Epsilon));
		}

		// compute integer coordinates
		int X0 = (int)gridPt.X;
		int Y0 = (int)gridPt.Y, Y1 = Y0 + 1;
		int Z0 = (int)gridPt.Z, Z1 = Z0 + 1;

		// return Outside if not in Grid
		if constexpr (!bClamped)
		{
			if (X0 < 0 || (X0 + 1) >= Dimensions.X ||
				Y0 < 0 || Y1 >= Dimensions.Y ||
				Z0 < 0 || Z1 >= Dimensions.Z)
			{
				return Outside;
			}
		}

		// convert double coords to [0,1] range
		RealType fAx = gridPt.X - (RealType)X0;
		RealType fAy = gridPt.Y - (RealType)Y0;
		RealType fAz = gridPt.Z - (RealType)Z0;
		RealType OneMinusfAx = (RealType)(1.0) - fAx;

		// compute trilinear interpolant. The code below tries to do this with the fewest 
		// number of variables, in hopes that optimizer will be clever about re-using registers, etc.
		// Commented code at bottom is fully-expanded version.
		RealType xa, xb;

		get_value_pair(X0, Y0, Z0, xa, xb);
		RealType yz = (1 - fAy) * (1 - fAz);
		RealType sum = (OneMinusfAx * xa + fAx * xb) * yz;

		get_value_pair(X0, Y0, Z1, xa, xb);
		yz = (1 - fAy) * (fAz);
		sum += (OneMinusfAx * xa + fAx * xb) * yz;

		get_value_pair(X0, Y1, Z0, xa, xb);
		yz = (fAy) * (1 - fAz);
		sum += (OneMinusfAx * xa + fAx * xb) * yz;

		get_value_pair(X0, Y1, Z1, xa, xb);
		yz = (fAy) * (fAz);
		sum += (OneMinusfAx * xa + fAx * xb) * yz;

		return sum;

		// fV### is Grid cell corner index
		//return
		//    fV000 * (1 - fAx) * (1 - fAy) * (1 - fAz) +
		//    fV001 * (1 - fAx) * (1 - fAy) * (fAz) +
		//    fV010 * (1 - fAx) * (fAy) * (1 - fAz) +
		//    fV011 * (1 - fAx) * (fAy) * (fAz) +
		//    fV100 * (fAx) * (1 - fAy) * (1 - fAz) +
		//    fV101 * (fAx) * (1 - fAy) * (fAz) +
		//    fV110 * (fAx) * (fAy) * (1 - fAz) +
		//    fV111 * (fAx) * (fAy) * (fAz);
	}


protected:
	void get_value_pair(int I, int J, int K, RealType& A, RealType& B) const
	{
		A = (RealType)Grid->GetValue(FVector3i(I, J, K));
		B = (RealType)Grid->GetValue(FVector3i(I + 1, J, K));
	}


public:
	TVector<RealType> Gradient(const TVector<RealType>& Pt) const
	{
		TVector<RealType> gridPt = TVector<RealType>(
			((Pt.X - GridOrigin.X) /  GetDim(CellSize, 0)),
			((Pt.Y - GridOrigin.Y) /  GetDim(CellSize, 1)),
			((Pt.Z - GridOrigin.Z) /  GetDim(CellSize, 2)));

		// clamp to Grid
		if (gridPt.X < 0 || gridPt.X >= Dimensions.X - 1 ||
			gridPt.Y < 0 || gridPt.Y >= Dimensions.Y - 1 ||
			gridPt.Z < 0 || gridPt.Z >= Dimensions.Z - 1)
		{
			return TVector<RealType>::Zero();
		}

		// compute integer coordinates
		int X0 = (int)gridPt.X;
		int Y0 = (int)gridPt.Y, Y1 = Y0 + 1;
		int Z0 = (int)gridPt.Z, Z1 = Z0 + 1;

		// convert RealType coords to [0,1] range
		RealType fAx = gridPt.X - (RealType)X0;
		RealType fAy = gridPt.Y - (RealType)Y0;
		RealType fAz = gridPt.Z - (RealType)Z0;

		RealType fV000, fV100;
		get_value_pair(X0, Y0, Z0, fV000, fV100);
		RealType fV010, fV110;
		get_value_pair(X0, Y1, Z0, fV010, fV110);
		RealType fV001, fV101;
		get_value_pair(X0, Y0, Z1, fV001, fV101);
		RealType fV011, fV111;
		get_value_pair(X0, Y1, Z1, fV011, fV111);

		// [TODO] can re-order this to vastly reduce number of ops!
		RealType gradX =
			-fV000 * (1 - fAy) * (1 - fAz) +
			-fV001 * (1 - fAy) * (fAz)+
			-fV010 * (fAy) * (1 - fAz) +
			-fV011 * (fAy) * (fAz)+
			fV100 * (1 - fAy) * (1 - fAz) +
			fV101 * (1 - fAy) * (fAz)+
			fV110 * (fAy) * (1 - fAz) +
			fV111 * (fAy) * (fAz);

		RealType gradY =
			-fV000 * (1 - fAx) * (1 - fAz) +
			-fV001 * (1 - fAx) * (fAz)+
			fV010 * (1 - fAx) * (1 - fAz) +
			fV011 * (1 - fAx) * (fAz)+
			-fV100 * (fAx) * (1 - fAz) +
			-fV101 * (fAx) * (fAz)+
			fV110 * (fAx) * (1 - fAz) +
			fV111 * (fAx) * (fAz);

		RealType gradZ =
			-fV000 * (1 - fAx) * (1 - fAy) +
			fV001 * (1 - fAx) * (1 - fAy) +
			-fV010 * (1 - fAx) * (fAy)+
			fV011 * (1 - fAx) * (fAy)+
			-fV100 * (fAx) * (1 - fAy) +
			fV101 * (fAx) * (1 - fAy) +
			-fV110 * (fAx) * (fAy)+
			fV111 * (fAx) * (fAy);

		return TVector<RealType>(gradX, gradY, gradZ);
	}

private:
	inline static RealType GetDim(CellSizeType CellSize, int32 Axis)
	{
		if constexpr (bScalarCellSize)
		{
			return CellSize;
		}
		else
		{
			return CellSize[Axis];
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE