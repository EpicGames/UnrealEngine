// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp's CachingMeshSDFImplicit

#pragma once

#include "MathUtil.h"
#include "SegmentTypes.h"
#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template<typename RealType>
struct TImplicitFunction3
{
	TUniqueFunction<RealType(TVector<RealType>)> Value();
};

template<typename RealType>
struct TBoundedImplicitFunction3
{
	TUniqueFunction<RealType(TVector<RealType>)> Value();
	TUniqueFunction<TAxisAlignedBox3<RealType>> Bounds();
};



/**
 * Implicit Distance-Field Point Object/Primitive, surface shape at isovalue=0 (defined by Distance-Radius) is a sphere
 */
template<typename RealType>
struct TImplicitPoint3
{
	TVector<RealType> Position;
	RealType Radius;

	RealType Value(const TVector<RealType>& Point) const
	{
		return Position.Distance(Point) - Radius;
	}

	TAxisAlignedBox3<RealType> Bounds() const
	{
		return  TAxisAlignedBox3<RealType>(Position, Radius);
	}
};
typedef TImplicitPoint3<float> FImplicitPoint3f;
typedef TImplicitPoint3<double> FImplicitPoint3d;



/**
 * Implicit Distance-Field Line Object/Primitive, surface shape at isovalue=0 (defined by Distance-Radius) is a capsule
 */
template<typename RealType>
struct TImplicitLine3
{
	TSegment3<RealType> Segment;
	RealType Radius;

	RealType Value(const TVector<RealType>& Point) const
	{
		RealType DistanceSqr = Segment.DistanceSquared(Point);
		return TMathUtil<RealType>::Sqrt(DistanceSqr) - Radius;
	}

	TAxisAlignedBox3<RealType> Bounds() const
	{
		return Segment.GetBounds(Radius);
	}
};
typedef TImplicitLine3<float> FImplicitLine3f;
typedef TImplicitLine3<double> FImplicitLine3d;







/**
 * Skeletal implicit line primitive. Field value is 1 along line and 0 at
 * distance=1 from the line. Scale factor is applied to distance.
 */
template<typename RealType>
struct TSkeletalImplicitLine3
{
	TSegment3<RealType> Segment;
	RealType Scale = 1.0;

	void SetScaleFromRadius(double TargetRadius, double DefaultIsoValue = 0.5)
	{
		Scale = TargetRadius / DefaultIsoValue;
	}

	float GetRadius(double DefaultIsoValue = 0.5) const
	{
		return Scale * DefaultIsoValue;
	}

	RealType Value(const TVector<RealType>& Point) const
	{
		RealType DistanceSqr = Segment.DistanceSquared(Point);
		DistanceSqr /= (Scale * Scale);

		RealType T = FMathd::Max(1.0 - DistanceSqr, 0.0);
		return T*T*T; // (1-x^2)^3
	}

	TAxisAlignedBox3<RealType> Bounds(double DefaultIsoValue = 0.5) const
	{
		return Segment.GetBounds(GetRadius(DefaultIsoValue));
	}
};
typedef TSkeletalImplicitLine3<float> FSkeletalImplicitLine3f;
typedef TSkeletalImplicitLine3<double> FSkeletalImplicitLine3d;




/**
 * This class converts the interval [-falloff,falloff] to [0,1],
 * Then applies Wyvill falloff function (1-t^2)^3.
 * The result is a skeletal-primitive-like shape with 
 * the distance=0 isocontour lying just before midway in
 * the range (at the .ZeroIsocontour constant)
 */
template<typename InputBoundedImplicitType, typename RealType>
struct TDistanceFieldToSkeletalField
{
	InputBoundedImplicitType* DistanceField = nullptr;
	RealType FalloffDistance = 10;

	TDistanceFieldToSkeletalField(InputBoundedImplicitType* DistanceField = nullptr, RealType FalloffDistance = 10) : DistanceField(DistanceField), FalloffDistance(FalloffDistance)
	{
		checkSlow(FalloffDistance > 0);
	}

	static constexpr RealType ZeroIsocontour = (RealType)0.421875; // (1-.5^2)^3

	TAxisAlignedBox3<RealType> Bounds()
	{
		checkSlow(DistanceField != nullptr);
		TAxisAlignedBox3<RealType> B = DistanceField->Bounds();
		B.Expand(FalloffDistance);
		return B;
	}

	RealType Value(const TVector<RealType> Pt)
	{
		checkSlow(DistanceField != nullptr);
		RealType Dist = DistanceField->Value(Pt);
		if (Dist > FalloffDistance)
			return 0;
		else if (Dist < -FalloffDistance)
			return 1.0;
		RealType t = (Dist + FalloffDistance) / (2 * FalloffDistance);
		RealType expr = 1 - (t * t);
		return expr * expr * expr; // == (1-t^2)^3
	}
};


/**
 * Boolean Union of N implicit functions (F_1 or F_2 or ... or F_N)
 * Or if bSubtract=true, Subtraction: (F_1 - (F_2 or F_3 or ... or F_N))
 * We expect this to be used with "Skeletal Field" inputs (e.g. TDistanceFieldToSkeletalField, TSkeletalImplicitLine3, etc)
 * where values are in the range [0,1] and the isosurface is ~ in the middle of that range
**/
template<typename InputBoundedImplicitType, typename RealType>
struct TSkeletalRicciNaryBlend3
{
	TArray<InputBoundedImplicitType*> Children;
	RealType BlendPower = 2.0;
	bool bSubtract = false;

	RealType Value(const TVector<RealType> Pt)
	{
		int N = Children.Num();
		checkSlow(N > 0);
		RealType f = Children[0]->Value(Pt);
		RealType Scale = bSubtract ? -1 : 1;
		if (BlendPower == 1.0)
		{
			for (int k = 1; k < N; ++k)
			{
				f += Children[k]->Value(Pt) * Scale;
			}
			f = TMathUtil<RealType>::Max(0, f);
		}
		else if (BlendPower == 2.0)
		{
			f = f * f;
			for (int k = 1; k < N; ++k)
			{
				RealType v = Children[k]->Value(Pt);
				f += v * v * Scale;
			}
			f = TMathUtil<RealType>::Sqrt(TMathUtil<RealType>::Max(0, f));
		}
		else
		{
			f = TMathUtil<RealType>::Pow(f, BlendPower);
			for (int k = 1; k < N; ++k)
			{
				RealType v = Children[k]->Value(Pt);
				f += TMathUtil<RealType>::Pow(v, BlendPower) * Scale;
			}
			f = TMathUtil<RealType>::Pow(TMathUtil<RealType>::Max(0, f), 1.0 / BlendPower);
		}
		return f;
	}

	TAxisAlignedBox3<RealType> Bounds()
	{
		TAxisAlignedBox3<RealType> Box = Children[0]->Bounds();
		for (int k = 1, N = Children.Num(); k < N; ++k)
		{
			Box.Contain(Children[k]->Bounds());
		}
		return Box;
	}
};


} // end namespace UE::Geometry
} // end namespace UE
