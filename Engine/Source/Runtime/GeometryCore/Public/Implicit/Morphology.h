// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp / gsShapeModels MeshMorphologyOp

#pragma once

#include "MeshAdapter.h"

#include "Spatial/MeshAABBTree3.h"

#include "Implicit/SparseNarrowBandMeshSDF.h"
#include "Implicit/GridInterpolant.h"
#include "Implicit/ImplicitFunctions.h"

#include "Generators/MarchingCubes.h"
#include "MeshQueries.h"

namespace UE
{
namespace Geometry
{

template<typename TriangleMeshType>
class TImplicitMorphology
{
public:
	/** Morphology operation types */
	enum class EMorphologyOp
	{
		/** Expand the shapes outward */
		Dilate = 0,

		/** Shrink the shapes inward */
		Contract = 1,

		/** Dilate and then contract, to delete small negative features (sharp inner corners, small holes) */
		Close = 2,

		/** Contract and then dilate, to delete small positive features (sharp outer corners, small isolated pieces) */
		Open = 3
	};

	///
	/// Inputs
	///
	const TriangleMeshType* Source = nullptr;
	TMeshAABBTree3<TriangleMeshType>* SourceSpatial = nullptr;
	EMorphologyOp MorphologyOp = EMorphologyOp::Dilate;

	// Distance of offset; should be positive
	double Distance = 1.0;

	// size of the cells used when sampling the distance field
	double GridCellSize = 1.0;

	// size of the cells used when meshing the output (marching cubes' cube size)
	double MeshCellSize = 1.0;

	/** Whether to use a custom bounding box instead of the input mesh bounds to define the domain to solidify */
	bool bUseCustomBounds = false;

	/** Custom bounds to use, if bUseCustomBounds == true; ignored otherwise */
	FAxisAlignedBox3d CustomBounds;

	// Set cell sizes to hit the target voxel counts along the max dimension of the bounds
	void SetCellSizesAndDistance(FAxisAlignedBox3d Bounds, double DistanceIn, int TargetInputVoxelCount, int TargetOutputVoxelCount)
	{
		Distance = DistanceIn;
		SetGridCellSize(Bounds, DistanceIn, TargetInputVoxelCount);
		SetMeshCellSize(Bounds, DistanceIn, TargetOutputVoxelCount);
	}

	// Set input grid cell size to hit the target voxel counts along the max dimension of the bounds
	void SetGridCellSize(FAxisAlignedBox3d Bounds, double DistanceIn, int TargetInputVoxelCount)
	{
		int UseTargetVoxelCount = FMath::Min(MaxTargetVoxelCount, TargetInputVoxelCount);
		GridCellSize = (Bounds.MaxDim() + (!bUseCustomBounds ? DistanceIn * 2.0 : 0.0)) / double(UseTargetVoxelCount);
	}

	// Set output meshing cell size to hit the target voxel counts along the max dimension of the bounds
	void SetMeshCellSize(FAxisAlignedBox3d Bounds, double DistanceIn, int TargetInputVoxelCount)
	{
		int UseTargetVoxelCount = FMath::Min(MaxTargetVoxelCount, TargetInputVoxelCount);
		MeshCellSize = (Bounds.MaxDim() + (!bUseCustomBounds ? DistanceIn * 2.0 : 0.0)) / double(UseTargetVoxelCount);
	}

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []()
	{
		return false;
	};

protected:
	// Stores result (returned as a const FMeshShapeGenerator)
	FMarchingCubes MarchingCubes;

	// computed in first pass, re-used in second
	double NarrowBandMaxDistance;

public:
	bool Validate()
	{
		bool bValidMeshAndSpatial = Source != nullptr && SourceSpatial != nullptr && SourceSpatial->IsValid(false);
		bool bValidParams = Distance > 0 && GridCellSize > 0 && MeshCellSize > 0 && FMath::IsFinite(MeshCellSize);
		return bValidMeshAndSpatial && bValidParams;
	}

	const FMeshShapeGenerator& Generate()
	{
		MarchingCubes.Reset();
		if ((Source && Source->TriangleCount() == 0) || !ensure(Validate()))
		{
			// return an empty result if input is empty or parameters are not valid
			return MarchingCubes;
		}

		double UnsignedOffset = FMathd::Abs(Distance);
		double SignedOffset = UnsignedOffset;
		switch (MorphologyOp)
		{
		case TImplicitMorphology<TriangleMeshType>::EMorphologyOp::Dilate:
		case TImplicitMorphology<TriangleMeshType>::EMorphologyOp::Close:
			SignedOffset = -SignedOffset;
			break;
		}

		ComputeFirstPass(UnsignedOffset, SignedOffset);

		if (MorphologyOp == TImplicitMorphology<TriangleMeshType>::EMorphologyOp::Close || MorphologyOp == TImplicitMorphology<TriangleMeshType>::EMorphologyOp::Open)
		{
			ComputeSecondPass(UnsignedOffset, -SignedOffset);
		}

		return MarchingCubes;
	}

protected:

	template<typename MeshType>
	using TMeshSDF = TSparseNarrowBandMeshSDF<MeshType>; 

	void ComputeFirstPass(double UnsignedOffset, double SignedOffset)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_FirstPass);

		typedef TMeshSDF<TriangleMeshType>   MeshSDFType;

		MeshSDFType ComputedSDF;
		ComputedSDF.Mesh = Source;

		ComputedSDF.Spatial = SourceSpatial;
		ComputedSDF.ComputeMode = MeshSDFType::EComputeModes::NarrowBand_SpatialFloodFill;

		double UseGridCellSize = GetSafeCellSize(2*UnsignedOffset + SourceSpatial->GetBoundingBox().MaxDim(), GridCellSize, 2);
		ComputedSDF.CellSize = (float)UseGridCellSize;
		NarrowBandMaxDistance = UnsignedOffset + ComputedSDF.CellSize;
		ComputedSDF.NarrowBandMaxDistance = NarrowBandMaxDistance;
		ComputedSDF.ExactBandWidth = FMath::CeilToInt32(ComputedSDF.NarrowBandMaxDistance / ComputedSDF.CellSize);

		// for meshes with long triangles relative to the width of the narrow band, don't use the AABB tree
		double AvgEdgeLen = TMeshQueries<TriangleMeshType>::AverageEdgeLength(*Source);
		if (!ComputedSDF.ShouldUseSpatial(ComputedSDF.ExactBandWidth, ComputedSDF.CellSize, AvgEdgeLen))
		{
			ComputedSDF.Spatial = nullptr;
			ComputedSDF.ComputeMode = MeshSDFType::EComputeModes::NarrowBandOnly;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_FirstPass_ComputeSDF);
			ComputedSDF.Compute(SourceSpatial->GetBoundingBox());
		}

		TTriLinearGridInterpolant<MeshSDFType> Interpolant = ComputedSDF.MakeInterpolant();

		MarchingCubes.IsoValue = SignedOffset;
		if (bUseCustomBounds)
		{
			MarchingCubes.Bounds = CustomBounds;
		}
		else
		{
			MarchingCubes.Bounds = SourceSpatial->GetBoundingBox();
			MarchingCubes.Bounds.Expand(GridCellSize);
			if (MarchingCubes.IsoValue < 0)
			{
				MarchingCubes.Bounds.Expand(ComputedSDF.NarrowBandMaxDistance);
			}
		}
		MarchingCubes.RootMode = ERootfindingModes::SingleLerp;
		MarchingCubes.CubeSize = GetSafeCellSize(MarchingCubes.Bounds.MaxDim(), MeshCellSize, 1);

		MarchingCubes.CancelF = CancelF;

		if (CancelF())
		{
			return;
		}

		MarchingCubes.Implicit = [Interpolant](const FVector3d& Pt)
		{
			return -Interpolant.Value(Pt);
		};
		MarchingCubes.bEnableValueCaching = false;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_FirstPass_GenerateMesh);
			MarchingCubes.Generate();
		}

		// TODO: refactor FMarchingCubes to not retain the implicit function, or refactor this function so the implicit function isn't invalid after returning,
		/// ..... then remove this line
		MarchingCubes.Implicit = nullptr;
	}

	void ComputeSecondPass(double UnsignedOffset, double SignedOffset)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_SecondPass);
		
		typedef TMeshSDF<TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d>>   MeshSDFType;

		if (MarchingCubes.Triangles.Num() == 0)
		{
			MarchingCubes.Reset();
			return;
		}

		TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d> MCAdapter(&MarchingCubes.Vertices, &MarchingCubes.Triangles);
		TMeshAABBTree3<TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d>> SecondSpatial(&MCAdapter, false);

		FAxisAlignedBox3d Bounds = MarchingCubes.Bounds;
		if (!bUseCustomBounds)
		{
			Bounds.Expand(MeshCellSize); // (because mesh may spill one cell over bounds)
		}

		MeshSDFType SecondSDF;
		SecondSDF.Mesh = &MCAdapter;

		// Adjust cell size to not overflow w/ the added UnsignedOffset
		double UseGridCellSize = GetSafeCellSize(2*UnsignedOffset + Bounds.MaxDim(), GridCellSize, 2);

		SecondSDF.CellSize = (float)UseGridCellSize;
		SecondSDF.Spatial = nullptr;


		SecondSDF.NarrowBandMaxDistance = UnsignedOffset + SecondSDF.CellSize;
		SecondSDF.ExactBandWidth = FMath::CeilToInt32(SecondSDF.NarrowBandMaxDistance / SecondSDF.CellSize);

		if (SecondSDF.ExactBandWidth > 1) // for larger band width, prefer using the AABB tree to do one distance per cell.  TODO: tune?
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_SecondPass_BuildSpatial);
			SecondSpatial.Build();
			SecondSDF.Spatial = &SecondSpatial;
			SecondSDF.ComputeMode = MeshSDFType::EComputeModes::NarrowBand_SpatialFloodFill;
			if (!bUseCustomBounds)
			{
				Bounds = SecondSpatial.GetBoundingBox(); // Use the tighter bounds from the AABB tree since we have it
			}
		}
		else
		{
			SecondSDF.ComputeMode = MeshSDFType::EComputeModes::NarrowBandOnly;
		}


		if (CancelF())
		{
			return;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_SecondPass_ComputeSDF);
			SecondSDF.Compute(Bounds);
		}
		TTriLinearGridInterpolant<MeshSDFType> Interpolant = SecondSDF.MakeInterpolant();

		MarchingCubes.Reset();
		MarchingCubes.IsoValue = SignedOffset;
		MarchingCubes.Bounds = Bounds;
		if (!bUseCustomBounds)
		{
			MarchingCubes.Bounds.Expand(UseGridCellSize);
			if (MarchingCubes.IsoValue < 0)
			{
				MarchingCubes.Bounds.Expand(NarrowBandMaxDistance);
			}
			// Make sure the CubeSize is still safe after expanding the bounds
			MarchingCubes.CubeSize = GetSafeCellSize(MarchingCubes.Bounds.MaxDim(), MarchingCubes.CubeSize, 1);
		}


		if (CancelF())
		{
			return;
		}

		MarchingCubes.Implicit = [Interpolant](const FVector3d& Pt)
		{
			return -Interpolant.Value(Pt);
		};
		MarchingCubes.bEnableValueCaching = false;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_Morphology_SecondPass_GenerateMesh);
			MarchingCubes.Generate();
		}

		// TODO: refactor FMarchingCubes to not retain the implicit function, or refactor this function so the implicit function isn't invalid after returning,
		/// ..... then remove this line
		MarchingCubes.Implicit = nullptr;
	}

private:
	// Set a max target voxel count s.t. the VoxelCount^3 does not overflow int32 linearized grid indices
	// (and to reduce the chance of running out of memory for a very dense grid, generally)
	static constexpr int32 MaxTargetVoxelCount = 1200;

	// Adjust cell size so that a cell count based on (BoundsWidth/InitialCellSize + ExtraCellCount) should not
	// (too far) exceed MaxTargetVoxelCount.  (Assumes ExtraCellCount is smaller MaxTargetVoxelCount)
	static double GetSafeCellSize(double BoundsWidth, double InitialCellSize, int32 ExtraCellCount)
	{
		if (BoundsWidth + (double)ExtraCellCount * InitialCellSize > InitialCellSize * (double)MaxTargetVoxelCount)
		{
			return BoundsWidth / (double)(MaxTargetVoxelCount - ExtraCellCount);
		}
		else
		{
			return InitialCellSize;
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE