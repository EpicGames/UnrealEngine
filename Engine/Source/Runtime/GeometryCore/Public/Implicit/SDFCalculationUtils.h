// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{
	using namespace UE::Math;

	// These are utility functions shared by the various SDF calculation implementations
	// (CachingMeshSDF, SweepignMeshSDF, SparseNarrowBandMeshSDF)


	GEOMETRYCORE_API double PointTriangleDistance(const FVector3d& x0, const FVector3d& x1, const FVector3d& x2, const FVector3d& x3);

	GEOMETRYCORE_API float PointTriangleDistance(const FVector3f& x0, const FVector3f& x1, const FVector3f& x2, const FVector3f& x3);


	GEOMETRYCORE_API bool PointInTriangle2d(double X0, double Y0, double X1, double Y1, double X2, double Y2, double X3, double Y3, double& A, double& B, double& C);

	// find distance x0 is from segment x1-x2
	template<typename RealType> 
	RealType PointSegmentDistance(const TVector<RealType>& x0, const TVector<RealType>& x1, const TVector<RealType>& x2)
	{
		TVector<RealType> DX = x2 - x1;
		RealType m2 = DX.SquaredLength();
		// find parameter value of closest point on segment
		RealType s12 = (DX.Dot(x2 - x0) / m2);
		if (s12 < 0)
		{
			s12 = 0;
		}
		else if (s12 > 1)
		{
			s12 = 1;
		}
		// and find the distance
		return Distance(x0, s12*x1 + (1.0 - s12)*x2);
	}


	// calculate twice signed area of triangle (0,0)-(X1,Y1)-(X2,Y2)
	// return an SOS-determined sign (-1, +1, or 0 only if it's a truly degenerate triangle)
	template<typename RealType> 
	int Orientation(RealType X1, RealType Y1, RealType X2, RealType Y2, RealType& TwiceSignedArea)
	{
		TwiceSignedArea = Y1 * X2 - X1 * Y2;
		if (TwiceSignedArea > 0) return 1;
		else if (TwiceSignedArea < 0) return -1;
		else if (Y2 > Y1) return 1;
		else if (Y2 < Y1) return -1;
		else if (X1 > X2) return 1;
		else if (X1 < X2) return -1;
		else return 0; // only true when X1==X2 and Y1==Y2
	}




}
}