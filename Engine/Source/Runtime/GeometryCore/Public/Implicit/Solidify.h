// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/MeshAABBTree3.h"
#include "Spatial/FastWinding.h"

#include "Generators/MarchingCubes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Use marching cubes to remesh a triangle mesh to a solid surface
 * Uses fast winding number to decide what is inside vs outside
 */
template<typename TriangleMeshType>
class TImplicitSolidify
{
public:

	TImplicitSolidify(const TriangleMeshType* Source = nullptr, TMeshAABBTree3<TriangleMeshType>* SourceSpatial = nullptr, TFastWindingTree<TriangleMeshType>* SourceWinding = nullptr)
		: Source(Source), SourceSpatial(SourceSpatial), SourceWinding(SourceWinding)
	{
	}

	///
	/// Inputs
	///

	const TriangleMeshType* Source = nullptr;
	TMeshAABBTree3<TriangleMeshType>* SourceSpatial = nullptr;
	TFastWindingTree<TriangleMeshType>* SourceWinding = nullptr;

	/** Inside/outside winding number threshold */
	double WindingThreshold = .5;

	/** Whether to use a custom bounding box instead of the input mesh bounds to define the domain to solidify */
	bool bUseCustomBounds = false;

	/** Custom bounds to use, if bUseCustomBounds == true; ignored otherwise */
	FAxisAlignedBox3d CustomBounds;

	/** How much to extend bounds considered by marching cubes outside the original surface bounds. Ignored if bUseCustomBounds == true. */
	double ExtendBounds = 1;

	/** What to do if the surface extends outside the marching cubes bounds -- if true, puts a solid surface at the boundary */
	bool bSolidAtBoundaries = true;

	/** How many binary search steps to do when placing surface at boundary */
	int SurfaceSearchSteps = 4;

	/** size of the cells used when meshing the output (marching cubes' cube size) */
	double MeshCellSize = 1.0;

	/**
	 * Set cell size to hit the target voxel count along the max dimension of the bounds
	 */
	void SetCellSizeAndExtendBounds(FAxisAlignedBox3d Bounds, double ExtendBoundsIn, int TargetOutputVoxelCount)
	{
		ExtendBounds = !bUseCustomBounds ? ExtendBoundsIn : 0.0;
		MeshCellSize = (Bounds.MaxDim() + ExtendBounds * 2.0) / double(TargetOutputVoxelCount);
	}

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []()
	{
		return false;
	};
	
protected:

	FMarchingCubes MarchingCubes;

public:

	/**
	 * @return true if input parameters are valid
	 */
	bool Validate()
	{
		bool bValidMeshAndSpatial = Source != nullptr && SourceSpatial != nullptr && SourceSpatial->IsValid(false);
		bool bValidWinding = SourceWinding != nullptr;
		bool bValidParams = SurfaceSearchSteps >= 0 && MeshCellSize > 0 && FMath::IsFinite(MeshCellSize);
		return bValidMeshAndSpatial && bValidWinding && bValidParams;
	}

	/**
	 * 
	 */
	const FMeshShapeGenerator& Generate()
	{
		MarchingCubes.Reset();
		if (!ensure(Validate()))
		{
			// give up and return and empty result on invalid parameters
			return MarchingCubes;
		}

		FAxisAlignedBox3d UseBounds = bUseCustomBounds ? CustomBounds : SourceSpatial->GetBoundingBox();
		if (!bUseCustomBounds)
		{
			UseBounds.Expand(ExtendBounds);
		}
		
		MarchingCubes.CubeSize = MeshCellSize;

		MarchingCubes.Bounds = UseBounds;
		// expand marching cubes bounds beyond the 'internal' bounds to ensure we sample outside the bounds, if solid-at-boundaries is requested
		if (bSolidAtBoundaries)
		{
			MarchingCubes.Bounds.Expand(MeshCellSize * .1);
		}

		MarchingCubes.RootMode = ERootfindingModes::Bisection;
		MarchingCubes.RootModeSteps = SurfaceSearchSteps;
		MarchingCubes.IsoValue = WindingThreshold;
		MarchingCubes.CancelF = CancelF;

		if (bSolidAtBoundaries)
		{
			MarchingCubes.Implicit = [this, UseBounds](const FVector3d& Pos)
			{
				return UseBounds.Contains(Pos) ? SourceWinding->FastWindingNumber(Pos) : -(WindingThreshold + 1);
			};
		}
		else
		{
			MarchingCubes.Implicit = [this](const FVector3d& Pos)
			{
				return SourceWinding->FastWindingNumber(Pos);
			};
		}

		TArray<FVector3d> MCSeeds;
		for ( int32 VertIdx : Source->VertexIndicesItr() )
		{
			FVector3d Vertex = Source->GetVertex(VertIdx);
			// Only add vertices that are inside the spatial bounds
			// In the non-custom-bounds case, this will only filter out unreferenced vertices, so should not miss surfaces.
			// However, in the case of custom bounds smaller than the mesh bounds, it may cause us to miss surfaces.
			// TODO: consider adding additional sampling options to better support this case
			if (MarchingCubes.Bounds.Contains(Vertex))
			{
				MCSeeds.Add(Vertex);
			}
		}
		MarchingCubes.GenerateContinuation(MCSeeds);

		return MarchingCubes;
	}
};







/**
 * Use marching cubes to remesh an arbitrary function that provides a winding-number like scalar value to a solid surface
 */
class FWindingNumberBasedSolidify
{
public:

	FWindingNumberBasedSolidify(
		TUniqueFunction<double(const FVector3d&)> WindingFunctionIn,
		const FAxisAlignedBox3d& BoundsIn,
		const TArray<FVector3d>& SeedPointsIn)
	{
		this->WindingFunction = MoveTemp(WindingFunctionIn);
		this->FunctionBounds = BoundsIn;
		this->SeedPoints = SeedPointsIn;
	}

	virtual ~FWindingNumberBasedSolidify() = default;

	///
	/// Inputs
	///

	/** External Winding-Number Function */
	TUniqueFunction<double(const FVector3d&)> WindingFunction = [](const FVector3d& Pos) { return 0.0; };

	/** Bounds within which we will mesh things */
	FAxisAlignedBox3d FunctionBounds;

	/** Seed points for meshing */
	TArray<FVector3d> SeedPoints;

	/** Inside/outside winding number threshold */
	double WindingThreshold = .5;

	/** How much to extend bounds considered by marching cubes outside the original surface bounds */
	double ExtendBounds = 1;

	/** What to do if the surface extends outside the marching cubes bounds -- if true, puts a solid surface at the boundary */
	bool bSolidAtBoundaries = true;

	/** How many binary search steps to do when placing surface at boundary */
	int SurfaceSearchSteps = 4;

	/** size of the cells used when meshing the output (marching cubes' cube size) */
	double MeshCellSize = 1.0;

	/**
	 * Set cell size to hit the target voxel count along the max dimension of the bounds
	 */
	void SetCellSizeAndExtendBounds(const FAxisAlignedBox3d& Bounds, double ExtendBoundsIn, int TargetOutputVoxelCount)
	{
		ExtendBounds = ExtendBoundsIn;
		MeshCellSize = (Bounds.MaxDim() + ExtendBounds * 2.0) / double(TargetOutputVoxelCount);
	}

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []()
	{
		return false;
	};
	
protected:

	FMarchingCubes MarchingCubes;

public:

	/**
	 * @return true if input parameters are valid
	 */
	bool Validate()
	{
		bool bValidParams = SurfaceSearchSteps >= 0 && MeshCellSize > 0 && FMath::IsFinite(MeshCellSize);
		return bValidParams;
	}

	/**
	 * 
	 */
	const FMeshShapeGenerator& Generate()
	{
		MarchingCubes.Reset();
		if (!ensure(Validate()))
		{
			// give up and return and empty result on invalid parameters
			return MarchingCubes;
		}

		FAxisAlignedBox3d InternalBounds = FunctionBounds;
		InternalBounds.Expand(ExtendBounds);
		
		MarchingCubes.CubeSize = MeshCellSize;

		MarchingCubes.Bounds = InternalBounds;

		// expand marching cubes bounds beyond the 'internal' bounds to ensure we sample outside the bounds, if solid-at-boundaries is requested
		if (bSolidAtBoundaries)
		{
			MarchingCubes.Bounds.Expand(MeshCellSize * .1);
		}

		MarchingCubes.RootMode = ERootfindingModes::Bisection;
		MarchingCubes.RootModeSteps = SurfaceSearchSteps;
		MarchingCubes.IsoValue = WindingThreshold;
		MarchingCubes.CancelF = CancelF;

		if (bSolidAtBoundaries)
		{
			MarchingCubes.Implicit = [this, InternalBounds](const FVector3d& Pos)
			{
				return InternalBounds.Contains(Pos) ? WindingFunction(Pos) : -(WindingThreshold + 1);
			};
		}
		else
		{
			MarchingCubes.Implicit = [this](const FVector3d& Pos)
			{
				return WindingFunction(Pos);
			};
		}

		TArray<TVector<double>> MCSeeds;
		for ( const FVector3d& SeedPoint : SeedPoints )
		{
			MCSeeds.Add(SeedPoint);
		}
		MarchingCubes.GenerateContinuation(MCSeeds);

		return MarchingCubes;
	}
};






} // end namespace UE::Geometry
} // end namespace UE
