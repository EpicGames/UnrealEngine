// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshSignedDistanceGrid

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Util/IndexUtil.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/BlockedLayout3.h"
#include "Spatial/BlockedDenseGrid3.h"
#include "Async/ParallelFor.h"
#include "Implicit/GridInterpolant.h"
#include "Implicit/SDFCalculationUtils.h"
#include "Containers/StaticArray.h"

#include <atomic> // note: UE FGenericPlatformAtomics are deprecated. 

namespace UE
{
namespace Geometry
{


/**
 * Compute discretely-sampled (ie gridded) signed distance field for a mesh within a specified narrow band.
 * 
 * The basic approach is, first compute exact Distances in a narrow band, and then
 * The resulting unsigned Grid is then signed using ray-intersection counting, which
 * is also computed on the Grid, so no BVH is necessary
 * 
 * The underlying grid data structures are blocked grids where spatial blocks are allocated
 * as needed, so the memory footprint will generally be much lower than the TSweepingMeshSDF, or TCachingMeshSDF.
 * But if distances in the full grid domain are desired (not just a narrow band), then the TSweepingMeshSDF should be used.
 *
 * You can optionally provide a spatial data structure to allow faster computation of
 * narrow-band Distances; this is most beneficial if we want a wider band 
 * 
 * Caveats:
 *  - the "narrow band" is based on triangle bounding boxes, so it is not necessarily
 *    that "narrow" if you have large triangles on a diagonal to Grid axes
 *
 * 
 * This code is based on the implementation found at https://github.com/christopherbatty/SDFGen
 */
template <class TriangleMeshType>
class TSparseNarrowBandMeshSDF
{
public:
	// INPUTS

	const TriangleMeshType* Mesh = nullptr;
	TMeshAABBTree3<TriangleMeshType>* Spatial = nullptr;       // required for the NarrowBand_SpatialFloodFill method.
	TUniqueFunction<bool(const FVector3d&)> IsInsideFunction;  // if not supplied, the intersection counting method is used.

	// Relates the grids index space, to mesh distance units.
	float CellSize;

	// Width of the band around triangles for which exact Distances are computed
	// (In fact this is conservative, the band is often larger locally)
	int ExactBandWidth = 1;

	// Bounds of Grid will be expanded this much in positive and negative directions.
	// Useful for if you want field to extend outwards.
	FVector3d ExpandBounds = FVector3d::Zero();

	// Most of this parallelizes very well, makes a huge speed difference
	bool bUseParallel = true;

	// If the number of cells in any dimension may exceed this, CellSize will be automatically increased to keep cell count reasonable
	int ApproxMaxCellsPerDimension = 4096;

	// The narrow band is always computed exactly, and the full Grid will be signed if bComputeSigns is true.
	enum EComputeModes
	{
		NarrowBandOnly = 1,
		NarrowBand_SpatialFloodFill = 2
	};
	EComputeModes ComputeMode = EComputeModes::NarrowBandOnly;

	// how wide of narrow band should we compute. This value is 
	// currently only used NarrowBand_SpatialFloodFill, as
	// we can efficiently explore the space
	// (in that case ExactBandWidth is only used to initialize the grid extents, and this is used instead during the flood fill)
	double NarrowBandMaxDistance = 0;

	// If true, sign of full grid will be computed using either IsInsideFunction (if provided) or crossing as controlled by the InsideMode
	bool bComputeSigns = true;

	// What counts as "inside" the mesh. Crossing count does not use triangle
	// Orientation, so inverted faces are fine, but overlapping shells or self intersections
	// will be filled using even/odd rules (as seen along X axis...)
	// Winding count is basically mesh winding number, handles overlap shells and
	// self-intersections, but inverted shells are 'subtracted', and inverted faces are a disaster.
	// Both modes handle internal cavities, neither handles open sheets.
	enum EInsideModes
	{
		CrossingCount = 0,
		WindingCount = 1
	};
	EInsideModes InsideMode = EInsideModes::WindingCount;

	// Implementation computes the triangle closest to each Grid cell, can
	// return this Grid if desired (only reason not to is avoid hanging onto memory)
	bool bWantClosestTriGrid = false;

	// Grid of per-cell crossing or parity counts
	bool bWantIntersectionsGrid = true;

	/** if this function returns true, the calculation will abort */
	TFunction<bool()> CancelF = []()
	{
		return false;
	};

	// OUTPUTS

	FVector3f GridOrigin;
	FBlockedGrid3f Grid;

protected:
	// grid of closest triangle indices; only available if bWantClosestTriGrid is set to true.  Access via GetClosestTriGrid()
	FBlockedGrid3i ClosestTriGrid;
	// grid of intersection counts; only available if bWantIntersectionsGrid is set to true.  Access via GetIntersectionsGrid()
	FBlockedGrid3i IntersectionsGrid;


protected:
	// structs used in a scatter / gather of triangles into the grid blocks they overlap.
	
	// Grid that holds one atomic<int> in each block
	struct FScatterCounter : public TBlockedGrid3Layout<FBlockedGrid3f::BlockSize>
	{
		typedef TBlockedGrid3Layout<FBlockedGrid3f::BlockSize> MyLayout;
		FScatterCounter(const FVector3i& Dims, bool bInitParallel)
			: MyLayout(Dims)
		{
			const FVector3i BlockDims = MyLayout::GetBlockDimensions();
			const int32 NumBlocks = BlockDims[0] * BlockDims[1] * BlockDims[2];
			NumObjectsPerBlock.SetNum(NumBlocks);
			std::atomic<int>* PerBlock = NumObjectsPerBlock.GetData();
			ParallelFor(NumBlocks, [PerBlock](int32 id)
			{
				PerBlock[id] = 0;
			}, !bInitParallel);
		}
		
		// increment the counter for the indicated block
		void AtomicIncrement(const int32 BlockIndex)
		{
			std::atomic<int>& NumObjects = NumObjectsPerBlock.GetData()[BlockIndex];
			NumObjects.fetch_add(1, std::memory_order_relaxed);
		}
		
		TArray<std::atomic<int>> NumObjectsPerBlock;
	};

	// sparse grid that stores array of TriIDs each blocks.
	struct FTriIDBlockGrid : public TBlockedGrid3Layout<FBlockedGrid3f::BlockSize>
	{
		typedef TBlockedGrid3Layout<FBlockedGrid3f::BlockSize> MyLayout;

		FTriIDBlockGrid(const FScatterCounter& TriCounter, bool bInitParallel)
			: MyLayout(TriCounter.GetDimensions())
		{
			const FVector3i DimAsBlocks = MyLayout::GetBlockDimensions();
			const int32 NumBlocks = DimAsBlocks.X * DimAsBlocks.Y * DimAsBlocks.Z;
			TrisInBlock.SetNum(NumBlocks);
			PerBlockNum.SetNum(NumBlocks);
			const std::atomic<int>* PerBlockCounted = TriCounter.NumObjectsPerBlock.GetData();
			std::atomic<int>* SizePerBlock = PerBlockNum.GetData();
			ParallelFor(NumBlocks, [this, SizePerBlock, PerBlockCounted, &TriCounter](int32 id)
			{
				TArray<int32>& Tris = TrisInBlock[id];
				const int32 n = PerBlockCounted[id];
				Tris.AddUninitialized(n);
				SizePerBlock[id] = 0;
			}, !bInitParallel);
		}



		void AddTriID(const int32& BlockIndex, const int32& TriId)
		{
			TArray<int32>& Tris = TrisInBlock.GetData()[BlockIndex];
			std::atomic<int>& Num = PerBlockNum.GetData()[BlockIndex];

			int32 i = Num.fetch_add(1, std::memory_order_relaxed);;
			Tris.GetData()[i] = TriId;
		}
		TArray<int32> GetOccupiedBlockIDs() const 
		{
			const int32 NumBlocks = TrisInBlock.Num();
			int32 NumBlocksWithTris = 0;
			for (int32 i = 0; i < NumBlocks; ++i)
			{
				if (TrisInBlock[i].Num() != 0)
				{
					NumBlocksWithTris++;
				}
			}
			
			TArray<int32> TmpArray;
			TmpArray.Reserve(NumBlocksWithTris);
			for (int32 i = 0; i < NumBlocks; ++i)
			{
				if (TrisInBlock[i].Num() != 0)
				{
					TmpArray.Add(i);
				}
			}
			return TmpArray;
		};

		TArray<std::atomic<int>>  PerBlockNum;
		TArray<TArray<int32>>     TrisInBlock;
	};

	struct FTriBBox
	{
		FVector3i BoxMin = FVector3i(INT32_MAX, INT32_MAX, INT32_MAX);
		FVector3i BoxMax = FVector3i(INT32_MIN, INT32_MIN, INT32_MIN);

		bool IsValid() const
		{
			return ((BoxMax[0] >= BoxMin[0]) && (BoxMax[1] >= BoxMin[1]) && (BoxMax[2] >= BoxMin[2]));
		}

		// reduce this box by clamping with specified min/max
		void Clamp(const FVector3i& OtherMin, const FVector3i& OtherMax)
		{	
			BoxMin[0] = FMath::Clamp(BoxMin[0], OtherMin[0], OtherMax[0]);
			BoxMin[1] = FMath::Clamp(BoxMin[1], OtherMin[1], OtherMax[1]);
			BoxMin[2] = FMath::Clamp(BoxMin[2], OtherMin[2], OtherMax[2]);

			BoxMax[0] = FMath::Clamp(BoxMax[0], OtherMin[0], OtherMax[0]);
			BoxMax[1] = FMath::Clamp(BoxMax[1], OtherMin[1], OtherMax[1]);
			BoxMax[2] = FMath::Clamp(BoxMax[2], OtherMin[2], OtherMax[2]);
		}
	};

public:

	/**
	 * @param Mesh Triangle mesh to build an SDF around
	 * @param CellSize Spacing between Grid points
	 * @param Spatial Optional AABB tree; note it *must* be provided if ComputeMode is set to NarrowBand_SpatialFloodFill
	 */
	TSparseNarrowBandMeshSDF(
		const TriangleMeshType* Mesh = nullptr, 
		float CellSize = 1.f, 
		TMeshAABBTree3<TriangleMeshType>* Spatial = nullptr ) 
		: Mesh(Mesh), Spatial(Spatial), CellSize(CellSize)
	{
	}

	TTriLinearGridInterpolant<TSparseNarrowBandMeshSDF> MakeInterpolant()
	{
		return TTriLinearGridInterpolant<TSparseNarrowBandMeshSDF>(this, (FVector3d)GridOrigin, CellSize, Dimensions());
	}

	/**
	 * Encodes heuristic for deciding whether it will be faster to use
	 *  - NarrowBand_SpatialFloodFill
	 *  - Or NarrowBandOnly
	 * The heuristic is to use the Spatial method unless you have relatively long edges relative to the width
	 * of the narrow band.  For a 1-cell wide band, or one where few triangles interact w/ each cell in the band
	 * the spatial overhead will probably not be worthwhile.
	 * TODO: test and tune this ...
	 */
	static bool ShouldUseSpatial(int32 ExactCells, double CellSize, double AvgEdgeLen)
	{
		double EdgesPerCell = CellSize / AvgEdgeLen;
		double EdgesPerBand = ExactCells * EdgesPerCell;
		return ExactCells > 1 && EdgesPerBand >= .25;
	}

	bool Validate(const FAxisAlignedBox3d& Bounds)
	{
		if (Bounds.IsEmpty() || !FMath::IsFinite(Bounds.MaxDim()))
		{
			return false;
		}
		if (CellSize <= 0 || !FMath::IsFinite(CellSize))
		{
			return false;
		}
		if (ApproxMaxCellsPerDimension < 5)
		{
			return false;
		}
		return true;
	}

	/**
	 * Compute the SDF
	 * @param Bounds Bounding box for the mesh data (passed as param it is usually already available, depending on TriangleMeshType and whether an AABB tree was provided)
	 * @return false if cancelled or failed (e.g. due to invalid arguments); true otherwise
	 */
	bool Compute(FAxisAlignedBox3d Bounds)
	{
		if (!ensure(Validate(Bounds)))
		{
			return false;
		}

		if (!ensureMsgf(ExactBandWidth*2 < ApproxMaxCellsPerDimension, TEXT("Cannot request band wider than half the max cells per dimension")))
		{
			ExactBandWidth = FMath::Max(ApproxMaxCellsPerDimension/2-1, 1);
		}

		double MaxDim = MaxElement(Bounds.Max - Bounds.Min + ExpandBounds * 2.0);
		if (!ensureMsgf(MaxDim / CellSize <= ApproxMaxCellsPerDimension - 2 * ExactBandWidth, TEXT("SDF resolution clamped to avoid excessive memory use")))
		{
			CellSize = float ( MaxDim / (ApproxMaxCellsPerDimension - 2 * ExactBandWidth) );
			if (!ensure(CellSize > 0 && FMath::IsFinite(CellSize)))
			{
				return false;
			}
		}

		float fBufferWidth = float(ExactBandWidth) * CellSize;
		if (ComputeMode == EComputeModes::NarrowBand_SpatialFloodFill)
		{
			fBufferWidth = (float)FMath::Max(fBufferWidth, float(NarrowBandMaxDistance));
		}
		GridOrigin = (FVector3f)Bounds.Min - fBufferWidth * FVector3f::One() - (FVector3f)ExpandBounds;
		FVector3f max = (FVector3f)Bounds.Max + fBufferWidth * FVector3f::One() + (FVector3f)ExpandBounds;
		int32 NI = (int32)((max.X - GridOrigin.X) / CellSize) + 1;
		int32 NJ = (int32)((max.Y - GridOrigin.Y) / CellSize) + 1;
		int32 NK = (int32)((max.Z - GridOrigin.Z) / CellSize) + 1;

		
		if (ComputeMode == EComputeModes::NarrowBand_SpatialFloodFill)
		{
			if (!ensureMsgf(Spatial && NarrowBandMaxDistance != 0, TEXT("SweepingMeshSDF::Compute: must set Spatial data structure and band max distance")))
			{
				return false;
			}
			make_level_set3_parallel_floodfill(GridOrigin, CellSize, NI, NJ, NK, Grid, NarrowBandMaxDistance);
		}
		else // In general this path is faster than NarrowBand_SpatialFloodFill as it is lockless.
		{
			if (Spatial != nullptr) 
			{
				make_level_set3_parallel_spatial(GridOrigin, CellSize, NI, NJ, NK, Grid, ExactBandWidth);
			}
			else
			{
				make_level_set3_parallel(GridOrigin, CellSize, NI, NJ, NK, Grid, ExactBandWidth);
			}
		}

		if (bComputeSigns)
		{
			update_signs(GridOrigin, CellSize, NI, NJ, NK, Grid, IntersectionsGrid);
		}
		

		CleanupUnwanted();

		return !CancelF();
	}


	/** @return the number of cells in each direction*/
	FVector3i Dimensions() const
	{
		return Grid.GetDimensions();
	}

	/** @return a grid that can be sampled within the narrow band to find the ID of the closest triangle.  Note: this is populated during Compute()*/
	const FBlockedGrid3i& GetClosestTriGrid() const
	{
		checkf(bWantClosestTriGrid, TEXT("Set bWantClosestTriGrid=true to return this value"));
		return ClosestTriGrid;
	}

	/** @return a grid that can be sampled within the narrow band to find cells that actually contain mesh triangles.  Note: this is populated during Compute()*/
	const FBlockedGrid3i& GetIntersectionsGrid() const
	{
		checkf(bWantIntersectionsGrid, TEXT("Set bWantIntersectionsGrid=true to return this value"));
		return IntersectionsGrid;
	}

	/** @return the distance value at the specified grid cell.*/
	float At(int32 I, int32 J, int32 K) const
	{
		return Grid.GetValue(I, J, K);
	}

	/** @return the distance value at the specified grid cell.*/
	float operator[](const FVector3i& ijk) const
	{
		return Grid.GetValue(ijk);
	}
	
	/** @return the distance value at the specified grid cell.*/
	float GetValue(const FVector3i& ijk) const // TTriLinearGridInterpolant interface 
	{
		return Grid.GetValue(ijk);
	}

	/** @return the physical space location of specified grid point.*/
	FVector3f CellCenter(int32 I, int32 J, int32 K) const
	{
		return cell_center(FVector3i(I, J, K));
	}



private:
	
	// iterate over a bounded grid region applying a functor at each cell
	template <typename FunctorType>
	void inclusive_loop_3d(const FVector3i& start, const FVector3i& inclusive_end, FunctorType func)
	{
		FVector3i ijk;
		for (ijk[2] = start[2]; ijk[2] <= inclusive_end[2]; ++ijk[2])
		{
			for (ijk[1] = start[1]; ijk[1] <= inclusive_end[1]; ++ijk[1])
			{
				for (ijk[0] = start[0]; ijk[0] <= inclusive_end[0]; ++ijk[0])
				{
					func(ijk);
				}
			}
		}
	}

	FVector3f cell_center(const FVector3i& IJK) const
	{
		return FVector3f((float)IJK.X * CellSize + GridOrigin[0],
						 (float)IJK.Y * CellSize + GridOrigin[1],
						 (float)IJK.Z * CellSize + GridOrigin[2]);
	}

	float upper_bound(const FVector3i& GridDimensions, float DX) const
	{
		return float(GridDimensions[0] + GridDimensions[1] + GridDimensions[2]) * DX;
	}

	// if provided, uses IsInsideFunction to determine sign, otherwise uses a counting method.
	void update_signs(const FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, FBlockedGrid3f& Distances, FBlockedGrid3i& IntersectionCountGrid)
	{
	
		typedef FBlockedGrid3f::BlockData3Type    FloatBlockData3Type;


		if (IsInsideFunction)  // Compute signs using inside/outside oracle.
		{
			
			TArray<FloatBlockData3Type*> DistanceBlocks = Distances.GetAllocatedBlocks();
			const int32 NumBlocksAllocated = DistanceBlocks.Num();

			// evaluate the inside/outside oracle at each cell within the allocated blocks.
			ParallelFor(NumBlocksAllocated, [this, &DistanceBlocks, &Distances, Origin, DX](int32 AllocatedBlockIdx)
			{

				FloatBlockData3Type& DistanceBlock = *DistanceBlocks[AllocatedBlockIdx];

				const FVector3i BlockIJK = Distances.BlockIndexToBlockIJK(DistanceBlock.Id);
				const FVector3i BlockOrigin = FloatBlockData3Type::BlockSize * BlockIJK;

				const int32 ElemCount = FloatBlockData3Type::ElemCount;
				for (int32 LocalIndex = 0; LocalIndex < ElemCount; ++LocalIndex)
				{
					const FVector3i GridCoord = DistanceBlock.ToLocalIJK(LocalIndex) + BlockOrigin;
					const FVector3d Pos((float)GridCoord.X * DX + Origin[0], (float)GridCoord.Y * DX + Origin[1], (float)GridCoord.Z * DX + Origin[2]);
					if (this->IsInsideFunction(Pos))
					{
						float& dist = DistanceBlock.At(LocalIndex);
						dist *= -1;
					}
				}
			}, !this->bUseParallel);
			
			if (CancelF())
			{
				return;
			}

			// update default value for unallocated blocks: evaluate the inside/outside oracle at the center of each.

			const int32 NumBlocks = Distances.GetNumBlocks();
			const FVector3d BlockCenter(0.5 * DX * (double)FloatBlockData3Type::BlockSize);
			ParallelFor(NumBlocks, [this, &Distances, Origin, DX, BlockCenter](int32 BlockIndex)
				{
					const FVector3i BlockIJK = Distances.BlockIndexToBlockIJK(BlockIndex);
					if (!Distances.IsBlockAllocated(BlockIJK))
					{
						const FVector3i BlockOrigin = FloatBlockData3Type::BlockSize * BlockIJK;
						const FVector3d BlockOriginPos((float)BlockOrigin.X * DX + Origin.X, (float)BlockOrigin.Y * DX + Origin.Y, (float)BlockOrigin.Z * DX + Origin.Z);
						const FVector3d Pos = BlockOriginPos + BlockCenter;
						if (this->IsInsideFunction(Pos))
						{
							Distances.ProcessBlockDefaultValue(BlockIJK, [](float& value) {value = -value; });
						}
					}

				}, !this->bUseParallel);

		}
		else // use intersection counts and the specified InsideMode to determine inside/outside regions.
		{
			// intersection_count(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
			
			compute_intersections(Origin, DX, NI, NJ, NK, Distances, IntersectionCountGrid);
			if (CancelF())
			{
				return;
			}

			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, IntersectionCountGrid, Distances);
		}
	} // update_signs

	// compute AABBs for the tris in the mesh.  the resulting TArray is indexed by the same TID as the mesh
	// ( note, if the mesh doesn't have compact TIDs, the boxes that correspond to gaps in the TIDs will be in a state IsInvalid() == true.)
	TArray<FTriBBox> compute_tri_bboxes(FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, int32 ExactBand)
	{
		const double invdx = 1.0 / DX;
		const FVector3d org((double)Origin[0], (double)Origin[1], (double)Origin[2]);

		TArray<FTriBBox> TriBBoxes; TriBBoxes.SetNum(Mesh->MaxTriangleID());

		auto ComputeTriBoundingBox = [this, ExactBand, NI, NJ, NK, invdx, org](const int32 TID, FVector3i& BBMin, FVector3i& BBMax)
									{
										FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
										Mesh->GetTriVertices(TID, xp, xq, xr);

										// real IJK coordinates of xp/xq/xr

										const FVector3d fp = (xp - org) * invdx;
										const FVector3d fq = (xq - org) * invdx;
										const FVector3d fr = (xr - org) * invdx;

										// clamped integer bounding box of triangle plus/minus exact-band
										BBMin[0] = FMath::Clamp(((int)FMath::Min3(fp[0], fq[0], fr[0])) - ExactBand, 0, NI - 1);
										BBMin[1] = FMath::Clamp(((int)FMath::Min3(fp[1], fq[1], fr[1])) - ExactBand, 0, NJ - 1);
										BBMin[2] = FMath::Clamp(((int)FMath::Min3(fp[2], fq[2], fr[2])) - ExactBand, 0, NK - 1);

										BBMax[0] = FMath::Clamp(((int)FMath::Max3(fp[0], fq[0], fr[0])) + ExactBand + 1, 0, NI - 1);
										BBMax[1] = FMath::Clamp(((int)FMath::Max3(fp[1], fq[1], fr[1])) + ExactBand + 1, 0, NJ - 1);
										BBMax[2] = FMath::Clamp(((int)FMath::Max3(fp[2], fq[2], fr[2])) + ExactBand + 1, 0, NK - 1);
									};

		ParallelFor(Mesh->MaxTriangleID(), [this, &ComputeTriBoundingBox, &TriBBoxes](int TID)
		{
			if (!Mesh->IsTriangle(TID)) return; // skipped BBoxes will be left in 'bbox.IsInvalid() == true' state

			FVector3i& BoxMin = TriBBoxes.GetData()[TID].BoxMin;
			FVector3i& BoxMax = TriBBoxes.GetData()[TID].BoxMax;
			ComputeTriBoundingBox(TID, BoxMin, BoxMax);
		});

		return TriBBoxes;
	}

	// scatter box ids (based on the overlap of bbox with grid block) to return a grid that holds IDs in block-level TArrays. 
	FTriIDBlockGrid scatter_tris(const TArray<FTriBBox>& TriBBoxes, int32 NI, int32 NJ, int32 NK)
	{
		bool bCancel = false;

		// count number of tris that will scatter to each block.
		FScatterCounter TriCounter(FVector3i(NI, NJ, NK), this->bUseParallel);
		{
			const int32 NumBBoxes = TriBBoxes.Num();
			ParallelFor(NumBBoxes, [this, &TriBBoxes, &TriCounter, &bCancel](const  int32 TID)
			{
				constexpr int32 BlockSize = FBlockedGrid3f::BlockSize;

				const FTriBBox& TriBBox = TriBBoxes.GetData()[TID];
				if (!TriBBox.IsValid())
				{
					return;
				}
				if (TID % 100 == 0)
				{
					bCancel = this->CancelF();
				}
				if (bCancel)
				{
					return;
				}

				// convert bbox to block-level ijk
				FVector3i box_min(TriBBox.BoxMin[0] / BlockSize, TriBBox.BoxMin[1] / BlockSize, TriBBox.BoxMin[2] / BlockSize);
				FVector3i box_max(TriBBox.BoxMax[0] / BlockSize, TriBBox.BoxMax[1] / BlockSize, TriBBox.BoxMax[2] / BlockSize);

				// increment the count for each block this tri scatters to.
				inclusive_loop_3d(box_min, box_max, 
								[&TriCounter](const FVector3i& block_ijk) 
								{
									const int32 BlockIndex = TriCounter.BlockIJKToBlockIndex(block_ijk);
									TriCounter.AtomicIncrement(BlockIndex); 
								});
				
			}, !this->bUseParallel);
		}

		// scatter the tris into the TriIDBlockedGrid
		FTriIDBlockGrid TriIDBlockGrid(TriCounter, this->bUseParallel);
		{
			const int32 NumBBoxes = TriBBoxes.Num();
			ParallelFor(NumBBoxes, [this, &TriBBoxes, &TriIDBlockGrid, &bCancel](const  int32 TID)
			{
				constexpr int32 BlockSize = FBlockedGrid3f::BlockSize;

				const FTriBBox& TriBBox = TriBBoxes.GetData()[TID];
				if (!TriBBox.IsValid())
				{
					return;
				}
				if (TID % 100 == 0)
				{
					bCancel = this->CancelF();
				}
				if (bCancel)
				{
					return;
				}

				// convert bbox to block-level ijk
				FVector3i box_min(TriBBox.BoxMin[0] / BlockSize, TriBBox.BoxMin[1] / BlockSize, TriBBox.BoxMin[2] / BlockSize);
				FVector3i box_max(TriBBox.BoxMax[0] / BlockSize, TriBBox.BoxMax[1] / BlockSize, TriBBox.BoxMax[2] / BlockSize);

				// increment the count for each block this tri scatters to.
				inclusive_loop_3d(box_min, box_max, 
								[&TriIDBlockGrid, TID](const FVector3i& block_ijk) 
								{
										const int32 BlockIndex = TriIDBlockGrid.BlockIJKToBlockIndex(block_ijk);
										TriIDBlockGrid.AddTriID(BlockIndex, TID); // blocks have already been allocated, and this is a threadsafe add to an array
								});
			
			}, !this->bUseParallel);
		}

		return TriIDBlockGrid;
	}

	// lockless method - uses a scatter/ gather pattern to implement a painters algorithm (rasterizing all triangles but retaining only the min). 
	void make_level_set3_parallel(const FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, FBlockedGrid3f& DistanceGrid, int32 ExactBand)
	{
		const float upper_bound_value = upper_bound(FVector3i(NI, NJ, NK), DX);

		DistanceGrid.Reset(NI, NJ, NK, upper_bound_value);
		ClosestTriGrid.Reset(NI, NJ, NK, -1);

		// bboxes for each tri.
		const TArray<FTriBBox> TriBBoxes = compute_tri_bboxes(Origin, DX, NI, NJ, NK, ExactBand);

		// blocked grid of tri ids - identifies the tris that overlap with each grid block.
		const FTriIDBlockGrid TIDBlockGrid = scatter_tris(TriBBoxes, NI, NJ, NK);

		// identify the blocks that are occupied by tris
		const TArray<int32> OccupiedBlocks = TIDBlockGrid.GetOccupiedBlockIDs();

		if (this->CancelF())
		{
			return;
		}


		const double invdx = 1.0 / DX;
		const FVector3d org((double)Origin[0], (double)Origin[1], (double)Origin[2]);

		// parallelize over occupied blocks.
		// within each block, loop over the associated triangles, rasterizing distance (retaining the min).
		{

			// actually raster the tris into the distance and closest tri grid
			const int32 NumOccupied = OccupiedBlocks.Num();
			ParallelFor(NumOccupied, [this, &TriBBoxes, &OccupiedBlocks, &TIDBlockGrid, &DistanceGrid, ExactBand, org, invdx, DX](const int32& OccupiedId)
			{
				const FVector3i GridDimensions = DistanceGrid.GetDimensions();
				const int32 BlockIndex = OccupiedBlocks.GetData()[OccupiedId];
				const TArray<int32>& TrisToRasterize = TIDBlockGrid.TrisInBlock[BlockIndex];

				const FVector3i BlockIJK = DistanceGrid.BlockIndexToBlockIJK(BlockIndex);
				// allocates block on demand.  doing so here assumes the parallel memory allocator is up to it..
				FBlockedGrid3f::BlockData3Type& DistanceBlock   = DistanceGrid.TouchBlockData(BlockIJK);
				FBlockedGrid3i::BlockData3Type& ClosestTriBlock = ClosestTriGrid.TouchBlockData(BlockIJK);
			
				// bounding box for this block [inclusive, inclusive]
				
				const FVector3i BlockMin = FBlockedGrid3f::BlockSize * BlockIJK;
				const FVector3i BlockMax( FMath::Min(BlockMin[0] + FBlockedGrid3f::BlockSize, GridDimensions[0]) -1,
										  FMath::Min(BlockMin[1] + FBlockedGrid3f::BlockSize, GridDimensions[1]) -1,
										  FMath::Min(BlockMin[2] + FBlockedGrid3f::BlockSize, GridDimensions[2]) -1);

				for (const int32 TID : TrisToRasterize)
				{ 

					FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
					Mesh->GetTriVertices(TID, xp, xq, xr);

					// real IJK coordinates of xp/xq/xr

					const FVector3d fp = (xp - org) * invdx;
					const FVector3d fq = (xq - org) * invdx;
					const FVector3d fr = (xr - org) * invdx;

					// clamped integer bounding box of triangle plus exact-band
					FTriBBox tri_bbox = TriBBoxes.GetData()[TID];

					// clamp against the block
					tri_bbox.Clamp(BlockMin, BlockMax);

					// iter over the bbox region, min-compositing the distance from this tri into the grid. 
					inclusive_loop_3d(tri_bbox.BoxMin, tri_bbox.BoxMax, 
									[&](const FVector3i& ijk)
									{
										const FVector3i local_coords = ijk - BlockMin;
										const int32 local_index = DistanceBlock.ToLinear(local_coords[0], local_coords[1], local_coords[2]);
										const FVector3d cellpos((double)ijk[0], (double)ijk[1], (double)ijk[2]);
										float d = float(DX * PointTriangleDistance(cellpos, fp, fq, fr));

										float& grid_dist = DistanceBlock.At(local_index);
										if (d < grid_dist)
										{
											grid_dist = d;
											ClosestTriBlock.At(local_index) = TID;
										}
									});
				}

			}, !this->bUseParallel);
		};
	}   // make_level_set3_parallel

	// lockless method - uses a scatter/ gather pattern along with a spatial acceleration structure to find the closest triangle (relative to each narrow band cell)
	void make_level_set3_parallel_spatial(FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, FBlockedGrid3f& DistanceGrid, int32 ExactBand)
	{
		const float upper_bound_value = upper_bound(FVector3i(NI, NJ, NK), DX);

		DistanceGrid.Reset(NI, NJ, NK, upper_bound_value);
		ClosestTriGrid.Reset(NI, NJ, NK, -1);

		// bboxes for each tri.
		const TArray<FTriBBox> TriBBoxes = compute_tri_bboxes(Origin, DX, NI, NJ, NK, ExactBand);

		// blocked grid of tri IDs - identifies the tris that overlap with each grid block.
		const FTriIDBlockGrid TIDBlockGrid = scatter_tris(TriBBoxes, NI, NJ, NK);

		// identify the blocks that are occupied by tris
		const TArray<int32> OccupiedBlocks = TIDBlockGrid.GetOccupiedBlockIDs();


		if (this->CancelF())
		{
			return;
		}

		// parallelize over occupied blocks.  
		// within the block, 
		//   1st, mark overlapped cells by scattering the tri bboxes identified with block
		//   2nd, gather the closest distance value into each overlapped cell using the spatial lookup. 
		
		{
			const double max_dist = ExactBand * (DX * FMathd::Sqrt2);
		
			// actually raster the tris bbox
			const int32 NumOccupied = OccupiedBlocks.Num();
			ParallelFor(NumOccupied, [this, &Origin, &OccupiedBlocks, &TIDBlockGrid, &DistanceGrid, &TriBBoxes, DX, upper_bound_value,  max_dist](const int32& OccupiedId)
				{
					const FVector3i GridDimensions       = DistanceGrid.GetDimensions();
					const int32 BlockIndex               = OccupiedBlocks.GetData()[OccupiedId];
					const TArray<int32>& TrisToRasterize = TIDBlockGrid.TrisInBlock[BlockIndex];

					const FVector3i BlockIJK    = DistanceGrid.BlockIndexToBlockIJK(BlockIndex);
					const FVector3i BlockOrigin = FBlockedGrid3f::BlockData3Type::BlockSize * BlockIJK;
					// allocates block on demand.  doing so here assumes the parallel memory allocator is up to it..
					FBlockedGrid3f::BlockData3Type& DistanceBlock   = DistanceGrid.TouchBlockData(BlockIJK);
					FBlockedGrid3i::BlockData3Type& ClosestTriBlock = ClosestTriGrid.TouchBlockData(BlockIJK);

					FBlockedGrid3b::BlockData3Type BitBlock(false, -1);
					// bounding box for this block [inclusive, inclusive]
					const FVector3i BlockMin = FBlockedGrid3f::BlockSize * BlockIJK;
					const FVector3i BlockMax( FMath::Min(BlockMin[0] + FBlockedGrid3f::BlockSize, GridDimensions[0]) - 1,
						                      FMath::Min(BlockMin[1] + FBlockedGrid3f::BlockSize, GridDimensions[1]) - 1,
						                      FMath::Min(BlockMin[2] + FBlockedGrid3f::BlockSize, GridDimensions[2]) - 1);

					for (const int32 TID : TrisToRasterize)
					{
						FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
						Mesh->GetTriVertices(TID, xp, xq, xr);

						// clamped integer bounding box of triangle plus exact-band
						FTriBBox tri_bbox = TriBBoxes.GetData()[TID];

						// clamp against the block
						tri_bbox.Clamp(BlockMin, BlockMax);
						
						inclusive_loop_3d(tri_bbox.BoxMin, tri_bbox.BoxMax, 
										[&](const FVector3i& ijk)
										{
											const FVector3i local_coords = ijk - BlockMin;
											const int32 local_index = DistanceBlock.ToLinear(local_coords[0], local_coords[1], local_coords[2]);
											BitBlock.At(local_index) = true;
										});
					}

					

					for (int32 local_index = 0; local_index < FBlockedGrid3f::BlockData3Type::ElemCount; ++local_index)
					{
						if (BitBlock.At(local_index) == true)
						{
							const FVector3i GridCoord = DistanceBlock.ToLocalIJK(local_index) + BlockOrigin;

							const FVector3d Pos((float)GridCoord.X* DX + Origin[0], (float)GridCoord.Y* DX + Origin[1], (float)GridCoord.Z* DX + Origin[2]);
							double dsqr;
							int near_tid = this->Spatial->FindNearestTriangle(Pos, dsqr, max_dist);
							if (near_tid == IndexConstants::InvalidID)
							{
								DistanceBlock.At(local_index) = upper_bound_value; 
							}
							else
							{
								const float dist = (float)FMath::Sqrt(dsqr);
								DistanceBlock.At(local_index) = dist;  
								ClosestTriBlock.At(local_index) = near_tid;
							}
						}
						
					}

				}, !this->bUseParallel);
		}
	}

	// use the mesh vertex points as seeds, and flood fill distance until reaching the exact band width. this uses some thread-locking, but can be fast if the width is not large compared to block size
	void make_level_set3_parallel_floodfill(FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, FBlockedGrid3f& DistanceGrid, double NarrowBandWidth)
	{
	
		typedef FBlockedGrid3f::BlockData3Type   FloatBlockData3Type;
		typedef FBlockedGrid3b::BlockData3Type   BoolBlockData3Type;
		typedef FBlockedGrid3i::BlockData3Type   IntBlockData3Type;
		
		typedef BoolBlockData3Type::BitArrayConstIterator  BitArrayConstIter;

		const double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		const double invdx = 1.0 / DX;
		const float upper_bound_value = upper_bound(FVector3i(NI, NJ, NK), DX);

		DistanceGrid.Reset(NI, NJ, NK, upper_bound_value);
		ClosestTriGrid.Reset(NI, NJ, NK, -1);
		
		TArray<FCriticalSection> BlockLocks;  BlockLocks.SetNum(DistanceGrid.GetNumBlocks());
		auto BlockLockProvider = [&BlockLocks](int32 block_id)->FCriticalSection&
		{
			return BlockLocks.GetData()[block_id];
		};

		// unsigned distance populate the distance grid and the closest tri grid using a limited flood fill, starting with
		// the locations of the mesh verts.
		{
			FBlockedGrid3b VisitedGrid(NI, NJ, NK, false);
			int32 CurCandidateID = 0;
			FBlockedGrid3b CandidateGrids[2] = {FBlockedGrid3b(NI, NJ, NK, false), FBlockedGrid3b(NI, NJ, NK, false) };
		
			

			// populate the CandidateGrid with all the mesh verts
			{
				
				FBlockedGrid3b& CandidateGrid = CandidateGrids[CurCandidateID];
				
			
				bool bCancel = false;
				ParallelFor(Mesh->MaxVertexID(), [this, &bCancel, &CandidateGrid, &BlockLockProvider, NI, NJ, NK, Origin, DX](const int32 VID)
				{
					const double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
					const double invdx = 1.0 / DX;
					
					if (!Mesh->IsVertex(VID))
					{
						return;
					}
					if (VID % 1000 == 0)
					{
						bCancel = this->CancelF();
					}
					if (bCancel)
					{
						return;
					}
					const FVector3d v = Mesh->GetVertex(VID);
					
					// real IJK coordinates of v
					const double  fi = (v.X - ox) * invdx, fj = (v.Y - oy) * invdx, fk = (v.Z - oz) * invdx;
					
					const FVector3i Idx( FMath::Clamp((int32)fi, 0, NI - 1),
										 FMath::Clamp((int32)fj, 0, NJ - 1),
										 FMath::Clamp((int32)fk, 0, NK - 1));

					CandidateGrid.SetValueWithLock(Idx.X, Idx.Y, Idx.Z, true, BlockLockProvider);

				}, !this->bUseParallel);

				if (bCancel)
				{
					return;
				}
			}

			const double max_dist = NarrowBandWidth;
			const double max_query_dist = max_dist + (2.0 * DX * FMathd::Sqrt2);

			int32 NumCandidateBlocks = CandidateGrids[CurCandidateID].GetAllocatedBlocks().Num();
			while (NumCandidateBlocks > 0)
			{
				const FBlockedGrid3b& CandidateGrid = CandidateGrids[CurCandidateID];
				FBlockedGrid3b& NextCandidateGrid = CandidateGrids[1-CurCandidateID];

				VisitedGrid.PreAllocateFromSourceGrid(CandidateGrid);
				DistanceGrid.PreAllocateFromSourceGrid(CandidateGrid);
				ClosestTriGrid.PreAllocateFromSourceGrid(CandidateGrid);
				NextCandidateGrid.Reset(NI, NJ, NK, false);

				// for each block with candidate locations, generate seed points and flood fill within the block and capture fill that exits the block
				// in the NextCandidateGrid.

				TArray<const BoolBlockData3Type*> CandidateBlocks = CandidateGrid.GetAllocatedConstBlocks();

				ParallelFor(NumCandidateBlocks, [this, max_query_dist, &BlockLockProvider, &VisitedGrid, &DistanceGrid, &NextCandidateGrid, &CandidateBlocks](const int32 cb)
				{
					const BoolBlockData3Type& CandidateBlock = *CandidateBlocks[cb];
					const int32 BlockIdx = CandidateBlock.Id;

					FloatBlockData3Type& DistanceBlock = *DistanceGrid.GetBlockData(BlockIdx);
					IntBlockData3Type& ClosestTriBlock = *ClosestTriGrid.GetBlockData(BlockIdx);
					BoolBlockData3Type& VistedBlock    = *VisitedGrid.GetBlockData(BlockIdx);

					const FVector3i BlockIJK    = DistanceGrid.BlockIndexToBlockIJK(BlockIdx);
					const FVector3i BlockOrigin = FloatBlockData3Type::BlockSize * BlockIJK;

					// the candidates were scattered from neighboring blocks, but if this block has already been processed some 
					// of them may have been visited.

					// make a mask that is the subtraction of the visited cells from the candidate cells.
					BoolBlockData3Type::BlockDataBitMask Mask = CandidateBlock.BitArray;
					Mask.CombineWithBitwiseXOR(VistedBlock.BitArray, EBitwiseOperatorFlags::MaintainSize);
					Mask.CombineWithBitwiseAND(CandidateBlock.BitArray, EBitwiseOperatorFlags::MaintainSize);

					// inter over unvisited candidate cells, updating result grids and creating a list of seed points. 
					TArray<FVector3i> SeedLocalCoords;
					for (BitArrayConstIter CIter(Mask); CIter; ++CIter)
					{
						const int32 LocalIndex = CIter.GetIndex();
						VistedBlock.BitArray[LocalIndex] = true;

						const FVector3i LocalCoords = DistanceBlock.ToLocalIJK(LocalIndex);
						const FVector3i GridCoords = LocalCoords + BlockOrigin;
						const FVector3d Pos(cell_center(GridCoords));

						double dsqr;
						const int32 near_tid = this->Spatial->FindNearestTriangle(Pos, dsqr, max_query_dist);
						if (near_tid != IndexConstants::InvalidID)
						{
							const float dist = (float)FMath::Sqrt(dsqr);
							DistanceBlock.At(LocalIndex) = dist;
							ClosestTriBlock.At(LocalIndex) = near_tid;

							SeedLocalCoords.Add(LocalCoords);
						}
					}
	
					auto IsInBlock = [](const FVector3i& LocalCoords)
										{
											bool bInBlock = true;
											bInBlock = LocalCoords.X > -1 && LocalCoords.Y > -1 && LocalCoords.Z > -1;
											bInBlock = bInBlock && LocalCoords.X < FloatBlockData3Type::BlockSize&& LocalCoords.Y < FloatBlockData3Type::BlockSize && LocalCoords.Z < FloatBlockData3Type::BlockSize;
											return bInBlock;
										};

					// flood fill from the seeds within the block.  if flood fill exits the block, mark the locations in the NextCanidateGrid for the next pass.
					while(SeedLocalCoords.Num() > 0)
					{
						const FVector3i Seed = SeedLocalCoords.Pop(EAllowShrinking::No);
						for (const FVector3i& idx_offset : IndexUtil::GridOffsets26)
						{
							const FVector3i LocalCoords = Seed + idx_offset;
							const FVector3i GridCoords = LocalCoords + BlockOrigin;
							if (IsInBlock(LocalCoords))
							{
								const int32 local_index = FloatBlockData3Type::ToLinear(LocalCoords.X, LocalCoords.Y, LocalCoords.Z);
								if ((bool)VistedBlock.BitArray[local_index] == false)
								{
									VistedBlock.BitArray[local_index] = true;

									const FVector3d Pos(cell_center(GridCoords));
								
									double dsqr;
									const int32 near_tid = this->Spatial->FindNearestTriangle(Pos, dsqr, max_query_dist);
									if (near_tid != IndexConstants::InvalidID)
									{
										const float dist = (float)FMath::Sqrt(dsqr);
										DistanceBlock.At(local_index) = dist; 
										ClosestTriBlock.At(local_index) = near_tid; 

										SeedLocalCoords.Add(LocalCoords);
									}
								
								}
							}
							else // outside this block - mark the location as a candidate for the next pass.
							{
								if (NextCandidateGrid.IsValidIJK(GridCoords))
								{
									const bool bIsCanidate = true;
									NextCandidateGrid.SetValueWithLock(GridCoords.X, GridCoords.Y, GridCoords.Z, bIsCanidate, BlockLockProvider);
								}
							}
						}
					}


				},
				!this->bUseParallel);

				CurCandidateID = 1-CurCandidateID;
				NumCandidateBlocks = CandidateGrids[CurCandidateID].GetAllocatedBlocks().Num();

				if (CancelF())
				{
					return;
				}
			}

		}
	}   // end make_level_set_3


	void CleanupUnwanted()
	{
		if (!bWantClosestTriGrid)
		{
			ClosestTriGrid.Reset();
		}

		if (!bWantIntersectionsGrid)
		{
			IntersectionsGrid.Reset();
		}
	}



	// fill the intersection Grid w/ number of intersections in each cell
	void compute_intersections(FVector3f Origin, float DX, int32 NI, int32 NJ, int32 NK, const FBlockedGrid3f& DistanceGrid, FBlockedGrid3i& IntersectionCountGrid)
	{
		
		// grid of atomic<int>, pre-allocated to match the topology of the distance grid. 
		struct FAtomicGrid3i : public TBlockedGrid3Layout<FBlockedGrid3f::BlockSize>
		{
			typedef TStaticArray<std::atomic<int32>, FBlockedGrid3f::BlockElemCount>  BlockAtomicData;
			typedef TBlockedGrid3Layout<FBlockedGrid3f::BlockSize>  MyLayout;
			
			FAtomicGrid3i(const FBlockedGrid3f& Grid3f, const bool bThreadedBuild)
			 : MyLayout(Grid3f.GetDimensions())
			{
				const int32 NumBlocks = Grid3f.GetNumBlocks();
				Blocks.SetNum(NumBlocks);
				// we use a parallel memory allocator, so we allow allocation to occur inside this parallel loop.
				ParallelFor(NumBlocks, [this, &Grid3f](int32 i)
				{
					if (Grid3f.GetBlockData(i).IsValid())
					{
						Blocks[i] = MakeUnique<BlockAtomicData>();
						BlockAtomicData& Data= *Blocks[i];
						for (int32 j = 0; j < FBlockedGrid3i::BlockElemCount; ++j)
						{
							Data[j] = 0;
						}
					}
				}, !bThreadedBuild);
			}
			TArray<TUniquePtr<BlockAtomicData>> Blocks;
		} AtomicGrid3i(DistanceGrid, this->bUseParallel);

		bool bCancel= false;

		// this is what we will do for each triangle. There are no Grid-reads, only Grid-writes, 
		// since we use atomic_increment, it is always thread-safe
		ParallelFor(Mesh->MaxTriangleID(),
			[this, Origin, DX, NI, NJ, NK, &IntersectionCountGrid, &AtomicGrid3i, &bCancel](int32 TID)
			{
				double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
				double invdx = 1.0 / DX;

				if (!Mesh->IsTriangle(TID))
				{
					return;
				}
				if (TID % 100 == 0 && CancelF() == true)
				{
					bCancel = true;
				}
				if (bCancel)
				{
					return;
				}

				FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
				Mesh->GetTriVertices(TID, xp, xq, xr);


				bool neg_x = false;
				if (InsideMode == EInsideModes::WindingCount)
				{
					FVector3d n = VectorUtil::NormalDirection(xp, xq, xr);
					neg_x = n.X > 0;
				}

				// real IJK coordinates of xp/xq/xr
				double fip = (xp[0] - ox) * invdx, fjp = (xp[1] - oy) * invdx, fkp = (xp[2] - oz) * invdx;
				double fiq = (xq[0] - ox) * invdx, fjq = (xq[1] - oy) * invdx, fkq = (xq[2] - oz) * invdx;
				double fir = (xr[0] - ox) * invdx, fjr = (xr[1] - oy) * invdx, fkr = (xr[2] - oz) * invdx;

				// recompute J/K integer bounds of triangle w/o exact band
				int32 j0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fjp, fjq, fjr)),  0, NJ - 1);
				int32 j1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fjp, fjq, fjr)), 0, NJ - 1);
				int32 k0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fkp, fkq, fkr)),  0, NK - 1);
				int32 k1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fkp, fkq, fkr)), 0, NK - 1);

				
				auto AtomicIncDec = [&AtomicGrid3i, neg_x](const int32 i, const int32 j, const int32 k)
										{
											int32 BlockIndex, LocalIndex;
											AtomicGrid3i.GetBlockAndLocalIndex(i, j, k, BlockIndex, LocalIndex);
											
											// note the grid has been pre-allocated, so the blocks will exist
											auto& ABlock = *AtomicGrid3i.Blocks[BlockIndex];
											std::atomic<int32>& value = ABlock[LocalIndex];
											if (neg_x)
											{
												value.fetch_sub(1, std::memory_order_relaxed);
											}
											else
											{
												value.fetch_add(1, std::memory_order_relaxed);
											}
										};

				// and do intersection counts
				for (int K = k0; K <= k1; ++K)
				{
					for (int J = j0; J <= j1; ++J)
					{
						double A, B, C;
						if (PointInTriangle2d(J, K, fjp, fkp, fjq, fkq, fjr, fkr, A, B, C))
						{
							double fi = A * fip + B * fiq + C * fir; // intersection I coordinate
							int i_interval = FMath::CeilToInt32(fi); // intersection is in (i_interval-1,i_interval]
							if (i_interval < 0)
							{
								AtomicIncDec(0, J, K);
							}
							else if (i_interval < NI)
							{
								AtomicIncDec(i_interval, J, K);
							}
							else
							{
								// we ignore intersections that are beyond the +x side of the Grid
							}
						}
					}
				}
			}, !this->bUseParallel);

			IntersectionCountGrid.Reset(NI, NJ, NK, 0);
			// copy result back to the intersection grid
			const int32 NumBlocks = AtomicGrid3i.Blocks.Num();
			ParallelFor(NumBlocks, [&IntersectionCountGrid, &AtomicGrid3i](const int32 blockid)
			{
				TUniquePtr<typename FAtomicGrid3i::BlockAtomicData>& AtomicBlockPtr = AtomicGrid3i.Blocks[blockid];
				
				if (AtomicBlockPtr.IsValid())
				{
					FVector3i BlockIJK = AtomicGrid3i.BlockIndexToBlockIJK(blockid);
					FBlockedGrid3i::BlockData3Type& BlockData = IntersectionCountGrid.TouchBlockData(BlockIJK);	
					const typename FAtomicGrid3i::BlockAtomicData& AtomicBlockData = *AtomicBlockPtr;

					for (int32 j = 0; j < FBlockedGrid3i::BlockElemCount; ++j)
					{
						const int32 Count = AtomicBlockData[j];
						BlockData.At(j) = Count;
					}

					// delete the atomic block after copying the data.
					AtomicBlockPtr.Reset();
				}
			
			}, !this->bUseParallel);
	}


	// iterate through each x-row of Grid and set unsigned distances to be negative (inside the mesh), based on the IntersectionCount
	void compute_signs(int32 NI, int32 NJ, int32 NK, const FBlockedGrid3i& IntersectionCountGrid, FBlockedGrid3f& Distances)
	{
		if (bUseParallel)
		{
			bool bCancel = false;

			const FVector3i BlockDims = Distances.GetBlockDimensions();
			// can process each x block-row in parallel
			ParallelFor(BlockDims.Y * BlockDims.Z, [this, BlockDims, &Distances, &IntersectionCountGrid, &bCancel](int32 BlockRowID)
			{
				const FVector3i GridDims = Distances.GetDimensions();
				if (BlockRowID % 10 == 0 && CancelF() == true)
				{
					bCancel = true;
				}

				if (bCancel)
				{
					return;
				}

				constexpr int32 BlockSize = FBlockedGrid3f::BlockSize;
				TStaticArray<int32, BlockSize*BlockSize> BlockCrossSection;
				for (int32 i = 0, I = BlockSize * BlockSize; i < I; ++i) 
				{ 
					BlockCrossSection[i] = 0;
				}

				// 2d array linearized as BlockRowID = BlockJ + BlockDims[1]*BlockK
				const int32 BlockJ = BlockRowID % BlockDims[1];
				const int32 BlockK = BlockRowID / BlockDims[1];
				FVector3i BlockIJK(0, BlockJ, BlockK);
				for (  BlockIJK[0] = 0; BlockIJK[0] < BlockDims[0]; ++BlockIJK[0]) {
					
					if (Distances.IsBlockAllocated(BlockIJK))
					{
						checkSlow(IntersectionCountGrid.IsBlockAllocated(BlockIJK));

						FBlockedGrid3f::BlockData3Type& DistanceBlockData = Distances.TouchBlockData(BlockIJK);
						const FBlockedGrid3i::BlockData3Type& IntersectionCountBlockData = *IntersectionCountGrid.GetBlockData(BlockIJK);

						// note, some data values stored in the blocks are formally outside the dimensions of the grids.. 
						// so this loop potentially touches some regions that will never be accessed. 
						for (int32 k = 0; k < BlockSize; ++k)
						{
							for (int32 j = 0; j < BlockSize; ++j)
							{
								int32& total_count = BlockCrossSection[j + k * BlockSize];

								for (int32 i = 0; i < BlockSize; ++i)
								{
									const int32 LocalIndex = TBlockData3Layout<BlockSize>::ToLinear(i, j, k);

									total_count += IntersectionCountBlockData.At(LocalIndex);

									if (
										(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
										(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
										)
									{
										// we are inside the mesh
										float& d = DistanceBlockData.At(LocalIndex);
										d = -FMath::Abs(d);
									}

								}
							}
						}
							
					}
					else
					{
						checkSlow(!IntersectionCountGrid.IsBlockAllocated(BlockIJK));
						
						// the block isn't allocated - so it should all share the same state (inside or outside) for water-tight meshes,
						// but the counting method used for sign assignment may produce odd results for non water-tight meshes where a block
						// far from the narrow band may have both inside and outside elements.  
			
						// count the number of "inside" cells on the (already processed) neighbor block face adjacent to BlockIJK
						int32 NumInsideCells = 0;
						{
							for (int32 index = 0, IndexMax = BlockSize * BlockSize; index < IndexMax; ++index)
							{
								const int32 total_count = BlockCrossSection[index];
								if (
									(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
									(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
									)
								{
									NumInsideCells++;
								}
							}
					
						}
						
						switch (NumInsideCells)
						{
							case  0:
							{
								// neighbor is all outside - this unallocated block can be represented by existing positive value ( do nothing)
								break;
							}
							case (BlockSize * BlockSize):
							{
								// neighbor is all inside - the unallocated block can be represented by single negative 
								// we are inside the mesh
								Distances.ProcessBlockDefaultValue(BlockIJK, [](float& value) {value = -FMath::Abs(value); });
								break;
							}
							default:
							{
								// allocate the block (unique blocks can be allocated in parallel)
								FBlockedGrid3f::BlockData3Type& DistanceBlockData = Distances.TouchBlockData(BlockIJK);

								for (int32 k = 0; k < BlockSize; ++k)
								{
									for (int32 j = 0; j < BlockSize; ++j)
									{
										const int32 total_count = BlockCrossSection[j + k * BlockSize];

										for (int32 i = 0; i < BlockSize; ++i)
										{
											const int32 LocalIndex = TBlockData3Layout<BlockSize>::ToLinear(i, j, k);

											if (
												(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
												(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
												)
											{
												// we are inside the mesh
												float& d = DistanceBlockData.At(LocalIndex);
												d = -FMath::Abs(d);
											}

										}
									}
								}

								break;
							}
						}
					}
						
				}
			}, !this->bUseParallel);

		}
		else
		{
			for (int K = 0; K < NK; ++K)
			{
				if (CancelF())
				{
					return;
				}

				for (int J = 0; J < NJ; ++J)
				{
					int total_count = 0;
					for (int I = 0; I < NI; ++I)
					{
						total_count += IntersectionCountGrid.GetValue(I, J, K);
						if (
							(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
							(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
							)
						{
							// we are inside the mesh
							Distances.ProcessValue(I, J, K, [](float& value){value = -value;});
						}
					}
				}
			}
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE