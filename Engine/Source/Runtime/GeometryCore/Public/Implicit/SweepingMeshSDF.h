// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshSignedDistanceGrid

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Util/IndexUtil.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/DenseGrid3.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopeLock.h"
#include "Implicit/GridInterpolant.h"
#include "Math/NumericLimits.h"

namespace UE
{
namespace Geometry
{


/**
 * Compute discretely-sampled (ie gridded) signed distance field for a mesh
 * The basic approach is, first compute exact Distances in a narrow band, and then
 * extend out to rest of Grid using fast "sweeping" (ie like a distance transform).
 * The resulting unsigned Grid is then signed using ray-intersection counting, which
 * is also computed on the Grid, so no BVH is necessary
 * 
 * If you set ComputeMode to NarrowBandOnly, result is a narrow-band signed distance field.
 * This is quite a bit faster as the sweeping is the most computationally-intensive step.
 *
 * You can optionally provide a spatial data structure to allow faster computation of
 * narrow-band Distances; this is most beneficial if we want a wider band 
 * 
 * Caveats:
 *  - the "narrow band" is based on triangle bounding boxes, so it is not necessarily
 *    that "narrow" if you have large triangles on a diagonal to Grid axes
 *
 * TODO: a number of utility function could & should be shared between this and CachingMeshSDF!
 * 
 * This code is based on the implementation found at https://github.com/christopherbatty/SDFGen
 */
template <class TriangleMeshType, bool bScalarCellSize = true>
class TSweepingMeshSDF
{
public:

	// Type to use for CellSize
	using CellSizeType = std::conditional_t<bScalarCellSize, float, FVector3f>;
	// Type to use for CellSize, but in double precision
	using CellSizeTyped = std::conditional_t<bScalarCellSize, double, FVector3d>;
	// Unit cube, in CellSizeType
	static inline CellSizeType UnitCellSize() { return CellSizeType(1); };

	// INPUTS

	const TriangleMeshType* Mesh;
	TMeshAABBTree3<TriangleMeshType>* Spatial;
	CellSizeType CellSize;

	void SetCellSize(float InCellSize)
	{
		CellSize = CellSizeType(InCellSize);
	}

	// Width of the band around triangles for which exact Distances are computed
	// (In fact this is conservative, the band is often larger locally)
	int ExactBandWidth = 1;

	// Bounds of Grid will be expanded this much in positive and negative directions.
	// Useful for if you want field to extend outwards.
	FVector3d ExpandBounds = FVector3d::Zero();

	// Most of this parallelizes very well, makes a huge speed difference
	bool bUseParallel = true;

	// If the number of cells in any dimension may exceed this, CellSize will be automatically increased to keep cell count reasonable
	int ApproxMaxCellsPerDimension = 4096;


	// The narrow band is always computed exactly, and the full Grid is always signed.
	// Can also fill in the rest of the full Grid with fast sweeping. This is 
	// quite computationally intensive, though, and not parallelizable 
	// (time only depends on Grid resolution)
	enum EComputeModes
	{
		FullGrid = 0,
		NarrowBandOnly = 1,
		NarrowBand_SpatialFloodFill = 2
	};
	EComputeModes ComputeMode = EComputeModes::NarrowBandOnly;

	// how wide of narrow band should we compute. This value is 
	// currently only used NarrowBand_SpatialFloodFill, as
	// we can efficiently explore the space
	// (in that case ExactBandWidth is only used to initialize the grid extents, and this is used instead during the flood fill)
	double NarrowBandMaxDistance = 0;

	// should we try to compute signs? if not, Grid remains unsigned
	bool bComputeSigns = true;

	// What counts as "inside" the mesh. Crossing count does not use triangle
	// Orientation, so inverted faces are fine, but overlapping shells or self intersections
	// will be filled using even/odd rules (as seen along X axis...)
	// Winding count is basically mesh winding number, handles overlap shells and
	// self-intersections, but inverted shells are 'subtracted', and inverted faces are a disaster.
	// Both modes handle internal cavities, neither handles open sheets.
	enum EInsideModes
	{
		CrossingCount = 0,
		WindingCount = 1
	};
	EInsideModes InsideMode = EInsideModes::WindingCount;

	// Implementation computes the triangle closest to each Grid cell, can
	// return this Grid if desired (only reason not to is avoid hanging onto memory)
	bool bWantClosestTriGrid = false;

	// Grid of per-cell crossing or parity counts
	bool bWantIntersectionsGrid = false;

	/** if this function returns true, we should abort calculation */
	TFunction<bool()> CancelF = []()
	{
		return false;
	};

	// OUTPUTS

	FVector3f GridOrigin;
	FDenseGrid3f Grid;

protected:
	// grid of closest triangle indices; only available if bWantClosestTriGrid is set to true.  Access via GetClosestTriGrid()
	FDenseGrid3i ClosestTriGrid;
	// grid of intersection counts; only available if bWantIntersectionsGrid is set to true.  Access via GetIntersectionsGrid()
	FDenseGrid3i IntersectionsGrid;

public:

	/**
	 * @param Mesh Triangle mesh to build an SDF around
	 * @param InCellSize Spacing between Grid points
	 * @param Spatial Optional AABB tree; note it *must* be provided if ComputeMode is set to NarrowBand_SpatialFloodFill
	 */
	TSweepingMeshSDF(const TriangleMeshType* Mesh = nullptr, CellSizeType InCellSize = UnitCellSize(), TMeshAABBTree3<TriangleMeshType>* Spatial = nullptr) : Mesh(Mesh), Spatial(Spatial), CellSize(InCellSize), GridOrigin(0)
	{
	}

	/**
	 * Empty out all computed result data (leaves input data alone)
	 */
	void Empty()
	{
		Grid.Resize(0, 0, 0, EAllowShrinking::Yes);
		ClosestTriGrid.Resize(0, 0, 0, EAllowShrinking::Yes);
		IntersectionsGrid.Resize(0, 0, 0, EAllowShrinking::Yes);
	}

	// Make an interpolating sampler of the sdf data. Note the lifetime is dependent on this class.
	template<typename InterpolantRealType = double>
	TTriLinearGridInterpolant<FDenseGrid3f, InterpolantRealType, bScalarCellSize> MakeInterpolant()
	{
		using InterpVecType = TVector<InterpolantRealType>;
		return TTriLinearGridInterpolant<FDenseGrid3f, InterpolantRealType, bScalarCellSize>(&Grid, (InterpVecType)GridOrigin, (InterpVecType)CellSize, Dimensions());
	}

	/**
	 * Encodes heuristic for deciding whether it will be faster to use
	 *  - NarrowBand_SpatialFloodFill
	 *  - Or NarrowBandOnly
	 * The heuristic is to use the Spatial method unless you have relatively long edges relative to the width
	 * of the narrow band.  For a 1-cell wide band, or one where few triangles interact w/ each cell in the band
	 * the spatial overhead will probably not be worthwhile.
	 * TODO: test and tune this ...
	 */
	static bool ShouldUseSpatial(int ExactCells, double CellSize, double AvgEdgeLen)
	{
		double EdgesPerCell = CellSize / AvgEdgeLen;
		double EdgesPerBand = ExactCells * EdgesPerCell;
		return ExactCells > 1 && EdgesPerBand >= .25;
	}

	bool Validate(const FAxisAlignedBox3d& Bounds)
	{
		if (Bounds.IsEmpty() || !FMath::IsFinite(Bounds.MaxDim()))
		{
			return false;
		}
		if (!IsValid(CellSize))
		{
			return false;
		}
		if (ApproxMaxCellsPerDimension < 5)
		{
			return false;
		}
		return true;
	}

	/**
	 * Compute the SDF enclosing the given bounds, with a 'safe' buffer zone
	 * @param Bounds Bounding box for the mesh data (passed as param it is usually already available, depending on TriangleMeshType and whether an AABB tree was provided)
	 * @return false if cancelled or failed (e.g. due to invalid arguments); true otherwise
	 */
	bool Compute(FAxisAlignedBox3d Bounds)
	{
		if (!ensure(Validate(Bounds)))
		{
			return false;
		}

		if (!ensureMsgf(ExactBandWidth*2 < ApproxMaxCellsPerDimension, TEXT("Cannot request band wider than half the max cells per dimension")))
		{
			ExactBandWidth = FMath::Max(ApproxMaxCellsPerDimension/2-1, 1);
		}

		FVector3d ExpandedDiagonal = (Bounds.Diagonal() + ExpandBounds * 2.0);
		FVector3d ApproxVoxelsPerDim = (FVector3d)(ExpandedDiagonal / CellSize);
		int32 MaxVoxelsDim = VectorUtil::Max3Index(ApproxVoxelsPerDim);
		if (!ensureMsgf(ApproxVoxelsPerDim[MaxVoxelsDim] <= ApproxMaxCellsPerDimension - 2 * ExactBandWidth, TEXT("SDF resolution clamped to avoid excessive memory use")))
		{
			float CellSizeFactor = float(ExpandedDiagonal[MaxVoxelsDim] / (ApproxMaxCellsPerDimension - 2 * ExactBandWidth)) / GetDim(CellSize, MaxVoxelsDim);
			CellSize *= CellSizeFactor;
			if (!ensure(IsValid(CellSize)))
			{
				return false;
			}
		}

		CellSizeType fBufferWidth = float(ExactBandWidth) * CellSize;
		if (ComputeMode == EComputeModes::NarrowBand_SpatialFloodFill)
		{
			if constexpr (bScalarCellSize)
			{
				fBufferWidth = (float)FMath::Max(fBufferWidth, float(NarrowBandMaxDistance));
			}
			else
			{
				for (int32 Idx = 0; Idx < 3; ++Idx)
				{
					fBufferWidth = FMath::Max(fBufferWidth, float(NarrowBandMaxDistance));
				}
			}
		}
		FVector3f UseOrigin = (FVector3f)Bounds.Min - fBufferWidth * FVector3f::One() - (FVector3f)ExpandBounds;
		FVector3f max = (FVector3f)Bounds.Max + fBufferWidth * FVector3f::One() + (FVector3f)ExpandBounds;
		FVector3i UseDims(
			(int)((max.X - UseOrigin.X) / GetDim(CellSize, 0)) + 1,
			(int)((max.Y - UseOrigin.Y) / GetDim(CellSize, 1)) + 1,
			(int)((max.Z - UseOrigin.Z) / GetDim(CellSize, 2)) + 1
		);

		return Compute(UseOrigin, UseDims);
	}
	
	/**
	 * Compute the SDF with exactly specified grid origin and dimensions
	 * @param InGridOrigin Origin of the output grid
	 * @param InDimensions Dimensions of the output grid
	 * @return false if cancelled or failed (e.g. due to invalid arguments); true otherwise
	 */
	bool Compute(FVector3f InGridOrigin, FVector3i InDimensions)
	{
		GridOrigin = InGridOrigin;
		int NI = InDimensions.X;
		int NJ = InDimensions.Y;
		int NK = InDimensions.Z;

		// dimensions must be positive
		if (NI < 0 || NJ < 0 || NK < 0)
		{
			return false;
		}

		if (ComputeMode == EComputeModes::NarrowBand_SpatialFloodFill)
		{
			if (!ensureMsgf(Spatial && NarrowBandMaxDistance != 0, TEXT("SweepingMeshSDF::Compute: must set Spatial data structure and band max distance")))
			{
				return false;
			}
			make_level_set3_parallel_floodfill(GridOrigin, CellSize, NI, NJ, NK, Grid);
		}
		else
		{
			if (bUseParallel)
			{
				if (Spatial != nullptr) // TODO: is this path better than NarrowBand_SpatialFloodFill path sometimes?  when?
				{
					make_level_set3_parallel_spatial(GridOrigin, CellSize, NI, NJ, NK, Grid, ExactBandWidth);
				}
				else
				{
					make_level_set3_parallel(GridOrigin, CellSize, NI, NJ, NK, Grid, ExactBandWidth);
				}
			}
			else
			{
				make_level_set3(GridOrigin, CellSize, NI, NJ, NK, Grid, ExactBandWidth);
			}
		}

		CleanupUnwanted();

		return !CancelF();
	}



	FVector3i Dimensions() const
	{
		return Grid.GetDimensions();
	}


	const FDenseGrid3i& GetClosestTriGrid() const
	{
		checkf(bWantClosestTriGrid, TEXT("Set bWantClosestTriGrid=true to return this value"));
		return ClosestTriGrid;
	}
	const FDenseGrid3i& GetIntersectionsGrid() const
	{
		checkf(bWantIntersectionsGrid, TEXT("Set bWantIntersectionsGrid=true to return this value"));
		return IntersectionsGrid;
	}

	float At(int I, int J, int K) const
	{
		return Grid.GetValue(I, J, K);
	}

	float operator[](const FVector3i& Idx) const
	{
		return Grid.GetValue(Idx);
	}

	FVector3f CellCenter(int I, int J, int K) const
	{
		return cell_center(FVector3i(I, J, K));
	}

	float GetValue(FVector3i Idx) const // TTriLinearGridInterpolant interface 
	{
		return Grid.GetValue(Idx);
	}

private:
	FVector3f cell_center(FVector3i IJK) const
	{
		return FVector3f((float)IJK.X * GetDim(CellSize, 0) + GridOrigin[0],
			(float)IJK.Y * GetDim(CellSize, 1) + GridOrigin[1],
			(float)IJK.Z * GetDim(CellSize, 2) + GridOrigin[2]);
	}

	float upper_bound(const FDenseGrid3f& GridIn) const
	{
		return float(GridIn.GetDimensions().X) * GetDim(CellSize, 0) + float(GridIn.GetDimensions().Y) * GetDim(CellSize, 1) + float(GridIn.GetDimensions().Z) * GetDim(CellSize, 2);
	}

	float cell_tri_dist(const FVector3i& Idx, int TID) const
	{
		FVector3d xp, xq, xr;
		FVector3d c = cell_center(Idx);
		Mesh->GetTriVertices(TID, xp, xq, xr);
		return (float)PointTriangleDistance(c, xp, xq, xr);
	}


	void make_level_set3(FVector3f Origin, CellSizeType DX, int NI, int NJ, int NK, FDenseGrid3f& Distances, int ExactBand)
	{
		Distances.Resize(NI, NJ, NK);
		Distances.Assign(upper_bound(Distances)); // upper bound on distance

		// closest triangle id for each Grid cell
		FDenseGrid3i& closest_tri = ClosestTriGrid;
		ClosestTriGrid.Resize(NI, NJ, NK);
		ClosestTriGrid.Assign(-1);

		// intersection_count(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
		FDenseGrid3i& intersection_count = IntersectionsGrid;
		IntersectionsGrid.Resize(NI, NJ, NK);
		IntersectionsGrid.Assign(0);

		// Compute narrow-band Distances. For each triangle, we find its Grid-coord-bbox,
		// and compute exact Distances within that box. The intersection_count Grid
		// is also filled in this computation
		CellSizeTyped ddx = (CellSizeTyped)DX;
		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		FVector3d xp, xq, xr;
		for (int TID = 0; TID < Mesh->MaxTriangleID(); TID++)
		{
			if (!Mesh->IsTriangle(TID))
			{
				continue;
			}
			if (TID % 100 == 0 && CancelF())
			{
				break;
			}
			Mesh->GetTriVertices(TID, xp, xq, xr);

			// real IJK coordinates of xp/xq/xr
			double fip = (xp[0] - ox) / GetDim(ddx, 0), fjp = (xp[1] - oy) / GetDim(ddx, 1), fkp = (xp[2] - oz) / GetDim(ddx, 2);
			double fiq = (xq[0] - ox) / GetDim(ddx, 0), fjq = (xq[1] - oy) / GetDim(ddx, 1), fkq = (xq[2] - oz) / GetDim(ddx, 2);
			double fir = (xr[0] - ox) / GetDim(ddx, 0), fjr = (xr[1] - oy) / GetDim(ddx, 1), fkr = (xr[2] - oz) / GetDim(ddx, 2);

			// clamped integer bounding box of triangle plus exact-band
			int i0 = FMath::Clamp(((int)FMath::Min3(fip, fiq, fir)) - ExactBand, 0, NI - 1);
			int i1 = FMath::Clamp(((int)FMath::Max3(fip, fiq, fir)) + ExactBand + 1, 0, NI - 1);
			int j0 = FMath::Clamp(((int)FMath::Min3(fjp, fjq, fjr)) - ExactBand, 0, NJ - 1);
			int j1 = FMath::Clamp(((int)FMath::Max3(fjp, fjq, fjr)) + ExactBand + 1, 0, NJ - 1);
			int k0 = FMath::Clamp(((int)FMath::Min3(fkp, fkq, fkr)) - ExactBand, 0, NK - 1);
			int k1 = FMath::Clamp(((int)FMath::Max3(fkp, fkq, fkr)) + ExactBand + 1, 0, NK - 1);

			// compute distance for each tri inside this bounding box
			// note: this can be very conservative if the triangle is large and on diagonal to Grid axes
			for (int K = k0; K <= k1; ++K) {
				for (int J = j0; J <= j1; ++J) {
					for (int I = i0; I <= i1; ++I) {
						FVector3d gx((float)I * GetDim(DX, 0) + Origin[0], (float)J * GetDim(DX, 1) + Origin[1], (float)K * GetDim(DX, 2) + Origin[2]);
						float d = (float)PointTriangleDistance(gx, xp, xq, xr);
						if (d < Distances.At(I, J, K)) {
							Distances.At(I, J, K) = d;
							closest_tri.At(I, J, K) = TID;
						}
					}
				}
			}
		}
		if (CancelF())
		{
			return;
		}

		if (bComputeSigns == true)
		{
			compute_intersections(Origin, DX, NI, NJ, NK, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (ComputeMode == EComputeModes::FullGrid)
			{
				// and now we fill in the rest of the Distances with fast sweeping
				for (int pass = 0; pass < 2; ++pass)
				{
					sweep_pass(Origin, DX, Distances, closest_tri);
					if (CancelF())
					{
						return;
					}
				}
			}
			else
			{
				// nothing!
			}


			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, Distances, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (!bWantIntersectionsGrid)
			{
				// empty grid
				intersection_count.Resize(0,0,0,EAllowShrinking::Yes);
			}
		}

		if (!bWantClosestTriGrid)
		{
			// empty grid
			closest_tri.Resize(0,0,0,EAllowShrinking::Yes);
		}

	}   // end make_level_set_3






	void make_level_set3_parallel(FVector3f Origin, CellSizeType DX, int NI, int NJ, int NK, FDenseGrid3f& Distances, int ExactBand)
	{
		Distances.Resize(NI, NJ, NK);
		Distances.Assign(upper_bound(Grid)); // upper bound on distance

		// closest triangle id for each Grid cell
		FDenseGrid3i& closest_tri = ClosestTriGrid;
		ClosestTriGrid.Resize(NI, NJ, NK);
		ClosestTriGrid.Assign(-1);

		// intersection_count(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
		FDenseGrid3i& intersection_count = IntersectionsGrid;
		IntersectionsGrid.Resize(NI, NJ, NK);
		IntersectionsGrid.Assign(0);

		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		CellSizeTyped invdx = CellSizeTyped(UnitCellSize()) / (CellSizeTyped)DX;

		// Compute narrow-band Distances. For each triangle, we find its Grid-coord-bbox,
		// and compute exact Distances within that box.

		// To compute in parallel, we need to safely update Grid cells. Current strategy is
		// to use a critical section to control access to Grid. Partitioning the Grid into a few regions,
		// each w/ a separate lock, improves performance somewhat.
		int wi = NI / 2, wj = NJ / 2, wk = NK / 2;
		FCriticalSection GridSections[8];

		bool bAbort = false;
		ParallelFor(Mesh->MaxTriangleID(), [this, &Origin, DX, NI, NJ, NK, &Distances, ExactBand, &closest_tri, &intersection_count, ox, oy, oz, invdx, wi, wj, wk, &GridSections, &bAbort](int TID)
		{
			if (!Mesh->IsTriangle(TID))
			{
				return;
			}
			if (TID % 100 == 0)
			{
				bAbort = CancelF();
			}
			if (bAbort)
			{
				return;
			}

			FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
			Mesh->GetTriVertices(TID, xp, xq, xr);

			// real IJK coordinates of xp/xq/xr
			double fip = (xp[0] - ox) * GetDim(invdx, 0), fjp = (xp[1] - oy) * GetDim(invdx, 1), fkp = (xp[2] - oz) * GetDim(invdx, 2);
			double fiq = (xq[0] - ox) * GetDim(invdx, 0), fjq = (xq[1] - oy) * GetDim(invdx, 1), fkq = (xq[2] - oz) * GetDim(invdx, 2);
			double fir = (xr[0] - ox) * GetDim(invdx, 0), fjr = (xr[1] - oy) * GetDim(invdx, 1), fkr = (xr[2] - oz) * GetDim(invdx, 2);

			// clamped integer bounding box of triangle plus exact-band
			int i0 = FMath::Clamp(((int)FMath::Min3(fip, fiq, fir)) - ExactBand, 0, NI - 1);
			int i1 = FMath::Clamp(((int)FMath::Max3(fip, fiq, fir)) + ExactBand + 1, 0, NI - 1);
			int j0 = FMath::Clamp(((int)FMath::Min3(fjp, fjq, fjr)) - ExactBand, 0, NJ - 1);
			int j1 = FMath::Clamp(((int)FMath::Max3(fjp, fjq, fjr)) + ExactBand + 1, 0, NJ - 1);
			int k0 = FMath::Clamp(((int)FMath::Min3(fkp, fkq, fkr)) - ExactBand, 0, NK - 1);
			int k1 = FMath::Clamp(((int)FMath::Max3(fkp, fkq, fkr)) + ExactBand + 1, 0, NK - 1);

			// compute distance for each tri inside this bounding box
			// note: this can be very conservative if the triangle is large and on diagonal to Grid axes
			for (int K = k0; K <= k1; ++K) {
				for (int J = j0; J <= j1; ++J) {
					int base_idx = ((J < wj) ? 0 : 1) | ((K < wk) ? 0 : 2);    // construct index into spinlocks array

					for (int I = i0; I <= i1; ++I) {
						FVector3d gx((float)I * GetDim(DX, 0) + Origin[0], (float)J * GetDim(DX, 1) + Origin[1], (float)K * GetDim(DX, 2) + Origin[2]);
						float d = (float)PointTriangleDistance(gx, xp, xq, xr);
						if (d < Distances.At(I, J, K)) {
							int lock_idx = base_idx | ((I < wi) ? 0 : 4);
							
							FScopeLock GridLock(&GridSections[lock_idx]);
							if (d < Distances.At(I, J, K)) {    // have to check again in case Grid changed in another thread...
								Distances.At(I, J, K) = d;
								closest_tri.At(I, J, K) = TID;
							}
						}
					}

				}
			}
		});
		if (CancelF())
		{
			return;
		}


		if (bComputeSigns == true)
		{
			compute_intersections(Origin, DX, NI, NJ, NK, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (ComputeMode == EComputeModes::FullGrid) {
				// and now we fill in the rest of the Distances with fast sweeping
				for (int pass = 0; pass < 2; ++pass) {
					sweep_pass(Origin, DX, Distances, closest_tri);
					if (CancelF())
						return;
				}
			}
			else
			{
				// nothing!
			}

			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, Distances, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (!bWantIntersectionsGrid)
			{
				// empty grid
				intersection_count.Resize(0, 0, 0, EAllowShrinking::Yes);
			}
		}

		if (!bWantClosestTriGrid)
		{
			// empty grid
			closest_tri.Resize(0, 0, 0, EAllowShrinking::Yes);
		}
	}   // end make_level_set_3



	void make_level_set3_parallel_spatial(FVector3f Origin, CellSizeType DX, int NI, int NJ, int NK, FDenseGrid3f& Distances, int ExactBand)
	{
		Distances.Resize(NI, NJ, NK);
		float upper_bound = this->upper_bound(Distances);
		Distances.Assign(upper_bound); // upper bound on distance

		// closest triangle id for each Grid cell
		FDenseGrid3i& closest_tri = ClosestTriGrid;
		ClosestTriGrid.Resize(NI, NJ, NK);
		ClosestTriGrid.Assign(-1);

		// intersection_count(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
		FDenseGrid3i& intersection_count = IntersectionsGrid;
		IntersectionsGrid.Resize(NI, NJ, NK);
		IntersectionsGrid.Assign(0);

		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		CellSizeTyped invdx = CellSizeTyped(UnitCellSize()) / (CellSizeTyped)DX;

		// Compute narrow-band Distances. For each triangle, we find its Grid-coord-bbox,
		// and compute exact Distances within that box.

		// To compute in parallel, we need to safely update Grid cells. Current strategy is
		// to use a spinlock to control access to Grid. Partitioning the Grid into a few regions,
		// each w/ a separate spinlock, improves performance somewhat. Have also tried having a
		// separate spinlock per-row, this resulted in a few-percent performance improvement.
		// Also tried pre-sorting triangles into disjoint regions, this did not help much except
		// on "perfect" cases like a sphere. 
		bool bAbort = false;
		ParallelFor(Mesh->MaxTriangleID(), [this, &Origin, DX, NI, NJ, NK, &Distances, ExactBand, upper_bound, &closest_tri, &intersection_count, ox, oy, oz, invdx, &bAbort](int TID)
		{
			if (!Mesh->IsTriangle(TID))
			{
				return;
			}
			if (TID % 100 == 0)
			{
				bAbort = CancelF();
			}
			if (bAbort)
			{
				return;
			}

			FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
			Mesh->GetTriVertices(TID, xp, xq, xr);

			// real IJK coordinates of xp/xq/xr
			double fip = (xp[0] - ox) * GetDim(invdx, 0), fjp = (xp[1] - oy) * GetDim(invdx, 1), fkp = (xp[2] - oz) * GetDim(invdx, 2);
			double fiq = (xq[0] - ox) * GetDim(invdx, 0), fjq = (xq[1] - oy) * GetDim(invdx, 1), fkq = (xq[2] - oz) * GetDim(invdx, 2);
			double fir = (xr[0] - ox) * GetDim(invdx, 0), fjr = (xr[1] - oy) * GetDim(invdx, 1), fkr = (xr[2] - oz) * GetDim(invdx, 2);

			// clamped integer bounding box of triangle plus exact-band
			int i0 = FMath::Clamp(((int)FMath::Min3(fip, fiq, fir)) - ExactBand, 0, NI - 1);
			int i1 = FMath::Clamp(((int)FMath::Max3(fip, fiq, fir)) + ExactBand + 1, 0, NI - 1);
			int j0 = FMath::Clamp(((int)FMath::Min3(fjp, fjq, fjr)) - ExactBand, 0, NJ - 1);
			int j1 = FMath::Clamp(((int)FMath::Max3(fjp, fjq, fjr)) + ExactBand + 1, 0, NJ - 1);
			int k0 = FMath::Clamp(((int)FMath::Min3(fkp, fkq, fkr)) - ExactBand, 0, NK - 1);
			int k1 = FMath::Clamp(((int)FMath::Max3(fkp, fkq, fkr)) + ExactBand + 1, 0, NK - 1);

			// compute distance for each tri inside this bounding box
			// note: this can be very conservative if the triangle is large and on diagonal to Grid axes
			for (int K = k0; K <= k1; ++K) {
				for (int J = j0; J <= j1; ++J) {
					for (int I = i0; I <= i1; ++I) {
						Distances.At(I, J, K) = 1;
					}
				}
			}
		});

		double max_dist = ExactBand * (double)GetDiagLength(DX);
		check(Grid.Num() <= MAX_int32);
		ParallelFor(static_cast<int32>(Grid.Num()), [this, &Origin, DX, NI, NJ, NK, &Distances, max_dist, upper_bound, &closest_tri](int LinearIdx)
		{
			FVector3i Idx = Grid.GetCoords(LinearIdx);
			if (Distances[Idx] == 1) {
				int I = Idx.X, J = Idx.Y, K = Idx.Z;
				FVector3d p((float)I * GetDim(DX, 0) + Origin[0], (float)J * GetDim(DX, 1) + Origin[1], (float)K * GetDim(DX, 2) + Origin[2]);
				double dsqr;
				int near_tid = Spatial->FindNearestTriangle(p, dsqr, max_dist);
				if (near_tid == IndexConstants::InvalidID) {
					Distances[Idx] = upper_bound;
					return;
				}
				Distances[Idx] = (float)FMath::Sqrt(dsqr);
				closest_tri[Idx] = near_tid;
			}
		});



		if (CancelF())
		{
			return;
		}

		if (bComputeSigns == true)
		{
			compute_intersections(Origin, DX, NI, NJ, NK, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (ComputeMode == EComputeModes::FullGrid) {
				// and now we fill in the rest of the Distances with fast sweeping
				for (int pass = 0; pass < 2; ++pass) {
					sweep_pass(Origin, DX, Distances, closest_tri);
					if (CancelF())
						return;
				}
			}
			else {
				// nothing!
			}

			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, Distances, intersection_count);
			if (CancelF())
			{
				return;
			}

			if (!bWantIntersectionsGrid)
			{
				// empty grid
				intersection_count.Resize(0, 0, 0, EAllowShrinking::Yes);
			}
		}

		if (!bWantClosestTriGrid)
		{
			// empty grid
			closest_tri.Resize(0, 0, 0, EAllowShrinking::Yes);
		}

	}   // end make_level_set_3














	void make_level_set3_parallel_floodfill(FVector3f Origin, CellSizeType DX, int NI, int NJ, int NK, FDenseGrid3f& Distances)
	{
		Distances.Resize(NI, NJ, NK);
		float upper_bound = this->upper_bound(Distances);
		Distances.Assign(upper_bound); // upper bound on distance

		// closest triangle id for each Grid cell
		FDenseGrid3i& closest_tri = ClosestTriGrid;
		ClosestTriGrid.Resize(NI, NJ, NK);
		ClosestTriGrid.Assign(-1);

		// intersection_count(I,J,K) is # of tri intersections in (I-1,I]x{J}x{K}
		FDenseGrid3i& intersection_count = IntersectionsGrid;
		IntersectionsGrid.Resize(NI, NJ, NK);
		IntersectionsGrid.Assign(0);

		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		CellSizeTyped invdx = CellSizeTyped(UnitCellSize()) / (CellSizeTyped)DX;

		// the steps below that populate the grid will potentially touch the same cells at the
		// same time, so we need to lock them. However locking the entire grid for each cell 
		// basically means the threads are constantly fighting. So we split the grid up into
		// chunks for locking
		int32 NumSections = 256;		// a bit arbitrary, stopped seeing improvement on a 64-core machine at this point
		TArray<FCriticalSection> GridSections;
		GridSections.SetNum(NumSections);
		int64 TotalGridCellCount = NI * NJ * NK;
		int64 SectionSize = FMath::CeilToInt(float(TotalGridCellCount) / (float)NumSections);
		
		// this returns the FCriticalSection to use for the given span of values
		auto GetGridSectionLock = [this, SectionSize, &Distances, &GridSections](FVector3i CellGridIndex) -> FCriticalSection*
		{
			int64 CellLinearIndex = Distances.GetIndex(CellGridIndex);
			int64 SectionIndex = CellLinearIndex / SectionSize;
			checkSlow(SectionIndex <= MAX_int32);
			return &GridSections[(int32)SectionIndex];
		};

		// per-grid-chunk queue, each one of these cooresponds to a GridSections lock
		TArray<TArray<int>> SectionQueues;
		SectionQueues.SetNum(NumSections);
		auto AddToSectionQueue = [SectionSize, &SectionQueues](int64 CellLinearIndex)
		{
			int64 SectionIndex = CellLinearIndex / SectionSize;
			checkSlow(SectionIndex <= MAX_int32);
			checkSlow(CellLinearIndex <= MAX_int32);
			SectionQueues[(int32)SectionIndex].Add((int32)CellLinearIndex);
		};


		TArray<int> PendingCellQueue;
		TArray<bool> done; 
		check(Distances.Num() <= MAX_int32);
		done.SetNumZeroed(static_cast<int32>(Distances.Num()));

		// this lambda combines the per-chunk queues into a single queue which we can pass to a ParallelFor
		auto FlattenSectionQueues = [&PendingCellQueue, &SectionQueues]()
		{
			PendingCellQueue.Reset();
			for (TArray<int>& SectionQueue : SectionQueues)
			{
				for (int Value : SectionQueue)
				{
					PendingCellQueue.Add(Value);
				}
				SectionQueue.Reset();
			}
		};

		// compute values at vertices
		bool bAbort = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_SweepingMeshSDF_Distances);
			ParallelFor(Mesh->MaxVertexID(), [&](int vid)
			{
				if (!Mesh->IsVertex(vid))
				{
					return;
				}
				if (vid % 100 == 0)
				{
					bAbort = CancelF();
				}
				if (bAbort)
				{
					return;
				}

				FVector3d v = Mesh->GetVertex(vid);
				// real IJK coordinates of v
				double fi = (v.X - ox) * GetDim(invdx, 0), fj = (v.Y - oy) * GetDim(invdx, 1), fk = (v.Z - oz) * GetDim(invdx, 2);
				FVector3i Idx(
					FMath::Clamp((int)fi, 0, NI - 1),
					FMath::Clamp((int)fj, 0, NJ - 1),
					FMath::Clamp((int)fk, 0, NK - 1));

				{
					FScopeLock GridLock(GetGridSectionLock(Idx));
					if (Distances[Idx] < upper_bound)
					{
						return;
					}
					FVector3d p = (FVector3d)cell_center(Idx);
					double dsqr;
					int near_tid = Spatial->FindNearestTriangle(p, dsqr);
					Distances[Idx] = (float)FMathd::Sqrt(dsqr);
					closest_tri[Idx] = near_tid;
					int idx_linear = static_cast<int32>(Distances.GetIndex(Idx));
					if (done[idx_linear] == false)
					{
						done[idx_linear] = true;
						AddToSectionQueue(idx_linear);
					}
				}
			}, !bUseParallel);
		}
		if (CancelF())
		{
			return;
		}

		FlattenSectionQueues();

		// we could do this parallel w/ some kind of producer-consumer...
		FAxisAlignedBox3i Bounds = Distances.BoundsInclusive();
		double max_dist = NarrowBandMaxDistance;
		double max_query_dist = max_dist + GetDiagLength(DX);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_SweepingMeshSDF_FloodFill);
			while (PendingCellQueue.Num() > 0)
			{
				ParallelFor(PendingCellQueue.Num(), [&](int QIdx)
				{
					int cur_linear_index = PendingCellQueue[QIdx];
					FVector3i cur_idx = Distances.GetCoords(cur_linear_index);
					for (FVector3i idx_offset : IndexUtil::GridOffsets26) 
					{
						FVector3i nbr_idx = cur_idx + idx_offset;
						if (Bounds.Contains(nbr_idx) == false)
						{
							continue;
						}
						int nbr_linear_idx = static_cast<int32>(Distances.GetIndex(nbr_idx));

						// Note: this is technically unsafe to do because other threads might be writing to this memory location
						// which could in theory mean that the value being read is garbage or stale. However the only possible values are true and false. 
						// If we get an incorrect false, we will just do an extra unnecessary spatial query, but because we lock before
						// any writes, that value will just be discarded. If we get an incorrect true, we potentially skip a grid cell
						// we need. However in this context we are doing a flood-fill and so except at the very edges of the populated cells,
						// each cell will be considered multiple times as we touch it's neighbours. In practice we have not observed
						// this to be a problem, however locking here dramatically slows the algorithm down (eg by 3-5x on a 64-core machine)
						if (done[nbr_linear_idx])
						{
							continue;
						}

						FVector3d p = (FVector3d)cell_center(nbr_idx);
						double dsqr;
						int near_tid = Spatial->FindNearestTriangle(p, dsqr, max_query_dist);
						if (near_tid == -1)
						{
							FScopeLock GridLock(GetGridSectionLock(nbr_idx));
							done[nbr_linear_idx] = true;
							continue;
						}
						double dist = FMathd::Sqrt(dsqr);

						{	
							// locked section -- if index not already done, update the grid
							FScopeLock GridLock(GetGridSectionLock(nbr_idx));
							if (done[nbr_linear_idx] == false)
							{
								Distances[nbr_linear_idx] = (float)dist;
								closest_tri[nbr_linear_idx] = near_tid;
								done[nbr_linear_idx] = true;
								if (dist < max_dist)
								{
									AddToSectionQueue(nbr_linear_idx);
								}
							}
						}
					}
				}, !bUseParallel);

				FlattenSectionQueues();
			}
		}
		if (CancelF())
		{
			return;
		}


		if (bComputeSigns == true)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Geometry_SweepingMeshSDF_Signs);
			compute_intersections(Origin, DX, NI, NJ, NK, intersection_count);
			if (CancelF())
			{
				return;
			}			

			if (ComputeMode == EComputeModes::FullGrid)
			{
				// and now we fill in the rest of the Distances with fast sweeping
				for (int pass = 0; pass < 2; ++pass)
				{
					sweep_pass(Origin, DX, Distances, closest_tri);
					if (CancelF())
					{
						return;
					}
				}
			}
			else
			{
				// nothing!
			}

			// then figure out signs (inside/outside) from intersection counts
			compute_signs(NI, NJ, NK, Distances, intersection_count);
			if (CancelF())
			{
				return;
			}

		}

	}   // end make_level_set_3


	void CleanupUnwanted()
	{
		if (!bWantIntersectionsGrid)
		{
			IntersectionsGrid.Resize(0, 0, 0, EAllowShrinking::Yes);
		}
		if (!bWantClosestTriGrid)
		{
			ClosestTriGrid.Resize(0, 0, 0, EAllowShrinking::Yes);
		}
	}




	// sweep through Grid in different directions, Distances and closest tris
	void sweep_pass(FVector3f Origin, CellSizeType DX, FDenseGrid3f& Distances, FDenseGrid3i& closest_tri)
	{
		sweep(Distances, closest_tri, Origin, DX, +1, +1, +1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, -1, -1, -1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, +1, +1, -1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, -1, -1, +1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, +1, -1, +1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, -1, +1, -1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, +1, -1, -1);
		if (CancelF()) return;
		sweep(Distances, closest_tri, Origin, DX, -1, +1, +1);
	}


	// single sweep pass
	void sweep(FDenseGrid3f& phi, FDenseGrid3i& closest_tri, FVector3f Origin, CellSizeType DX, int di, int dj, int dk)
	{
		int i0, i1;
		if (di > 0) { i0 = 1; i1 = phi.GetDimensions().X; }
		else { i0 = phi.GetDimensions().X - 2; i1 = -1; }
		int j0, j1;
		if (dj > 0) { j0 = 1; j1 = phi.GetDimensions().Y; }
		else { j0 = phi.GetDimensions().Y - 2; j1 = -1; }
		int k0, k1;
		if (dk > 0) { k0 = 1; k1 = phi.GetDimensions().Z; }
		else { k0 = phi.GetDimensions().Z - 2; k1 = -1; }
		for (int K = k0; K != k1; K += dk)
		{
			if (CancelF())
			{
				return;
			}
			for (int J = j0; J != j1; J += dj)
			{
				for (int I = i0; I != i1; I += di)
				{
					FVector3d gx(float(I) * GetDim(DX, 0) + Origin[0], float(J) * GetDim(DX, 1) + Origin[1], float(K) * GetDim(DX, 2) + Origin[2]);
					check_neighbour(phi, closest_tri, gx, I, J, K, I - di, J, K);
					check_neighbour(phi, closest_tri, gx, I, J, K, I, J - dj, K);
					check_neighbour(phi, closest_tri, gx, I, J, K, I - di, J - dj, K);
					check_neighbour(phi, closest_tri, gx, I, J, K, I, J, K - dk);
					check_neighbour(phi, closest_tri, gx, I, J, K, I - di, J, K - dk);
					check_neighbour(phi, closest_tri, gx, I, J, K, I, J - dj, K - dk);
					check_neighbour(phi, closest_tri, gx, I, J, K, I - di, J - dj, K - dk);
				}
			}
		}
	}



	void check_neighbour(FDenseGrid3f& phi, FDenseGrid3i& closest_tri, const FVector3d& gx, int i0, int j0, int k0, int i1, int j1, int k1)
	{
		if (closest_tri.At(i1, j1, k1) >= 0)
		{
			FVector3d xp, xq, xr;
			Mesh->GetTriVertices(closest_tri.At(i1, j1, k1), xp, xq, xr);
			float d = (float)PointTriangleDistance(gx, xp, xq, xr);
			if (d < phi.At(i0, j0, k0)) {
				phi.At(i0, j0, k0) = d;
				closest_tri.At(i0, j0, k0) = closest_tri.At(i1, j1, k1);
			}
		}
	}




	// fill the intersection Grid w/ number of intersections in each cell
	void compute_intersections(FVector3f Origin, CellSizeType DX, int32 NI, int32 NJ, int32 NK, FDenseGrid3i& IntersectionCount)
	{
		double ox = (double)Origin[0], oy = (double)Origin[1], oz = (double)Origin[2];
		CellSizeTyped invdx = CellSizeTyped(UnitCellSize()) / (CellSizeTyped)DX;

		bool cancelled = false;

		// this is what we will do for each triangle. There are no Grid-reads, only Grid-writes, 
		// since we use atomic_increment, it is always thread-safe
		ParallelFor(Mesh->MaxTriangleID(),
			[this, &Origin, &NI, &NJ, &NK,
			&IntersectionCount, &ox, &oy, &oz, &invdx, &cancelled](int32 TID)
			{
				if (!Mesh->IsTriangle(TID))
				{
					return;
				}
				if (TID % 100 == 0 && CancelF() == true)
				{
					cancelled = true;
				}
				if (cancelled)
				{
					return;
				}

				FVector3d xp = FVector3d::Zero(), xq = FVector3d::Zero(), xr = FVector3d::Zero();
				Mesh->GetTriVertices(TID, xp, xq, xr);


				bool neg_x = false;
				if (InsideMode == EInsideModes::WindingCount)
				{
					FVector3d n = VectorUtil::NormalDirection(xp, xq, xr);
					neg_x = n.X > 0;
				}

				// real IJK coordinates of xp/xq/xr
				double fip = (xp[0] - ox) * GetDim(invdx, 0), fjp = (xp[1] - oy) * GetDim(invdx, 1), fkp = (xp[2] - oz) * GetDim(invdx, 2);
				double fiq = (xq[0] - ox) * GetDim(invdx, 0), fjq = (xq[1] - oy) * GetDim(invdx, 1), fkq = (xq[2] - oz) * GetDim(invdx, 2);
				double fir = (xr[0] - ox) * GetDim(invdx, 0), fjr = (xr[1] - oy) * GetDim(invdx, 1), fkr = (xr[2] - oz) * GetDim(invdx, 2);

				// recompute J/K integer bounds of triangle w/o exact band
				int32 j0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fjp, fjq, fjr)), 0, NJ - 1);
				int32 j1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fjp, fjq, fjr)), 0, NJ - 1);
				int32 k0 = FMath::Clamp(FMath::CeilToInt32(FMath::Min3(fkp, fkq, fkr)), 0, NK - 1);
				int32 k1 = FMath::Clamp(FMath::FloorToInt32(FMath::Max3(fkp, fkq, fkr)), 0, NK - 1);

				// and do intersection counts
				for (int K = k0; K <= k1; ++K)
				{
					for (int J = j0; J <= j1; ++J)
					{
						double A, B, C;
						if (PointInTriangle2d(J, K, fjp, fkp, fjq, fkq, fjr, fkr, A, B, C))
						{
							double fi = A * fip + B * fiq + C * fir; // intersection I coordinate
							int i_interval = FMath::CeilToInt32(fi); // intersection is in (i_interval-1,i_interval]
							if (i_interval < 0)
							{
								DenseGrid::AtomicIncDec(IntersectionCount, 0, J, K, neg_x);
							}
							else if (i_interval < NI)
							{
								DenseGrid::AtomicIncDec(IntersectionCount, i_interval, J, K, neg_x);
							}
							else
							{
								// we ignore intersections that are beyond the +x side of the Grid
							}
						}
					}
				}
			}, !bUseParallel);
	}





	// iterate through each x-row of Grid and set unsigned distances to be negative
	// inside the mesh, based on the IntersectionCount
	void compute_signs(int NI, int NJ, int NK, FDenseGrid3f& Distances, FDenseGrid3i& IntersectionCount)
	{
		if (bUseParallel)
		{
			// can process each x-row in parallel
			ParallelFor(NJ * NK, [this, &NI, &NJ, &NK, &Distances, &IntersectionCount](int32 vi)
				{
					if (CancelF())
					{
						return;
					}

					int32 J = vi % NJ, K = vi / NJ;
					int32 total_count = 0;
					for (int I = 0; I < NI; ++I) {
						total_count += IntersectionCount.At(I, J, K);
						if (
							(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
							(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
							)
						{
							Distances.At(I, J, K) = -Distances.At(I, J, K); // we are inside the mesh
						}
					}
				}, !bUseParallel);

		}
		else
		{
			for (int K = 0; K < NK; ++K)
			{
				if (CancelF())
				{
					return;
				}

				for (int J = 0; J < NJ; ++J)
				{
					int total_count = 0;
					for (int I = 0; I < NI; ++I)
					{
						total_count += IntersectionCount.At(I, J, K);
						if (
							(InsideMode == EInsideModes::WindingCount && total_count > 0) ||
							(InsideMode == EInsideModes::CrossingCount && total_count % 2 == 1)
							)
						{
							Distances.At(I, J, K) = -Distances.At(I, J, K); // we are inside the mesh
						}
					}
				}
			}
		}
	}












public:

	// calculate twice signed area of triangle (0,0)-(X1,Y1)-(X2,Y2)
	// return an SOS-determined sign (-1, +1, or 0 only if it's a truly degenerate triangle)
	static int	Orientation(double X1, double Y1, double X2, double Y2, double& TwiceSignedArea)
	{
		TwiceSignedArea = Y1 * X2 - X1 * Y2;
		if (TwiceSignedArea > 0) return 1;
		else if (TwiceSignedArea < 0) return -1;
		else if (Y2 > Y1) return 1;
		else if (Y2 < Y1) return -1;
		else if (X1 > X2) return 1;
		else if (X1 < X2) return -1;
		else return 0; // only true when X1==X2 and Y1==Y2
	}


	// robust test of (X0,Y0) in the triangle (X1,Y1)-(X2,Y2)-(X3,Y3)
	// if true is returned, the barycentric coordinates are set in A,B,C.
	static bool PointInTriangle2d(double X0, double Y0, double X1, double Y1, double X2, double Y2, double X3, double Y3, double& A, double& B, double& C)
	{
		A = B = C = 0;
		X1 -= X0; X2 -= X0; X3 -= X0;
		Y1 -= Y0; Y2 -= Y0; Y3 -= Y0;
		int signa = Orientation(X2, Y2, X3, Y3, A);
		if (signa == 0) return false;
		int signb = Orientation(X3, Y3, X1, Y1, B);
		if (signb != signa) return false;
		int signc = Orientation(X1, Y1, X2, Y2, C);
		if (signc != signa) return false;
		double sum = A + B + C;
		// if the SOS signs match and are nonzero, there's no way all of A, B, and C are zero.
		// TODO: is this mathematically impossible? can we just remove the check?
		checkf(sum != 0, TEXT("TCachingMeshSDF::PointInTriangle2d: impossible config?"));
		A /= sum;
		B /= sum;

		C /= sum;
		return true;
	}

	// find distance x0 is from segment x1-x2
	static float PointSegmentDistance(const FVector3f& x0, const FVector3f& x1, const FVector3f& x2)
	{
		FVector3f DX = x2 - x1;
		float m2 = DX.SquaredLength();
		// find parameter value of closest point on segment
		float s12 = (DX.Dot(x2 - x0) / m2);
		if (s12 < 0)
		{
			s12 = 0;
		}
		else if (s12 > 1)
		{
			s12 = 1;
		}
		// and find the distance
		return Distance(x0, s12*x1 + (1.0 - s12)*x2);
	}


	// find distance x0 is from segment x1-x2
	static double PointSegmentDistance(const FVector3d& x0, const FVector3d& x1, const FVector3d& x2)
	{
		FVector3d DX = x2 - x1;
		double m2 = DX.SquaredLength();
		// find parameter value of closest point on segment
		double s12 = (DX.Dot(x2 - x0) / m2);
		if (s12 < 0)
		{
			s12 = 0;
		}
		else if (s12 > 1)
		{
			s12 = 1;
		}
		// and find the distance
		return Distance(x0, s12*x1 + (1.0 - s12)*x2);
	}



	// find distance x0 is from triangle x1-x2-x3
	static float PointTriangleDistance(const FVector3f& x0, const FVector3f& x1, const FVector3f& x2, const FVector3f& x3)
	{
		// first find barycentric coordinates of closest point on infinite plane
		FVector3f x13 = (x1 - x3);
		FVector3f x23 = (x2 - x3);
		FVector3f x03 = (x0 - x3);
		float m13 = x13.SquaredLength(), m23 = x23.SquaredLength(), d = x13.Dot(x23);
		float invdet = 1.0f / FMath::Max(m13 * m23 - d * d, 1e-30f);
		float a = x13.Dot(x03), b = x23.Dot(x03);
		// the barycentric coordinates themselves
		float w23 = invdet * (m23 * a - d * b);
		float w31 = invdet * (m13 * b - d * a);
		float w12 = 1 - w23 - w31;
		if (w23 >= 0 && w31 >= 0 && w12 >= 0) // if we're inside the triangle
		{
			return Distance(x0, w23*x1 + w31*x2 + w12*x3);
		}
		else // we have to clamp to one of the edges
		{
			if (w23 > 0) // this rules out edge 2-3 for us
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x1, x3));
			}
			else if (w31 > 0) // this rules out edge 1-3
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x2, x3));
			}
			else // w12 must be >0, ruling out edge 1-2
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x3), PointSegmentDistance(x0, x2, x3));
			}
		}
	}


	// find distance x0 is from triangle x1-x2-x3
	static double PointTriangleDistance(const FVector3d& x0, const FVector3d& x1, const FVector3d& x2, const FVector3d& x3)
	{
		// first find barycentric coordinates of closest point on infinite plane
		FVector3d x13 = (x1 - x3);
		FVector3d x23 = (x2 - x3);
		FVector3d x03 = (x0 - x3);
		double m13 = x13.SquaredLength(), m23 = x23.SquaredLength(), d = x13.Dot(x23);
		double invdet = 1.0 / FMath::Max(m13 * m23 - d * d, 1e-30);
		double a = x13.Dot(x03), b = x23.Dot(x03);
		// the barycentric coordinates themselves
		double w23 = invdet * (m23 * a - d * b);
		double w31 = invdet * (m13 * b - d * a);
		double w12 = 1 - w23 - w31;
		if (w23 >= 0 && w31 >= 0 && w12 >= 0) // if we're inside the triangle
		{
			return Distance(x0, w23*x1 + w31*x2 + w12*x3);
		}
		else // we have to clamp to one of the edges
		{
			if (w23 > 0) // this rules out edge 2-3 for us
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x1, x3));
			}
			else if (w31 > 0) // this rules out edge 1-3
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x2), PointSegmentDistance(x0, x2, x3));
			}
			else // w12 must be >0, ruling out edge 1-2
			{
				return FMath::Min(PointSegmentDistance(x0, x1, x3), PointSegmentDistance(x0, x2, x3));
			}
		}
	}

	// Helper methods to access CellSize in either scalar or vector mode
	inline static double GetDim(CellSizeTyped CellSize, int32 Axis)
	{
		if constexpr (bScalarCellSize)
		{
			return (double)CellSize;
		}
		else
		{
			return (double)CellSize[Axis];
		}
	}
	inline static float GetDim(CellSizeType CellSize, int32 Axis)
	{
		if constexpr (bScalarCellSize)
		{
			return (float)CellSize;
		}
		else
		{
			return (float)CellSize[Axis];
		}
	}
	inline static float GetDiagLength(CellSizeType CellSize)
	{
		if constexpr (bScalarCellSize)
		{
			return CellSize * FMathf::Sqrt3;
		}
		else
		{
			return CellSize.Length();
		}
	}
	inline static bool IsValid(CellSizeType CellSize)
	{
		if constexpr (std::is_floating_point_v<CellSizeType>)
		{
			return CellSize > 0 && FMath::IsFinite(CellSize);
		}
		else
		{
			return CellMinDim(CellSize) > 0 && FMath::IsFinite(CellSize[0]) && FMath::IsFinite(CellSize[1]) && FMath::IsFinite(CellSize[2]);
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE