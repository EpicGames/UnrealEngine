// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBase.h"
#include "Math/IntVector.h"
#include "Serialization/Archive.h"
#include <limits>


namespace IndexConstants
{
	inline constexpr int InvalidID = -1;
}

namespace UE
{
namespace Geometry
{

/**
 * 2-index tuple. Ported from g3Sharp library, with the intention of
 * maintaining compatibility with existing g3Sharp code. Has an API
 * similar to WildMagic, GTEngine, Eigen, etc.
 */
struct FIndex2i
{
	union
	{
		struct
		{
			int A, B;
		};

		UE_DEPRECATED(all, "For internal use only")
		int AB[2] = { IndexConstants::InvalidID, IndexConstants::InvalidID };
	};

	constexpr FIndex2i() = default;
	constexpr FIndex2i(int ValA, int ValB)
		: A(ValA), B(ValB)
	{}

	constexpr static FIndex2i Zero()
	{
		return FIndex2i(0, 0);
	}
	constexpr static FIndex2i Max()
	{
		return FIndex2i(TNumericLimits<int>::Max(), TNumericLimits<int>::Max());
	}
	constexpr static FIndex2i Invalid()
	{
		return FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID);
	}

	int& operator[](int Idx)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AB[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const int& operator[](int Idx) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AB[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline bool operator==(const FIndex2i& Other) const
	{
		return A == Other.A && B == Other.B;
	}

	inline bool operator!=(const FIndex2i& Other) const
	{
		return A != Other.A || B != Other.B;
	}

	int IndexOf(int Value) const
	{
		return (A == Value) ? 0 : ((B == Value) ? 1 : -1);
	}

	bool Contains(int Value) const
	{
		return (A == Value) || (B == Value);
	}

	/** @return whichever of A or B is not Value, or IndexConstants::InvalidID if neither is Value */
	int OtherElement(int Value) const
	{
		if (A == Value)
		{
			return B;
		}
		else if (B == Value)
		{
			return A;
		}
		else
		{
			return IndexConstants::InvalidID;
		}
	}

	inline void Swap()
	{
		::Swap(A, B);
	}

	inline void Sort()
	{
		if (A > B)
		{
			Swap();
		}
	}

	/**
	 * Serialization operator for FIndex2i.
	 *
	 * @param Ar Archive to serialize with.
	 * @param I Index to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIndex2i& I)
	{
		I.Serialize(Ar);
		return Ar;
	}

	/** Serialize FIndex2i to an archive. */
	void Serialize(FArchive& Ar)
	{
		Ar << A;
		Ar << B;
	}
};

inline uint32 GetTypeHash(const FIndex2i& Index)
{
	// (this is how FIntVector and all the other FVectors do their hash functions)
	// Note: this assumes there's no padding that could contain non compared data.
	return FCrc::MemCrc_DEPRECATED(&Index, sizeof(FIndex2i));
}



/**
 * 3-index tuple. Ported from g3Sharp library, with the intention of
 * maintaining compatibility with existing g3Sharp code. Has an API
 * similar to WildMagic, GTEngine, Eigen, etc.
 *
 * Implicit casts to/from FIntVector are defined.
 */
struct FIndex3i
{
	union
	{
		struct
		{
			int A;
			int B;
			int C;
		};

		UE_DEPRECATED(all, "For internal use only")
		int ABC[3] = { IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID };
	};

	constexpr FIndex3i() = default;
	constexpr FIndex3i(int ValA, int ValB, int ValC)
		: A(ValA), B(ValB), C(ValC)
	{}

	constexpr static FIndex3i Zero()
	{
		return FIndex3i(0, 0, 0);
	}
	constexpr static FIndex3i Max()
	{
		return FIndex3i(TNumericLimits<int>::Max(), TNumericLimits<int>::Max(), TNumericLimits<int>::Max());
	}
	constexpr static FIndex3i Invalid()
	{
		return FIndex3i(IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID);
	}

	int& operator[](int Idx)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ABC[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const int& operator[](int Idx) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ABC[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool operator==(const FIndex3i& Other) const
	{
		return A == Other.A && B == Other.B && C == Other.C;
	}

	bool operator!=(const FIndex3i& Other) const
	{
		return A != Other.A || B != Other.B || C != Other.C;
	}

	int IndexOf(int Value) const
	{
		return (A == Value) ? 0 : ((B == Value) ? 1 : (C == Value ? 2 : -1));
	}

	bool Contains(int Value) const
	{
		return (A == Value) || (B == Value) || (C == Value);
	}

	// Sort the indices from lowest to highest
	void Sort()
	{
		if (A > B)
		{
			Swap(A, B);
		}
		if (B > C)
		{
			Swap(B, C);
			if (A > B)
			{
				Swap(A, B);
			}
		}
	}

	/** @return offset triplet, with the OffsetIndicesBy value added to each index */
	[[nodiscard]] FIndex3i GetOffsetBy(int32 OffsetIndicesBy) const
	{
		return FIndex3i(A + OffsetIndicesBy, B + OffsetIndicesBy, C + OffsetIndicesBy);
	}

	/**
	 * @return shifted triplet such that A=WantIndex0Value, and B,C values maintain the same relative ordering
	 */
	FIndex3i GetCycled(int32 WantIndex0Value) const
	{
		if (B == WantIndex0Value)
		{
			return FIndex3i(B, C, A);
		}
		else if (C == WantIndex0Value)
		{
			return FIndex3i(C, A, B);
		}
		return FIndex3i(A,B,C);
	}



	operator FIntVector() const
	{
		return FIntVector(A, B, C);
	}
	FIndex3i(const FIntVector& Vec)
	{
		A = Vec.X;
		B = Vec.Y;
		C = Vec.Z;
	}

	/**
	 * Serialization operator for FIndex3i.
	 *
	 * @param Ar Archive to serialize with.
	 * @param I Index to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIndex3i& I)
	{
		I.Serialize(Ar);
		return Ar;
	}

	/** Serialize FIndex3i to an archive */
	void Serialize(FArchive& Ar)
	{
		Ar << A;
		Ar << B;
		Ar << C;
	}
};

inline uint32 GetTypeHash(const FIndex3i& Index)
{
	// (this is how FIntVector and all the other FVectors do their hash functions)
	// Note: this assumes there's no padding that could contain uncompared data.
	return FCrc::MemCrc_DEPRECATED(&Index, sizeof(FIndex3i));
}



/**
 * 4-index tuple. Ported from g3Sharp library, with the intention of
 * maintaining compatibility with existing g3Sharp code. Has an API
 * similar to WildMagic, GTEngine, Eigen, etc.
 */

struct FIndex4i
{
	union
	{
		struct
		{
			int A, B, C, D;
		};

		UE_DEPRECATED(all, "For internal use only")
		int ABCD[4];
	};

	FIndex4i()
	{
	}
	FIndex4i(int ValA, int ValB, int ValC, int ValD)
	{
		this->A = ValA;
		this->B = ValB;
		this->C = ValC;
		this->D = ValD;
	}

	static FIndex4i Zero()
	{
		return FIndex4i(0, 0, 0, 0);
	}
	static FIndex4i Max()
	{
		return FIndex4i(TNumericLimits<int>::Max(), TNumericLimits<int>::Max(), TNumericLimits<int>::Max(), TNumericLimits<int>::Max());
	}
	static FIndex4i Invalid()
	{
		return FIndex4i(IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID);
	}

	int& operator[](int Idx)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ABCD[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const int& operator[](int Idx) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ABCD[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool operator==(const FIndex4i& Other) const
	{
		return A == Other.A && B == Other.B && C == Other.C && D == Other.D;
	}

	bool operator!=(const FIndex4i& Other) const
	{
		return A != Other.A || B != Other.B || C != Other.C || D != Other.D;
	}

	int IndexOf(int Value) const
	{
		return (A == Value) ? 0 : ((B == Value) ? 1 : ((C == Value) ? 2 : ((D == Value) ? 3 : -1)));
	}

	bool Contains(int Idx) const
	{
		return A == Idx || B == Idx || C == Idx || D == Idx;
	}

	/**
	 * Serialization operator for FIndex4i.
	 *
	 * @param Ar Archive to serialize with.
	 * @param I Index to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIndex4i& I)
	{
		I.Serialize(Ar);
		return Ar;
	}

	/** Serialize FIndex3i to an archive */
	void Serialize(FArchive& Ar)
	{
		Ar << A;
		Ar << B;
		Ar << C;
		Ar << D;
	}
};

inline uint32 GetTypeHash(const FIndex4i& Index)
{
	// (this is how FIntVector and all the other FVectors do their hash functions)
	// Note: this assumes there's no padding that could contain uncompared data.
	return FCrc::MemCrc_DEPRECATED(&Index, sizeof(FIndex4i));
}


} // end namespace UE::Geometry
} // end namespace UE

template<> struct TCanBulkSerialize<UE::Geometry::FIndex2i> { enum { Value = true }; };
template<> struct TCanBulkSerialize<UE::Geometry::FIndex3i> { enum { Value = true }; };
template<> struct TCanBulkSerialize<UE::Geometry::FIndex4i> { enum { Value = true }; };

