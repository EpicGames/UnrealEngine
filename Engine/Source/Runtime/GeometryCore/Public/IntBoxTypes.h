// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntVectorTypes.h"

namespace UE
{
namespace Geometry
{

struct FInterval1i
{
	int32 Min;
	int32 Max;

	FInterval1i() :
		FInterval1i(Empty())
	{
	}

	FInterval1i(const int32& Min, const int32& Max)
	{
		this->Min = Min;
		this->Max = Max;
	}

	static FInterval1i Empty()
	{
		return FInterval1i(TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max());
	}

	int32 Center() const
	{
		return (Min + Max) / 2;
	}

	int32 Extent() const
	{
		return (Max - Min) / 2;
	}
	int32 Length() const
	{
		return Max - Min;
	}

	inline bool IsEmpty() const
	{
		return Max < Min;
	}

	void Expand(int32 Radius)
	{
		Max += Radius;
		Min -= Radius;
	}

	void Contain(int32 V)
	{
		if (V < Min)
		{
			Min = V;
		}
		if (V > Max)
		{
			Max = V;
		}
	}
};




struct FAxisAlignedBox2i
{
	FVector2i Min;
	FVector2i Max;

	FAxisAlignedBox2i() :
		FAxisAlignedBox2i(Empty())
	{
	}

	FAxisAlignedBox2i(const FVector2i& Min, const FVector2i& Max)
		: Min(Min), Max(Max)
	{
	}

	FAxisAlignedBox2i(int32 SquareSize)
		: Min((int32)0, (int32)0), Max(SquareSize, SquareSize)
	{
	}
	FAxisAlignedBox2i(int32 Width, int32 Height)
		: Min((int32)0, (int32)0), Max(Width, Height)
	{
	}

	static FAxisAlignedBox2i Empty()
	{
		return FAxisAlignedBox2i(
			FVector2i(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max()),
			FVector2i(-TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max()));
	}

	bool operator==(const FAxisAlignedBox2i& Other) const
	{
		return Max == Other.Max && Min == Other.Min;
	}
	bool operator!=(const FAxisAlignedBox2i& Other) const
	{
		return Max != Other.Max || Min != Other.Min;
	}

	/**
	 * Corners are ordered to follow the perimeter of the bounding rectangle, starting from the (Min.X, Min.Y) corner and ending at (Min.X, Max.Y)
	 * @param Index which corner to return, must be in range [0,3]
	 * @return Corner of the bounding rectangle
	 */
	FVector2i GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 3);
		int32 X = ((Index % 3) == 0) ? (Min.X) : (Max.X);
		int32 Y = ((Index & 2) == 0) ? (Min.Y) : (Max.Y);
		return FVector2i(X, Y);
	}

	inline bool IsEmpty() const
	{
		return Max.X < Min.X || Max.Y < Min.Y;
	}

	void Expand(int32 Radius)
	{
		Max.X += Radius;
		Max.Y += Radius;
		Min.X -= Radius;
		Min.Y -= Radius;
	}

	int32 Area() const
	{
		const int32 XLength = TMathUtil<int32>::Max(Max.X - Min.X, 0);
		const int32 YLength = TMathUtil<int32>::Max(Max.Y - Min.Y, 0);
		return XLength * YLength;
	}

	FVector2i Diagonal() const
	{
		return FVector2i(Max.X - Min.X, Max.Y - Min.Y);
	}

	bool Contains(const FVector2i& V) const
	{
		return (Min.X <= V.X) && (Min.Y <= V.Y) && (Max.X >= V.X) && (Max.Y >= V.Y);
	}

	void Contain(const FVector2i& V)
	{
		if (V.X < Min.X)
		{
			Min.X = V.X;
		}
		if (V.X > Max.X)
		{
			Max.X = V.X;
		}
		if (V.Y < Min.Y)
		{
			Min.Y = V.Y;
		}
		if (V.Y > Max.Y)
		{
			Max.Y = V.Y;
		}
	}
};






struct FAxisAlignedBox3i
{
	FVector3i Min;
	FVector3i Max;

	FAxisAlignedBox3i() : 
		FAxisAlignedBox3i(FAxisAlignedBox3i::Empty())
	{
	}

	FAxisAlignedBox3i(const FVector3i& Min, const FVector3i& Max)
	{
		this->Min = Min;
		this->Max = Max;
	}

	bool operator==(const FAxisAlignedBox3i& Other) const
	{
		return Max == Other.Max && Min == Other.Min;
	}
	bool operator!=(const FAxisAlignedBox3i& Other) const
	{
		return Max != Other.Max || Min != Other.Min;
	}

	int32 Width() const
	{
		return TMathUtil<int32>::Max(Max.X - Min.X, 0);
	}

	int32 Height() const
	{
		return TMathUtil<int32>::Max(Max.Y - Min.Y, 0);
	}

	int32 Depth() const
	{
		return TMathUtil<int32>::Max(Max.Z - Min.Z, 0);
	}

	int32 Volume() const
	{
		return Width() * Height() * Depth();
	}

	FVector3i Diagonal() const
	{
		return FVector3i(Max.X - Min.X, Max.Y - Min.Y, Max.Z - Min.Z);
	}

	bool Contains(const FVector3i& V) const
	{
		return (Min.X <= V.X) && (Min.Y <= V.Y) && (Min.Z <= V.Z) && (Max.X >= V.X) && (Max.Y >= V.Y) && (Max.Z >= V.Z);
	}

	void Contain(const FVector3i& V)
	{
		if (V.X < Min.X)
		{
			Min.X = V.X;
		}
		if (V.X > Max.X)
		{
			Max.X = V.X;
		}
		if (V.Y < Min.Y)
		{
			Min.Y = V.Y;
		}
		if (V.Y > Max.Y)
		{
			Max.Y = V.Y;
		}
		if (V.Z < Min.Z)
		{
			Min.Z = V.Z;
		}
		if (V.Z > Max.Z)
		{
			Max.Z = V.Z;
		}
	}

	/**
	* @param Index corner index in range 0-7
	* @return Corner point on the box identified by the given index. See diagram in OrientedBoxTypes.h for index/corner mapping.
	*/
	FVector3i GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 7);
		int32 X = (((Index & 1) != 0) ^ ((Index & 2) != 0)) ? (Max.X) : (Min.X);
		int32 Y = ((Index / 2) % 2 == 0) ? (Min.Y) : (Max.Y);
		int32 Z = (Index < 4) ? (Min.Z) : (Max.Z);
		return FVector3i(X, Y, Z);
	}

	static FAxisAlignedBox3i Empty()
	{
		return FAxisAlignedBox3i(
			FVector3i(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max()),
			FVector3i(-TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max()));
	}

	static FAxisAlignedBox3i Infinite()
	{
		return FAxisAlignedBox3i(
			FVector3i(-TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max(), -TNumericLimits<int32>::Max()),
			FVector3i(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max()) );
	}
};



} // end namespace UE::Geometry
} // end namespace UE