// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/IntVector.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include <sstream>


namespace UE {
namespace Geometry {


/**
* 2D 32-bit integer Vector
*/
struct FVector2i
{
	union
	{
		struct
		{
			int32 X, Y;
		};

		UE_DEPRECATED(all, "For internal use only")
		int32 XY[2] = { {}, {} };
	};

	constexpr FVector2i()
		: X(0), Y(0)
	{
	}

	constexpr FVector2i(int32 ValX, int32 ValY)
		: X(ValX), Y(ValY)
	{
	}

	constexpr FVector2i(const int32* Data)
		: X(Data[0]), Y(Data[1])
	{
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	constexpr FVector2i(const FVector2i& Vec) = default;
	constexpr FVector2i& operator=(const FVector2i& V2) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	explicit constexpr operator const int32*() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	explicit constexpr operator int32*()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	explicit operator FVector2f() const
	{
		return FVector2f((float)X, (float)Y);
	}
	explicit operator FVector2d() const
	{
		return FVector2d((double)X, (double)Y);
	}
	explicit operator FIntVector2() const
	{
		return FIntVector2(X, Y);
	}

	explicit FVector2i(const FVector& Vec)
		: X((int32)Vec.X), Y((int32)Vec.Y)
	{
	}

	explicit FVector2i(const FVector2f& Vec)
		: X((int32)Vec.X), Y((int32)Vec.Y)
	{
	}

	explicit FVector2i(const FVector2d& Vec)
		: X((int32)Vec.X), Y((int32)Vec.Y)
	{
	}

	explicit FVector2i(const FIntVector2& Vec)
		: X(Vec.X), Y(Vec.Y)
	{
	}

	constexpr static FVector2i Zero()
	{
		return FVector2i(0, 0);
	}
	constexpr static FVector2i One()
	{
		return FVector2i(1, 1);
	}
	constexpr static FVector2i UnitX()
	{
		return FVector2i(1, 0);
	}
	constexpr static FVector2i UnitY()
	{
		return FVector2i(0, 1);
	}

	constexpr int32& operator[](int Idx)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	constexpr const int32& operator[](int Idx) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XY[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	constexpr int32 SquaredLength() const
	{
		return X * X + Y * Y;
	}

	constexpr int32 DistanceSquared(const FVector2i& V2) const
	{
		int32 dx = V2.X - X;
		int32 dy = V2.Y - Y;
		return dx * dx + dy * dy;
	}

	constexpr int32 Dot(const FVector2i& V2) const
	{
		return X * V2.X + Y * V2.Y;
	}

	constexpr FVector2i operator-() const
	{
		return FVector2i(-X, -Y);
	}

	constexpr FVector2i operator+(const FVector2i& V2) const
	{
		return FVector2i(X + V2.X, Y + V2.Y);
	}

	constexpr FVector2i operator-(const FVector2i& V2) const
	{
		return FVector2i(X - V2.X, Y - V2.Y);
	}

	constexpr FVector2i operator+(const int32& Scalar) const
	{
		return FVector2i(X + Scalar, Y + Scalar);
	}

	constexpr FVector2i operator-(const int32& Scalar) const
	{
		return FVector2i(X - Scalar, Y - Scalar);
	}

	constexpr FVector2i operator*(const int32& Scalar) const
	{
		return FVector2i(X * Scalar, Y * Scalar);
	}

	constexpr FVector2i operator*(const FVector2i& V2) const // component-wise
	{
		return FVector2i(X * V2.X, Y * V2.Y);
	}

	constexpr FVector2i operator/(const int32& Scalar) const
	{
		return FVector2i(X / Scalar, Y / Scalar);
	}

	constexpr FVector2i operator/(const FVector2i& V2) const // component-wise
	{
		return FVector2i(X / V2.X, Y / V2.Y);
	}

	constexpr FVector2i& operator+=(const FVector2i& V2)
	{
		X += V2.X;
		Y += V2.Y;
		return *this;
	}

	constexpr FVector2i& operator-=(const FVector2i& V2)
	{
		X -= V2.X;
		Y -= V2.Y;
		return *this;
	}

	constexpr FVector2i& operator*=(const int32& Scalar)
	{
		X *= Scalar;
		Y *= Scalar;
		return *this;
	}

	constexpr FVector2i& operator/=(const int32& Scalar)
	{
		X /= Scalar;
		Y /= Scalar;
		return *this;
	}

	constexpr bool operator==(const FVector2i& Other) const
	{
		return X == Other.X && Y == Other.Y;
	}

	constexpr bool operator!=(const FVector2i& Other) const
	{
		return X != Other.X || Y != Other.Y;
	}
};

inline FVector2i operator*(int32 Scalar, const FVector2i& V)
{
	return FVector2i(Scalar * V.X, Scalar * V.Y);
}

inline std::ostream& operator<<(std::ostream& os, const FVector2i& Vec)
{
	os << Vec.X << " " << Vec.Y;
	return os;
}

inline uint32 GetTypeHash(const FVector2i& Vector)
{
	return FCrc::MemCrc_DEPRECATED(&Vector, sizeof(FVector2i));
}



/**
* 3D 32-bit integer Vector
*/
struct FVector3i
{
	union
	{
		struct
		{
			int32 X, Y, Z;
		};

		UE_DEPRECATED(all, "For internal use only")
		int32 XYZ[3] = { {}, {}, {} };
	};

	constexpr FVector3i()
		: X(0), Y(0), Z(0)
	{
	}

	constexpr FVector3i(int32 ValX, int32 ValY, int32 ValZ)
		: X(ValX), Y(ValY), Z(ValZ)
	{
	}

	constexpr FVector3i(const int32* Data)
		: X(Data[0]), Y(Data[1]), Z(Data[2])
	{
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	constexpr FVector3i(const FVector3i& Vec) = default;
	FVector3i& operator=(const FVector3i& V2) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	explicit constexpr operator const int32*() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
	explicit constexpr operator int32*()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	explicit operator FVector3f() const
	{
		return FVector3f((float)X, (float)Y, (float)Z);
	}
	explicit operator FVector3d() const
	{
		return FVector3d((double)X, (double)Y, (double)Z);
	}

	explicit operator FIntVector() const
	{
		return FIntVector(X, Y, Z);
	}

	explicit FVector3i(const FVector3f& Vec)
		: X((int32)Vec.X), Y((int32)Vec.Y), Z((int32)Vec.Z)
	{
	}

	explicit FVector3i(const FVector3d& Vec)
		: X((int32)Vec.X), Y((int32)Vec.Y), Z((int32)Vec.Z)
	{
	}

	explicit FVector3i(const FIntVector& Vec)
		: X(Vec.X), Y(Vec.Y), Z(Vec.Z)
	{
	}

	static FVector3i Zero()
	{
		return FVector3i(0, 0, 0);
	}
	static FVector3i One()
	{
		return FVector3i(1, 1, 1);
	}
	static FVector3i UnitX()
	{
		return FVector3i(1, 0, 0);
	}
	static FVector3i UnitY()
	{
		return FVector3i(0, 1, 0);
	}
	static FVector3i UnitZ()
	{
		return FVector3i(0, 0, 1);
	}
	static FVector3i MaxVector()
	{
		return FVector3i(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max());
	}
	
	int32& operator[](int Idx)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const int32& operator[](int Idx) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZ[Idx];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	constexpr int32 SquaredLength() const
	{
		return X * X + Y * Y + Z * Z;
	}

	constexpr int32 DistanceSquared(const FVector3i& V2) const
	{
		int32 dx = V2.X - X;
		int32 dy = V2.Y - Y;
		int32 dz = V2.Z - Z;
		return dx * dx + dy * dy + dz * dz;
	}

	constexpr FVector3i operator-() const
	{
		return FVector3i(-X, -Y, -Z);
	}

	constexpr FVector3i operator+(const FVector3i& V2) const
	{
		return FVector3i(X + V2.X, Y + V2.Y, Z + V2.Z);
	}

	constexpr FVector3i operator-(const FVector3i& V2) const
	{
		return FVector3i(X - V2.X, Y - V2.Y, Z - V2.Z);
	}

	constexpr FVector3i operator+(const int32& Scalar) const
	{
		return FVector3i(X + Scalar, Y + Scalar, Z + Scalar);
	}

	constexpr FVector3i operator-(const int32& Scalar) const
	{
		return FVector3i(X - Scalar, Y - Scalar, Z - Scalar);
	}

	constexpr FVector3i operator*(const int32& Scalar) const
	{
		return FVector3i(X * Scalar, Y * Scalar, Z * Scalar);
	}

	constexpr FVector3i operator*(const FVector3i& V2) const // component-wise
	{
		return FVector3i(X * V2.X, Y * V2.Y, Z * V2.Z);
	}

	constexpr FVector3i operator/(const int32& Scalar) const
	{
		return FVector3i(X / Scalar, Y / Scalar, Z / Scalar);
	}

	constexpr FVector3i operator/(const FVector3i& V2) const // component-wise
	{
		return FVector3i(X / V2.X, Y / V2.Y, Z / V2.Z);
	}

	constexpr FVector3i& operator+=(const FVector3i& V2)
	{
		X += V2.X;
		Y += V2.Y;
		Z += V2.Z;
		return *this;
	}

	constexpr FVector3i& operator-=(const FVector3i& V2)
	{
		X -= V2.X;
		Y -= V2.Y;
		Z -= V2.Z;
		return *this;
	}

	constexpr FVector3i& operator*=(const int32& Scalar)
	{
		X *= Scalar;
		Y *= Scalar;
		Z *= Scalar;
		return *this;
	}

	constexpr FVector3i& operator/=(const int32& Scalar)
	{
		X /= Scalar;
		Y /= Scalar;
		Z /= Scalar;
		return *this;
	}

	constexpr int32 Dot(const FVector3i& V2) const
	{
		return X * V2.X + Y * V2.Y + Z * V2.Z;
	}

	constexpr bool operator==(const FVector3i& Other) const
	{
		return X == Other.X && Y == Other.Y && Z == Other.Z;
	}

	constexpr bool operator!=(const FVector3i& Other) const
	{
		return X != Other.X || Y != Other.Y || Z != Other.Z;
	}

	constexpr bool operator<(const FVector3i& Other) const
	{
		if ( X != Other.X ) return X < Other.X;
		else if (Y != Other.Y) return Y < Other.Y;
		else if (Z != Other.Z) return Z < Other.Z;
		else return false;
	}
};


inline FVector3i operator*(int32 Scalar, const FVector3i& V)
{
	return FVector3i(Scalar * V.X, Scalar * V.Y, Scalar * V.Z);
}


inline FVector3i Min(const FVector3i& V0, const FVector3i& V1)
{
	return FVector3i(TMathUtil<int32>::Min(V0.X, V1.X),
		TMathUtil<int32>::Min(V0.Y, V1.Y),
		TMathUtil<int32>::Min(V0.Z, V1.Z));
}

inline FVector3i Max(const FVector3i& V0, const FVector3i& V1)
{
	return FVector3i(TMathUtil<int32>::Max(V0.X, V1.X),
		TMathUtil<int32>::Max(V0.Y, V1.Y),
		TMathUtil<int32>::Max(V0.Z, V1.Z));
}



inline std::ostream& operator<<(std::ostream& os, const FVector3i& Vec)
{
	os << Vec.X << " " << Vec.Y << " " << Vec.Z;
	return os;
}

inline uint32 GetTypeHash(const FVector3i& Vector)
{
	return FCrc::MemCrc_DEPRECATED(&Vector, sizeof(FVector3i));
}



} // end namespace UE::Geometry
} // end namespace UE


