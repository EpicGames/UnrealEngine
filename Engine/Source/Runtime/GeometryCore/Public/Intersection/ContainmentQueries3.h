// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "CapsuleTypes.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "HalfspaceTypes.h"
#include "Implicit/GridInterpolant.h"
#include "IntVectorTypes.h"
#include "Intersection/IntersectionQueries3.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "OrientedBoxTypes.h"
#include "SphereTypes.h"
#include "Templates/UnrealTemplate.h"
#include "VectorTypes.h"


namespace UE
{
	namespace Math { template <typename T> struct TTransform; }

	namespace Geometry
	{
		template <class GridType, typename RealType, bool bScalarCellSize> class TTriLinearGridInterpolant;
		template <typename T> struct TCapsule3;
		template <typename T> struct THalfspace3;
		template <typename T> struct TSphere3;
		//
		// Sphere Containment Queries
		//

		/** @return true if InnerSphere is fully contained within OuterSphere */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TSphere3<RealType>& OuterSphere, const TSphere3<RealType>& InnerSphere);

		/** @return true if InnerCapsule is fully contained within OuterSphere */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TSphere3<RealType>& OuterSphere, const TCapsule3<RealType>& InnerCapsule);

		/** @return true if InnerBox is fully contained within OuterSphere */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TSphere3<RealType>& OuterSphere, const TOrientedBox3<RealType>& InnerBox);

		/** @return true if all all points in range-based for over EnumerablePts are inside OuterSphere */
		template<typename RealType, typename EnumerablePointsType, typename E = decltype(DeclVal<EnumerablePointsType>().begin())>
		bool IsInside(const TSphere3<RealType>& OuterSphere, EnumerablePointsType EnumerablePts)
		{
			for (TVector<RealType> Point : EnumerablePts)
			{
				if (OuterSphere.Contains(Point) == false)
				{
					return false;
				}
			}
			return true;
		}

		/** @return true if the AABB of InnerGrid is completely inside OuterSphere */
		template<typename RealType, typename GridType>
		bool IsInside(const TSphere3<RealType>& OuterSphere, const TTriLinearGridInterpolant<GridType>& InnerGrid)
		{
			return IsInside(OuterSphere, TOrientedBox3<RealType>(InnerGrid.Bounds()));
		}



		//
		// Capsule Containment Queries
		//

		/** @return true if InnerCapsule is fully contained within OuterCapsule */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TCapsule3<RealType>& OuterCapsule, const TCapsule3<RealType>& InnerCapsule);

		/** @return true if InnerSphere is fully contained within OuterCapsule */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TCapsule3<RealType>& OuterCapsule, const TSphere3<RealType>& InnerSphere);

		/** @return true if InnerBox is fully contained within OuterCapsule */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TCapsule3<RealType>& OuterCapsule, const TOrientedBox3<RealType>& InnerBox);

		/** @return true if all all points in range-based for over EnumerablePts are inside OuterCapsule */
		template<typename RealType, typename EnumerablePointsType, typename E = decltype(DeclVal<EnumerablePointsType>().begin())>
		bool IsInside(const TCapsule3<RealType>& OuterCapsule, EnumerablePointsType EnumerablePts)
		{
			for (TVector<RealType> Point : EnumerablePts)
			{
				if (OuterCapsule.Contains(Point) == false)
				{
					return false;
				}
			}
			return true;
		}

		/** @return true if the AABB of InnerGrid is completely inside OuterCapsule */
		template<typename RealType, typename GridType>
		bool IsInside(const TCapsule3<RealType>& OuterCapsule, const TTriLinearGridInterpolant<GridType>& InnerGrid)
		{
			return IsInside(OuterCapsule, TOrientedBox3<RealType>(InnerGrid.Bounds()));
		}



		//
		// OrientedBox Containment Queries
		//

		/** @return true if InnerBox is fully contained within OuterBox */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TOrientedBox3<RealType>& OuterBox, const TOrientedBox3<RealType>& InnerBox);

		/** @return true if InnerSphere is fully contained within OuterBox */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TOrientedBox3<RealType>& OuterBox, const TSphere3<RealType>& InnerSphere);

		/** @return true if InnerCapsule is fully contained within OuterBox */
		template<typename RealType>
		GEOMETRYCORE_API bool IsInside(const TOrientedBox3<RealType>& OuterBox, const TCapsule3<RealType>& InnerCapsule);

		/** @return true if all all points in range-based for over EnumerablePts are inside OuterBox */
		template<typename RealType, typename EnumerablePointsType, typename E = decltype(DeclVal<EnumerablePointsType>().begin())>
		bool IsInside(const TOrientedBox3<RealType>& OuterBox, EnumerablePointsType EnumerablePts)
		{
			for (TVector<RealType> Point : EnumerablePts)
			{
				if (OuterBox.Contains(Point) == false)
				{
					return false;
				}
			}
			return true;
		}

		/** @return true if the AABB of InnerGrid is completely inside OuterBox */
		template<typename RealType, typename GridType>
		bool IsInside(const TOrientedBox3<RealType>& OuterBox, const TTriLinearGridInterpolant<GridType>& InnerGrid)
		{
			return IsInside(OuterBox, TOrientedBox3<RealType>(InnerGrid.Bounds()));
		}


		//
		// Convex Hull/Volume containment queries
		//

		/** 
		 * Test if the convex volume defined by a set of Halfspaces contains InnerSphere. 
		 * Each Halfspace normal should point "outwards", ie it defines the outer halfspace that is not inside the Convex Volume.
		 * @return false if InnerSphere intersects any of the Halfspaces
		 */
		template<typename RealType>
		bool IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TSphere3<RealType>& InnerSphere);

		/** 
		 * Test if the convex volume defined by a set of Halfspaces contains InnerCapsule. 
		 * Each Halfspace normal should point "outwards", ie it defines the outer halfspace that is not inside the Convex Volume.
		 * @return false if InnerCapsule intersects any of the Halfspaces
		 */
		template<typename RealType>
		bool IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TCapsule3<RealType>& InnerCapsule);

		/** 
		 * Test if the convex volume defined by a set of Halfspaces contains InnerBox. 
		 * Each Halfspace normal should point "outwards", ie it defines the outer halfspace that is not inside the Convex Volume.
		 * @return false if InnerBox intersects any of the Halfspaces
		 */
		template<typename RealType>
		bool IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TOrientedBox3<RealType>& InnerBox);

		/** 
		 * Test if the convex volume defined by a set of Halfspaces contains InnerSphere. 
		 * Each Halfspace normal should point "outwards", ie it defines the outer halfspace that is not inside the Convex Volume.
		 * @return false any of the Halfspaces contain any of the Points
		 */
		template<typename RealType, typename EnumerablePointsType, typename E = decltype(DeclVal<EnumerablePointsType>().begin())>
		bool IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, EnumerablePointsType EnumerablePts);

		template<typename RealType, typename GridType>
		bool IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TTriLinearGridInterpolant<GridType>& InnerGrid)
		{
			return IsInsideHull(Halfspaces, TOrientedBox3<RealType>(InnerGrid.Bounds()));
		}


		//
		// Signed Distance Field containment queries
		//

		/**
		 * Test if the axis-aligned bounding box of InnerSphere is completely inside the negative region of the signed distance field discretized on OuterGrid
		 */
		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TSphere3<RealType>& InnerSphere);

		/**
		 * Test if the axis-aligned bounding box of InnerCapsule is completely inside the negative region of the signed distance field discretized on OuterGrid
		 */
		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TCapsule3<RealType>& InnerCapsule);

		/**
		 * Test if the axis-aligned bounding box of InnerBox is completely inside the negative region of the signed distance field discretized on OuterGrid
		 */
		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TOrientedBox3<RealType>& InnerBox);

		/**
		 * Test if all enumerable points are inside the negative region of the signed distance field discretized on OuterGrid
		 */
		template<typename RealType, typename GridType, typename EnumerablePointsType, typename E = decltype(DeclVal<EnumerablePointsType>().begin())>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, EnumerablePointsType EnumerablePts);

		/**
		 * Test if the axis-aligned bounding box of InnerGrid is completely inside the negative region of the signed distance field discretized on OuterGrid
		 */
		template<typename RealType, typename GridType1, typename GridType2>
		bool IsInside(const TTriLinearGridInterpolant<GridType1>& OuterGrid, const TTransform<RealType>& OuterGridTransform, 
					  const TTriLinearGridInterpolant<GridType2>& InnerGrid, const TTransform<RealType>& InnerGridTransform )
		{
			return IsInside(OuterGrid, OuterGridTransform * InnerGridTransform.Inverse(), TOrientedBox3<RealType>(InnerGrid.Bounds()));
		}


		// Cylinder

		/**
		 * Test if QueryPoint is inside (or on) the given cylinder.
		 */
		template<typename RealType>
		bool DoesCylinderContainPoint(const TVector<RealType>& CylinderCenter, const TVector<RealType>& NormalizedCylinderAxis,
			RealType CylinderRadius, RealType CylindeHeight, const TVector<RealType>& QueryPoint);
	}
}



template<typename RealType>
bool UE::Geometry::IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TSphere3<RealType>& InnerSphere)
{
	for (const THalfspace3<RealType>& Halfspace : Halfspaces)
	{
		if (UE::Geometry::TestIntersection(Halfspace, InnerSphere))
		{
			return false;
		}
	}
	return true;
}


template<typename RealType>
bool UE::Geometry::IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TCapsule3<RealType>& InnerCapsule)
{
	for (const THalfspace3<RealType>& Halfspace : Halfspaces)
	{
		if (UE::Geometry::TestIntersection(Halfspace, InnerCapsule))
		{
			return false;
		}
	}
	return true;
}


template<typename RealType>
bool UE::Geometry::IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, const TOrientedBox3<RealType>& InnerBox)
{
	for (const THalfspace3<RealType>& Halfspace : Halfspaces)
	{
		if (UE::Geometry::TestIntersection(Halfspace, InnerBox))
		{
			return false;
		}
	}
	return true;
}


template<typename RealType, typename EnumerablePointsType, typename E>
bool UE::Geometry::IsInsideHull(TArrayView<THalfspace3<RealType>> Halfspaces, EnumerablePointsType EnumerablePts)
{
	for (const THalfspace3<RealType>& Halfspace : Halfspaces)
	{
		for (TVector<RealType> Point : EnumerablePts)
		{
			if (Halfspace.Contains(Point))
			{
				return false;
			}
		}
	}
	return true;
}



namespace UE
{
	namespace Geometry
	{
		//
		// Signed distance field containment tests
		// 
		// To test if an object is inside an SDF, we first rasterize the object's AABB onto the SDF grid. If all of the AABB cells have negative SDF values,
		// we report that the object is inside. Note this is conservative, not exact -- so an object may actually be inside the implicit surface defined by
		// the SDF, but if its AABB is not fully inside then we will report "not inside."
		//

		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const UE::Geometry::TAxisAlignedBox3<RealType>& InnerAABB)
		{
			const FVector3i MinCell = OuterGrid.Cell(InnerAABB.Min);
			FVector3i MaxCell = OuterGrid.Cell(InnerAABB.Max);

			for (int32 Dim = 0; Dim < 3; ++Dim)
			{
				if (MinCell[Dim] < 0 || MaxCell[Dim] >= OuterGrid.Dimensions[Dim])
				{
					return false;	// AABB extends outside the grid
				}
			}

			MaxCell += FVector3i(1, 1, 1);

			for (int I = MinCell[0]; I < MaxCell[0]; ++I)
			{
				for (int J = MinCell[1]; J < MaxCell[1]; ++J)
				{
					for (int K = MinCell[2]; K < MaxCell[2]; ++K)
					{
						RealType GridCellValue = OuterGrid.Grid->GetValue(I, J, K);

						if (GridCellValue >= 0)	// one of the cells containing the AABB is outside the implicit surface
						{
							return false;
						}
					}
				}
			}

			return true;
		}

		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TSphere3<RealType>& InnerSphere)
		{
			const TAxisAlignedBox3<RealType> InnerAABB(InnerSphere.Center - InnerSphere.Radius * TVector<RealType>::OneVector,
				InnerSphere.Center + InnerSphere.Radius * TVector<RealType>::OneVector);

			const TAxisAlignedBox3<RealType> GridSpaceInnerAABB(InnerAABB, [&OuterGridTransform](const TVector<RealType>& Corner)
			{
				return OuterGridTransform.InverseTransformPosition(Corner);
			});

			return IsInside(OuterGrid, GridSpaceInnerAABB);
		}

		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TCapsule3<RealType>& InnerCapsule)
		{
			const TAxisAlignedBox3<RealType> GridSpaceInnerAABB(InnerCapsule.GetBounds(), [&OuterGridTransform](const TVector<RealType>& Corner)
			{
				return OuterGridTransform.InverseTransformPosition(Corner);
			});

			return IsInside(OuterGrid, GridSpaceInnerAABB);
		}

		template<typename RealType, typename GridType>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, const TOrientedBox3<RealType>& InnerBox)
		{
			TAxisAlignedBox3<RealType> GridSpaceInnerAABB;
			InnerBox.EnumerateCorners([&OuterGridTransform, &GridSpaceInnerAABB](const FVector3d& CornerPt)
			{
				GridSpaceInnerAABB.Contain(OuterGridTransform.InverseTransformPosition(CornerPt));
			});

			return IsInside(OuterGrid, GridSpaceInnerAABB);
		}

		template<typename RealType, typename GridType, typename EnumerablePointsType, typename E>
		bool IsInside(const TTriLinearGridInterpolant<GridType>& OuterGrid, const TTransform<RealType>& OuterGridTransform, EnumerablePointsType EnumerablePts)
		{
			TAxisAlignedBox3<RealType> GridSpaceInnerAABB;
			for (TVector<RealType> Point : EnumerablePts)
			{
				GridSpaceInnerAABB.Contain(OuterGridTransform.InverseTransformPosition(Point));
			}

			return IsInside(OuterGrid, GridSpaceInnerAABB);
		}

	}
}
