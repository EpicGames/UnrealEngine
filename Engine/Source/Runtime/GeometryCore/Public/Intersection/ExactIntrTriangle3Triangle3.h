// Copyright Epic Games, Inc. All Rights Reserved.

// Interface for exact predicates w/ Unreal Engine vector types

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "CompGeom/ExactPredicates.h"

namespace UE::Geometry {

// An exact-math version of TIntrTriangle3Triangle3, with less persistent state -- results are returned per-function rather than kept on the struct.
template <typename Real>
class TExactIntrTriangle3Triangle3
{
public:
	// Inputs
	TTriangle3<Real> Triangle0, Triangle1;

	// TODO: Implement the option to support coplanar intersection
	// bool bIgnoreCoplanar = true;

	TExactIntrTriangle3Triangle3() = default;
	TExactIntrTriangle3Triangle3(TTriangle3<Real> Triangle0, TTriangle3<Real> Triangle1) : Triangle0(Triangle0), Triangle1(Triangle1)
	{}

	bool Test(bool& bWasCoplanar)
	{
		return TestWithoutCoplanar(Triangle0, Triangle1, bWasCoplanar);
	}

	static bool FindWithoutCoplanar(const TTriangle3<Real>& Triangle0, const TTriangle3<Real>& Triangle1, TVector<Real>& OutSegA, TVector<Real>& OutSegB, bool& bWasCoplanar)
	{
		return FindOrTestWithoutCoplanar<true>(Triangle0, Triangle1, OutSegA, OutSegB, bWasCoplanar);
	}

	static bool TestWithoutCoplanar(const TTriangle3<Real>& Triangle0, const TTriangle3<Real>& Triangle1, bool& bWasCoplanar)
	{
		TVector<Real> Unused_SegA, Unused_SegB;
		return FindOrTestWithoutCoplanar<false>(Triangle0, Triangle1, Unused_SegA, Unused_SegB, bWasCoplanar);
	}


private:
	// Note: Currently also does not handle degenerate (collinear or collapsed-to-point) inputs, and will report such as coplanar
	template<bool bFindSegments>
	static bool FindOrTestWithoutCoplanar(const TTriangle3<Real>& Triangle0, const TTriangle3<Real>& Triangle1, TVector<Real>& OutSegA, TVector<Real>& OutSegB, bool& bWasCoplanar)
	{
		bWasCoplanar = false; // default to false; will be set to true if coplanar case detected

		// Currently implements the non-coplanar part of "Faster Triangle - Triangle Intersection Tests"
		// by Olivier Devillers, Philippe Guigue (2006)
		// Using exact predicate tests to detect intersection, and (inexact) plane intersection to find the OutSeg points

		// Compute stats about the signs of the signed distance of each triangle vertices relative to the other triangle's plane,
		// and return false in no-intersect or coplanar cases.
		const TTriangle3<Real>* Tris[2]{ &Triangle0, &Triangle1 };
		// For each triangle, count how many of its vertices had negative, zero, or positive signed distance (vs the other triangle's plane)
		int32 SignCounts[2][3]{ {0,0,0},{0,0,0} };
		// For each triangle, the index of a vertex in that triangle w/ negative, zero or positive signed distance, respectively.  -1 indicates no vertex had that sign.
		int32 SignSubIdx[2][3]{ {-1,-1,-1},{-1,-1,-1} };
		for (int32 TriIdx = 0; TriIdx < 2; ++TriIdx)
		{
			const TTriangle3<Real>& CurTri = *Tris[TriIdx];
			const TTriangle3<Real>& OtherTri = *Tris[1 - TriIdx];
			
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				Real VertSide = ExactPredicates::Orient3<Real>(OtherTri.V[0], OtherTri.V[1], OtherTri.V[2], CurTri.V[SubIdx]);
				int32 Sign = TMathUtil<Real>::SignAsInt(VertSide);
				SignCounts[TriIdx][Sign + 1]++;
				SignSubIdx[TriIdx][Sign + 1] = SubIdx;
			}
			if (SignCounts[TriIdx][0] == 3 || SignCounts[TriIdx][2] == 3)
			{
				return false;
			}
			if (SignCounts[TriIdx][1] == 3)
			{
				bWasCoplanar = true;
				return false;
			}
		}

		// Figure out a canonical ordering for the vertices, so the first vertex is always the one that is alone on a side of the other triangle (or if there is no such case, then is on the triangle)
		// and the other two vertices are on a relatively smaller-sign side (negative or zero)
		int32 IdxMap[2][3];
		int32 TrySignIdx[3]{ 2,0,1 }; // Look for the vertex that is alone on one side, testing the sides in order: Positive, Negative, Zero
		bool OtherTriNeedsOrientationFlip[2]{ false,false };
		for (int32 TriIdx = 0; TriIdx < 2; ++TriIdx)
		{
			for (int32 Idx = 0; Idx < 3; ++Idx)
			{
				int32 SignIdx = TrySignIdx[Idx];
				if (SignCounts[TriIdx][SignIdx] == 1)
				{
					int32 OffsideIdx = SignSubIdx[TriIdx][SignIdx];
					IdxMap[TriIdx][0] = OffsideIdx;
					IdxMap[TriIdx][1] = (OffsideIdx + 1) % 3;
					IdxMap[TriIdx][2] = (OffsideIdx + 2) % 3;
					// If the isolated vertex is on the negative side, or if it's zero and the other two are positive,
					// then we'll need to flip the other tri's orientation to ensure the isolated vertex is on the (more-)positive side instead
					OtherTriNeedsOrientationFlip[TriIdx] = SignIdx == 0 || (SignIdx == 1 && SignCounts[TriIdx][2] == 2);
					break;
				}
			}
		}
		for (int32 TriIdx = 0; TriIdx < 2; ++TriIdx)
		{
			if (OtherTriNeedsOrientationFlip[TriIdx])
			{
				Swap(IdxMap[1 - TriIdx][1], IdxMap[1 - TriIdx][2]);
			}
		}

		// Friendly names for the remapped triangle vertices, matching the Devillers & Guigue paper Fig. 2 (reference above)
		const TVector<Real>& P1 = Tris[0]->V[IdxMap[0][0]];
		const TVector<Real>& Q1 = Tris[0]->V[IdxMap[0][1]];
		const TVector<Real>& R1 = Tris[0]->V[IdxMap[0][2]];

		const TVector<Real>& P2 = Tris[1]->V[IdxMap[1][0]];
		const TVector<Real>& Q2 = Tris[1]->V[IdxMap[1][1]];
		const TVector<Real>& R2 = Tris[1]->V[IdxMap[1][2]];

		auto GetEdgePlaneCrossing = [](const TVector<Real>& A, const TVector<Real>& B, const TVector<Real>& N, const TVector<Real>& O)
		{
			Real AToPlane = N.Dot(O - A);
			Real ABLenInPlaneDir = N.Dot(B - A);
			// Note: ABLenInPlaneDir should be non-zero because we call this only in non-coplanar cases
			// However, it could still be zero due to floating point error if the vectors involved are very small ...
			// in that case for now, we fall back to the edge midpoint
			if (ABLenInPlaneDir == 0)
			{
				return (A + B) * .5;
			}
			// Find the edge crossing, with a clamp to ensure it remains at least on the edge in numerically bad (near-coplanar) cases
			Real FracMoveAlongAB = FMath::Clamp(AToPlane / ABLenInPlaneDir, (Real)0, (Real)1);
			return A * (1 - FracMoveAlongAB) + B * FracMoveAlongAB;
		};

		if constexpr (!bFindSegments)
		{
			bool bIntersects =
				ExactPredicates::Orient3<Real>(P1, Q1, P2, Q2) <= 0 &&
				ExactPredicates::Orient3<Real>(P1, R1, R2, P2) <= 0;
			return bIntersects;
		}
		else
		{
			// Decision tree from Fig. 5 of Devillers & Guigue paper (see reference above)
			if (ExactPredicates::Orient3<Real>(P1, Q1, R2, P2) <= 0)
			{
				if (ExactPredicates::Orient3<Real>(P1, Q1, Q2, P2) >= 0)
				{
					if (ExactPredicates::Orient3<Real>(P1, R1, Q2, P2) < 0)
					{
						// Intersection is k,j segment -- crossings of p2,q2 to p1,q1
						TVector<Real> N1 = VectorUtil::NormalDirection(P1, Q1, R1);
						TVector<Real> N2 = VectorUtil::NormalDirection(P2, Q2, R2);
						OutSegA = GetEdgePlaneCrossing(P2, Q2, N1, P1);
						OutSegB = GetEdgePlaneCrossing(P1, Q1, N2, P2);
					}
					else
					{
						// Intersection is i,j segment -- crossings of p1,r1 to p1,q1
						TVector<Real> N2 = VectorUtil::NormalDirection(P2, Q2, R2);
						OutSegA = GetEdgePlaneCrossing(P1, R1, N2, P2);
						OutSegB = GetEdgePlaneCrossing(P1, Q1, N2, P2);
					}
				}
				else
				{
					return false;
				}
			}
			else
			{
				if (ExactPredicates::Orient3<Real>(P1, R1, R2, P2) <= 0)
				{
					if (ExactPredicates::Orient3<Real>(P1, R1, Q2, P2) <= 0)
					{
						// Intersection is k,l segment -- crossings p2,q2 to p2, r2
						TVector<Real> N1 = VectorUtil::NormalDirection(P1, Q1, R1);
						OutSegA = GetEdgePlaneCrossing(P2, Q2, N1, P1);
						OutSegB = GetEdgePlaneCrossing(P2, R2, N1, P1);
					}
					else
					{
						// Intersection is i,l segment -- crossings of p1,r1 to p2,r2
						TVector<Real> N1 = VectorUtil::NormalDirection(P1, Q1, R1);
						TVector<Real> N2 = VectorUtil::NormalDirection(P2, Q2, R2);
						OutSegA = GetEdgePlaneCrossing(P1, R1, N2, P2);
						OutSegB = GetEdgePlaneCrossing(P2, R2, N1, P1);
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}
	}

};


typedef TExactIntrTriangle3Triangle3<float> FExactIntrTriangle3Triangle3f;
typedef TExactIntrTriangle3Triangle3<double> FExactIntrTriangle3Triangle3d;


}// namespace UE::Geometry
