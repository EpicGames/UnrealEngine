// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "CoreMinimal.h"
#include "SegmentTypes.h"

namespace UE
{
namespace Geometry
{
template <typename RealType> struct TAxisAlignedBox2;
template <typename T> struct TSegment2;

// Return true if Segment and Box intersect and false otherwise
template<typename Real>
GEOMETRYCORE_API bool TestIntersection(const TSegment2<Real>& Segment, const TAxisAlignedBox2<Real>& Box);

template<typename Real>
GEOMETRYCORE_API bool DoesTriangleIntersectCircle2D(
	const TVector2<Real>& A, const TVector2<Real>& B, const TVector2<Real>& C, 
	const TVector2<Real>& Center, double RadiusSquared);
		
} // namespace UE::Geometry
} // namespace UE
