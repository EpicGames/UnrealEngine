// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CapsuleTypes.h"
#include "CoreMinimal.h"
#include "HalfspaceTypes.h"
#include "OrientedBoxTypes.h"
#include "SphereTypes.h"
#include "VectorTypes.h"

namespace UE
{
	namespace Geometry
	{
		template <typename RealType> struct TOrientedBox3;
		template <typename T> struct TCapsule3;
		template <typename T> struct THalfspace3;
		template <typename T> struct TSphere3;
		//
		// Halfspace Intersection Queries
		//

		/** @return true if Halfspace and Sphere intersect */
		template<typename RealType>
		GEOMETRYCORE_API bool TestIntersection(const THalfspace3<RealType>& Halfspace, const TSphere3<RealType>& Sphere);

		/** @return true if Halfspace and Capsule intersect */
		template<typename RealType>
		GEOMETRYCORE_API bool TestIntersection(const THalfspace3<RealType>& Halfspace, const TCapsule3<RealType>& Capsule);

		/** @return true if Halfspace and Box intersect */
		template<typename RealType>
		GEOMETRYCORE_API bool TestIntersection(const THalfspace3<RealType>& Halfspace, const TOrientedBox3<RealType>& Box);


	}
}