// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "BoxTypes.h"


namespace UE
{
namespace Geometry
{

/**
 * FLinearIntersection contains intersection information returned by linear/primitive intersection functions
 */
struct FLinearIntersection
{
	bool intersects;
	int numIntersections;       // 0, 1, or 2
	FInterval1d parameter;      // t-values along ray
};


} // end namespace UE::Geometry
} // end namespace UE


/**
 * IntersectionUtil contains functions to compute intersections between geometric objects
 */
namespace IntersectionUtil
{
	using namespace UE::Math;
	using namespace UE::Geometry;

	template<typename RealType>
	bool RayTriangleTest(const TVector<RealType>& RayOrigin, const TVector<RealType>& RayDirection, const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		// same code as IntrRay3Triangle3, but can be called w/o constructing additional data structures
			
		// Compute the offset origin, edges, and normal.
		TVector<RealType> diff = RayOrigin - V0;
		TVector<RealType> edge1 = V1 - V0;
		TVector<RealType> edge2 = V2 - V0;
		TVector<RealType> normal = edge1.Cross(edge2);

		// Solve Q + t*D = b1*E1 + b2*E2 (Q = kDiff, D = ray direction,
		// E1 = kEdge1, E2 = kEdge2, N = Cross(E1,E2)) by
		//   |Dot(D,N)|*b1 = sign(Dot(D,N))*Dot(D,Cross(Q,E2))
		//   |Dot(D,N)|*b2 = sign(Dot(D,N))*Dot(D,Cross(E1,Q))
		//   |Dot(D,N)|*t = -sign(Dot(D,N))*Dot(Q,N)
		RealType DdN = RayDirection.Dot(normal);
		RealType sign;
		if (DdN > TMathUtil<RealType>::ZeroTolerance) 
		{
			sign = 1;
		}
		else if (DdN < -TMathUtil<RealType>::ZeroTolerance)
		{
			sign = -1;
			DdN = -DdN;
		}
		else 
		{
			// Ray and triangle are parallel, call it a "no intersection"
			// even if the ray does intersect.
			return false;
		}

		RealType DdQxE2 = sign * RayDirection.Dot(diff.Cross(edge2));
		if (DdQxE2 >= 0) 
		{
			RealType DdE1xQ = sign * RayDirection.Dot(edge1.Cross(diff));
			if (DdE1xQ >= 0) 
			{
				if (DdQxE2 + DdE1xQ <= DdN) 
				{
					// Line intersects triangle, check if ray does.
					RealType QdN = -sign * diff.Dot(normal);
					if (QdN >= 0) 
					{
						// Ray intersects triangle.
						return true;
					}
					// else: t < 0, no intersection
				}
				// else: b1+b2 > 1, no intersection
			}
			// else: b2 < 0, no intersection
		}
		// else: b1 < 0, no intersection
		return false;

	}



	/**
	 * Test if line intersects sphere
	 * @return true if line intersects sphere
	 */
	template<typename RealType>
	bool LineSphereTest(
		const TVector<RealType>& LineOrigin,
		const TVector<RealType>& LineDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius)
	{
		// adapted from GeometricTools GTEngine
		// https://www.geometrictools.com/GTEngine/Include/Mathematics/GteIntrLine3Sphere3.h

		// The sphere is (X-C)^T*(X-C)-1 = 0 and the line is X = P+t*D.
		// Substitute the line equation into the sphere equation to obtain a
		// quadratic equation Q(t) = t^2 + 2*a1*t + a0 = 0, where a1 = D^T*(P-C),
		// and a0 = (P-C)^T*(P-C)-1.

		TVector<RealType> diff = LineOrigin - SphereCenter;
		RealType a0 = diff.SquaredLength() - SphereRadius * SphereRadius;
		RealType a1 = LineDirection.Dot(diff);

		// Intersection occurs when Q(t) has real roots.
		RealType discr = a1 * a1 - a0;
		return (discr >= 0);
	}


	/**
	 * Intersect line with sphere and return intersection info (# hits, ray parameters)
	 * @return true if line intersects sphere
	 */
	template<typename RealType>
	bool LineSphereIntersection(
		const TVector<RealType>& LineOrigin,
		const TVector<RealType>& LineDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius,
		FLinearIntersection& ResultOut)
	{
		// adapted from GeometricTools GTEngine
		// https://www.geometrictools.com/GTEngine/Include/Mathematics/GteIntrLine3Sphere3.h

		// The sphere is (X-C)^T*(X-C)-1 = 0 and the line is X = P+t*D.
		// Substitute the line equation into the sphere equation to obtain a
		// quadratic equation Q(t) = t^2 + 2*a1*t + a0 = 0, where a1 = D^T*(P-C),
		// and a0 = (P-C)^T*(P-C)-1.
		TVector<RealType> diff = LineOrigin - SphereCenter;
		RealType a0 = diff.SquaredLength() - SphereRadius * SphereRadius;
		RealType a1 = LineDirection.Dot(diff);

		// Intersection occurs when Q(t) has real roots.
		RealType discr = a1 * a1 - a0;
		if (discr > 0) 
		{
			ResultOut.intersects = true;
			ResultOut.numIntersections = 2;
			RealType root = FMath::Sqrt(discr);
			ResultOut.parameter.Min = -a1 - root;
			ResultOut.parameter.Max = -a1 + root;
		}
		else if (discr < 0) 
		{
			ResultOut.intersects = false;
			ResultOut.numIntersections = 0;
		}
		else 
		{
			ResultOut.intersects = true;
			ResultOut.numIntersections = 1;
			ResultOut.parameter.Min = -a1;
			ResultOut.parameter.Max = -a1;
		}
		return ResultOut.intersects;
	}

	template<typename RealType>
	FLinearIntersection LineSphereIntersection(
		const TVector<RealType>& LineOrigin,
		const TVector<RealType>& LineDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius)
	{
		FLinearIntersection result;
		LineSphereIntersection(LineOrigin, LineDirection, SphereCenter, SphereRadius, result);
		return result;
	}



	/**
	 * @return true if ray intersects sphere
	 */
	template<typename RealType>
	bool RaySphereTest(
		const TVector<RealType>& RayOrigin,
		const TVector<RealType>& RayDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius)
	{
		// adapted from GeometricTools GTEngine
		// https://www.geometrictools.com/GTEngine/Include/Mathematics/GteIntrRay3Sphere3.h

		// The sphere is (X-C)^T*(X-C)-1 = 0 and the line is X = P+t*D.
		// Substitute the line equation into the sphere equation to obtain a
		// quadratic equation Q(t) = t^2 + 2*a1*t + a0 = 0, where a1 = D^T*(P-C),
		// and a0 = (P-C)^T*(P-C)-1.

		TVector<RealType> diff = RayOrigin - SphereCenter;
		RealType a0 = diff.SquaredLength() - SphereRadius * SphereRadius;
		if (a0 <= 0)
		{
			return true;  // P is inside the sphere.
		}
		// else: P is outside the sphere
		RealType a1 = RayDirection.Dot(diff);
		if (a1 >= 0)
		{
			return false;
		}

		// Intersection occurs when Q(t) has RealType roots.
		RealType discr = a1 * a1 - a0;
		return (discr >= 0);
	}


	/**
	 * Intersect ray with sphere and return intersection info (# hits, ray parameters)
	 * @return true if ray intersects sphere
	 */
	template<typename RealType>
	bool RaySphereIntersection(
		const TVector<RealType>& RayOrigin,
		const TVector<RealType>& RayDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius, 
		FLinearIntersection& Result)
	{
		// adapted from GeometricTools GTEngine
		// https://www.geometrictools.com/GTEngine/Include/Mathematics/GteIntrRay3Sphere3.h

		LineSphereIntersection(RayOrigin, RayDirection, SphereCenter, SphereRadius, Result);
		if (Result.intersects) 
		{
			// The line containing the ray intersects the sphere; the t-interval
			// is [t0,t1].  The ray intersects the sphere as long as [t0,t1]
			// overlaps the ray t-interval [0,+infinity).
			if (Result.parameter.Max < 0) 
			{
				Result.intersects = false;
				Result.numIntersections = 0;
			}
			else if (Result.parameter.Min < 0) 
			{
				Result.numIntersections--;
				Result.parameter.Min = Result.parameter.Max;
			}
		}
		return Result.intersects;
	}

	template<typename RealType>
	FLinearIntersection RaySphereIntersection(
		const TVector<RealType>& RayOrigin,
		const TVector<RealType>& RayDirection,
		const TVector<RealType>& SphereCenter,
		RealType SphereRadius)
	{
		FLinearIntersection result;
		LineSphereIntersection(RayOrigin, RayDirection, SphereCenter, SphereRadius, result);
		return result;
	}

	 /**
	  * Intersect ray with circle and return intersection info (# hits, ray parameters)
	  * @return true if ray intersects circle
	  */
	template <typename RealType>
	bool RayCircleIntersection(
		const TVector<RealType>& RayOrigin,
		const TVector<RealType>& RayDirection,
		const TVector<RealType>& CircleCenter,
		RealType CircleRadius,
		const TVector<RealType>& CircleNormal,
		FLinearIntersection& Result)
	{

		// if ray is parallel to circle, no hit
		if (FMath::IsNearlyZero(TVector<RealType>::DotProduct(CircleNormal, RayDirection)))
		{
			Result.intersects = false;
			Result.numIntersections = 0;
			return false;
		}

		TPlane<RealType> Plane(CircleCenter, CircleNormal);
		RealType Param = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);

		if (Param < 0)
		{
			Result.intersects = false;
			Result.numIntersections = 0;
			return false;
		}

		TVector<RealType> HitPoint = RayOrigin + RayDirection * Param;
		TVector<RealType> Offset = HitPoint - CircleCenter;
		const RealType CircleRadiusSquared = CircleRadius * CircleRadius;

		if (Offset.SizeSquared() <= CircleRadiusSquared)
		{
			Result.intersects = true;
			Result.numIntersections = 1;
			Result.parameter.Min = Result.parameter.Max = Param;
		}
		else
		{
			Result.intersects = false;
			Result.numIntersections = 0;
		}

		return Result.intersects;
	}
};
