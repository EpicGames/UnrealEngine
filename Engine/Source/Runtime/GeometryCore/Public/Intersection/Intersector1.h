// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Intersector1.h  ported from WildMagic5
=============================================================================*/

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * TIntersector1 computes the intersection of one-dimensional intervals [u0,u1] and [v0,v1].
 * The end points of the input intervals must be ordered u0 <= u1 and v0 <= v1.
 * Infinite and degenerate intervals are allowed.
 */
template<typename RealType>
class TIntersector1
{
private:
	/** intersection point/interval, access via GetIntersection */
	TInterval1<RealType> Intersections;

public:
	/** First interval */
	TInterval1<RealType> U;
	/** Second interval */
	TInterval1<RealType> V;

	/**
	 * Number of intersections found.
	 * 0: intervals do not overlap
	 * 1: intervals are touching at a single point
	 * 2: intervals intersect in an interval
	 */
	int NumIntersections = 0;

	TIntersector1(RealType u0, RealType u1, RealType v0, RealType v1) : Intersections(0,0), U(u0, u1), V(v0, v1)
	{
		check(u0 <= u1);
		check(v0 <= v1);
	}

	TIntersector1(const TInterval1<RealType>& u, const TInterval1<RealType>& v) : Intersections(0, 0), U(u), V(v)
	{
	}

	/**
	 * Fast check to see if intervals intersect, but does not calculate intersection interval
	 * @return true if intervals intersect
	 */
	bool Test() const
	{
		return U.Min <= V.Max && U.Max >= V.Min;
	}

	/**
	 * @return first or second point of intersection interval
	 */
	RealType GetIntersection(int i)
	{
		return i == 0 ? Intersections.Min : Intersections.Max;
	}

	/**
	 * Calculate the intersection interval
	 * @return true if the intervals intersect
	 */
	bool Find()
	{
		if ((U.Max < V.Min) || (U.Min > V.Max))
		{
			NumIntersections = 0;
		}
		else if (U.Max > V.Min) 
		{
			if (U.Min < V.Max) 
			{
				NumIntersections = 2;
				Intersections.Min = (U.Min < V.Min ? V.Min : U.Min);
				Intersections.Max = (U.Max > V.Max ? V.Max : U.Max);
				if (Intersections.Min == Intersections.Max) 
				{
					NumIntersections = 1;
				}
			}
			else 
			{
				// U.Min == V.Max
				NumIntersections = 1;
				Intersections.Min = U.Min;
			}
		}
		else 
		{
			// U.Max == V.Min
			NumIntersections = 1;
			Intersections.Min = U.Max;
		}

		return NumIntersections > 0;
	}

};

typedef TIntersector1<double> FIntersector1d;
typedef TIntersector1<float> FIntersector1f;

} // end namespace UE::Geometry
} // end namespace UE
