// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp IntrLine2Line2

#pragma once

#include "VectorTypes.h"
#include "LineTypes.h"
#include "VectorUtil.h"  // for EIntersectionType

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between two 2D lines
 */
template<typename RealType>
class TIntrLine2Line2
{
protected:
	// input data
	TLine2<RealType> Line1;
	TLine2<RealType> Line2;
	RealType dotThresh = TMathUtil<RealType>::ZeroTolerance;
	RealType DistThresh = TMathUtil<RealType>::ZeroTolerance;
	
public:
	// result data
	TVector2<RealType> Point;
	RealType Segment1Parameter;
	RealType Segment2Parameter;
	int Quantity = 0;
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;



	TIntrLine2Line2(const TLine2<RealType>& Line1In, const TLine2<RealType>& Line2In)
		: Line1(Line1In), Line2(Line2In)
	{
	}


	const TLine2<RealType>& GetLine1() const
	{
		return Line1;
	}

	void SetLine1(const TLine2<RealType>& Value)
	{
		Line1 = Value;
		Result = EIntersectionResult::NotComputed;
	}

	const TLine2<RealType>& GetLine2() const
	{
		return Line2;
	}

	void SetLine2(TLine2<RealType>& Value)
	{
		Line2 = Value;
		Result = EIntersectionResult::NotComputed;
	}


	RealType GetDotThreshold() const
	{
		return dotThresh;
	}

	RealType GetDistThreshold() const
	{
		return DistThresh;
	}

	void SetDotThreshold(RealType Value)
	{
		dotThresh = FMath::Max(Value, (RealType)0);
		Result = EIntersectionResult::NotComputed;
	}

	void SetDistThreshold(RealType Value)
	{
		DistThresh = FMath::Max(Value, (RealType)0);
		Result = EIntersectionResult::NotComputed;
	}
	

	bool IsSimpleIntersection() const
	{
		return Result == EIntersectionResult::Intersects && Type == EIntersectionType::Point;
	}


	TIntrLine2Line2& Compute()
	{
		Find();
		return *this;
	}


	bool Find()
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result == EIntersectionResult::Intersects);
		}

		// if either line direction is not a normalized vector, 
		//   results are garbage, so fail query
		if (IsNormalized(Line1.Direction) == false || IsNormalized(Line2.Direction) == false) 
		{
			Type = EIntersectionType::Empty;
			Result = EIntersectionResult::InvalidQuery;
			return false;
		}

		TVector2<RealType> s = TVector2<RealType>::Zero();
		Type = Classify(Line1.Origin, Line1.Direction,
			Line2.Origin, Line2.Direction, dotThresh, DistThresh, s);

		if (Type == EIntersectionType::Point) 
		{
			Quantity = 1;
			Point = Line1.Origin + s.X*Line1.Direction;
			Segment1Parameter = s.X;
			Segment2Parameter = s.Y;
		}
		else if (Type == EIntersectionType::Line) 
		{
			Quantity = TNumericLimits<int32>::Max();
		}
		else 
		{
			Quantity = 0;
		}

		Result = (Type != EIntersectionType::Empty) ?
			EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;
		return (Result == EIntersectionResult::Intersects);
	}



	static EIntersectionType Classify(
		const TVector2<RealType>& P0, const TVector2<RealType>& D0, 
		const TVector2<RealType>& P1, const TVector2<RealType>& D1, 
		RealType DotThreshold, RealType DistThreshold, TVector2<RealType>& s)
	{
		// Ensure DotThreshold is nonnegative.
		DotThreshold = FMath::Max(DotThreshold, (RealType)0);

		// The intersection of two lines is a solution to P0+s0*D0 = P1+s1*D1.
		// Rewrite this as s0*D0 - s1*D1 = P1 - P0 = Q.  If D0.Dot(Perp(D1)) = 0,
		// the lines are parallel.  Additionally, if Q.Dot(Perp(D1)) = 0, the
		// lines are the same.  If D0.Dot(Perp(D1)) is not zero, then
		//   s0 = Q.Dot(Perp(D1))/D0.Dot(Perp(D1))
		// produces the point of intersection.  Also,
		//   s1 = Q.Dot(Perp(D0))/D0.Dot(Perp(D1))

		TVector2<RealType> diff = P1 - P0;
		RealType D0DotPerpD1 = DotPerp(D0, D1);
		if (FMath::Abs(D0DotPerpD1) > DotThreshold) 
		{
			// Lines intersect in a single point.
			RealType invD0DotPerpD1 = 1.0 / D0DotPerpD1;
			RealType diffDotPerpD0 = DotPerp(diff, D0);
			RealType diffDotPerpD1 = DotPerp(diff, D1);
			s[0] = diffDotPerpD1 * invD0DotPerpD1;
			s[1] = diffDotPerpD0 * invD0DotPerpD1;
			return EIntersectionType::Point;
		}

		// Lines are parallel; check if they are within DistThresh apart
		RealType diffNDotPerpD1 = DotPerp(diff, D1);
		if (FMath::Abs(diffNDotPerpD1) <= DistThreshold) 
		{
			// Lines are collinear.
			return EIntersectionType::Line;
		}

		// Lines are parallel, but distinct.
		return EIntersectionType::Empty;
	}

};

typedef TIntrLine2Line2<double> FIntrLine2Line2d;
typedef TIntrLine2Line2<float> FIntrLine2Line2f;

} // end namespace UE::Geometry
} // end namespace UE
