// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic TIntrLine2Triangle2

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "LineTypes.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"

#include "Intersection/Intersector1.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 2D Line and 2D Triangle
 */
template <typename Real>
class TIntrLine2Triangle2
{
protected:
	// Input
	TLine2<Real> Line;
	TTriangle2<Real> Triangle;

public:
	// Output
	int Quantity = 0;
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;

	TVector2<Real> Point0;
	TVector2<Real> Point1;
	double Param0;
	double Param1;

	TLine2<Real> GetLine() const
	{
		return Line;
	}
	TTriangle2<Real> GetTriangle() const
	{
		return Triangle;
	}
	void SetLine(const TLine2<Real>& LineIn)
	{
		Result = EIntersectionResult::NotComputed;
		Line = LineIn;
	}
	void SetTriangle(const TTriangle2<Real>& TriangleIn)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle = TriangleIn;
	}

	bool IsSimpleIntersection()
	{
		return Result == EIntersectionResult::Intersects && Type == EIntersectionType::Point;
	}

	TIntrLine2Triangle2()
	{}
	TIntrLine2Triangle2(TLine2<Real> l, TTriangle2<Real> t) : Line(l), Triangle(t)
	{
	}


	TIntrLine2Triangle2* Compute()
	{
		Find();
		return this;
	}


	bool Find()
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result == EIntersectionResult::Intersects);
		}

		// if either Line Direction is not a normalized vector, 
		//   results are garbage, so fail query
		if (IsNormalized(Line.Direction) == false)
		{
			Type = EIntersectionType::Empty;
			Result = EIntersectionResult::InvalidQuery;
			return false;
		}

		TVector<Real> Dist;
		FVector3i Sign;
		int Positive = 0, Negative = 0, Zero = 0;
		TriangleLineRelations(Line.Origin, Line.Direction, Triangle, Dist, Sign, Positive, Negative, Zero);

		if (Positive == 3 || Negative == 3)
		{
			// No intersections.
			Quantity = 0;
			Type = EIntersectionType::Empty;
		}
		else
		{
			TVector2<Real> param;
			GetInterval(Line.Origin, Line.Direction, Triangle, Dist, Sign, param);

			TIntersector1<Real> intr(param[0], param[1], -TMathUtil<Real>::MaxReal, +TMathUtil<Real>::MaxReal);
			intr.Find();

			Quantity = intr.NumIntersections;
			if (Quantity == 2)
			{
				// Segment intersection.
				Type = EIntersectionType::Segment;
				Param0 = intr.GetIntersection(0);
				Point0 = Line.Origin + Param0 * Line.Direction;
				Param1 = intr.GetIntersection(1);
				Point1 = Line.Origin + Param1 * Line.Direction;
			}
			else if (Quantity == 1)
			{
				// Point intersection.
				Type = EIntersectionType::Point;
				Param0 = intr.GetIntersection(0);
				Point0 = Line.Origin + Param0 * Line.Direction;
			}
			else
			{
				// No intersections.
				Type = EIntersectionType::Empty;
			}
		}

		Result = (Type != EIntersectionType::Empty) ?
			EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;
		return (Result == EIntersectionResult::Intersects);
	}



	static void TriangleLineRelations(
		const TVector2<Real>& Origin, const TVector2<Real>& Direction, const TTriangle2<Real>& Tri,
		TVector<Real>& Dist, FVector3i& Sign, int& Positive, int& Negative, int& Zero, Real Tolerance = TMathUtil<Real>::ZeroTolerance)
	{
		Positive = 0;
		Negative = 0;
		Zero = 0;
		for (int i = 0; i < 3; ++i)
		{
			TVector2<Real> diff = Tri.V[i] - Origin;
			Dist[i] = DotPerp(diff, Direction);
			if (Dist[i] > Tolerance)
			{
				Sign[i] = 1;
				++Positive;
			}
			else if (Dist[i] < -Tolerance)
			{
				Sign[i] = -1;
				++Negative;
			}
			else
			{
				Dist[i] = 0.0;
				Sign[i] = 0;
				++Zero;
			}
		}
	}


	static bool GetInterval(const TVector2<Real>& Origin, const TVector2<Real>& Direction, const TTriangle2<Real>& Tri,
							const TVector<Real>& Dist, const FVector3i& Sign, TVector2<Real>& param)
	{
		// Project Triangle onto Line.
		TVector<Real> proj;
		int i;
		for (i = 0; i < 3; ++i)
		{
			TVector2<Real> diff = Tri.V[i] - Origin;
			proj[i] = Direction.Dot(diff);
		}

		// Compute transverse intersections of Triangle edges with Line.
		double numer, denom;
		int i0, i1;
		int quantity = 0;
		for (i0 = 2, i1 = 0; i1 < 3; i0 = i1++)
		{
			if (Sign[i0] * Sign[i1] < 0)
			{
				numer = Dist[i0] * proj[i1] - Dist[i1] * proj[i0];
				denom = Dist[i0] - Dist[i1];
				param[quantity++] = numer / denom;
			}
		}

		// Check for grazing contact.
		if (quantity < 2)
		{
			for (i = 0; i < 3; i++)
			{
				if (Sign[i] == 0)
				{
					if (quantity == 2) // all sign==0 case
					{
						// Sort
						if (param[0] > param[1])
						{
							Swap(param[0], param[1]);
						}
						// Expand range as needed with new param
						double extraparam = proj[i];
						if (extraparam < param[0])
						{
							param[0] = extraparam;
						}
						else if (extraparam > param[1])
						{
							param[1] = extraparam;
						}
					}
					else
					{
						param[quantity++] = proj[i];
					}
				}
			}
		}

		
		// (we expect GetInterval only to be called if there was some intersection)
		if (!ensureMsgf(quantity > 0, TEXT("TIntrLine2Triangle2.GetInterval: need at least one intersection")))
		{
			return false;
		}

		// Sort.
		if (quantity == 2)
		{
			if (param[0] > param[1])
			{
				double save = param[0];
				param[0] = param[1];
				param[1] = save;
			}
		}
		else
		{
			param[1] = param[0];
		}

		return true;
	}




};

typedef TIntrLine2Triangle2<float> FIntrLine2Triangle2f;
typedef TIntrLine2Triangle2<double> FIntrLine2Triangle2d;

} // end namespace UE::Geometry
} // end namespace UE
