// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic IntrRay3Box3, simplified for Axis-Aligned Bounding Box

#pragma once

#include "BoxTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 3D ray and 3D axis-aligned box
 */
template <typename RealType>
class TIntrRay3AxisAlignedBox3
{
public:

	// @todo port object version



	/**
	 * Test if ray intersects box
	 * @param Ray query ray
	 * @param Box query box
	 * @param ExpandExtents box is expanded by this amount in each direction, useful for dealing with float imprecision
	 * @return true if ray intersects box
	 */
	static bool TestIntersection(const TRay<RealType>& Ray, const TAxisAlignedBox3<RealType>& Box, RealType ExpandExtents = 0)
	{
		TVector<RealType> WdU = TVector<RealType>::Zero();
		TVector<RealType> AWdU = TVector<RealType>::Zero();
		TVector<RealType> DdU = TVector<RealType>::Zero();
		TVector<RealType> ADdU = TVector<RealType>::Zero();
		RealType RHS;

		TVector<RealType> diff = Ray.Origin - Box.Center();
		TVector<RealType> extent = Box.Extents() + ExpandExtents;

		WdU.X = Ray.Direction.X;
		AWdU.X = FMath::Abs(WdU.X);
		DdU.X = diff.X;
		ADdU.X = FMath::Abs(DdU.X);
		if (ADdU.X > extent.X && DdU.X * WdU.X >= (RealType)0) 
		{
			return false;
		}

		WdU.Y = Ray.Direction.Y;
		AWdU.Y = FMath::Abs(WdU.Y);
		DdU.Y = diff.Y;
		ADdU.Y = FMath::Abs(DdU.Y);
		if (ADdU.Y > extent.Y && DdU.Y * WdU.Y >= (RealType)0) 
		{
			return false;
		}

		WdU.Z = Ray.Direction.Z;
		AWdU.Z = FMath::Abs(WdU.Z);
		DdU.Z = diff.Z;
		ADdU.Z = FMath::Abs(DdU.Z);
		if (ADdU.Z > extent.Z && DdU.Z * WdU.Z >= (RealType)0) 
		{
			return false;
		}

		TVector<RealType> WxD = Ray.Direction.Cross(diff);
		TVector<RealType> AWxDdU = TVector<RealType>::Zero();

		AWxDdU.X = FMath::Abs(WxD.X);
		RHS = extent.Y * AWdU.Z + extent.Z * AWdU.Y;
		if (AWxDdU.X > RHS) 
		{
			return false;
		}

		AWxDdU.Y = FMath::Abs(WxD.Y);
		RHS = extent.X * AWdU.Z + extent.Z * AWdU.X;
		if (AWxDdU.Y > RHS) 
		{
			return false;
		}

		AWxDdU.Z = FMath::Abs(WxD.Z);
		RHS = extent.X * AWdU.Y + extent.Y * AWdU.X;
		if (AWxDdU.Z > RHS) 
		{
			return false;
		}

		return true;
	}



	/**
	 * Find intersection of ray with AABB and returns ray T-value of intersection point (or TNumericLimits::Max() on miss)
	 * @param Ray query ray
	 * @param Box query box
	 * @param RayParamOut ray intersect T-value, or TNumericLimits::Max()
	 * @return true if ray intersects box
	 */
	inline static bool FindIntersection(const TRay<RealType>& Ray, const TAxisAlignedBox3<RealType>& Box, RealType& RayParamOut)
	{
		return FindIntersection(Ray.Origin, Ray.Direction, Box, RayParamOut);
	}
	/**
	 * Find intersection of ray with AABB and returns ray T-value of intersection point (or TNumericLimits::Max() on miss)
	 * @param Origin query ray origin
	 * @param Direction query ray direction
	 * @param Box query box
	 * @param RayParamOut ray intersect T-value, or TNumericLimits::Max()
	 * @return true if ray intersects box
	 */
	static bool FindIntersection(const TVector<RealType>& Origin, const TVector<RealType>& Direction, const TAxisAlignedBox3<RealType>& Box, RealType& RayParamOut)
	{
		RealType RayParam0 = 0.0;
		RealType RayParam1 = TNumericLimits<RealType>::Max();
		int Quantity = 0;
		TVector<RealType> Point0 = TVector<RealType>::Zero();
		TVector<RealType> Point1 = TVector<RealType>::Zero();
		EIntersectionType Type = EIntersectionType::Empty;
		DoClipping(RayParam0, RayParam1, Origin, Direction, Box,
			true, Quantity, Point0, Point1, Type);

		if (Type != EIntersectionType::Empty) 
		{
			RayParamOut = RayParam0;
			return true;
		}
		else 
		{
			RayParamOut = TNumericLimits<RealType>::Max();
			return false;
		}
	}




protected:

	// internal functions

	static inline bool DoClipping(RealType& t0, RealType& t1,
		const TVector<RealType>& RayOrigin, const TVector<RealType>& RayDirection,
		const TAxisAlignedBox3<RealType>& Box, bool solid, 
		int& quantity, TVector<RealType>& Point0, TVector<RealType>& Point1, EIntersectionType& intrType)
	{
		TVector<RealType> BOrigin = RayOrigin - Box.Center();
		TVector<RealType> extent = Box.Extents();

		RealType saveT0 = t0, saveT1 = t1;
		bool notAllClipped =
			Clip(+RayDirection.X, -BOrigin.X - extent.X, t0, t1) &&
			Clip(-RayDirection.X, +BOrigin.X - extent.X, t0, t1) &&
			Clip(+RayDirection.Y, -BOrigin.Y - extent.Y, t0, t1) &&
			Clip(-RayDirection.Y, +BOrigin.Y - extent.Y, t0, t1) &&
			Clip(+RayDirection.Z, -BOrigin.Z - extent.Z, t0, t1) &&
			Clip(-RayDirection.Z, +BOrigin.Z - extent.Z, t0, t1);

		if (notAllClipped && (solid || t0 != saveT0 || t1 != saveT1)) 
		{
			if (t1 > t0) 
			{
				intrType = EIntersectionType::Segment;
				quantity = 2;
				Point0 = RayOrigin + t0 * RayDirection;
				Point1 = RayOrigin + t1 * RayDirection;
			}
			else 
			{
				intrType = EIntersectionType::Point;
				quantity = 1;
				Point0 = RayOrigin + t0 * RayDirection;
			}
		}
		else 
		{
			quantity = 0;
			intrType = EIntersectionType::Empty;
		}

		return intrType != EIntersectionType::Empty;
	}

	static inline bool Clip(RealType denom, RealType numer, RealType& t0, RealType& t1)
	{
		// Return value is 'true' if line segment intersects the current test
		// plane.  Otherwise 'false' is returned in which case the line segment
		// is entirely clipped.

		if (denom > (RealType)0)
		{
			if (numer > denom*t1)
			{
				return false;
			}
			if (numer > denom*t0)
			{
				t0 = numer / denom;
			}
			return true;
		}
		else if (denom < (RealType)0)
		{
			if (numer > denom*t0)
			{
				return false;
			}
			if (numer > denom*t1)
			{
				t1 = numer / denom;
			}
			return true;
		}
		else
		{
			return numer <= (RealType)0;
		}
	}


};

typedef TIntrRay3AxisAlignedBox3<float> FIntrRay3AxisAlignedBox3f;
typedef TIntrRay3AxisAlignedBox3<double> FIntrRay3AxisAlignedBox3d;

} // end namespace UE::Geometry
} // end namespace UE
