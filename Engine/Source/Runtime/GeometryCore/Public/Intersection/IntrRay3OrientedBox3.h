// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "IntrRay3AxisAlignedBox3.h"
#include "OrientedBoxTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 3D ray and 3D oriented box
 */
template <typename RealType>
class TIntrRay3OrientedBox3
{
public:

	/**
	 * Find intersection of ray with oriented box
	 * @param Ray query ray
	 * @param Box query box
	 * @param RayParamOut ray intersect T-value, or TNumericLimits::Max() on miss
	 * @return true if ray intersects box
	 * 
	 * Based on GeometricTools Engine implementation:
	 *		https://www.geometrictools.com/GTE/Mathematics/IntrRay3OrientedBox3.h
	 */

	static bool FindIntersection(
		const TRay<RealType>& Ray, 
		const TOrientedBox3<RealType>& Box,
		RealType& RayParamOut)
	{
		const TVector<RealType> OriginOffset = Ray.Origin - Box.Frame.Origin;

		const TVector<RealType> Axis0 = Box.Frame.GetAxis(0);
		const TVector<RealType> Axis1 = Box.Frame.GetAxis(1);
		const TVector<RealType> Axis2 = Box.Frame.GetAxis(2);

		const TVector<RealType> Origin = TVector<RealType>(
			TVector<RealType>::DotProduct(OriginOffset, Axis0),
			TVector<RealType>::DotProduct(OriginOffset, Axis1),
			TVector<RealType>::DotProduct(OriginOffset, Axis2));

		const TVector<RealType> Direction = TVector<RealType>(
			TVector<RealType>::DotProduct(Ray.Direction, Axis0),
			TVector<RealType>::DotProduct(Ray.Direction, Axis1),
			TVector<RealType>::DotProduct(Ray.Direction, Axis2));

		const TRay<RealType> AdjustedRay(Origin, Direction);

		const TAxisAlignedBox3<RealType> AxisAlignedBox(
			TVector<RealType>(-Box.Extents.X, -Box.Extents.Y, -Box.Extents.Z),
			TVector<RealType>(Box.Extents.X, Box.Extents.Y, Box.Extents.Z));

		return TIntrRay3AxisAlignedBox3<RealType>::FindIntersection(AdjustedRay, AxisAlignedBox, RayParamOut);
	}
};

typedef TIntrRay3OrientedBox3<float> FIntrRay3OrientedBox3f;
typedef TIntrRay3OrientedBox3<double> FIntrRay3OrientedBox3d;

} // end namespace UE::Geometry
} // end namespace UE
