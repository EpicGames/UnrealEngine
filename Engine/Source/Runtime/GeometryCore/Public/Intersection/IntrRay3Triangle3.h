// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic IntrRay3Triangle3

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"
#include "Math/Ray.h"

#include "CompGeom/ExactPredicates.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

// Ray representation with additional data to support 'watertight' raycasts, i.e. raycasts that cannot slip between adjacent triangles w/ floating point error
template<typename Real>
struct TWatertightRay3
{
	FIntVector3 DimRemap;
	TVector<Real> Shear;
	TVector<Real> Origin;
	TVector<Real> Direction;

	TWatertightRay3() = default;
	TWatertightRay3(const TVector<Real>& InOrigin, const TVector<Real>& InDirection)
	{
		Init(InOrigin, InDirection);
	}
	explicit TWatertightRay3(const TRay<Real>& InRay)
	{
		Init(InRay.Origin, InRay.Direction);
	}

	void Init(const TVector<Real>& InOrigin, const TVector<Real>& InDirection)
	{
		// By convention, remap the max dimension to 'Z', and the other two to X and Y
		static int32 NextDim[3]{ 1, 2, 0 };
		DimRemap.Z = VectorUtil::Max3Index(InDirection.GetAbs());
		DimRemap.X = NextDim[DimRemap.Z];
		DimRemap.Y = NextDim[DimRemap.X];
		// Preserve winding
		if (InDirection[DimRemap.Z] < 0)
		{
			Swap(DimRemap.X, DimRemap.Y);
		}
		// Direction of ray must not be a zero vector
		checkSlow(InDirection[DimRemap.Z] != (Real)0.0);
		// Compute transform to ray space
		Shear = TVector<Real>(InDirection[DimRemap.X], InDirection[DimRemap.Y], (Real)1.0) / InDirection[DimRemap.Z];
		// Copy standard origin/direction (unchanged)
		Origin = InOrigin;
		Direction = InDirection;
	}
};

/**
 * Compute intersection between 3D ray and 3D triangle
 */
template <typename Real, typename RayType = TRay<Real>>
class TIntrRay3Triangle3
{
public:
	// Input
	RayType Ray;
	TTriangle3<Real> Triangle;

	// Output
	Real RayParameter;
	FVector3d TriangleBaryCoords;
	EIntersectionType IntersectionType;


	TIntrRay3Triangle3(const RayType& RayIn, const TTriangle3<Real>& TriangleIn)
	{
		Ray = RayIn;
		Triangle = TriangleIn;
	}
	
	// watertight intersection test
	inline static bool TestIntersection(const TWatertightRay3<Real>& InRay, const TTriangle3<Real>& InTriangle, EIntersectionType& OutIntersectionType)
	{
		Real UnusedRayParameter;
		FVector3d UnusedTriangleBaryCoords;
		return FindWatertightIntersectionInternal<false>(InRay, InTriangle, UnusedRayParameter, UnusedTriangleBaryCoords, OutIntersectionType);
	}
	
	static bool TestIntersection(const TRay<Real>& InRay, const TTriangle3<Real>& InTriangle, EIntersectionType& OutIntersectionType)
	{
		// Compute the offset origin, edges, and normal.
		TVector<Real> Diff = InRay.Origin - InTriangle.V[0];
		TVector<Real> Edge1 = InTriangle.V[1] - InTriangle.V[0];
		TVector<Real> Edge2 = InTriangle.V[2] - InTriangle.V[0];
		TVector<Real> Normal = Edge1.Cross(Edge2);

		// Solve Q + t*D = b1*E1 + b2*E2 (Q = kDiff, D = ray direction,
		// E1 = kEdge1, E2 = kEdge2, N = Cross(E1,E2)) by
		//   |Dot(D,N)|*b1 = sign(Dot(D,N))*Dot(D,Cross(Q,E2))
		//   |Dot(D,N)|*b2 = sign(Dot(D,N))*Dot(D,Cross(E1,Q))
		//   |Dot(D,N)|*t = -sign(Dot(D,N))*Dot(Q,N)
		Real DdN = InRay.Direction.Dot(Normal);
		Real Sign;
		if (DdN > TMathUtil<Real>::ZeroTolerance)
		{
			Sign = (Real)1;
		}
		else if (DdN < -TMathUtil<Real>::ZeroTolerance)
		{
			Sign = (Real)-1;
			DdN = -DdN;
		}
		else
		{
			// Ray and triangle are parallel, call it a "no intersection"
			// even if the ray does intersect.
			OutIntersectionType = EIntersectionType::Empty;
			return false;
		}

		Real DdQxE2 = Sign * InRay.Direction.Dot(Diff.Cross(Edge2));
		if (DdQxE2 >= (Real)0)
		{
			Real DdE1xQ = Sign * InRay.Direction.Dot(Edge1.Cross(Diff));
			if (DdE1xQ >= (Real)0)
			{
				if (DdQxE2 + DdE1xQ <= DdN)
				{
					// Line intersects triangle, check if ray does.
					Real QdN = -Sign * Diff.Dot(Normal);
					if (QdN >= (Real)0)
					{
						// Ray intersects triangle.
						OutIntersectionType = EIntersectionType::Point;
						return true;
					}
					// else: t < 0, no intersection
				}
				// else: b1+b2 > 1, no intersection
			}
			// else: b2 < 0, no intersection
		}
		// else: b1 < 0, no intersection

		OutIntersectionType = EIntersectionType::Empty;
		return false;
	}


	/**
	 * @return true if ray intersects triangle
	 */
	bool Test()
	{
		return TestIntersection(Ray, Triangle, IntersectionType);
	}

	
	/**
	 * Find intersection point
	 * @return true if ray intersects triangle
	 */
	static bool FindIntersection(const TRay<Real>& InRay, const TTriangle3<Real>& InTriangle,
						Real& OutRayParameter, FVector3d& OutTriangleBaryCoords, EIntersectionType& OutIntersectionType)
	{
		// Compute the offset origin, edges, and normal.
		TVector<Real> Diff = InRay.Origin - InTriangle.V[0];
		TVector<Real> Edge1 = InTriangle.V[1] - InTriangle.V[0];
		TVector<Real> Edge2 = InTriangle.V[2] - InTriangle.V[0];
		TVector<Real> Normal = Edge1.Cross(Edge2);

		// Solve Q + t*D = b1*E1 + b2*E2 (Q = kDiff, D = ray direction,
		// E1 = kEdge1, E2 = kEdge2, N = Cross(E1,E2)) by
		//   |Dot(D,N)|*b1 = sign(Dot(D,N))*Dot(D,Cross(Q,E2))
		//   |Dot(D,N)|*b2 = sign(Dot(D,N))*Dot(D,Cross(E1,Q))
		//   |Dot(D,N)|*t = -sign(Dot(D,N))*Dot(Q,N)
		Real DdN = InRay.Direction.Dot(Normal);
		Real Sign;
		if (DdN > TMathUtil<Real>::ZeroTolerance)
		{
			Sign = (Real)1;
		}
		else if (DdN < -TMathUtil<Real>::ZeroTolerance)
		{
			Sign = (Real)-1;
			DdN = -DdN;
		}
		else
		{
			// Ray and triangle are parallel, call it a "no intersection"
			// even if the ray does intersect.
			OutIntersectionType = EIntersectionType::Empty;
			return false;
		}

		Real DdQxE2 = Sign * InRay.Direction.Dot(Diff.Cross(Edge2));
		if (DdQxE2 >= (Real)0)
		{
			Real DdE1xQ = Sign * InRay.Direction.Dot(Edge1.Cross(Diff));
			if (DdE1xQ >= (Real)0)
			{
				if (DdQxE2 + DdE1xQ <= DdN)
				{
					// Line intersects triangle, check if ray does.
					Real QdN = -Sign * Diff.Dot(Normal);
					if (QdN >= (Real)0)
					{
						// Ray intersects triangle.
						Real Inv = ((Real)1) / DdN;
						OutRayParameter = QdN * Inv;
						OutTriangleBaryCoords.Y = DdQxE2 * Inv;
						OutTriangleBaryCoords.Z = DdE1xQ * Inv;
						OutTriangleBaryCoords.X = (Real)1 - OutTriangleBaryCoords.Y - OutTriangleBaryCoords.Z;
						OutIntersectionType = EIntersectionType::Point;
						return true;
					}
					// else: t < 0, no intersection
				}
				// else: b1+b2 > 1, no intersection
			}
			// else: b2 < 0, no intersection
		}
		// else: b1 < 0, no intersection

		OutIntersectionType = EIntersectionType::Empty;
		return false;
	}


	// Find intersection point in a 'consistent' way, such that rays should not 'leak' between adjacent triangles.
	// Note this may return true but report an intersection point slightly outside the triangle, due to numerical precision.
	// Rays parallel to the triangle are still considered not intersecting.
	// @return true if ray intersects triangle
	inline static bool FindIntersection(const TWatertightRay3<Real>& InRay, const TTriangle3<Real>& InTriangle,
		Real& OutRayParameter, FVector3d& OutTriangleBaryCoords, EIntersectionType& OutIntersectionType)
	{
		return FindWatertightIntersectionInternal<true>(InRay, InTriangle, OutRayParameter, OutTriangleBaryCoords, OutIntersectionType);
	}

	bool Find()
	{
		return FindIntersection(Ray, Triangle, RayParameter, TriangleBaryCoords, IntersectionType);
	}


private:
	// Watertight raycast implementation
	template<bool bNeedResult> // if bNeedResult is false, we will not compute the OutRayParameter and OutTriangleBaryCoords
	static bool FindWatertightIntersectionInternal(const TWatertightRay3<Real>& InRay, const TTriangle3<Real>& InTriangle,
		Real& OutRayParameter, FVector3d& OutTriangleBaryCoords, EIntersectionType& OutIntersectionType)
	{
		OutIntersectionType = EIntersectionType::Empty;

		// transform the input triangle to sheared space w/ the ray through the origin the along Z axis,
		// so the line of the ray intersects the triangle if the 2D projection of the triangle touches the origin
		TTriangle3<Real> OriginRelTri(InTriangle.V[0] - InRay.Origin, InTriangle.V[1] - InRay.Origin, InTriangle.V[2] - InRay.Origin);
		TTriangle2<Real> Tri2D;
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			TVector<Real> V = OriginRelTri.V[Idx];
			Tri2D.V[Idx] = TVector2<Real>(V[InRay.DimRemap.X] - InRay.Shear.X * V[InRay.DimRemap.Z], V[InRay.DimRemap.Y] - InRay.Shear.Y * V[InRay.DimRemap.Z]);
		}

		// Use exact predicate for orientation test of triangle segments vs the origin
		// Note: This is just A.X*B.Y-A.Y*B.X, with some special handling when the result is zero to ensure the sign of the result is accurate
		TVector<Real> BaryScaled(
			ExactPredicates::Orient2Origin(Tri2D.V[2], Tri2D.V[1]),
			ExactPredicates::Orient2Origin(Tri2D.V[0], Tri2D.V[2]),
			ExactPredicates::Orient2Origin(Tri2D.V[1], Tri2D.V[0])
		);

		if ((BaryScaled.X < 0 || BaryScaled.Y < 0 || BaryScaled.Z < 0) &&
			(BaryScaled.X > 0 || BaryScaled.Y > 0 || BaryScaled.Z > 0))
		{
			return false;
		}

		Real Det = BaryScaled.X + BaryScaled.Y + BaryScaled.Z;
		if (Det == 0)
		{
			return false;
		}

		Real ScaledZ[3];
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			ScaledZ[Idx] = InRay.Shear.Z * OriginRelTri.V[Idx][InRay.DimRemap.Z];
		}
		// T parameter in scaled space
		Real ScaledT =
			BaryScaled.X * ScaledZ[0] +
			BaryScaled.Y * ScaledZ[1] +
			BaryScaled.Z * ScaledZ[2];

		if ((ScaledT < 0 && Det > 0) || (ScaledT > 0 && Det < 0))
		{
			return false;
		}
		if constexpr (bNeedResult)
		{
			Real InvDet = (Real)1.0 / Det;

			OutTriangleBaryCoords = BaryScaled * InvDet;
			OutRayParameter = ScaledT * InvDet;
		}
		OutIntersectionType = EIntersectionType::Point;
		return true;
	}

};

typedef TWatertightRay3<float> FWatertightRay3f;
typedef TWatertightRay3<double> FWatertightRay3d;
typedef TIntrRay3Triangle3<float> FIntrRay3Triangle3f;
typedef TIntrRay3Triangle3<double> FIntrRay3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
