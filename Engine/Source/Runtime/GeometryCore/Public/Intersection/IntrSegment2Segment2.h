// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic IntrSegment2Segment2

#pragma once

#include "SegmentTypes.h"
#include "Intersection/Intersector1.h"
#include "Intersection/IntersectionUtil.h"
#include "Intersection/IntrLine2Line2.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "VectorUtil.h"  // for EIntersectionType

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

// ported from WildMagic5
//
//
// RealType IntervalThreshold
// 		The intersection testing uses the center-extent form for line segments.
// 		If you start with endpoints (Vector2<Real>) and create Segment2<Real>
// 		objects, the conversion to center-extent form can contain small
// 		numerical round-off errors.  Testing for the intersection of two
// 		segments that share an endpoint might lead to a failure due to the
// 		round-off errors.  To allow for this, you may specify a small positive
// 		threshold that slightly enlarges the intervals for the segments.  The
// 		default value is zero.
//
// RealType DotThreshold
// 		The computation for determining whether the linear components are
// 		parallel might contain small floating-point round-off errors.  The
// 		default threshold is TMathUtil<RealType>::ZeroTolerance.  If you set the value,
// 		pass in a nonnegative number.
//
//
// The intersection set:  Let q = Quantity.  The cases are
//
//   q = 0: The segments do not intersect.  Type is Empty
//
//   q = 1: The segments intersect in a single point.  Type is Point
//          Intersection point is Point0.
//          
//   q = 2: The segments are collinear and intersect in a segment.
//			Type is Segment. Points are Point0 and Point1

/**
 * Calculate intersection between two 2D line segments
 */
template<typename RealType>
class TIntrSegment2Segment2
{
protected:
	// inputs
	TSegment2<RealType> Segment1;
	TSegment2<RealType> Segment2;
	RealType IntervalThreshold = 0;
	RealType DotThreshold = TMathUtil<RealType>::ZeroTolerance;

public:
	// outputs
	int Quantity = 0;
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;
	// these values are all on segment 1, unlike many other tests!!
	TVector2<RealType> Point0;
	TVector2<RealType> Point1;     // only set if Quantity == 2, ie segment overlap

	RealType Parameter0;
	RealType Parameter1;     // only set if Quantity == 2, ie segment overlap



	TIntrSegment2Segment2(const TSegment2<RealType>& Segment1In, const TSegment2<RealType>& Segment2In)
		: Segment1(Segment1In), Segment2(Segment2In)
	{
	}


	const TSegment2<RealType>& GetSegment1() const
	{
		return Segment1;
	}

	void SetSegment1(const TSegment2<RealType>& Value)
	{
		Segment1 = Value;
		Result = EIntersectionResult::NotComputed;
	}

	const TSegment2<RealType>& GetSegment2() const
	{
		return Segment2;
	}

	void SetSegment2(const TSegment2<RealType>& Value)
	{
		Segment2 = Value;
		Result = EIntersectionResult::NotComputed;
	}

	RealType GetIntervalThreshold() const
	{
		return IntervalThreshold;
	}

	void SetIntervalThreshold(RealType Value)
	{
		IntervalThreshold = FMath::Max(Value, (RealType)0);
		Result = EIntersectionResult::NotComputed;
	}

	RealType GetDotThreshold() const
	{
		return DotThreshold;
	}

	void SetDotThreshold(RealType Value)
	{
		DotThreshold = FMath::Max(Value, (RealType)0);
		Result = EIntersectionResult::NotComputed;
	}

	bool IsSimpleIntersection() const
	{
		return Result == EIntersectionResult::Intersects && Type == EIntersectionType::Point;
	}


	TIntrSegment2Segment2& Compute()
	{
		Find();
		return *this;
	}

	// This is implemented in TSegment2::Intersects
	// bool Test()

	// Note: This implementation is identical to TSegment2::Intersects but also computes the intersection geometry
	bool Find()
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result == EIntersectionResult::Intersects);
		}

		// Special handling of degenerate segments; by convention if Direction was too small to normalize, Extent and Direction will be zero
		bool bIsPoint1 = Segment1.Extent == 0;
		bool bIsPoint2 = Segment2.Extent == 0;
		int IsPointCount = (int)bIsPoint1 + (int)bIsPoint2;
		auto SetPointIntersection = [this](TVector2<RealType> Point, RealType Param) -> void
		{
			Result = EIntersectionResult::Intersects;
			Type = EIntersectionType::Point;
			Quantity = 1;
			Point0 = Point;
			Parameter0 = Param;
		};
		auto SetNoIntersection = [this]() -> void
		{
			Quantity = 0;
			Result = EIntersectionResult::NoIntersection;
			Type = EIntersectionType::Empty;
		};
		if (IsPointCount == 2) // both degenerate -- check if they're within IntervalThreshold of each other
		{
			bool bIntersects = UE::Geometry::DistanceSquared(Segment1.Center, Segment2.Center) <= IntervalThreshold * IntervalThreshold;
			if (bIntersects)
			{
				SetPointIntersection((Segment1.Center + Segment2.Center) * .5, 0);
			}
			else
			{
				SetNoIntersection();
			}
			return bIntersects;
		}
		else if (IsPointCount == 1) // one degenerate -- check if it's within IntervalThreshold of the non-degenerate segment
		{
			RealType DistSq;
			RealType Param = 0;
			TVector2<RealType> Point;
			if (bIsPoint1)
			{
				DistSq = Segment2.DistanceSquared(Segment1.Center);
				Point = Segment1.Center;
			}
			else // bIsPoint2
			{
				DistSq = Segment1.DistanceSquared(Segment2.Center);
				Param = Segment1.Project(Segment2.Center);
				Point = Segment2.Center;
			}
			bool bIntersects = DistSq <= IntervalThreshold * IntervalThreshold;
			if (bIntersects)
			{
				SetPointIntersection(Point, Param);
			}
			else
			{
				SetNoIntersection();
			}
			return bIntersects;
		}

		// If neither segment is zero, directions should always be normalized (will hold as long as segment is created/updated via the standard functions)
		checkSlow(IsNormalized(Segment1.Direction) && IsNormalized(Segment2.Direction));

		TVector2<RealType> s = TVector2<RealType>::Zero();
		Type = TIntrLine2Line2<RealType>::Classify(Segment1.Center, Segment1.Direction,
			Segment2.Center, Segment2.Direction,
			DotThreshold, IntervalThreshold, s);

		if (Type == EIntersectionType::Point)
		{
			// Test whether the line-line intersection is on the segments.
			if (FMath::Abs(s[0]) <= Segment1.Extent + IntervalThreshold
				&& FMath::Abs(s[1]) <= Segment2.Extent + IntervalThreshold)
			{
				SetPointIntersection(Segment1.Center + s[0] * Segment1.Direction, s[0]);
			}
			else
			{
				SetNoIntersection();
			}
		}
		else if (Type == EIntersectionType::Line)
		{
			// Compute the location of Segment2 endpoints relative to Segment1.
			TVector2<RealType> diff = Segment2.Center - Segment1.Center;
			RealType t1 = Segment1.Direction.Dot(diff);
			// Note: IntervalThreshold not used here; this is a bit inconsistent but simplifies the code
			// If you add it, make sure that segments that intersect without it don't end up with too-wide intersection intervals
			RealType tmin = t1 - Segment2.Extent;
			RealType tmax = t1 + Segment2.Extent;
			TIntersector1<RealType> calc(-Segment1.Extent, Segment1.Extent, tmin, tmax);
			calc.Find();
			Quantity = calc.NumIntersections;
			if (Quantity == 2) 
			{
				Type = EIntersectionType::Segment;
				Parameter0 = calc.GetIntersection(0);
				Point0 = Segment1.Center +
					Parameter0 * Segment1.Direction;
				Parameter1 = calc.GetIntersection(1);
				Point1 = Segment1.Center +
					Parameter1 * Segment1.Direction;
			}
			else if (Quantity == 1)
			{
				Type = EIntersectionType::Point;
				Parameter0 = calc.GetIntersection(0);
				Point0 = Segment1.Center +
					Parameter0 * Segment1.Direction;
			}
			else 
			{
				Type = EIntersectionType::Empty;
			}
		}
		else
		{
			Quantity = 0;
		}

		Result = (Type != EIntersectionType::Empty) ?
			EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;

		// for debugging...
		//SanityCheck();

		return (Result == EIntersectionResult::Intersects);
	}


protected:
	void SanityCheck()
	{
		if (Quantity == 0) 
		{
			check(Type == EIntersectionType::Empty);
			check(Result == EIntersectionResult::NoIntersection);
		}
		else if (Quantity == 1) 
		{
			check(Type == EIntersectionType::Point);
			check(Segment1.DistanceSquared(Point0) < TMathUtil<RealType>::ZeroTolerance);
			check(Segment2.DistanceSquared(Point0) < TMathUtil<RealType>::ZeroTolerance);
		}
		else if (Quantity == 2) 
		{
			check(Type == EIntersectionType::Segment);
			check(Segment1.DistanceSquared(Point0) < TMathUtil<RealType>::ZeroTolerance);
			check(Segment1.DistanceSquared(Point1) < TMathUtil<RealType>::ZeroTolerance);
			check(Segment2.DistanceSquared(Point0) < TMathUtil<RealType>::ZeroTolerance);
			check(Segment2.DistanceSquared(Point1) < TMathUtil<RealType>::ZeroTolerance);
		}
	}
};

typedef TIntrSegment2Segment2<double> FIntrSegment2Segment2d;
typedef TIntrSegment2Segment2<float> FIntrSegment2Segment2f;

} // end namespace UE::Geometry
} // end namespace UE
