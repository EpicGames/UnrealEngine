// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic TIntrSegment2Triangle2

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "SegmentTypes.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"

#include "Intersection/IntrLine2Triangle2.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 2D segment and 2D triangle.
 * Note that if Segment.Extent is zero, this will test if the Segment Center is in the triangle.
 */
template <typename Real>
class TIntrSegment2Triangle2
{
protected:
	// Input
	TSegment2<Real> Segment;
	TTriangle2<Real> Triangle;

public:
	// Output
	int Quantity = 0;
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;

	bool IsSimpleIntersection()
	{
		return Result == EIntersectionResult::Intersects && Type == EIntersectionType::Point;
	}


	TVector2<Real> Point0;
	TVector2<Real> Point1;
	double Param0;
	double Param1;

	TSegment2<Real> GetSegment() const
	{
		return Segment;
	}
	TTriangle2<Real> GetTriangle() const
	{
		return Triangle;
	}
	void SetSegment(const TSegment2<Real>& SegmentIn)
	{
		Result = EIntersectionResult::NotComputed;
		Segment = SegmentIn;
	}
	void SetTriangle(const TTriangle2<Real>& TriangleIn)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle = TriangleIn;
	}

	TIntrSegment2Triangle2()
	{}
	TIntrSegment2Triangle2(TSegment2<Real> Seg, TTriangle2<Real> Tri) : Segment(Seg), Triangle(Tri)
	{
	}


	TIntrSegment2Triangle2* Compute(Real Tolerance = TMathUtil<Real>::ZeroTolerance)
	{
		Find(Tolerance);
		return this;
	}


	bool Find(Real Tolerance = TMathUtil<Real>::ZeroTolerance)
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result == EIntersectionResult::Intersects);
		}

		// if segment extent is zero, segment is just a point, so just check the center vs the tri
		// (Note this is also the case where Segment.Direction is the zero vector)
		if (Segment.Extent == (Real)0)
		{
			int pos = 0, neg = 0;
			for (int TriPrev = 2, TriIdx = 0; TriIdx < 3; TriPrev = TriIdx++)
			{
				TVector2<Real> ToPt = Segment.Center - Triangle.V[TriIdx];
				TVector2<Real> EdgePerp = PerpCW(Triangle.V[TriIdx] - Triangle.V[TriPrev]);
				Real EdgeLen = Normalize(EdgePerp);
				if (EdgeLen == 0) // triangle is just a line segment; try edgeperp = one of the other edges
				{
					TVector2<Real> OtherV = Triangle.V[(TriIdx + 1) % 3];
					EdgePerp = OtherV - Triangle.V[TriIdx];
					EdgeLen = Normalize(EdgePerp);
					if (EdgeLen == 0) // triangle is just a point; go by distance between vertex and center
					{
						if ( DistanceSquared(Triangle.V[0], Segment.Center) <= Tolerance*Tolerance)
						{
							pos = neg = 0; // closer than tolerance; accept collision
						}
						else
						{
							pos = neg = 1;
						}
						break;
					}
					else // valid edge, test from the other side of the degenerate tri as well
					{
						TVector2<Real> ToPtFromOther = Segment.Center - OtherV;
						TVector2<Real> BackwardsEdgePerp = -EdgePerp;
						Real OtherSideSign = BackwardsEdgePerp.Dot(ToPtFromOther);
						if (OtherSideSign < -Tolerance)
						{
							neg++;
						}
						else if (OtherSideSign > Tolerance)
						{
							pos++;
						}
					}
				}
				
				Real SideSign = EdgePerp.Dot(ToPt);
				if (SideSign < -Tolerance)
				{
					neg++;
				}
				else if (SideSign > Tolerance)
				{
					pos++;
				}
			}
			if (pos == 0 || neg == 0)
			{
				// note: intersection is a single point but conceptually a segment intersection
				//  -- the whole (degenerate) segment intersects the triangle
				Type = EIntersectionType::Segment;
				Result = EIntersectionResult::Intersects;
				Quantity = 2;
				Point0 = Segment.Center;
				Param0 = 0;
				Point1 = Segment.Center;
				Param1 = 0;
				return true;
			}
			else
			{
				Type = EIntersectionType::Empty;
				Result = EIntersectionResult::NoIntersection;
				return false;
			}
		}
		
		// FSegment2d Direction should always be normalized unless Extent is 0 (handled above)
		checkSlow(IsNormalized(Segment.Direction));

		TVector<Real> dist;
		FVector3i sign;
		int positive = 0, negative = 0, zero = 0;
		TIntrLine2Triangle2<Real>::TriangleLineRelations(Segment.Center, Segment.Direction, Triangle,
			dist, sign, positive, negative, zero, Tolerance);

		if (positive == 3 || negative == 3)
		{
			// No intersections.
			Quantity = 0;
			Type = EIntersectionType::Empty;
		}
		else 
		{
			TVector2<Real> param;
			TIntrLine2Triangle2<Real>::GetInterval(Segment.Center, Segment.Direction, Triangle, dist, sign, param);

			TIntersector1<Real> intr(param[0], param[1], -Segment.Extent, +Segment.Extent);
			intr.Find();

			Quantity = intr.NumIntersections;
			if (Quantity == 2)
			{
				// Segment intersection.
				Type = EIntersectionType::Segment;
				Param0 = intr.GetIntersection(0);
				Point0 = Segment.Center + Param0 * Segment.Direction;
				Param1 = intr.GetIntersection(1);
				Point1 = Segment.Center + Param1 * Segment.Direction;
			}
			else if (Quantity == 1) 
			{
				// Point intersection.
				Type = EIntersectionType::Point;
				Param0 = intr.GetIntersection(0);
				Point0 = Segment.Center + Param0 * Segment.Direction;
			}
			else {
				// No intersections.
				Type = EIntersectionType::Empty;
			}
		}

		Result = (Type != EIntersectionType::Empty) ?
			EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;
		return (Result == EIntersectionResult::Intersects);
	}




};

typedef TIntrSegment2Triangle2<float> FIntrSegment2Triangle2f;
typedef TIntrSegment2Triangle2<double> FIntrSegment2Triangle2d;

} // end namespace UE::Geometry
} // end namespace UE
