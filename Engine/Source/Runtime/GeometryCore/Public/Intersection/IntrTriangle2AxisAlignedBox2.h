// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "BoxTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between a 2D triangle and a 2D axis-aligned box
 */
template <typename Real>
class TIntrTriangle2AxisAlignedBox2
{
protected:
	// Input
	TTriangle2<Real> Triangle;
	TAxisAlignedBox2<Real> Box;
	bool bTriangleIsOriented = true;

public:
	// Output
	// TODO: Unlike other TIntrPrim2Prim2 classes, this only computes the Boolean yes/no Test()
	// But in the future we should add the option to Compute() the intersection polygon as well (e.g. refer to IntrTriangle2Triangle.h)

	TIntrTriangle2AxisAlignedBox2()
	{}
	TIntrTriangle2AxisAlignedBox2(TTriangle2<Real> Triangle, TAxisAlignedBox2<Real> Box, bool bTriangleIsOriented = true)
		: Triangle(Triangle), Box(Box), bTriangleIsOriented(bTriangleIsOriented)
	{
	}

	TTriangle2<Real> GetTriangle() const
	{
		return Triangle;
	}
	bool IsTriangleOriented() const
	{
		return bTriangleIsOriented;
	}
	TTriangle2<Real> GetBox() const
	{
		return Box;
	}
	void SetTriangle(const TTriangle2<Real>& TriangleIn)
	{
		// TODO: Reset any result variable(s), once support for computing full intersection result is added
		Triangle = TriangleIn;
	}
	void SetTriangleOriented(const bool bIsOrientedIn)
	{
		bTriangleIsOriented = bIsOrientedIn;
	}
	void SetBox(const TAxisAlignedBox2<Real>& BoxIn)
	{
		// TODO: Reset any result variable(s), once support for computing full intersection result is added
		Box = BoxIn;
	}
	

	bool Test()
	{
		TAxisAlignedBox2<Real> TriBox(Triangle.V[0], Triangle.V[0]);
		TriBox.Contain(Triangle.V[1]);
		TriBox.Contain(Triangle.V[2]);

		if (!TriBox.Intersects(Box))
		{
			// iff we're separated by a bounding box axis, the bounds don't intersect
			return false;
		}

		Real OrientationSign = 1;
		Real SignedArea = Triangle.SignedArea();
		if (SignedArea < 0)
		{
			if (bTriangleIsOriented && SignedArea < -TMathUtil<Real>::ZeroTolerance)
			{
				// if we're using oriented triangles and this one is inverted, it can't intersect
				return false;
			}
			OrientationSign = -1;
		}

		// test if a box is entirely in the half-plane starting at EdgeVertex, going in the EdgePerp direction
		auto BoxOutsideEdge = [this](TVector2<Real> EdgeVertex, TVector2<Real> EdgePerp)
		{
			for (int32 CornerIdx = 0; CornerIdx < 4; CornerIdx++)
			{
				bool bOutside = EdgePerp.Dot(Box.GetCorner(CornerIdx) - EdgeVertex) > 0;
				if (!bOutside)
				{
					return false;
				}
			}
			return true;
		};

		// Test triangle edges for separation
		for (int32 LeadVert = 0, PrevVert = 2; LeadVert < 3; PrevVert = LeadVert++)
		{
			TVector2<Real> Edge = Triangle.V[LeadVert] - Triangle.V[PrevVert];
			Real SquaredEdgeLen = Edge.SquaredLength();
			if (SquaredEdgeLen < TMathUtil<Real>::ZeroTolerance)
			{
				continue;
			}

			if (BoxOutsideEdge(Triangle.V[LeadVert], PerpCW(Edge) * OrientationSign))
			{
				// separated by a triangle edge
				return false;
			}
		}

		// no separating axis found
		return true;
	}


};

typedef TIntrTriangle2AxisAlignedBox2<float> FIntrTriangle2AxisAlignedBox2f;
typedef TIntrTriangle2AxisAlignedBox2<double> FIntrTriangle2AxisAlignedBox2d;

} // end namespace UE::Geometry
} // end namespace UE
