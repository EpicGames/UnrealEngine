// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic TIntrTriangle2Triangle2

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 2D triangles
 */
template <typename Real>
class TIntrTriangle2Triangle2
{
protected:
	// Input
	TTriangle2<Real> Triangle0, Triangle1;

public:
	// Output
	int Quantity = 0;
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;

	bool IsSimpleIntersection()
	{
		return Result == EIntersectionResult::Intersects && Type == EIntersectionType::Point;
	}

	// intersection polygon - this array will always be 6 elements long,
	// however only the first Quantity vertices will be valid
	TVector2<Real> Points[6];

	TIntrTriangle2Triangle2()
	{}
	TIntrTriangle2Triangle2(TTriangle2<Real> T0, TTriangle2<Real> T1) : Triangle0(T0), Triangle1(T1)
	{
	}

	TTriangle2<Real> GetTriangle0() const
	{
		return Triangle0;
	}
	TTriangle2<Real> GetTriangle1() const
	{
		return Triangle1;
	}
	void SetTriangle0(const TTriangle2<Real>& Triangle0In)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle0 = Triangle0In;
	}
	void SetTriangle1(const TTriangle2<Real>& Triangle1In)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle1 = Triangle1In;
	}

	bool Test()
	{
		int i0, i1;
		TVector2<Real> dir;

		// Test edges of Triangle0 for separation.
		for (i0 = 0, i1 = 2; i0 < 3; i1 = i0++)
		{
			// Test axis V0[i1] + t*perp(V0[i0]-V0[i1]), perp(x,y) = (y,-x).
			dir.X = Triangle0.V[i0].Y - Triangle0.V[i1].Y;
			dir.Y = Triangle0.V[i1].X - Triangle0.V[i0].X;
			if (WhichSide(Triangle1, Triangle0.V[i1], dir) > 0)
			{
				// Triangle1 is entirely on positive side of Triangle0 edge.
				return false;
			}
		}

		// Test edges of Triangle1 for separation.
		for (i0 = 0, i1 = 2; i0 < 3; i1 = i0++)
		{
			// Test axis V1[i1] + t*perp(V1[i0]-V1[i1]), perp(x,y) = (y,-x).
			dir.X = Triangle1.V[i0].Y - Triangle1.V[i1].Y;
			dir.Y = Triangle1.V[i1].X - Triangle1.V[i0].X;
			if (WhichSide(Triangle0, Triangle1.V[i1], dir) > 0) {
				// Triangle0 is entirely on positive side of Triangle1 edge.
				return false;
			}
		}

		return true;
	}





	TIntrTriangle2Triangle2* Compute()
	{
		Find();
		return this;
	}


	bool Find()
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result == EIntersectionResult::Intersects);
		}

		// The potential intersection is initialized to Triangle1.  The set of
		// vertices is refined based on clipping against each edge of Triangle0.
		Quantity = 3;
		for (int i = 0; i < 3; ++i)
		{
			Points[i] = Triangle1.V[i];
		}

		for (int i1 = 2, i0 = 0; i0 < 3; i1 = i0++)
		{
			// Clip against edge <V0[i1],V0[i0]>.
			TVector2<Real> N(
				Triangle0.V[i1].Y - Triangle0.V[i0].Y,
				Triangle0.V[i0].X - Triangle0.V[i1].X);
			double c = N.Dot(Triangle0.V[i1]);
			ClipConvexPolygonAgainstLine(N, c, Quantity, Points);
			if (Quantity == 0)
			{
				// Triangle completely clipped, no intersection occurs.
				Type = EIntersectionType::Empty;
			}
			else if (Quantity == 1)
			{
				Type = EIntersectionType::Point;
			}
			else if (Quantity == 2)
			{
				Type = EIntersectionType::Segment;
			}
			else
			{
				Type = EIntersectionType::Polygon;
			}
		}

		Result = (Type != EIntersectionType::Empty) ?
			EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;
		return (Result == EIntersectionResult::Intersects);
	}




	static int WhichSide(const TTriangle2<Real>& V, const TVector2<Real>& P, const TVector2<Real>& D)
	{
		// Vertices are projected to the form P+t*D.  Return value is +1 if all
		// t > 0, -1 if all t < 0, 0 otherwise, in which case the line splits the
		// triangle.

		int positive = 0, negative = 0, zero = 0;
		for (int i = 0; i < 3; ++i)
		{
			double t = D.Dot(V.V[i] - P);
			if (t > (double)0)
			{
				++positive;
			}
			else if (t < (double)0)
			{
				++negative;
			}
			else
			{
				++zero;
			}

			if (positive > 0 && negative > 0)
			{
				return 0;
			}
		}
		return (zero == 0 ? (positive > 0 ? 1 : -1) : 0);
	}


private:
	// quantity, V initially are input polygon vertex count and vertices;
	// on return, they are the clipped polygon vertex count and vertices
	static void ClipConvexPolygonAgainstLine(const TVector2<Real>& N, double c, int& quantity, TVector2<Real> V[6])
	{
		// The input vertices are assumed to be in counterclockwise order.  The
		// ordering is an invariant of this function.

		// Test on which side of line the vertices are.
		int positive = 0, negative = 0, pIndex = -1;
		double test[6];
		int i;
		for (i = 0; i < quantity; ++i)
		{
			test[i] = N.Dot(V[i]) - c;
			if (test[i] > (double)0)
			{
				positive++;
				if (pIndex < 0)
				{
					pIndex = i;
				}
			}
			else if (test[i] < (double)0)
			{
				negative++;
			}
		}

		if (positive > 0)
		{
			if (negative > 0)
			{
				// Line transversely intersects polygon.
				TVector2<Real> CV[6];
				int cQuantity = 0, cur, prv;
				double t;

				if (pIndex > 0)
				{
					// First clip vertex on line.
					cur = pIndex;
					prv = cur - 1;
					t = test[cur] / (test[cur] - test[prv]);
					CV[cQuantity++] = V[cur] + t * (V[prv] - V[cur]);

					// Vertices on positive side of line.
					while (cur < quantity && test[cur] >(double)0)
					{
						CV[cQuantity++] = V[cur++];
					}

					// Last clip vertex on line.
					if (cur < quantity)
					{
						prv = cur - 1;
					}
					else
					{
						cur = 0;
						prv = quantity - 1;
					}
					t = test[cur] / (test[cur] - test[prv]);
					CV[cQuantity++] = V[cur] + t * (V[prv] - V[cur]);
				}
				else  // pIndex is 0
				{
					// Vertices on positive side of line.
					cur = 0;
					while (cur < quantity && test[cur] >(double)0)
					{
						CV[cQuantity++] = V[cur++];
					}

					// Last clip vertex on line.
					prv = cur - 1;
					t = test[cur] / (test[cur] - test[prv]);
					CV[cQuantity++] = V[cur] + t * (V[prv] - V[cur]);

					// Skip vertices on negative side.
					while (cur < quantity && test[cur] <= (double)0)
					{
						++cur;
					}

					// First clip vertex on line.
					if (cur < quantity)
					{
						prv = cur - 1;
						t = test[cur] / (test[cur] - test[prv]);
						CV[cQuantity++] = V[cur] + t * (V[prv] - V[cur]);

						// Vertices on positive side of line.
						while (cur < quantity && test[cur] >(double)0)
						{
							CV[cQuantity++] = V[cur++];
						}
					}
					else
					{
						// cur = 0
						prv = quantity - 1;
						t = test[0] / (test[0] - test[prv]);
						CV[cQuantity++] = V[0] + t * (V[prv] - V[0]);
					}
				}

				quantity = cQuantity;
				for (int Idx = 0; Idx < cQuantity; Idx++)
				{
					V[Idx] = CV[Idx];
				}
			}
			// else polygon fully on positive side of line, nothing to do.
		}
		else
		{
			// Polygon does not intersect positive side of line, clip all.
			quantity = 0;
		}
	}



};

typedef TIntrTriangle2Triangle2<float> FIntrTriangle2Triangle2f;
typedef TIntrTriangle2Triangle2<double> FIntrTriangle2Triangle2d;

} // end namespace UE::Geometry
} // end namespace UE
