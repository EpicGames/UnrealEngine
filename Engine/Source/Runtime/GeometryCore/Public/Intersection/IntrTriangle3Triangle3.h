// Copyright Epic Games, Inc. All Rights Reserved.

// Port of WildMagic TIntrTriangle3Triangle3


#pragma once

#include "VectorTypes.h"
#include "PlaneTypes.h"
#include "TriangleTypes.h"
#include "VectorUtil.h"
#include "IndexTypes.h"

#include "Intersection/IntrSegment2Triangle2.h"
#include "Intersection/IntrTriangle2Triangle2.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Compute intersection between 3D triangles
 * use Test() for fast boolean query, does not compute intersection info
 * use Find() to compute full information
 * By default fully-contained co-planar triangles are not reported as intersecting.
 *  Call SetReportCoplanarIntersection(true) to handle this case (more expensive)
 */
template <typename Real>
class TIntrTriangle3Triangle3
{
protected:
	// Input
	TTriangle3<Real> Triangle0, Triangle1;
	Real Tolerance = TMathUtil<Real>::ZeroTolerance;

	// If true, will return intersection polygons for co-planar triangles.
	// This is somewhat expensive, default is false.
	// Note that when false, co-planar intersections will **NOT** be reported as intersections
	bool bReportCoplanarIntersection = false;

public:
	// Output

	// result flags
	EIntersectionResult Result = EIntersectionResult::NotComputed;
	EIntersectionType Type = EIntersectionType::Empty;

	// intersection points (for point, line, polygon)
	// only first Quantity elements are relevant
	int Quantity = 0;
	TVector<Real> Points[6];

	TIntrTriangle3Triangle3()
	{}
	TIntrTriangle3Triangle3(const TTriangle3<Real>& T0, const TTriangle3<Real>& T1)
	{
		Triangle0 = T0;
		Triangle1 = T1;
	}

	/**
	 * Store an externally-computed segment intersection result
	 */
	inline void SetResult(const TVector<Real>& A, const TVector<Real>& B)
	{
		Result = EIntersectionResult::Intersects;
		Type = EIntersectionType::Segment;
		Points[0] = A;
		Points[1] = B;
		Quantity = 2;
	}

	/**
	 * Store an externally-computed no-intersection result
	 */
	inline void SetResultNone()
	{
		Result = EIntersectionResult::NoIntersection;
		Type = EIntersectionType::Empty;
		Quantity = 0;
	}

	/**
	 * Store an externally-computed binary yes/no result
	 */
	inline void SetResult(bool IsIntersecting)
	{
		Result = IsIntersecting ? EIntersectionResult::Intersects : EIntersectionResult::NoIntersection;
		Type = IsIntersecting ? EIntersectionType::Unknown : EIntersectionType::Empty;
		Quantity = 0;
	}

	TTriangle3<Real> GetTriangle0() const
	{
		return Triangle0;
	}
	TTriangle3<Real> GetTriangle1() const
	{
		return Triangle1;
	}
	bool GetReportCoplanarIntersection() const
	{
		return bReportCoplanarIntersection;
	}
	Real GetTolerance() const
	{
		return Tolerance;
	}
	void SetTriangle0(const TTriangle3<Real>& Triangle0In)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle0 = Triangle0In;
	}
	void SetTriangle1(const TTriangle3<Real>& Triangle1In)
	{
		Result = EIntersectionResult::NotComputed;
		Triangle1 = Triangle1In;
	}
	void SetReportCoplanarIntersection(bool bReportCoplanarIntersectionIn)
	{
		Result = EIntersectionResult::NotComputed;
		bReportCoplanarIntersection = bReportCoplanarIntersectionIn;
	}
	void SetTolerance(Real ToleranceIn)
	{
		Result = EIntersectionResult::NotComputed;
		Tolerance = ToleranceIn;
	}

	TIntrTriangle3Triangle3* Compute()
	{
		Find();
		return this;
	}


	bool Find()
	{
		if (Result != EIntersectionResult::NotComputed)
		{
			return (Result != EIntersectionResult::NoIntersection);
		}


		// in this code the results get initialized in subroutines, so we
		// set the default value here...
		Result = EIntersectionResult::NoIntersection;


		int i, iM, iP;

		// Get the plane of Triangle0.
		TPlane3<Real> Plane0(Triangle0.V[0], Triangle0.V[1], Triangle0.V[2]);

		if (Plane0.Normal == TVector<Real>::Zero())
		{
			// This function was written assuming triangle 0 has a valid normal;
			//  if it's degenerate, try swapping the triangles
			TPlane3<Real> Plane1(Triangle1.V[0], Triangle1.V[1], Triangle1.V[2]);
			if (Plane1.Normal != TVector<Real>::Zero())
			{
				// Find the intersection with the triangles swapped
				TIntrTriangle3Triangle3<Real> SwappedTrisIntr(Triangle1, Triangle0);
				SwappedTrisIntr.Tolerance = Tolerance;
				SwappedTrisIntr.bReportCoplanarIntersection = bReportCoplanarIntersection;
				bool bRes = SwappedTrisIntr.Find();
				// copy results back
				Result = SwappedTrisIntr.Result;
				Type = SwappedTrisIntr.Type;
				Quantity = SwappedTrisIntr.Quantity;
				for (int PtIdx = 0; PtIdx < 6; PtIdx++)
				{
					Points[PtIdx] = SwappedTrisIntr.Points[PtIdx];
				}
				return bRes;
			}
			else
			{
				// both triangles degenerate; we don't handle this
				return false;
			}
		}

		// Compute the signed distances of Triangle1 vertices to Plane0.  Use
		// an epsilon-thick plane test.
		int pos1, neg1, zero1;
		FIndex3i sign1;
		TVector<Real> dist1;
		TrianglePlaneRelations(Triangle1, Plane0, dist1, sign1, pos1, neg1, zero1, Tolerance);

		if (pos1 == 3 || neg1 == 3)
		{
			// Triangle1 is fully on one side of Plane0.
			return false;
		}

		if (zero1 == 3)
		{
			// Triangle1 is contained by Plane0.
			if (bReportCoplanarIntersection)
			{
				return GetCoplanarIntersection(Plane0, Triangle0, Triangle1);
			}
			return false;
		}

		// Check for grazing contact between Triangle1 and Plane0.
		if (pos1 == 0 || neg1 == 0)
		{
			if (zero1 == 2)
			{
				// An edge of Triangle1 is in Plane0.
				for (i = 0; i < 3; ++i)
				{
					if (sign1[i] != 0)
					{
						iM = (i + 2) % 3;
						iP = (i + 1) % 3;
						return IntersectsSegment(Plane0, Triangle0, Triangle1.V[iM], Triangle1.V[iP]);
					}
				}
			}
			else // zero1 == 1
			{
			 // A vertex of Triangle1 is in Plane0.
				for (i = 0; i < 3; ++i)
				{
					if (sign1[i] == 0)
					{
						return ContainsPoint(Triangle0, Plane0, Triangle1.V[i]);
					}
				}
			}
		}

		// At this point, Triangle1 transversely intersects plane 0.  Compute the
		// line segment of intersection.  Then test for intersection between this
		// segment and triangle 0.
		Real t;
		TVector<Real> intr0, intr1;
		if (zero1 == 0)
		{
			int iSign = (pos1 == 1 ? +1 : -1);
			for (i = 0; i < 3; ++i)
			{
				if (sign1[i] == iSign)
				{
					iM = (i + 2) % 3;
					iP = (i + 1) % 3;
					t = dist1[i] / (dist1[i] - dist1[iM]);
					intr0 = Triangle1.V[i] + t * (Triangle1.V[iM] - Triangle1.V[i]);
					t = dist1[i] / (dist1[i] - dist1[iP]);
					intr1 = Triangle1.V[i] + t * (Triangle1.V[iP] - Triangle1.V[i]);
					return IntersectsSegment(Plane0, Triangle0, intr0, intr1);
				}
			}
		}

		// zero1 == 1
		for (i = 0; i < 3; ++i)
		{
			if (sign1[i] == 0)
			{
				iM = (i + 2) % 3;
				iP = (i + 1) % 3;
				t = dist1[iM] / (dist1[iM] - dist1[iP]);
				intr0 = Triangle1.V[iM] + t * (Triangle1.V[iP] - Triangle1.V[iM]);
				return IntersectsSegment(Plane0, Triangle0, Triangle1.V[i], intr0);
			}
		}

		// should never get here...
		ensure(false);
		return false;
	}


	bool Test()
	{
		// Get edge vectors for Triangle0.
		TVector<Real> E0[3];
		E0[0] = Triangle0.V[1] - Triangle0.V[0];
		E0[1] = Triangle0.V[2] - Triangle0.V[1];
		E0[2] = Triangle0.V[0] - Triangle0.V[2];

		// Get normal vector of Triangle0.
		TVector<Real> N0 = UnitCross(E0[0], E0[1]);

		// Project Triangle1 onto normal line of Triangle0, test for separation.
		Real N0dT0V0 = N0.Dot(Triangle0.V[0]);
		Real min1, max1;
		ProjectOntoAxis(Triangle1, N0, min1, max1);
		if (N0dT0V0 < min1 - Tolerance || N0dT0V0 > max1 + Tolerance)
		{
			return false;
		}

		// Get edge vectors for Triangle1.
		TVector<Real> E1[3];
		E1[0] = Triangle1.V[1] - Triangle1.V[0];
		E1[1] = Triangle1.V[2] - Triangle1.V[1];
		E1[2] = Triangle1.V[0] - Triangle1.V[2];

		// Get normal vector of Triangle1.
		TVector<Real> N1 = UnitCross(E1[0], E1[1]);

		TVector<Real> dir;
		Real min0, max0;
		int i0, i1;

		TVector<Real> N0xN1 = UnitCross(N0, N1);
		if (N0xN1.Dot(N0xN1) >= Tolerance)
		{
			// Triangles are not parallel.

			// Project Triangle0 onto normal line of Triangle1, test for
			// separation.
			Real N1dT1V0 = N1.Dot(Triangle1.V[0]);
			ProjectOntoAxis(Triangle0, N1, min0, max0);
			if (N1dT1V0 < min0 - Tolerance || N1dT1V0 > max0 + Tolerance)
			{
				return false;
			}

			// Directions E0[i0]xE1[i1].
			for (i1 = 0; i1 < 3; ++i1)
			{
				for (i0 = 0; i0 < 3; ++i0)
				{
					dir = UnitCross(E0[i0], E1[i1]);
					ProjectOntoAxis(Triangle0, dir, min0, max0);
					ProjectOntoAxis(Triangle1, dir, min1, max1);
					if (max0 < min1 - Tolerance || max1 < min0 - Tolerance)
					{
						return false;
					}
				}
			}

			// The test query does not know the intersection set.
			Type = EIntersectionType::Unknown;
		}
		else  // Triangles are parallel (and, in fact, coplanar).
		{ 
			if (!bReportCoplanarIntersection)
			{
				return false;
			}

			// Directions N0xE0[i0].
			for (i0 = 0; i0 < 3; ++i0)
			{
				dir = UnitCross(N0, E0[i0]);
				ProjectOntoAxis(Triangle0, dir, min0, max0);
				ProjectOntoAxis(Triangle1, dir, min1, max1);
				if (max0 < min1 - Tolerance || max1 < min0 - Tolerance)
				{
					return false;
				}
			}

			// Directions N1xE1[i1].
			for (i1 = 0; i1 < 3; ++i1)
			{
				dir = UnitCross(N1, E1[i1]);
				ProjectOntoAxis(Triangle0, dir, min0, max0);
				ProjectOntoAxis(Triangle1, dir, min1, max1);
				if (max0 < min1 - Tolerance || max1 < min0 - Tolerance)
				{
					return false;
				}
			}

			// The test query does not know the intersection set.
			Type = EIntersectionType::Plane;
		}

		return true;
	}



	static bool Intersects(const TTriangle3<Real>& Triangle0, const TTriangle3<Real>& Triangle1, Real Tolerance = TMathUtil<Real>::ZeroTolerance)
	{
		// Get edge vectors for Triangle0.
		TVector<Real> E0[3];
		E0[0] = Triangle0.V[1] - Triangle0.V[0];
		E0[1] = Triangle0.V[2] - Triangle0.V[1];
		E0[2] = Triangle0.V[0] - Triangle0.V[2];

		// Get normal vector of Triangle0.
		TVector<Real> N0 = UnitCross(E0[0], E0[1]);

		// Project Triangle1 onto normal line of Triangle0, test for separation.
		Real N0dT0V0 = N0.Dot(Triangle0.V[0]);
		Real min1, max1;
		ProjectOntoAxis(Triangle1, N0, min1, max1);
		if (N0dT0V0 < min1-Tolerance || N0dT0V0 > max1+Tolerance) {
			return false;
		}

		// Get edge vectors for Triangle1.
		TVector<Real> E1[3];
		E1[0] = Triangle1.V[1] - Triangle1.V[0];
		E1[1] = Triangle1.V[2] - Triangle1.V[1];
		E1[2] = Triangle1.V[0] - Triangle1.V[2];

		// Get normal vector of Triangle1.
		TVector<Real> N1 = UnitCross(E1[0], E1[1]);

		TVector<Real> dir;
		Real min0, max0;
		int i0, i1;

		TVector<Real> N0xN1 = UnitCross(N0, N1);
		if (N0xN1.Dot(N0xN1) >= Tolerance) {
			// Triangles are not parallel.

			// Project Triangle0 onto normal line of Triangle1, test for
			// separation.
			Real N1dT1V0 = N1.Dot(Triangle1.V[0]);
			ProjectOntoAxis(Triangle0, N1, min0, max0);
			if (N1dT1V0 < min0 - Tolerance || N1dT1V0 > max0 + Tolerance) {
				return false;
			}

			// Directions E0[i0]xE1[i1].
			for (i1 = 0; i1 < 3; ++i1) {
				for (i0 = 0; i0 < 3; ++i0) {
					dir = UnitCross(E0[i0], E1[i1]);
					ProjectOntoAxis(Triangle0, dir, min0, max0);
					ProjectOntoAxis(Triangle1, dir, min1, max1);
					if (max0 < min1 - Tolerance || max1 < min0 - Tolerance) {
						return false;
					}
				}
			}

		}
		else { // Triangles are parallel (and, in fact, coplanar).
		 // Directions N0xE0[i0].
			for (i0 = 0; i0 < 3; ++i0) {
				dir = UnitCross(N0, E0[i0]);
				ProjectOntoAxis(Triangle0, dir, min0, max0);
				ProjectOntoAxis(Triangle1, dir, min1, max1);
				if (max0 < min1 - Tolerance || max1 < min0 - Tolerance) {
					return false;
				}
			}

			// Directions N1xE1[i1].
			for (i1 = 0; i1 < 3; ++i1) {
				dir = UnitCross(N1, E1[i1]);
				ProjectOntoAxis(Triangle0, dir, min0, max0);
				ProjectOntoAxis(Triangle1, dir, min1, max1);
				if (max0 < min1 - Tolerance || max1 < min0 - Tolerance) {
					return false;
				}
			}
		}

		return true;
	}




	static void ProjectOntoAxis(const TTriangle3<Real>& triangle, const TVector<Real>& axis, Real& fmin, Real& fmax)
	{
		Real dot0 = axis.Dot(triangle.V[0]);
		Real dot1 = axis.Dot(triangle.V[1]);
		Real dot2 = axis.Dot(triangle.V[2]);

		fmin = dot0;
		fmax = fmin;

		if (dot1 < fmin)
		{
			fmin = dot1;
		}
		else if (dot1 > fmax)
		{
			fmax = dot1;
		}

		if (dot2 < fmin)
		{
			fmin = dot2;
		}
		else if (dot2 > fmax)
		{
			fmax = dot2;
		}
	}



	static void TrianglePlaneRelations(const TTriangle3<Real>& triangle, const TPlane3<Real>& plane,
									   TVector<Real>& distance, FIndex3i& sign, int& positive, int& negative, int& zero,
									   Real Tolerance)
	{
		// Compute the signed distances of triangle vertices to the plane.  Use
		// an epsilon-thick plane test.
		positive = 0;
		negative = 0;
		zero = 0;
		for (int i = 0; i < 3; ++i)
		{
			distance[i] = plane.DistanceTo(triangle.V[i]);
			if (distance[i] > Tolerance)
			{
				sign[i] = 1;
				positive++;
			}
			else if (distance[i] < -Tolerance)
			{
				sign[i] = -1;
				negative++;
			}
			else
			{
				sign[i] = 0;
				zero++;
			}
		}
	}


	/**
	 * Solve a common sub-problem for triangle-triangle intersection --
	 *   find the sub-segment (or point) where a triangle intersects a coplanar segment
	 *
	 * @param plane The plane the triangle is on
	 * @param triangle The triangle to intersect
	 * @param end0 First point of line segment
	 * @param end1 Second point of line segment
	 * @param OutA First point of intersection between line segment and triangle (if any)
	 * @param OutB Second point of intersection between line segment and triangle (if any)
	 * @param Tolerance Tolerance to use for segment-triangle intersection
	 * @return Number of points representing the intersection result (0 for none, 1 for point, 2 for segment)
	 */
	static int IntersectTriangleWithCoplanarSegment(
		const TPlane3<Real>& plane, const TTriangle3<Real>& triangle, const TVector<Real>& end0, const TVector<Real>& end1,
		TVector<Real>& OutA, TVector<Real>& OutB, Real Tolerance)
	{
		// Compute the 2D representations of the triangle vertices and the
		// segment endpoints relative to the plane of the triangle.  Then
		// compute the intersection in the 2D space.

		// Project the triangle and segment onto the coordinate plane most
		// aligned with the plane normal.
		int maxNormal = 0;
		Real fmax = FMath::Abs(plane.Normal.X);
		Real absMax = FMath::Abs(plane.Normal.Y);
		if (absMax > fmax)
		{
			maxNormal = 1;
			fmax = absMax;
		}
		absMax = FMath::Abs(plane.Normal.Z);
		if (absMax > fmax) {
			maxNormal = 2;
		}

		TTriangle2<Real> projTri;
		TVector2<Real> projEnd0, projEnd1;
		int i;

		if (maxNormal == 0)
		{
			// Project onto yz-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri.V[i] = GetYZ(triangle.V[i]);
			}
			projEnd0.X = end0.Y;
			projEnd0.Y = end0.Z;
			projEnd1.X = end1.Y;
			projEnd1.Y = end1.Z;
		}
		else if (maxNormal == 1)
		{
			// Project onto xz-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri.V[i] = GetXZ(triangle.V[i]);
			}
			projEnd0.X = end0.X;
			projEnd0.Y = end0.Z;
			projEnd1.X = end1.X;
			projEnd1.Y = end1.Z;
		}
		else
		{
			// Project onto xy-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri.V[i] = GetXY(triangle.V[i]);
			}
			projEnd0.X = end0.X;
			projEnd0.Y = end0.Y;
			projEnd1.X = end1.X;
			projEnd1.Y = end1.Y;
		}

		TSegment2<Real> projSeg(projEnd0, projEnd1);
		TIntrSegment2Triangle2<Real> calc(projSeg, projTri);
		if (!calc.Find(Tolerance))
		{
			return 0;
		}

		int Quantity = 0;

		TVector2<Real> intr[2];
		if (calc.Type == EIntersectionType::Segment)
		{
			Quantity = 2;
			intr[0] = calc.Point0;
			intr[1] = calc.Point1;
		}
		else
		{
			checkSlow(calc.Type == EIntersectionType::Point);
			//"Intersection must be a point\n";
			Quantity = 1;
			intr[0] = calc.Point0;
		}

		TVector<Real>* OutPts[2]{ &OutA, &OutB };

		// Unproject the segment of intersection.
		if (maxNormal == 0)
		{
			Real invNX = ((Real)1) / plane.Normal.X;
			for (i = 0; i < Quantity; ++i)
			{
				Real y = intr[i].X;
				Real z = intr[i].Y;
				Real x = invNX * (plane.Constant - plane.Normal.Y * y - plane.Normal.Z * z);
				*OutPts[i] = TVector<Real>(x, y, z);
			}
		}
		else if (maxNormal == 1)
		{
			Real invNY = ((Real)1) / plane.Normal.Y;
			for (i = 0; i < Quantity; ++i)
			{
				Real x = intr[i].X;
				Real z = intr[i].Y;
				Real y = invNY * (plane.Constant - plane.Normal.X * x - plane.Normal.Z * z);
				*OutPts[i] = TVector<Real>(x, y, z);
			}
		}
		else
		{
			Real invNZ = ((Real)1) / plane.Normal.Z;
			for (i = 0; i < Quantity; ++i)
			{
				Real x = intr[i].X;
				Real y = intr[i].Y;
				Real z = invNZ * (plane.Constant - plane.Normal.X * x - plane.Normal.Y * y);
				*OutPts[i] = TVector<Real>(x, y, z);
			}
		}

		return Quantity;
	}


protected:


	bool ContainsPoint(const TTriangle3<Real>& triangle, const TPlane3<Real>& plane, const TVector<Real>& point)
	{
		// Generate a coordinate system for the plane.  The incoming triangle has
		// vertices <V0,V1,V2>.  The incoming plane has unit-length normal N.
		// The incoming point is P.  V0 is chosen as the origin for the plane. The
		// coordinate axis directions are two unit-length vectors, U0 and U1,
		// constructed so that {U0,U1,N} is an orthonormal set.  Any point Q
		// in the plane may be written as Q = V0 + x0*U0 + x1*U1.  The coordinates
		// are computed as x0 = Dot(U0,Q-V0) and x1 = Dot(U1,Q-V0).
		TVector<Real> U0, U1;
		VectorUtil::MakePerpVectors(plane.Normal, U0, U1);

		// Compute the planar coordinates for the points P, V1, and V2.  To
		// simplify matters, the origin is subtracted from the points, in which
		// case the planar coordinates are for P-V0, V1-V0, and V2-V0.
		TVector<Real> PmV0 = point - triangle.V[0];
		TVector<Real> V1mV0 = triangle.V[1] - triangle.V[0];
		TVector<Real> V2mV0 = triangle.V[2] - triangle.V[0];

		// The planar representation of P-V0.
		TVector2<Real> ProjP(U0.Dot(PmV0), U1.Dot(PmV0));

		// The planar representation of the triangle <V0-V0,V1-V0,V2-V0>.
		TTriangle2<Real> ProjT(TVector2<Real>::Zero(), TVector2<Real>(U0.Dot(V1mV0), U1.Dot(V1mV0)), TVector2<Real>(U0.Dot(V2mV0), U1.Dot(V2mV0)));

		// Test whether P-V0 is in the triangle <0,V1-V0,V2-V0>.
		if (ProjT.IsInsideOrOn_Oriented(ProjP) <= 0)
		{
			Result = EIntersectionResult::Intersects;
			Type = EIntersectionType::Point;
			Quantity = 1;
			Points[0] = point;
			return true;
		}

		return false;
	}


	bool IntersectsSegment(const TPlane3<Real>& plane, const TTriangle3<Real>& triangle, const TVector<Real>& end0, const TVector<Real>& end1)
	{
		Quantity = IntersectTriangleWithCoplanarSegment(plane, triangle, end0, end1, Points[0], Points[1], Tolerance);
		if (Quantity > 0)
		{
			Result = EIntersectionResult::Intersects;
			Type = Quantity == 2 ? EIntersectionType::Segment : EIntersectionType::Point;
			return true;
		}
		else
		{
			Result = EIntersectionResult::NoIntersection;
			Type = EIntersectionType::Empty;
			return false;
		}
	}




	bool GetCoplanarIntersection(const TPlane3<Real>& plane, const TTriangle3<Real>& tri0, const TTriangle3<Real>& tri1)
	{
		// Project triangles onto coordinate plane most aligned with plane
		// normal.
		int maxNormal = 0;
		Real fmax = FMath::Abs(plane.Normal.X);
		Real absMax = FMath::Abs(plane.Normal.Y);
		if (absMax > fmax)
		{
			maxNormal = 1;
			fmax = absMax;
		}
		absMax = FMath::Abs(plane.Normal.Z);
		if (absMax > fmax)
		{
			maxNormal = 2;
		}

		TTriangle2<Real> projTri0, projTri1;
		int i;

		if (maxNormal == 0)
		{
			// Project onto yz-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri0.V[i] = GetYZ(tri0.V[i]);
				projTri1.V[i] = GetYZ(tri1.V[i]);
			}
		}
		else if (maxNormal == 1)
		{
			// Project onto xz-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri0.V[i] = GetXZ(tri0.V[i]);
				projTri1.V[i] = GetXZ(tri1.V[i]);
			}
		}
		else
		{
			// Project onto xy-plane.
			for (i = 0; i < 3; ++i)
			{
				projTri0.V[i] = GetXY(tri0.V[i]);
				projTri1.V[i] = GetXY(tri1.V[i]);
			}
		}

		// 2D triangle intersection routines require counterclockwise ordering.
		TVector2<Real> save;
		TVector2<Real> edge0 = projTri0.V[1] - projTri0.V[0];
		TVector2<Real> edge1 = projTri0.V[2] - projTri0.V[0];
		if (DotPerp(edge0, edge1) < (Real)0)
		{
			// Triangle is clockwise, reorder it.
			save = projTri0.V[1];
			projTri0.V[1] = projTri0.V[2];
			projTri0.V[2] = save;
		}

		edge0 = projTri1.V[1] - projTri1.V[0];
		edge1 = projTri1.V[2] - projTri1.V[0];
		if (DotPerp(edge0, edge1) < (Real)0)
		{
			// Triangle is clockwise, reorder it.
			save = projTri1.V[1];
			projTri1.V[1] = projTri1.V[2];
			projTri1.V[2] = save;
		}

		TIntrTriangle2Triangle2<Real> intr(projTri0, projTri1);
		if (!intr.Find()) // TODO: pass tolerance through?  note this is sos'd currently so it doesn't have a tolerance concept
		{
			return false;
		}

		// Map 2D intersections back to the 3D triangle space.
		Quantity = intr.Quantity;
		if (maxNormal == 0)
		{
			Real invNX = ((Real)1) / plane.Normal.X;
			for (i = 0; i < Quantity; i++)
			{
				Real y = intr.Points[i].X;
				Real z = intr.Points[i].Y;
				Real x = invNX * (plane.Constant - plane.Normal.Y * y - plane.Normal.Z * z);
				Points[i] = TVector<Real>(x, y, z);
			}
		}
		else if (maxNormal == 1)
		{
			Real invNY = ((Real)1) / plane.Normal.Y;
			for (i = 0; i < Quantity; i++)
			{
				Real x = intr.Points[i].X;
				Real z = intr.Points[i].Y;
				Real y = invNY * (plane.Constant - plane.Normal.X * x - plane.Normal.Z * z);
				Points[i] = TVector<Real>(x, y, z);
			}
		}
		else
		{
			Real invNZ = ((Real)1) / plane.Normal.Z;
			for (i = 0; i < Quantity; i++)
			{
				Real x = intr.Points[i].X;
				Real y = intr.Points[i].Y;
				Real z = invNZ * (plane.Constant - plane.Normal.X * x - plane.Normal.Y * y);
				Points[i] = TVector<Real>(x, y, z);
			}
		}

		Result = EIntersectionResult::Intersects;
		Type = EIntersectionType::Polygon;
		return true;
	}







};

typedef TIntrTriangle3Triangle3<float> FIntrTriangle3Triangle3f;
typedef TIntrTriangle3Triangle3<double> FIntrTriangle3Triangle3d;

} // end namespace UE::Geometry
} // end namespace UE
