// Copyright Epic Games, Inc. All Rights Reserved.

// line types ported from WildMagic and geometry3Sharp

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TLine2 is a two-dimensional infinite line.
 * The line is stored in (Center,Direction) form.
 */
template<typename T>
struct TLine2
{
	/** Origin / Center Point of Line */
	TVector2<T> Origin;

	/** Direction of Line, Normalized */
	TVector2<T> Direction;

	/**
	 * Construct default line along X axis
	 */
	TLine2()
	{
		Origin = TVector2<T>::Zero();
		Direction = TVector2<T>::UnitX();
	}

	/**
	 * Construct line with given Origin and Direction
	 */
	TLine2(const TVector2<T>& OriginIn, const TVector2<T>& DirectionIn)
		: Origin(OriginIn), Direction(DirectionIn)
	{
	}


	/**
	 * @return line between two points
	 */
	static TLine2<T> FromPoints(const TVector2<T>& Point0, const TVector2<T>& Point1)
	{
		return TLine2<T>(Point0, Normalized(Point1 - Point0) );
	}


	/**
	 * @return point on line at given line parameter value (distance along line from origin)
	 */
	inline TVector2<T> PointAt(T LineParameter) const
	{
		return Origin + LineParameter * Direction;
	}


	/**
	 * @return line parameter (ie distance from Origin) at nearest point on line to QueryPoint
	 */
	inline T Project(const TVector2<T>& QueryPoint) const
	{
		return (QueryPoint - Origin).Dot(Direction);
	}

	/**
	 * @return smallest squared distance from line to QueryPoint
	 */
	inline T DistanceSquared(const TVector2<T>& QueryPoint) const
	{
		T ParameterT = (QueryPoint - Origin).Dot(Direction);
		TVector2<T> proj = Origin + ParameterT * Direction;
		return (proj - QueryPoint).SquaredLength();
	}

	/**
	 * @return nearest point on line to QueryPoint
	 */
	inline TVector2<T> NearestPoint(const TVector2<T>& QueryPoint) const
	{
		T ParameterT = (QueryPoint - Origin).Dot(Direction);
		return Origin + ParameterT * Direction;
	}


	/**
	 * @return +1 if QueryPoint is "right" of line, -1 if "left" or 0 if "on" line (up to given tolerance)
	 */
	inline int WhichSide(const TVector2<T>& QueryPoint, T OnLineTolerance = 0) const
	{
		T x0 = QueryPoint.X - Origin.X;
		T y0 = QueryPoint.Y - Origin.Y;
		T x1 = Direction.X;
		T y1 = Direction.Y;
		T det = x0 * y1 - x1 * y0;
		return (det > OnLineTolerance ? +1 : (det < -OnLineTolerance ? -1 : 0));
	}


	/**
	 * Calculate intersection point between this line and another one
	 * @param OtherLine line to test against
	 * @param IntersectionPointOut intersection point is stored here, if found
	 * @param ParallelDotTolerance tolerance used to determine if lines are parallel (and hence cannot intersect)
	 * @return true if lines intersect and IntersectionPointOut was computed
	 */
	bool IntersectionPoint(const TLine2<T>& OtherLine, TVector2<T>& IntersectionPointOut, T ParallelDotTolerance = TMathUtil<T>::ZeroTolerance) const
	{
		// see IntrTLine2TLine2 for more detailed explanation
		TVector2<T> diff = OtherLine.Origin - Origin;
		T D0DotPerpD1 = DotPerp(Direction, OtherLine.Direction);
		if (TMathUtil<T>::Abs(D0DotPerpD1) > ParallelDotTolerance)                     // TLines intersect in a single point.
		{
			T invD0DotPerpD1 = ((T)1) / D0DotPerpD1;
			T diffDotPerpD1 = DotPerp(diff, OtherLine.Direction);
			T s = diffDotPerpD1 * invD0DotPerpD1;
			IntersectionPointOut = Origin + s * Direction;
			return true;
		}
		// TLines are parallel.
		return false;
	}
};


typedef TLine2<double> FLine2d;
typedef TLine2<float> FLine2f;






/**
 * TLine3 is a three-dimensional infinite line.
 * The line is stored in (Center,Direction) form.
 */
template<typename T>
struct TLine3
{
	/** Origin / Center Point of Line */
	TVector<T> Origin;

	/** Direction of Line, Normalized */
	TVector<T> Direction;

	/**
	 * Construct default line along X axis
	 */
	TLine3()
	{
		Origin = TVector<T>::Zero();
		Direction = TVector<T>::UnitX();
	}

	/**
	 * Construct line with given Origin and Direction
	 */
	TLine3(const TVector<T>& OriginIn, const TVector<T>& DirectionIn)
		: Origin(OriginIn), Direction(DirectionIn)
	{
	}


	/**
	 * @return line between two points
	 */
	static TLine3<T> FromPoints(const TVector<T>& Point0, const TVector<T>& Point1)
	{
		return TLine3<T>(Point0, Normalized(Point1 - Point0) );
	}


	/**
	 * @return point on line at given line parameter value (distance along line from origin)
	 */
	inline TVector<T> PointAt(T LineParameter) const
	{
		return Origin + LineParameter * Direction;
	}


	/**
	 * @return line parameter (ie distance from Origin) at nearest point on line to QueryPoint
	 */
	inline T Project(const TVector<T>& QueryPoint) const
	{
		return (QueryPoint - Origin).Dot(Direction);
	}

	/**
	 * @return smallest squared distance from line to QueryPoint
	 */
	inline T DistanceSquared(const TVector<T>& QueryPoint) const
	{
		T t = (QueryPoint - Origin).Dot(Direction);
		TVector<T> proj = Origin + t * Direction;
		return (proj - QueryPoint).SquaredLength();
	}

	/**
	 * @return nearest point on line to QueryPoint
	 */
	inline TVector<T> NearestPoint(const TVector<T>& QueryPoint) const
	{
		T ParameterT = (QueryPoint - Origin).Dot(Direction);
		return Origin + ParameterT * Direction;
	}


};

typedef TLine3<double> FLine3d;
typedef TLine3<float> FLine3f;

} // end namespace UE::Geometry
} // end namespace UE

