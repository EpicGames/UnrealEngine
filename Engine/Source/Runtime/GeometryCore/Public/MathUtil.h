// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBase.h"
#include "HAL/Platform.h"
#include "EngineDefines.h"

#include <cmath>
#include <cfloat>

/**
 * Math constants and utility functions, templated on float/double type
 */
template<typename RealType>
struct TMathUtilConstants;

template<>
struct TMathUtilConstants<float>
{
	/** Machine Epsilon - float approx 1e-7, double approx 2e-16 */
	static constexpr float Epsilon = FLT_EPSILON;
	/** Zero tolerance for math operations (eg like parallel tests) - float 1e-6, double 1e-8 */
	static constexpr float ZeroTolerance = 1e-06f;

	/** largest possible number for type */
	static constexpr float MaxReal = FLT_MAX;

	/** a very large value, but not too close to the max possible for the type */
	static constexpr float SafeLargeValue = UE_LARGE_WORLD_MAX;

	/** 3.14159... */
	static constexpr float Pi = 3.1415926535897932384626433832795f;
	static constexpr float FourPi = 4.0f * Pi;
	static constexpr float TwoPi = 2.0f*Pi;
	static constexpr float HalfPi = 0.5f*Pi;

	/** 1.0 / Pi */
	static constexpr float InvPi = 1.0f / Pi;
	/** 1.0 / (2*Pi) */
	static constexpr float InvTwoPi = 1.0f / TwoPi;

	/** pi / 180 */
	static constexpr float DegToRad = Pi / 180.0f;
	/** 180 / pi */
	static constexpr float RadToDeg = 180.0f / Pi;

	//static constexpr float LN_2;
	//static constexpr float LN_10;
	//static constexpr float INV_LN_2;
	//static constexpr float INV_LN_10;

	static constexpr float Sqrt2 = 1.4142135623730950488016887242097f;
	static constexpr float InvSqrt2 = 1.0f / Sqrt2;
	static constexpr float Sqrt3 = 1.7320508075688772935274463415059f;
	static constexpr float InvSqrt3 = 1.0f / Sqrt3;
};

template<>
struct TMathUtilConstants<double>
{
	/** Machine Epsilon - float approx 1e-7, double approx 2e-16 */
	static constexpr double Epsilon = DBL_EPSILON;
	/** Zero tolerance for math operations (eg like parallel tests) - float 1e-6, double 1e-8 */
	static constexpr double ZeroTolerance = 1e-08;

	/** largest possible number for type */
	static constexpr double MaxReal = DBL_MAX;

	/** a very large value, but not too close to the max possible for the type */
	static constexpr double SafeLargeValue = (double)FLT_MAX;

	/** 3.14159... */
	static constexpr double Pi = 3.1415926535897932384626433832795;
	static constexpr double FourPi = 4.0 * Pi;
	static constexpr double TwoPi = 2.0 * Pi;
	static constexpr double HalfPi = 0.5 * Pi;

	/** 1.0 / Pi */
	static constexpr double InvPi = 1.0 / Pi;
	/** 1.0 / (2*Pi) */
	static constexpr double InvTwoPi = 1.0 / TwoPi;

	/** pi / 180 */
	static constexpr double DegToRad = Pi / 180.0;
	/** 180 / pi */
	static constexpr double RadToDeg = 180.0 / Pi;

	//static constexpr double LN_2;
	//static constexpr double LN_10;
	//static constexpr double INV_LN_2;
	//static constexpr double INV_LN_10;

	static constexpr double Sqrt2 = 1.4142135623730950488016887242097;
	static constexpr double InvSqrt2 = 1.0 / Sqrt2;
	static constexpr double Sqrt3 = 1.7320508075688772935274463415059;
	static constexpr double InvSqrt3 = 1.0 / Sqrt3;
};


// we use TMathUtil<int> so we need to define these nonsense constants
template<>
struct TMathUtilConstants<int32>
{
	static constexpr int32 Epsilon = 0;
	static constexpr int32 ZeroTolerance = 0;
	static constexpr int32 MaxReal = MAX_int32;
	static constexpr int32 SafeLargeValue = MAX_int32/2;
	static constexpr int32 Pi = 3;
	static constexpr int32 FourPi = 4 * Pi;
	static constexpr int32 TwoPi = 2 * Pi;
	static constexpr int32 HalfPi = 1;
	static constexpr int32 InvPi = 1;
	static constexpr int32 InvTwoPi = 1;
	static constexpr int32 DegToRad = 1;
	static constexpr int32 RadToDeg = 1;
	static constexpr int32 Sqrt2 = 1;
	static constexpr int32 InvSqrt2 = 1;
	static constexpr int32 Sqrt3 = 2;
	static constexpr int32 InvSqrt3 = 1;
};


// we use TMathUtil<int> so we need to define these nonsense constants
template<>
struct TMathUtilConstants<int64>
{
	static constexpr int64 Epsilon = 0;
	static constexpr int64 ZeroTolerance = 0;
	static constexpr int64 MaxReal = MAX_int64;
	static constexpr int64 SafeLargeValue = MAX_int64/2;
	static constexpr int64 Pi = 3;
	static constexpr int64 FourPi = 4 * Pi;
	static constexpr int64 TwoPi = 2 * Pi;
	static constexpr int64 HalfPi = 1;
	static constexpr int64 InvPi = 1;
	static constexpr int64 InvTwoPi = 1;
	static constexpr int64 DegToRad = 1;
	static constexpr int64 RadToDeg = 1;
	static constexpr int64 Sqrt2 = 1;
	static constexpr int64 InvSqrt2 = 1;
	static constexpr int64 Sqrt3 = 2;
	static constexpr int64 InvSqrt3 = 1;
};



template<typename RealType>
class TMathUtil : public TMathUtilConstants<RealType>
{
public:
	static inline bool IsNaN(const RealType Value);
	static inline bool IsFinite(const RealType Value);
	static inline RealType Abs(const RealType Value);
	static inline RealType Clamp(const RealType Value, const RealType ClampMin, const RealType ClampMax);
	static inline RealType Sign(const RealType Value);
	static inline int32 SignAsInt(const RealType Value);
	static inline RealType SignNonZero(const RealType Value);
	static inline RealType Max(const RealType A, const RealType B);
	static inline RealType Max3(const RealType A, const RealType B, const RealType C);
	static inline int32 Max3Index(const RealType A, const RealType B, const RealType C);
	static inline RealType Min(const RealType A, const RealType B);
	static inline RealType Min3(const RealType A, const RealType B, const RealType C);
	static inline int32 Min3Index(const RealType A, const RealType B, const RealType C);
	/** compute min and max of a,b,c with max 3 comparisons (sometimes 2) */
	static inline void MinMax(RealType A, RealType B, RealType C, RealType& MinOut, RealType& MaxOut);
	/** Uses a quadratic blend to smooth the min transition when A and B are less than BlendExtent apart. */
	static inline RealType SmoothMin(RealType A, RealType B, RealType BlendExtent);
	/** Uses a quadratic blend to smooth the max transition when A and B are less than BlendExtent apart. */
	static inline RealType SmoothMax(RealType A, RealType B, RealType BlendExtent);
	static inline RealType Sqrt(const RealType Value);
	/** Compute cube root */
	static inline RealType Cbrt(const RealType Value);
	static inline RealType Tan(const RealType Value);
	static inline RealType Atan2(const RealType ValueY, const RealType ValueX);
	static inline RealType Sin(const RealType Value);
	static inline RealType Cos(const RealType Value);
	static inline RealType ACos(const RealType Value);
	static inline RealType Floor(const RealType Value);
	static inline RealType Ceil(const RealType Value);
	static inline RealType Round(const RealType Value);
	static inline RealType Pow(const RealType Value, const RealType Power);
	static inline RealType Exp(const RealType Power);
	static inline RealType Log(const RealType Value);
	static inline RealType Lerp(const RealType A, const RealType B, RealType Alpha);


	/**
	 * @return result of Atan2 shifted to [0,2pi]  (normal ATan2 returns in range [-pi,pi])
	 */
	static inline RealType Atan2Positive(const RealType Y, const RealType X);

private:
	TMathUtil() = delete;
};
typedef TMathUtil<float> FMathf;
typedef TMathUtil<double> FMathd;


template<typename RealType>
bool TMathUtil<RealType>::IsNaN(const RealType Value)
{
	return std::isnan(Value);
}


template<typename RealType>
bool TMathUtil<RealType>::IsFinite(const RealType Value)
{
	return std::isfinite(Value);
}


template<typename RealType>
RealType TMathUtil<RealType>::Abs(const RealType Value)
{
	return (Value >= (RealType)0) ? Value : -Value;
}


template<typename RealType>
RealType TMathUtil<RealType>::Clamp(const RealType Value, const RealType ClampMin, const RealType ClampMax)
{
	return (Value < ClampMin) ? ClampMin : ((Value > ClampMax) ? ClampMax : Value);
}

template<typename RealType>
int32 TMathUtil<RealType>::SignAsInt(const RealType Value)
{
	return (Value > (RealType)0) ? 1 : ((Value < (RealType)0) ? -1 : 0);
}

template<typename RealType>
RealType TMathUtil<RealType>::Sign(const RealType Value)
{
	return (RealType)SignAsInt(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::SignNonZero(const RealType Value)
{
	return (Value < (RealType)0) ? (RealType)-1 : (RealType)1;
}

template<typename RealType>
RealType TMathUtil<RealType>::Max(const RealType A, const RealType B)
{
	return (A >= B) ? A : B;
}

template<typename RealType>
RealType TMathUtil<RealType>::Max3(const RealType A, const RealType B, const RealType C)
{
	return Max(Max(A, B), C);
}

template<typename RealType>
int32 TMathUtil<RealType>::Max3Index(const RealType A, const RealType B, const RealType C)
{
	if (A >= B) 
	{
		return (A >= C) ? 0 : 2;
	}
	else
	{
		return (B >= C) ? 1 : 2;
	}
}

template<typename RealType>
RealType TMathUtil<RealType>::Min(const RealType A, const RealType B)
{
	return (A <= B) ? A : B;
}

template<typename RealType>
RealType TMathUtil<RealType>::Min3(const RealType A, const RealType B, const RealType C)
{
	return Min(Min(A, B), C);
}

template<typename RealType>
int32 TMathUtil<RealType>::Min3Index(const RealType A, const RealType B, const RealType C)
{
	if (A <= B) 
	{
		return (A <= C) ? 0 : 2;
	}
	else 
	{
		return (B <= C) ? 1 : 2;
	}
}

// compute min and max of a,b,c with max 3 comparisons (sometimes 2)
template<typename RealType>
void TMathUtil<RealType>::MinMax(RealType A, RealType B, RealType C, RealType& MinOut, RealType& MaxOut)
{
	if (A < B) {
		if (A < C) {
			MinOut = A; MaxOut = TMathUtil<RealType>::Max(B, C);
		}
		else {
			MinOut = C; MaxOut = B;
		}
	}
	else {
		if (A > C) {
			MaxOut = A; MinOut = TMathUtil<RealType>::Min(B, C);
		}
		else {
			MinOut = B; MaxOut = C;
		}
	}
}

template<typename RealType>
RealType TMathUtil<RealType>::SmoothMin(RealType A, RealType B, RealType BlendExtent)
{
	if (BlendExtent == 0)
	{
		return Min(A, B);
	}

	double H = (BlendExtent + B - A) / 2;
	return Min(B - H * Clamp(H / BlendExtent, 0.0, 1.0), A);
}

template<typename RealType>
RealType TMathUtil<RealType>::SmoothMax(RealType A, RealType B, RealType BlendExtent)
{
	if (BlendExtent == 0)
	{
		return Max(A, B);
	}

	double H = (BlendExtent + A - B) / 2;
	return Max(B + H * Clamp(H / BlendExtent, 0.0, 1.0), A);
}

template<typename RealType>
RealType TMathUtil<RealType>::Sqrt(const RealType Value)
{
	return sqrt(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Cbrt(const RealType Value)
{
	return cbrt(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Tan(const RealType Value)
{
	return tan(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Atan2(const RealType ValueY, const RealType ValueX)
{
	return atan2(ValueY, ValueX);
}

template<typename RealType>
RealType TMathUtil<RealType>::Sin(const RealType Value)
{
	return sin(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Cos(const RealType Value)
{
	return cos(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::ACos(const RealType Value)
{
	return acos(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Floor(const RealType Value)
{
	return floor(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Ceil(const RealType Value)
{
	return ceil(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Round(const RealType Value)
{
	return round(Value);
}

template<typename RealType>
RealType TMathUtil<RealType>::Pow(const RealType Value, const RealType Power)
{
	return pow(Value, Power);
}

template<typename RealType>
RealType TMathUtil<RealType>::Exp(const RealType Power)
{
	return exp(Power);
}

template<typename RealType>
RealType TMathUtil<RealType>::Log(const RealType Power)
{
	return log(Power);
}



template<typename RealType>
RealType TMathUtil<RealType>::Lerp(const RealType A, const RealType B, RealType Alpha)
{
	Alpha = Clamp(Alpha, (RealType)0, (RealType)1);
	return ((RealType)1 - Alpha)*A + (Alpha)*B;
}

template<typename RealType>
RealType TMathUtil<RealType>::Atan2Positive(const RealType Y, const RealType X)
{
	// @todo this is a float atan2 !!
	RealType Theta = TMathUtil<RealType>::Atan2(Y, X);
	if (Theta < 0)
	{
		return ((RealType)2 * TMathUtil<RealType>::Pi) + Theta;
	}
	return Theta;
}
