// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template <typename RealType>
struct TMatrix3
{
	TVector<RealType> Row0;
	TVector<RealType> Row1;
	TVector<RealType> Row2;

	TMatrix3()
	{
	}

	TMatrix3(RealType ConstantValue)
	{
		Row0 = TVector<RealType>(ConstantValue, ConstantValue, ConstantValue);
		Row1 = Row0;
		Row2 = Row0;
	}

	TMatrix3(RealType Diag0, RealType Diag1, RealType Diag2)
	{
		Row0 = TVector<RealType>(Diag0, 0, 0);
		Row1 = TVector<RealType>(0, Diag1, 0);
		Row2 = TVector<RealType>(0, 0, Diag2);
	}

	/**
	 * Construct outer-product of U*transpose(V) of U and V
	 * result is that Mij = u_i * v_j
	 */
	TMatrix3<RealType>(const UE::Math::TVector<RealType>& U, const UE::Math::TVector<RealType>& V)
		: Row0(U.X * V.X, U.X * V.Y, U.X * V.Z),
		  Row1(U.Y * V.X, U.Y * V.Y, U.Y * V.Z),
		  Row2(U.Z * V.X, U.Z * V.Y, U.Z * V.Z)
	{
	}

	TMatrix3(RealType M00, RealType M01, RealType M02, RealType M10, RealType M11, RealType M12, RealType M20, RealType M21, RealType M22)
		: Row0(M00, M01, M02),
		  Row1(M10, M11, M12),
		  Row2(M20, M21, M22)
	{
	}

	TMatrix3(const UE::Math::TVector<RealType>& V1, const UE::Math::TVector<RealType>& V2, const UE::Math::TVector<RealType>& V3, bool bRows)
	{
		if (bRows)
		{
			Row0 = V1;
			Row1 = V2;
			Row2 = V3;
		}
		else
		{
			Row0 = TVector<RealType>(V1.X, V2.X, V3.X);
			Row1 = TVector<RealType>(V1.Y, V2.Y, V3.Y);
			Row2 = TVector<RealType>(V1.Z, V2.Z, V3.Z);
		}
	}

	template<typename RealType2>
	explicit constexpr TMatrix3(const TMatrix3<RealType2>& Mat) :
		Row0(TVector<RealType>{Mat.Row0}),
		Row1(TVector<RealType>{Mat.Row1}),
		Row2(TVector<RealType>{Mat.Row2})
	{
	}

	static TMatrix3<RealType> Zero()
	{
		return TMatrix3<RealType>(0);
	}
	static TMatrix3<RealType> Identity()
	{
		return TMatrix3<RealType>(1, 1, 1);
	}

	RealType operator()(int Row, int Col) const
	{
		check(Row >= 0 && Row < 3 && Col >= 0 && Col < 3);
		if (Row == 0)
		{
			return Row0[Col];
		}
		else if (Row == 1)
		{
			return Row1[Col];
		}
		else
		{
			return Row2[Col];
		}
	}

	TMatrix3<RealType> operator*(RealType Scale) const
	{
		return TMatrix3<RealType>(
			Row0.X * Scale, Row0.Y * Scale, Row0.Z * Scale,
			Row1.X * Scale, Row1.Y * Scale, Row1.Z * Scale,
			Row2.X * Scale, Row2.Y * Scale, Row2.Z * Scale);
	}

	TVector<RealType> operator*(const UE::Math::TVector<RealType>& V) const
	{
		return TVector<RealType>(
			Row0.X * V.X + Row0.Y * V.Y + Row0.Z * V.Z,
			Row1.X * V.X + Row1.Y * V.Y + Row1.Z * V.Z,
			Row2.X * V.X + Row2.Y * V.Y + Row2.Z * V.Z);
	}

	TMatrix3<RealType> operator*(const TMatrix3<RealType>& Mat2) const
	{
		RealType M00 = Row0.X * Mat2.Row0.X + Row0.Y * Mat2.Row1.X + Row0.Z * Mat2.Row2.X;
		RealType M01 = Row0.X * Mat2.Row0.Y + Row0.Y * Mat2.Row1.Y + Row0.Z * Mat2.Row2.Y;
		RealType M02 = Row0.X * Mat2.Row0.Z + Row0.Y * Mat2.Row1.Z + Row0.Z * Mat2.Row2.Z;

		RealType M10 = Row1.X * Mat2.Row0.X + Row1.Y * Mat2.Row1.X + Row1.Z * Mat2.Row2.X;
		RealType M11 = Row1.X * Mat2.Row0.Y + Row1.Y * Mat2.Row1.Y + Row1.Z * Mat2.Row2.Y;
		RealType M12 = Row1.X * Mat2.Row0.Z + Row1.Y * Mat2.Row1.Z + Row1.Z * Mat2.Row2.Z;

		RealType M20 = Row2.X * Mat2.Row0.X + Row2.Y * Mat2.Row1.X + Row2.Z * Mat2.Row2.X;
		RealType M21 = Row2.X * Mat2.Row0.Y + Row2.Y * Mat2.Row1.Y + Row2.Z * Mat2.Row2.Y;
		RealType M22 = Row2.X * Mat2.Row0.Z + Row2.Y * Mat2.Row1.Z + Row2.Z * Mat2.Row2.Z;

		return TMatrix3<RealType>(M00, M01, M02, M10, M11, M12, M20, M21, M22);
	}

	TMatrix3<RealType> operator+(const TMatrix3<RealType>& Mat2) const 
	{
		return TMatrix3<RealType>(Row0 + Mat2.Row0, Row1 + Mat2.Row1, Row2 + Mat2.Row2, true);
	}

	TMatrix3<RealType> operator-(const TMatrix3<RealType>& Mat2) const 
	{
		return TMatrix3<RealType>(Row0 - Mat2.Row0, Row1 - Mat2.Row1, Row2 - Mat2.Row2, true);
	}

	inline TMatrix3<RealType>& operator*=(const RealType& Scalar)
	{
		Row0 *= Scalar;
		Row1 *= Scalar;
		Row2 *= Scalar;
		return *this;
	}

	inline TMatrix3<RealType>& operator+=(const TMatrix3<RealType>& Mat2)
	{
		Row0 += Mat2.Row0;
		Row1 += Mat2.Row1;
		Row2 += Mat2.Row2;
		return *this;
	}

	RealType InnerProduct(const TMatrix3<RealType>& Mat2) const
	{
		return Row0.Dot(Mat2.Row0) + Row1.Dot(Mat2.Row1) + Row2.Dot(Mat2.Row2);
	}

	RealType Trace() const
	{
		return Row0.X + Row1.Y + Row2.Z;
	}

	RealType Determinant() const
	{
		RealType a11 = Row0.X, a12 = Row0.Y, a13 = Row0.Z, a21 = Row1.X, a22 = Row1.Y, a23 = Row1.Z, a31 = Row2.X, a32 = Row2.Y, a33 = Row2.Z;
		RealType i00 = a33 * a22 - a32 * a23;
		RealType i01 = -(a33 * a12 - a32 * a13);
		RealType i02 = a23 * a12 - a22 * a13;
		return a11 * i00 + a21 * i01 + a31 * i02;
	}

	TMatrix3<RealType> Inverse() const
	{
		RealType a11 = Row0.X, a12 = Row0.Y, a13 = Row0.Z, a21 = Row1.X, a22 = Row1.Y, a23 = Row1.Z, a31 = Row2.X, a32 = Row2.Y, a33 = Row2.Z;
		RealType i00 = a33 * a22 - a32 * a23;
		RealType i01 = -(a33 * a12 - a32 * a13);
		RealType i02 = a23 * a12 - a22 * a13;

		RealType i10 = -(a33 * a21 - a31 * a23);
		RealType i11 = a33 * a11 - a31 * a13;
		RealType i12 = -(a23 * a11 - a21 * a13);

		RealType i20 = a32 * a21 - a31 * a22;
		RealType i21 = -(a32 * a11 - a31 * a12);
		RealType i22 = a22 * a11 - a21 * a12;

		RealType det = a11 * i00 + a21 * i01 + a31 * i02;
		ensure(TMathUtil<RealType>::Abs(det) >= TMathUtil<RealType>::Epsilon);
		det = 1.0 / det;
		return TMatrix3<RealType>(i00 * det, i01 * det, i02 * det, i10 * det, i11 * det, i12 * det, i20 * det, i21 * det, i22 * det);
	}

	TMatrix3<RealType> Transpose() const
	{
		return TMatrix3<RealType>(
			Row0.X, Row1.X, Row2.X,
			Row0.Y, Row1.Y, Row2.Y,
			Row0.Z, Row1.Z, Row2.Z);
	}

	UE::Math::TVector<RealType> TransformByTranspose(const UE::Math::TVector<RealType>& V) const
	{
		return UE::Math::TVector<RealType>(
			Row0.X * V.X + Row1.X * V.Y + Row2.X * V.Z,
			Row0.Y * V.X + Row1.Y * V.Y + Row2.Y * V.Z,
			Row0.Z * V.X + Row1.Z * V.Y + Row2.Z * V.Z);
	}

	// Computes |A|(A^-1)^T. 
	// This value is sometimes useful for computing rotated surface normals from a continuous deformation.
	// Since computing the 3x3 inverse using Cramer's rule divides by the 3x3 determinant in the final step, this 
	// function avoids computing the determinant at all, since it would just cancel out.
	TMatrix3<RealType> DeterminantTimesInverseTranspose() const
	{
		RealType a11 = Row0.X, a12 = Row0.Y, a13 = Row0.Z, a21 = Row1.X, a22 = Row1.Y, a23 = Row1.Z, a31 = Row2.X, a32 = Row2.Y, a33 = Row2.Z;
		
		RealType i00 = a33 * a22 - a32 * a23;
		RealType i01 = -(a33 * a12 - a32 * a13);
		RealType i02 = a23 * a12 - a22 * a13;

		RealType i10 = -(a33 * a21 - a31 * a23);
		RealType i11 = a33 * a11 - a31 * a13;
		RealType i12 = -(a23 * a11 - a21 * a13);

		RealType i20 = a32 * a21 - a31 * a22;
		RealType i21 = -(a32 * a11 - a31 * a12);
		RealType i22 = a22 * a11 - a21 * a12;

		return TMatrix3<RealType>(i00, i10, i20, i01, i11, i21, i02, i12, i22);
	}


	bool EpsilonEqual(const TMatrix3<RealType>& Mat2, RealType Epsilon) const
	{
		return VectorUtil::EpsilonEqual(Row0, Mat2.Row0, Epsilon) &&
			   VectorUtil::EpsilonEqual(Row1, Mat2.Row1, Epsilon) &&
			   VectorUtil::EpsilonEqual(Row2, Mat2.Row2, Epsilon);
	}

	static TMatrix3<RealType> AxisAngleR(const UE::Math::TVector<RealType>& Axis, RealType AngleRad)
	{
		RealType cs = TMathUtil<RealType>::Cos(AngleRad);
		RealType sn = TMathUtil<RealType>::Sin(AngleRad);
		RealType oneMinusCos = 1.0 - cs;
		RealType x2 = Axis[0] * Axis[0];
		RealType y2 = Axis[1] * Axis[1];
		RealType z2 = Axis[2] * Axis[2];
		RealType xym = Axis[0] * Axis[1] * oneMinusCos;
		RealType xzm = Axis[0] * Axis[2] * oneMinusCos;
		RealType yzm = Axis[1] * Axis[2] * oneMinusCos;
		RealType xSin = Axis[0] * sn;
		RealType ySin = Axis[1] * sn;
		RealType zSin = Axis[2] * sn;
		return TMatrix3<RealType>(
			x2 * oneMinusCos + cs, xym - zSin, xzm + ySin,
			xym + zSin, y2 * oneMinusCos + cs, yzm - xSin,
			xzm - ySin, yzm + xSin, z2 * oneMinusCos + cs);
	}

	static TMatrix3<RealType> AxisAngleD(const UE::Math::TVector<RealType>& Axis, RealType AngleDeg)
	{
		return AxisAngleR(Axis, TMathUtil<RealType>::DegreesToRadians(AngleDeg));
	}
};


template <typename RealType>
struct TMatrix2
{
	TVector2<RealType> Row0;
	TVector2<RealType> Row1;

	TMatrix2()
	{
	}

	TMatrix2(RealType ConstantValue)
	{
		Row0 = Row1 = TVector2<RealType>(ConstantValue, ConstantValue);
	}

	TMatrix2(RealType Diag0, RealType Diag1)
	{
		Row0 = TVector2<RealType>(Diag0, 0);
		Row1 = TVector2<RealType>(0, Diag1);
	}

	/**
	 * Construct outer-product of U*transpose(V) of U and V
	 * result is that Mij = u_i * v_j
	 */
	TMatrix2<RealType>(const TVector2<RealType>& U, const TVector2<RealType>& V)
		: Row0(U.X * V.X, U.X * V.Y),
		  Row1(U.Y * V.X, U.Y * V.Y)
	{
	}

	TMatrix2(RealType M00, RealType M01, RealType M10, RealType M11)
		: Row0(M00, M01),
		  Row1(M10, M11)
	{
	}

	TMatrix2(const TVector2<RealType>& V1, const TVector2<RealType>& V2, bool bRows)
	{
		if (bRows)
		{
			Row0 = V1;
			Row1 = V2;
		}
		else
		{
			Row0 = TVector2<RealType>(V1.X, V2.X);
			Row1 = TVector2<RealType>(V1.Y, V2.Y);
		}
	}

	static TMatrix2<RealType> Zero()
	{
		return TMatrix2<RealType>(0);
	}
	static TMatrix2<RealType> Identity()
	{
		return TMatrix2<RealType>(1, 1);
	}

	RealType operator()(int Row, int Col) const
	{
		check(Row >= 0 && Row < 2 && Col >= 0 && Col < 2);
		if (Row == 0)
		{
			return Row0[Col];
		}
		else
		{
			return Row1[Col];
		}
	}

	TMatrix2<RealType> operator*(RealType Scale) const
	{
		return TMatrix2<RealType>(
			Row0.X * Scale, Row0.Y * Scale,
			Row1.X * Scale, Row1.Y * Scale);
	}

	TVector2<RealType> operator*(const TVector2<RealType>& V) const
	{
		return TVector2<RealType>(
			Row0.X * V.X + Row0.Y * V.Y,
			Row1.X * V.X + Row1.Y * V.Y);
	}

	TMatrix2<RealType> operator*(const TMatrix2<RealType>& Mat2) const
	{
		RealType M00 = Row0.X * Mat2.Row0.X + Row0.Y * Mat2.Row1.X;
		RealType M01 = Row0.X * Mat2.Row0.Y + Row0.Y * Mat2.Row1.Y;

		RealType M10 = Row1.X * Mat2.Row0.X + Row1.Y * Mat2.Row1.X;
		RealType M11 = Row1.X * Mat2.Row0.Y + Row1.Y * Mat2.Row1.Y;

		return TMatrix2<RealType>(M00, M01, M10, M11);
	}

	TMatrix2<RealType> operator+(const TMatrix2<RealType>& Mat2)
	{
		return TMatrix2<RealType>(Row0 + Mat2.Row0, Row1 + Mat2.Row1, true);
	}

	TMatrix2<RealType> operator-(const TMatrix2<RealType>& Mat2)
	{
		return TMatrix2<RealType>(Row0 - Mat2.Row0, Row1 - Mat2.Row1, true);
	}

	inline TMatrix2<RealType>& operator*=(const RealType& Scalar)
	{
		Row0 *= Scalar;
		Row1 *= Scalar;
		return *this;
	}

	inline TMatrix2<RealType>& operator+=(const TMatrix2<RealType>& Mat2)
	{
		Row0 += Mat2.Row0;
		Row1 += Mat2.Row1;
		return *this;
	}

	RealType InnerProduct(const TMatrix2<RealType>& Mat2) const
	{
		return Row0.Dot(Mat2.Row0) + Row1.Dot(Mat2.Row1);
	}

	RealType Trace() const
	{
		return Row0.X + Row1.Y;
	}

	RealType Determinant() const
	{
		return Row0.X * Row1.Y - Row0.Y * Row1.X;
	}

	TMatrix2<RealType> Inverse() const
	{
		RealType Det = Determinant();
		ensure(TMathUtil<RealType>::Abs(Det) >= TMathUtil<RealType>::Epsilon);
		RealType DetInv = 1.0 / Det;
		return TMatrix2<RealType>(Row1.Y * DetInv, -Row0.Y * DetInv, -Row1.X * DetInv, Row0.X * DetInv);
	}

	TMatrix2<RealType> Transpose() const
	{
		return TMatrix2<RealType>(
			Row0.X, Row1.X,
			Row0.Y, Row1.Y);
	}

	bool EpsilonEqual(const TMatrix2<RealType>& Mat2, RealType Epsilon) const
	{
		return VectorUtil::EpsilonEqual(Row0, Mat2.Row0, Epsilon) &&
			   VectorUtil::EpsilonEqual(Row1, Mat2.Row1, Epsilon);
	}

	static TMatrix2<RealType> RotationRad(RealType AngleRad)
	{
		RealType cs = TMathUtil<RealType>::Cos(AngleRad);
		RealType sn = TMathUtil<RealType>::Sin(AngleRad);
		return TMatrix2<RealType>(cs, -sn, sn, cs);
	}

	static TMatrix2<RealType> RotationDeg(RealType AngleDeg)
	{
		return RotationRad(AngleDeg * TMathUtil<RealType>::DegToRad);
	}

	// Create a matrix that scales along the specified axis
	// @param Axis	Axis to scale along
	// @param Scale	Amount to scale
	// @param bNormalizeAxis Whether to normalize Axis. If false, assumes Axis was already normalized.
	static TMatrix2<RealType> AxisScale(TVector2<RealType> Axis, RealType Scale, bool bNormalizeAxis = true)
	{
		if (bNormalizeAxis)
		{
			Axis.Normalize();
		}
		RealType X2 = Axis.X * Axis.X;
		RealType Y2 = Axis.Y * Axis.Y;
		RealType XY = Axis.X * Axis.Y;
		return TMatrix2<RealType>(
			Scale * X2 + Y2, Scale * XY - XY,
			Scale * XY - XY, Scale * Y2 + X2);
	}

	/**
	 * Assumes we have a rotation matrix (uniform scale ok)
	 */
	RealType GetAngleRad()
	{
		return TMathUtil<RealType>::Atan2(Row1.X, Row0.X);
	}

};

template <typename RealType>
inline TMatrix3<RealType> operator*(RealType Scale, const TMatrix3<RealType>& Mat)
{
	return TMatrix3<RealType>(
		Mat.Row0.X * Scale, Mat.Row0.Y * Scale, Mat.Row0.Z * Scale,
		Mat.Row1.X * Scale, Mat.Row1.Y * Scale, Mat.Row1.Z * Scale,
		Mat.Row2.X * Scale, Mat.Row2.Y * Scale, Mat.Row2.Z * Scale);
}

template <typename RealType>
inline TMatrix2<RealType> operator*(RealType Scale, const TMatrix2<RealType>& Mat)
{
	return TMatrix2<RealType>(
		Mat.Row0.X * Scale, Mat.Row0.Y * Scale,
		Mat.Row1.X * Scale, Mat.Row1.Y * Scale);
}

// Skew-Symmetric matrix such that A X B = CrossProductMatrix(A) * B;
template <typename RealType>
inline static TMatrix3<RealType> CrossProductMatrix(const UE::Math::TVector<RealType>& A)
{
	RealType Zero(0);

	return TMatrix3<RealType>( Zero, -A[2],  A[1],
		                       A[2],  Zero, -A[0],
		                      -A[1],  A[0],  Zero);
}



typedef TMatrix3<float> FMatrix3f;
typedef TMatrix3<double> FMatrix3d;
typedef TMatrix2<float> FMatrix2f;
typedef TMatrix2<double> FMatrix2d;

} // end namespace UE::Geometry
} // end namespace UE