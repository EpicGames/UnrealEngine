// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TriangleTypes.h"
#include "VectorTypes.h"

#include "Math/IntVector.h"
#include "Math/Vector.h"
#include "IndexTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Most generic / lazy example of a triangle mesh adapter; possibly useful for prototyping / building on top of (but slower than making a more specific-case adapter)
 */
template <typename RealType>
struct TTriangleMeshAdapter
{
	TFunction<bool(int32 index)> IsTriangle;
	TFunction<bool(int32 index)> IsVertex;
	TFunction<int32()> MaxTriangleID;
	TFunction<int32()> MaxVertexID;
	TFunction<int32()> TriangleCount;
	TFunction<int32()> VertexCount;
	TFunction<uint64()> GetChangeStamp;
	TFunction<FIndex3i(int32)> GetTriangle;
	TFunction<TVector<RealType>(int32)> GetVertex;

	inline void GetTriVertices(int TID, UE::Math::TVector<RealType>& V0, UE::Math::TVector<RealType>& V1, UE::Math::TVector<RealType>& V2) const
	{
		FIndex3i TriIndices = GetTriangle(TID);
		V0 = GetVertex(TriIndices.A);
		V1 = GetVertex(TriIndices.B);
		V2 = GetVertex(TriIndices.C);
	}
};

typedef TTriangleMeshAdapter<double> FTriangleMeshAdapterd;
typedef TTriangleMeshAdapter<float> FTriangleMeshAdapterf;

/**
 * Generic way to add edge connectivity iteration support to any mesh that looks like a TTriangleMeshAdapter
 * The interface is designed to match a subset of the FDynamicMesh3 query methods
 * Note that unlike FDynamicMesh3, it does support edge-non-manifold mesh topology (i.e. edges with more than 2 triangles)
 */
class FTriangleMeshAdapterEdgeConnectivity
{
public:

	using FLocalIntArray = TArray<int32, TInlineAllocator<8>>;

	template<typename TriangleMeshAdapterType>
	void BuildEdgeConnectivity(const TriangleMeshAdapterType& InMesh)
	{
		EIDtoVIDs.Reset();
		EIDtoTIDs.Reset();
		EIDtoTIDSpillID.Reset();
		SpillIDtoTIDs.Reset();

		TArray<FLocalIntArray> VIDtoNbrVIDs; // temp arrays, 1:1 with VIDtoEIDs
		VIDtoEIDs.SetNum(InMesh.MaxVertexID());
		VIDtoNbrVIDs.SetNum(VIDtoEIDs.Num());
		TIDtoEIDs.SetNum(InMesh.MaxTriangleID());

		auto AddEdge = [&VIDtoNbrVIDs, this](int32 A, int32 B, int32 TID) -> int32
			{
				int32 FoundEID = INDEX_NONE;

				FLocalIntArray& Nbrs = VIDtoNbrVIDs[A];
				int32 BSubIdx = INDEX_NONE;
				if (Nbrs.Find(B, BSubIdx)) // edge already exists
				{
					FoundEID = VIDtoEIDs[A][BSubIdx];
					// if it was only on one triangle, add this as the second triangle
					if (EIDtoTIDs[FoundEID].B == INDEX_NONE)
					{
						EIDtoTIDs[FoundEID].B = TID;
					}
					else // non-manifold case; build the spill mapping to hold the (hopefully few) non-manifold edges' neighbors
					{
						if (EIDtoTIDSpillID.IsEmpty())
						{
							EIDtoTIDSpillID.Init(INDEX_NONE, EIDtoVIDs.Num());
						}
						int32& SpillID = EIDtoTIDSpillID[FoundEID];
						if (SpillID == INDEX_NONE)
						{
							SpillID = SpillIDtoTIDs.Emplace();
						}
						SpillIDtoTIDs[SpillID].Add(TID);
					}
				}
				// edge didn't exist yet; create it
				else
				{
					FIndex2i SortedAB{ A,B }; // to match FDynamicMesh3 convention, store the vertices on the edge in index-sorted order
					SortedAB.Sort();
					FoundEID = EIDtoVIDs.Add({ SortedAB.A,SortedAB.B });
					EIDtoTIDs.Add({ TID,INDEX_NONE });
					checkSlow(EIDtoTIDs.Num() == EIDtoVIDs.Num());

					BSubIdx = Nbrs.Add(B);
					VIDtoEIDs[A].Add(FoundEID);
					VIDtoNbrVIDs[B].Add(A);
					VIDtoEIDs[B].Add(FoundEID);
					if (!EIDtoTIDSpillID.IsEmpty())
					{
						EIDtoTIDSpillID.Add(INDEX_NONE);
					}
				}
				return FoundEID;
			};

		for (int32 TID = 0; TID < InMesh.MaxTriangleID(); ++TID)
		{
			if (!InMesh.IsTriangle(TID))
			{
				continue;
			}
			const FIndex3i Tri = InMesh.GetTriangle(TID);
			FIndex3i& TriEdges = TIDtoEIDs[TID];
			for (int32 SubIdx = 0, PrevIdx = 2; SubIdx < 3; PrevIdx = SubIdx++)
			{
				int32 EID = AddEdge(Tri[SubIdx], Tri[PrevIdx], TID);
				TriEdges[PrevIdx] = EID;
			}
		}
	}

	int32 MaxEdgeID() const
	{
		return EIDtoVIDs.Num();
	}

	bool IsEdge(int32 EID) const
	{
		return EIDtoVIDs.IsValidIndex(EID);
	}

	bool IsBoundaryEdge(int32 EID) const
	{
		return GetEdgeT(EID).B == INDEX_NONE;
	}

	FIndex2i GetEdgeV(int32 EID) const
	{
		return FIndex2i(EIDtoVIDs[EID].A, EIDtoVIDs[EID].B);
	}

	// Return up to two of the triangles on edge EID; to access all triangles of a non-manifold edge, use EnumerateEdgeTriangles
	FIndex2i GetEdgeT(int32 EID) const
	{
		return FIndex2i(EIDtoTIDs[EID].A, EIDtoTIDs[EID].B);
	}

	FIndex3i GetTriEdges(int32 TID) const
	{
		return TIDtoEIDs[TID];
	}

	// @return a single triangle connected to the given vertex, or INDEX_NONE if the vertex has no triangles
	int32 GetSingleVertexTriangle(int32 VID) const
	{
		if (VIDtoEIDs[VID].IsEmpty())
		{
			return INDEX_NONE;
		}
		return EIDtoTIDs[VIDtoEIDs[VID][0]].A;
	}

	/** Call VertexFunc for each one-ring vertex neighbour of a vertex. */
	void EnumerateVertexVertices(int32 VertexID, TFunctionRef<void(int32)> VertexFunc) const
	{
		for (int32 EID : VIDtoEIDs[VertexID])
		{
			const FIndex2i& VIDs = EIDtoVIDs[EID];
			if (VIDs.A == VertexID)
			{
				VertexFunc(VIDs.B);
			}
			else
			{
				VertexFunc(VIDs.A);
			}
		}
	}

	/** Call EdgeFunc for each one-ring edge of a vertex. */
	void EnumerateVertexEdges(int32 VertexID, TFunctionRef<void(int32)> EdgeFunc) const
	{
		for (int32 EID : VIDtoEIDs[VertexID])
		{
			EdgeFunc(EID);
		}
	}

	// Note: Use templated TTriangleMeshAdapterEdgeConnectivity for a version of this method matching the FDynamicMesh3 interface
	/** Call TriangleFunc for each one-ring triangle of a vertex. */
	template<typename TriangleMeshAdapterType>
	void EnumerateVertexTriangles(int32 VertexID, TFunctionRef<void(int32)> TriangleFunc, const TriangleMeshAdapterType& InMesh) const
	{
		for (int32 EID : VIDtoEIDs[VertexID])
		{
			const int32 OtherVID = EIDtoVIDs[EID].OtherElement(VertexID);
			checkSlow(OtherVID != INDEX_NONE);
			EnumerateEdgeTriangles(EID, [this, VertexID, OtherVID, &InMesh, &TriangleFunc](int32 TID)
				{
					FIndex3i Tri = InMesh.GetTriangle(TID);
					// Each triangle is associate with two edges on the vertex; we report the triangle only for the 'outgoing' edge
					bool bIsTriangleOutgoingEdge = false;
					for (int32 SubIdx = 0, PrevIdx = 2; SubIdx < 3; PrevIdx = SubIdx++)
					{
						if (Tri[PrevIdx] == VertexID && Tri[SubIdx] == OtherVID)
						{
							bIsTriangleOutgoingEdge = true;
						}
					}
					if (bIsTriangleOutgoingEdge)
					{
						TriangleFunc(TID);
					}
				}
			);
		}
	}

	/** Call TriangleFunc for each Triangle attached to the given EdgeID */
	void EnumerateEdgeTriangles(int32 EdgeID, TFunctionRef<void(int32)> TriangleFunc) const
	{
		const FIndex2i& EdgeT = EIDtoTIDs[EdgeID];
		TriangleFunc(EdgeT.A);
		if (EdgeT.B != INDEX_NONE)
		{
			TriangleFunc(EdgeT.B);

			// enumerate non-manifold triangles if present
			if (!EIDtoTIDSpillID.IsEmpty())
			{
				int32 SpillID = EIDtoTIDSpillID[EdgeID];
				if (SpillID != INDEX_NONE)
				{
					for (int32 TID : SpillIDtoTIDs[SpillID])
					{
						TriangleFunc(TID);
					}
				}
			}
		}
	}

private:
	TArray<FLocalIntArray> VIDtoEIDs; // Map VID -> one-ring neighborhood of EIDs
	TArray<FIndex2i> EIDtoVIDs; // Map EID -> pair of vertex IDs on the edge (lower index first)
	TArray<FIndex2i> EIDtoTIDs; // Map EID -> up to two triangle IDs on the edge. If a boundary edge, second index is INDEX_NONE
	TArray<FIndex3i> TIDtoEIDs; // Map TID -> the three edges on the triangle

	// These arrays will only be populated if the input mesh was 'edge-non-manifold' -- i.e., had more than two triangle on some edges
	// Each non-manifold edge has a 'Spill ID' corresponding to an array of all additional neighboring triangle IDs
	TArray<int32> EIDtoTIDSpillID; // Map EID -> index in SpillID array
	TArray<TArray<int32, TInlineAllocator<2>>> SpillIDtoTIDs; // Map SpillID -> additional neighboring triangles of edge
};

// Templated version of FTriangleMeshAdapterEdgeConnectivity, to provide an FDynamicMesh3-compatible interface for methods that require additional mesh data
template<typename TriangleMeshType>
class TTriangleMeshAdapterEdgeConnectivity : public FTriangleMeshAdapterEdgeConnectivity
{
public:
	TTriangleMeshAdapterEdgeConnectivity(const TriangleMeshType* InMesh) : SourceMesh(InMesh)
	{
		check(SourceMesh);
		BuildEdgeConnectivity(*SourceMesh);
	}

	void EnumerateVertexTriangles(int32 VertexID, TFunctionRef<void(int32)> TriangleFunc) const
	{
		FTriangleMeshAdapterEdgeConnectivity::EnumerateVertexTriangles(VertexID, TriangleFunc, *SourceMesh);
	}
private:
	const TriangleMeshType* SourceMesh = nullptr;
};

/**
 * Example function to generate a generic mesh adapter from arrays
 * @param Vertices Array of mesh vertices
 * @param Triangles Array of int-vectors, one per triangle, indexing into the vertices array
 */
inline FTriangleMeshAdapterd GetArrayMesh(TArray<FVector>& Vertices, TArray<FIntVector>& Triangles)
{
	return {
		[&](int) { return true; },
		[&](int) { return true; },
		[&]() { return Triangles.Num(); },
		[&]() { return Vertices.Num(); },
		[&]() { return Triangles.Num(); },
		[&]() { return Vertices.Num(); },
		[&]() { return 0; },
		[&](int Idx) { return FIndex3i(Triangles[Idx]); },
		[&](int Idx) { return FVector3d(Vertices[Idx]); }};
}


/**
 * Faster adapter specifically for the common index mesh case
 */
template<typename IndexType, typename OutRealType, typename InVectorType=FVector>
struct TIndexMeshArrayAdapter
{
	const TArray<InVectorType>* SourceVertices;
	const TArray<IndexType>* SourceTriangles;

	void SetSources(const TArray<InVectorType>* SourceVerticesIn, const TArray<IndexType>* SourceTrianglesIn)
	{
		SourceVertices = SourceVerticesIn;
		SourceTriangles = SourceTrianglesIn;
	}

	TIndexMeshArrayAdapter() : SourceVertices(nullptr), SourceTriangles(nullptr)
	{
	}

	TIndexMeshArrayAdapter(const TArray<InVectorType>* SourceVerticesIn, const TArray<IndexType>* SourceTrianglesIn) : SourceVertices(SourceVerticesIn), SourceTriangles(SourceTrianglesIn)
	{
	}

	inline bool IsTriangle(int32 Index) const
	{
		return SourceTriangles->IsValidIndex(Index * 3);
	}

	inline bool IsVertex(int32 Index) const
	{
		return SourceVertices->IsValidIndex(Index);
	}

	inline int32 MaxTriangleID() const
	{
		return SourceTriangles->Num() / 3;
	}

	inline int32 MaxVertexID() const
	{
		return SourceVertices->Num();
	}

	// Counts are same as MaxIDs, because these are compact meshes
	inline int32 TriangleCount() const
	{
		return SourceTriangles->Num() / 3;
	}

	inline int32 VertexCount() const
	{
		return SourceVertices->Num();
	}

	inline uint64 GetChangeStamp() const
	{
		return 1; // source data doesn't have a timestamp concept
	}

	inline FIndex3i GetTriangle(int32 Index) const
	{
		int32 Start = Index * 3;
		return FIndex3i((int)(*SourceTriangles)[Start], (int)(*SourceTriangles)[Start+1], (int)(*SourceTriangles)[Start+2]);
	}

	inline TVector<OutRealType> GetVertex(int32 Index) const
	{
		return TVector<OutRealType>((*SourceVertices)[Index]);
	}

	inline void GetTriVertices(int32 TriIndex, UE::Math::TVector<OutRealType>& V0, UE::Math::TVector<OutRealType>& V1, UE::Math::TVector<OutRealType>& V2) const
	{
		int32 Start = TriIndex * 3;
		V0 = TVector<OutRealType>((*SourceVertices)[(*SourceTriangles)[Start]]);
		V1 = TVector<OutRealType>((*SourceVertices)[(*SourceTriangles)[Start+1]]);
		V2 = TVector<OutRealType>((*SourceVertices)[(*SourceTriangles)[Start+2]]);
	}

};


/**
 * Second version of the above faster adapter
 *  -- for the case where triangle indices are packed into an integer vector type instead of flat
 */
template<typename IndexVectorType, typename OutRealType, typename InVectorType = FVector>
struct TIndexVectorMeshArrayAdapter
{
	const TArray<InVectorType>* SourceVertices;
	const TArray<IndexVectorType>* SourceTriangles;

	void SetSources(const TArray<InVectorType>* SourceVerticesIn, const TArray<IndexVectorType>* SourceTrianglesIn)
	{
		SourceVertices = SourceVerticesIn;
		SourceTriangles = SourceTrianglesIn;
	}

	TIndexVectorMeshArrayAdapter() : SourceVertices(nullptr), SourceTriangles(nullptr)
	{
	}

	TIndexVectorMeshArrayAdapter(const TArray<InVectorType>* SourceVerticesIn, const TArray<IndexVectorType>* SourceTrianglesIn) : SourceVertices(SourceVerticesIn), SourceTriangles(SourceTrianglesIn)
	{
	}

	inline bool IsTriangle(int32 Index) const
	{
		return SourceTriangles->IsValidIndex(Index);
	}

	inline bool IsVertex(int32 Index) const
	{
		return SourceVertices->IsValidIndex(Index);
	}

	inline int32 MaxTriangleID() const
	{
		return SourceTriangles->Num();
	}

	inline int32 MaxVertexID() const
	{
		return SourceVertices->Num();
	}

	// Counts are same as MaxIDs, because these are compact meshes
	inline int32 TriangleCount() const
	{
		return SourceTriangles->Num();
	}

	inline int32 VertexCount() const
	{
		return SourceVertices->Num();
	}

	inline uint64 GetChangeStamp() const
	{
		return 1; // source data doesn't have a timestamp concept
	}

	inline FIndex3i GetTriangle(int32 Index) const
	{
		const IndexVectorType& Tri = (*SourceTriangles)[Index];
		return FIndex3i((int)Tri[0], (int)Tri[1], (int)Tri[2]);
	}

	inline TVector<OutRealType> GetVertex(int32 Index) const
	{
		return TVector<OutRealType>((*SourceVertices)[Index]);
	}

	inline void GetTriVertices(int32 TriIndex, UE::Math::TVector<OutRealType>& V0, UE::Math::TVector<OutRealType>& V1, UE::Math::TVector<OutRealType>& V2) const
	{
		const IndexVectorType& Tri = (*SourceTriangles)[TriIndex];
		V0 = TVector<OutRealType>((*SourceVertices)[Tri[0]]);
		V1 = TVector<OutRealType>((*SourceVertices)[Tri[1]]);
		V2 = TVector<OutRealType>((*SourceVertices)[Tri[2]]);
	}

};

typedef TIndexMeshArrayAdapter<uint32, double> FIndexMeshArrayAdapterd;


/**
 * TMeshWrapperAdapterd<T> can be used to present an arbitrary Mesh / Adapter type as a FTriangleMeshAdapterd.
 * This is useful in cases where it would be difficult or undesirable to write code templated on
 * the standard "Mesh Type" signature. If the code is written for FTriangleMeshAdapterd then this
 * shim can be used to present any compatible mesh type as a FTriangleMeshAdapterd
 */ 
template <class WrappedMeshType>
struct TMeshWrapperAdapterd : public UE::Geometry::FTriangleMeshAdapterd
{
	WrappedMeshType* WrappedAdapter;

	TMeshWrapperAdapterd(WrappedMeshType* WrappedAdapterIn) : WrappedAdapter(WrappedAdapterIn)
	{
		IsTriangle = [this](int index) { return WrappedAdapter->IsTriangle(index); };
		IsVertex = [this](int index) { return WrappedAdapter->IsVertex(index); };
		MaxTriangleID = [this]() { return WrappedAdapter->MaxTriangleID(); };
		MaxVertexID = [this]() { return WrappedAdapter->MaxVertexID(); };
		TriangleCount = [this]() { return WrappedAdapter->TriangleCount(); };
		VertexCount = [this]() { return WrappedAdapter->VertexCount(); };
		GetChangeStamp = [this]() { return WrappedAdapter->GetChangeStamp(); };
		GetTriangle = [this](int32 TriangleID) { return WrappedAdapter->GetTriangle(TriangleID); };
		GetVertex = [this](int32 VertexID) { return WrappedAdapter->GetVertex(VertexID); };
	}
};



} // end namespace UE::Geometry
} // end namespace UE