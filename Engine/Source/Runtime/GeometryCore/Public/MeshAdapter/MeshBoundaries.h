// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "IndexTypes.h"
#include "VectorUtil.h"
#include "VertexConnectedComponents.h"

namespace UE
{
namespace Geometry
{

// Generic methods for computing and operating on mesh boundaries, for any mesh type that implements the MeshAdapter interface
namespace MeshBoundaries
{
	/**
	 * Compute a mapping to track vertices that are within a WeldThreshold distance of other vertices, intended for use with ComputeBoundaries
	 * The mapping will map each vertex to a representative vertex of its welded (within-distance) group (and to itself, if unwelded)
	 */
	template<typename MeshType>
	TArray<int32> ComputeVertexWeldMapping(const MeshType& Mesh, double WeldThreshold, TFunctionRef<bool(int32)> VertexIDFilter = [](int32) { return true; })
	{
		FVertexConnectedComponents WeldVerts(Mesh.MaxVertexID());
		WeldVerts.ConnectCloseFilteredVertices(Mesh, WeldThreshold, VertexIDFilter);
		TArray<int32> WeldIdx;
		WeldIdx.SetNumUninitialized(Mesh.MaxVertexID());
		ParallelFor(Mesh.MaxVertexID(), [&WeldIdx, &WeldVerts](int32 VID) { WeldIdx[VID] = WeldVerts.GetComponentThreadsafe(VID); });
		return WeldIdx;
	}

	struct FBoundaryFilterOptions
	{
		// Filter boundary loops with perimeter less than this
		double MinPerimeter = 0;
		// Filter boundary loops with fewer edges than this
		int32 MinEdges = 0;
	};

	/** 
	 * Compute open boundary loops of a mesh.
	 * 
	 * @param Mesh The mesh to compute boundaries of
	 * @param OutPackedBoundaryVertLoops The boundary loops packed as [NumVertsInLoop0, LoopVID0, LoopVID1, ... LoopVIDN, NumVertsInLoop1, etc]
	 * @param OptionalOutPackedBoundaryTriLoops If non-null, the boundary loops packed as [NumTrisInLoop0, LoopTri0, LoopTri1, ... LoopTriN, NumTrisInLoop1, etc]. Tris are ordered s.t. LoopTri0 has boundary edge (LoopVID0, LoopVID1).
	 * @param VertexWeldFn A function taking a vertex ID to a 'welded' ID (which must always be one of the IDs of the vertices that was welded, i.e. if vertex 3 and 5 are welded, the weld ID is either 3 or 5)
	 * @param FilterOptions Options filtering loops from inclusion in the output, e.g. if they are too small
	 * @return Number of boundary loops found
	 */
	template<typename MeshType>
	int32 ComputeBoundaryLoops(const MeshType& Mesh, TArray<int32>& OutPackedBoundaryVertLoops, TArray<int32>* OptionalOutPackedBoundaryTriLoops,
		TFunctionRef<int32(int32)> VertexWeldFn = [](int32 VID) {return VID;}, FBoundaryFilterOptions FilterOptions = FBoundaryFilterOptions())
	{
		OutPackedBoundaryVertLoops.Reset();
		if (OptionalOutPackedBoundaryTriLoops)
		{
			OptionalOutPackedBoundaryTriLoops->Reset();
		}

		// 1. Find the edges w/ no paired reverse edge (i.e. the potential hole boundary edges)

		TMultiMap<int32, FIndex2i> OpenEdges; // VID -> (VID, TID)
		// remove an edge from OpenEdges, assuming the vertices are already passed through VertexWeldFn
		auto RemoveEdge = [&OpenEdges](int32 A, int32 B)
		{
			int32 NumRemovedPairs = 0;
			for (TMultiMap<int32, FIndex2i>::TKeyIterator It = OpenEdges.CreateKeyIterator(A); It; ++It)
			{
				if (It.Value().A == B)
				{
					It.RemoveCurrent();
					return true;
				}
			}
			return false;
		};
		auto AddEdge = [&OpenEdges, &VertexWeldFn, &RemoveEdge](int32 A, int32 B, int32 TID)
		{
			A = VertexWeldFn(A);
			B = VertexWeldFn(B);
			if (A == B) // weld collapsed edge, skip it
			{
				return;
			}
			if (!RemoveEdge(B, A))
			{
				OpenEdges.Add(A, FIndex2i(B, TID));
			}
		};
		// Consider all tri edges as possible boundary edges
		// Note: This is necessary for meshes where we only have a vertex and triangle buffer --
		//  for dynamic mesh can filter to boundary edge and add a potentially much smaller subset of candidates to OpenEdges
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
		{
			if (!Mesh.IsTriangle(TID))
			{
				continue;
			}
			FIndex3i Tri = Mesh.GetTriangle(TID);
			for (int32 SubIdx = 0, Prev = 2; SubIdx < 3; Prev = SubIdx++)
			{
				AddEdge(Tri[Prev], Tri[SubIdx], TID);
			}
		}

		// 2. Walk the open boundary edges to find the loops that define holes

		const int32 MinEdgesPerLoop = FMath::Max(3, FilterOptions.MinEdges);
		int32 NumHoles = 0;
		while (true)
		{
			const auto& EdgeItr = OpenEdges.CreateConstIterator();
			if (!EdgeItr)
			{
				break;
			}

			// Set up to add a new loop starting at this index
			int32 BdryStartIdx = OutPackedBoundaryVertLoops.Add(INDEX_NONE);
			// Helper to clear the added data, e.g. if we don't find a valid loop
			auto ClearLoop = [BdryStartIdx, &OutPackedBoundaryVertLoops, &OptionalOutPackedBoundaryTriLoops]()
			{
				OutPackedBoundaryVertLoops.SetNum(BdryStartIdx);
				if (OptionalOutPackedBoundaryTriLoops)
				{
					OptionalOutPackedBoundaryTriLoops->SetNum(BdryStartIdx);
				}
			};

			int32 Start = EdgeItr.Key();
			FIndex2i WalkPair = EdgeItr.Value();
			int32 Walk = WalkPair.A;

			OutPackedBoundaryVertLoops.Add(Start);
			if (OptionalOutPackedBoundaryTriLoops)
			{
				OptionalOutPackedBoundaryTriLoops->Add(INDEX_NONE);
				OptionalOutPackedBoundaryTriLoops->Add(WalkPair.B);
				checkSlow(OptionalOutPackedBoundaryTriLoops->Num() == OutPackedBoundaryVertLoops.Num());
			}
			RemoveEdge(Start, Walk);
			while (Walk != Start) // this will either loop back to start or dead end (note every non-dead-end iter removes an edge from OpenEdges)
			{
				OutPackedBoundaryVertLoops.Add(Walk);
				FIndex2i* Next = OpenEdges.Find(Walk);
				if (!Next)
				{
					// dead-ended chain of edges, discard the vertices
					ClearLoop();
					break;
				}
				int32 NextVal = Next->A;
				if (OptionalOutPackedBoundaryTriLoops)
				{
					OptionalOutPackedBoundaryTriLoops->Add(Next->B);
				}
				
				RemoveEdge(Walk, NextVal);
				Walk = NextVal;
			}
			int32 Count = OutPackedBoundaryVertLoops.Num() - BdryStartIdx - 1;
			if (Count < MinEdgesPerLoop) // failed to add a boundary, try again next loop
			{
				ClearLoop();
				continue;
			}
			if (FilterOptions.MinPerimeter > 0)
			{
				double Perim = 0;
				FVector3d PrevPos = Mesh.GetVertex(OutPackedBoundaryVertLoops.Last());
				for (int32 Idx = BdryStartIdx + 1; Perim < FilterOptions.MinPerimeter && Idx < OutPackedBoundaryVertLoops.Num(); ++Idx)
				{
					FVector3d CurPos = Mesh.GetVertex(OutPackedBoundaryVertLoops[Idx]);
					Perim += FVector3d::Dist(PrevPos, CurPos);
					PrevPos = CurPos;
				}
				if (Perim < FilterOptions.MinPerimeter)
				{
					ClearLoop();
					continue;
				}
			}
			
			OutPackedBoundaryVertLoops[BdryStartIdx] = Count; // record length of added chain
			if (OptionalOutPackedBoundaryTriLoops)
			{
				(*OptionalOutPackedBoundaryTriLoops)[BdryStartIdx] = Count;
			}
			NumHoles++;
		}
		if (OptionalOutPackedBoundaryTriLoops)
		{
			check(OutPackedBoundaryVertLoops.Num() == OptionalOutPackedBoundaryTriLoops->Num());
		}

		return NumHoles;
	}

	/**
	 * Context passed to ExtrudeBoundaryLoopEdges callbacks, describing a single boundary-edge endpoint.
	 * Vertex IDs are assumed to be welded (ie. passed through the VertexWeldFn by ComputeBoundaryLoops)
	 */
	struct FBoundaryVertexContext
	{
		int32 WeldedVID; // Central vertex ID
		int32 WeldedPrev; // loop-previous vertex ID
		int32 WeldedNext; // loop-next vertex ID
		int32 EdgeTID; // source triangle of the currently-processed boundary edge
	};

	/**
	 * Paired VIDs returned by the ExtrudeBoundaryLoopEdges vertex callback: the near corner of the
	 * skirt quad at this endpoint (an existing VID in the mesh) and the offset skirt VID.
	 */
	struct FBoundaryOffsetVert
	{
		int32 BaseVID; // near corner of the skirt quad at this endpoint (must already exist in the mesh)
		int32 SkirtVID; // the offset vertex for this endpoint (typically created on first call, cached thereafter)
	};

	/**
	 * Add a strip of offset geometry along mesh boundary loops (as computed by ComputeBoundaryLoops above).
	 *
	 * Walks each boundary edge and, for each of its two endpoints, calls GetOrCreateOffsetVertFn to get boundary/offset vertex pairs,
	 * then calls AddTriFn twice to add a two-triangle quad extending from the boundary edge.
	 *
	 * @param Mesh The mesh to extend with offset geometry
	 * @param PackedBoundaryVertLoops The boundary loops packed as [NumVertsInLoop0, LoopVID0, LoopVID1, ... LoopVIDN, NumVertsInLoop1, etc]
	 * @param PackedBoundaryTriLoops The boundary loops packed as [NumTrisInLoop0, LoopTri0, LoopTri1, ... LoopTriN, NumTrisInLoop1, etc]. Tris are ordered s.t. LoopTri0 has boundary edge (LoopVID0, LoopVID1).
	 * @param GetOrCreateOffsetVertFn Called per boundary-edge-endpoint, returning the boundary vertex and its offset pair. Should add the offset vertices (or find already-added ones) as needed.
	 * @param AddTriFn Function to add a new triangle to the mesh; Called per emitted skirt triangle with the source TID of the driving boundary edge.
	 */
	template<typename MeshType>
	void ExtrudeBoundaryLoopEdges(MeshType& Mesh, TConstArrayView<int32> PackedBoundaryVertLoops, TConstArrayView<int32> PackedBoundaryTriLoops,
		TFunctionRef<FBoundaryOffsetVert(const FBoundaryVertexContext&)> GetOrCreateOffsetVertFn,
		TFunctionRef<void(const FIndex3i& Tri, int32 SourceTID)> AddTriFn)
	{
		if (!ensure(PackedBoundaryVertLoops.Num() == PackedBoundaryTriLoops.Num()))
		{
			return;
		}
		for (int32 LoopCountIdx = 0; LoopCountIdx < PackedBoundaryVertLoops.Num(); LoopCountIdx += PackedBoundaryVertLoops[LoopCountIdx] + 1)
		{
			const int32 LoopCount = PackedBoundaryVertLoops[LoopCountIdx];
			if (!ensure(PackedBoundaryTriLoops[LoopCountIdx] == LoopCount))
			{
				return;
			}
			const int32 LoopStartIdx = LoopCountIdx + 1;
			const int32 LoopLastIdx = LoopCountIdx + LoopCount;

			// For each boundary edge (Prev -> Cur), emit the two skirt-quad triangles. PackedBoundaryTriLoops[PrevIdx]
			// is the tri containing the edge from PrevIdx's vertex to CurIdx's vertex, per ComputeBoundaryLoops.
			for (int32 CurIdx = LoopStartIdx; CurIdx <= LoopLastIdx; ++CurIdx)
			{
				const int32 PrevIdx = (CurIdx == LoopStartIdx) ? LoopLastIdx : (CurIdx - 1);
				const int32 PrevPrevIdx = (PrevIdx == LoopStartIdx) ? LoopLastIdx : (PrevIdx - 1);
				const int32 NextIdx = (CurIdx == LoopLastIdx) ? LoopStartIdx : (CurIdx + 1);

				const int32 WeldedCur = PackedBoundaryVertLoops[CurIdx];
				const int32 WeldedPrev = PackedBoundaryVertLoops[PrevIdx];
				const int32 WeldedPrevPrev = PackedBoundaryVertLoops[PrevPrevIdx];
				const int32 WeldedNext = PackedBoundaryVertLoops[NextIdx];

				const int32 EdgeTID = PackedBoundaryTriLoops[PrevIdx];

				const FBoundaryOffsetVert CurOff  = GetOrCreateOffsetVertFn({ WeldedCur,  WeldedPrev,     WeldedNext, EdgeTID });
				const FBoundaryOffsetVert PrevOff = GetOrCreateOffsetVertFn({ WeldedPrev, WeldedPrevPrev, WeldedCur,  EdgeTID });

				AddTriFn(FIndex3i(CurOff.BaseVID, PrevOff.BaseVID, CurOff.SkirtVID), EdgeTID);
				AddTriFn(FIndex3i(CurOff.SkirtVID, PrevOff.BaseVID, PrevOff.SkirtVID), EdgeTID);
			}
		}
	}

}

} // end namespace UE::Geometry
} // end namespace UE
