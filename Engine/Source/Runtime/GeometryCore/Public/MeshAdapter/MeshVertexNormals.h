// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include <atomic>

#include "Algo/Accumulate.h"
#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "IndexTypes.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Templates/Tuple.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

namespace MeshNormals
{

	// helper methods for the below parallel normal compute methods
	namespace ParallelHelpers
	{
		// This is a workaround for some platforms not supporting C++20's fetch_add for floats
		// UE platforms should be on C++20, so hopefully we can remove this method in the future and just use .fetch_add directly
		template<typename AtomicFloatType = std::atomic<float>>
		inline void AtomicFloatFetchAdd(AtomicFloatType& Value, float ToAdd)
		{
			constexpr bool bHasAtomicFloatFetchAdd = requires(AtomicFloatType & AtomicFloat, float Param)
			{
				AtomicFloat.fetch_add(Param, std::memory_order_relaxed);
			};
			if constexpr (bHasAtomicFloatFetchAdd)
			{
				Value.fetch_add(ToAdd, std::memory_order_relaxed);
			}
			else
			{
				// Manual implementation of fetch_add
				float Old = Value.load();
				while (!Value.compare_exchange_weak(Old, Old + ToAdd, std::memory_order_relaxed)) {}
			}
		}

		// This is a workaround for some platforms not supporting C++20's std::atomic_ref
		// UE platforms should be on C++20, so hopefully we can remove this method in the future and just use std::atomic_ref directly
		// Atomic_Ref is a slight performance improvement for platforms with weak memory order such as ARM as we can avoid a full barrier with
		// relaxed ordering here.
		template<typename IntType = int>
		inline void AtomicIncrement(IntType& Value, std::memory_order MemoryOrder)
		{
#if defined(__cpp_lib_atomic_ref)
			std::atomic_ref<IntType>(Value).fetch_add(1, MemoryOrder);
#else
			FPlatformAtomics::InterlockedIncrement(&Value);
#endif
		}

		// This is a workaround for some platforms not supporting C++20's std::atomic_ref
		// UE platforms should be on C++20, so hopefully we can remove this method in the future and just use std::atomic_ref directly
		// Atomic_Ref is a slight performance improvement for platforms with weak memory order such as ARM as we can avoid a full barrier with
		// relaxed ordering here.
		template<typename IntType = int>
		inline void AtomicDecrement(IntType& Value, std::memory_order MemoryOrder)
		{
#if defined(__cpp_lib_atomic_ref)
			std::atomic_ref<IntType>(Value).fetch_sub(1, MemoryOrder);
#else
			FPlatformAtomics::InterlockedDecrement(&Value);
#endif
		}

		struct FVertexNormalDeviceBlockAllocationTag : FDefaultBlockAllocationTag
		{
			static constexpr const char* TagName = "GeometryVertexNormalBlock";

			using Allocator = FAlignedAllocator;
		};

		struct FVertexNormalLinearAllocator
		{
			FORCEINLINE static void* Malloc(SIZE_T Size, uint32 Alignment)
			{
				return TConcurrentLinearAllocator<FVertexNormalDeviceBlockAllocationTag>::Malloc(Size, Alignment);
			}

			FORCEINLINE static void Free(void* Pointer)
			{
				TConcurrentLinearAllocator<FVertexNormalDeviceBlockAllocationTag>::Free(Pointer);
			}
		};
	}

	/**
	 * Compute per-vertex smooth normals for a generic triangle mesh type, writing to a *pre-allocated* output normals array
	 * 
	 * @param Mesh Input mesh
	 * @param OutNormals Output normals array; must have Mesh.MaxVertexID() elements
	 * @param bRequireDeterministicNormals If false, uses a non-deterministic fast path with unordered float operations that can produce slightly different roundings
	 */
	template<typename MeshType, typename NormalsArrayType>
	void ComputeVertexNormals(const MeshType& Mesh, NormalsArrayType& OutNormals, bool bRequireDeterministicNormals = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GeometryCore::MeshNormals::RecomputeNormals);
		check(Mesh.MaxVertexID() == OutNormals.Num());

		if (bRequireDeterministicNormals)
		{
			TArray<UE::TConsumeAllMpmcQueue<TPair<int, FVector3f>, ParallelHelpers::FVertexNormalLinearAllocator>> VertexNormalsToSum;
			VertexNormalsToSum.SetNum(Mesh.MaxVertexID());

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryCore::MeshNormals::Tri_Normal_Parallel);
				ParallelFor(Mesh.MaxTriangleID(), [&](int TriangleID)
				{
					if (!Mesh.IsTriangle(TriangleID))
					{
						return;
					}

					const FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
					FVector3d A, B, C;
					Mesh.GetTriVertices(TriangleID, A, B, C);
					double TriArea;
					const FVector3f TriNormal = FVector3f(VectorUtil::NormalArea(A, B, C, TriArea));
					FVector3f TriWeights = FVector3f(VectorUtil::TriangleInternalAngles(A, B, C) * TriArea);

					for (int VertIndex = 0; VertIndex < 3; ++VertIndex)
					{
						const int VertexID = Triangle[VertIndex];
						const int ElementID = 3 * VertexID;

						const float VertexWeight = TriWeights[VertIndex];
						const FVector3f Normal = TriNormal * VertexWeight;

						VertexNormalsToSum[VertexID].ProduceItem(TPair<int, FVector3f>(TriangleID, Normal));
					}
				});
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryCore::MeshNormals::Average);
				ParallelFor(Mesh.MaxVertexID(), [&](int VertexID)
				{
					if (!Mesh.IsVertex(VertexID))
					{
						return;
					}

					TArray<TPair<int, FVector3f>, TInlineAllocator<8>> VertexNormals;
					VertexNormalsToSum[VertexID].ConsumeAllLifo([&](TPair<int, FVector3f>&& Item) { VertexNormals.Add(Item); });
					VertexNormals.Sort([](const TPair<int, FVector3f>& A, const TPair<int, FVector3f>& B)
					{
						return A.Key < B.Key;
					});

					const FVector3f Normal = Algo::Accumulate(VertexNormals, FVector3f::ZeroVector,
						[](FVector3f Result, const TPair<int, FVector3f>& Normal) -> FVector3f
						{
							return Result + Normal.Value;
						}
					);

					OutNormals[VertexID] = FVector3f(Normalized(Normal));
				});
			}
		}
		else // Non-deterministic fast path which uses unordered float operations which can produce slightly different roundings
		{
			TArray<std::atomic<float>> VertexNormals;
			VertexNormals.SetNumZeroed(Mesh.MaxVertexID() * 3);

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryCore::MeshNormals::Tri_Normal_Parallel);
				ParallelFor(Mesh.MaxTriangleID(), [&](int TriangleID)
				{
					if (!Mesh.IsTriangle(TriangleID))
					{
						return;
					}

					const FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
					FVector3d A, B, C;
					Mesh.GetTriVertices(TriangleID, A, B, C);
					double TriArea;
					const FVector3f TriNormal = FVector3f(VectorUtil::NormalArea(A, B, C, TriArea));

					FVector3f TriWeights = FVector3f(VectorUtil::TriangleInternalAngles(A, B, C) * TriArea);

					for (int VertIndex = 0; VertIndex < 3; ++VertIndex)
					{
						const int VertexID = Triangle[VertIndex];
						const int ElementID = 3 * VertexID;

						const float VertexWeight = TriWeights[VertIndex];
						const FVector3f Normal = TriNormal * VertexWeight;

						for (int FloatIndex = 0; FloatIndex < 3; ++FloatIndex)
						{
							ParallelHelpers::AtomicFloatFetchAdd(VertexNormals[ElementID + FloatIndex], Normal[FloatIndex]);
						}
					}
				});
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryCore::MeshNormals::Average);
				ParallelFor(Mesh.MaxVertexID(), [&](int VertexID)
				{
					if (!Mesh.IsVertex(VertexID))
					{
						return;
					}

					const FVector3f Normal(VertexNormals[VertexID * 3], VertexNormals[VertexID * 3 + 1], VertexNormals[VertexID * 3 + 2]);

					OutNormals[VertexID] = Normalized(Normal);
				});
			}
		}
	}

	/**
	 * Compute vertex normals where some vertices are implicitly welded.
	 * @param Mesh The mesh to compute normals for
	 * @param VertexIDWeldMap Mapping from Vertex ID to the representative 'welded' ID. A set of mutually welded vertices must always map to the same welded-ID, which must be one of the vertex IDs from that set.
	 * @param OutNormals Computed normals
	 * @param bRequireDeterministicNormals If false, uses a non-deterministic fast path with unordered float operations that can produce slightly different roundings
	 */
	template<typename MeshType, typename NormalsArrayType>
	void ComputeWeldedVertexNormals(const MeshType& Mesh, TFunctionRef<int32(int32)> VertexIDWeldMap, NormalsArrayType& OutNormals, bool bRequireDeterministicNormals = true)
	{
		// Minimal mesh wrapper struct to forward triangle IDs to welded IDs
		struct FWeldMesh
		{
			FWeldMesh(const MeshType& InMesh, const TFunctionRef<int32(int32)>& InVertexIDWeldMap) : Mesh(&InMesh), VertexIDWeldMap(&InVertexIDWeldMap){}

			int32 MaxVertexID() const
			{
				return Mesh->MaxVertexID();
			}
			int32 MaxTriangleID() const
			{
				return Mesh->MaxTriangleID();
			}
			bool IsTriangle(int32 TID) const
			{
				return Mesh->IsTriangle(TID);
			}
			bool IsVertex(int32 VID) const
			{
				return Mesh->IsVertex(VID);
			}
			void GetTriVertices(int32 TID, FVector3d& A, FVector3d& B, FVector3d& C) const
			{
				return Mesh->GetTriVertices(TID, A, B, C);
			}
			FIndex3i GetTriangle(int32 TID) const
			{
				FIndex3i Tri = Mesh->GetTriangle(TID);
				Tri.A = (*VertexIDWeldMap)(Tri.A);
				Tri.B = (*VertexIDWeldMap)(Tri.B);
				Tri.C = (*VertexIDWeldMap)(Tri.C);
				return Tri;
			}
		private:
			const MeshType* Mesh;
			const TFunctionRef<int32(int32)>* VertexIDWeldMap;
		};
		FWeldMesh WeldMesh(Mesh, VertexIDWeldMap);
		ComputeVertexNormals(WeldMesh, OutNormals, bRequireDeterministicNormals);
		// Copy welded IDs back out
		ParallelFor(Mesh.MaxVertexID(), [&OutNormals, &VertexIDWeldMap](int32 Idx)
		{
			int32 WeldIdx = VertexIDWeldMap(Idx);
			if (WeldIdx > INDEX_NONE && WeldIdx != Idx)
			{
				OutNormals[Idx] = OutNormals[WeldIdx];
			}
		});
	}
}

} // end namespace UE::Geometry
} // end namespace UE
