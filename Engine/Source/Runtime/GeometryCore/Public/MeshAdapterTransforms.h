// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "FrameTypes.h"
#include "MathUtil.h"
#include "VectorTypes.h"

#include "Async/ParallelFor.h"

/**
 * A variant of DynamicMesh module's MeshTransforms that is generalized for MeshAdapter-based meshes
 * Unlike a lot of MeshAdapter-based functions, most of these require Set* and *Normal functions to exist on the Adapter
 * Specifically, these apply to any mesh class that implements GetVertex, SetVertex, GetNormal, SetNormal, IsVertex, IsNormal, MaxVertexID, MaxNormalID
 *
 * Note that unlike a lot of MeshAdapter things, FDynamicMesh3 will need to be wrapped by an adapter too for anything that changes normals,
 *     because the FDynamicMesh3 interface for normals a little too involved to make it the access pattern for the adapter.
 *     For now, will leave the FDynamicMesh3 specific version inside the DynamicMesh module (TODO: consider reworking it to just create a tiny shim and call these functions?)
 */
namespace MeshAdapterTransforms
{
	using namespace UE::Geometry;
	using FFrame3d = UE::Geometry::FFrame3d;

	/**
	* Apply Translation to vertex positions of Mesh. Does not modify any other attributes.
	*/
	template<class TriangleMeshType>
	void Translate(TriangleMeshType& Mesh, const FVector3d& Translation)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid) 
		{
			if (Mesh.IsVertex(vid))
			{
				Mesh.SetVertex(vid, Mesh.GetVertex(vid) + Translation);
			}
		});
	}

	/**
	* Transform Mesh into local coordinates of Frame
	*/
	template<class TriangleMeshType>
	void WorldToFrameCoords(TriangleMeshType& Mesh, const FFrame3d& Frame)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid)
		{
			if (Mesh.IsVertex(vid))
			{
				FVector3d Position = Mesh.GetVertex(vid);
				Mesh.SetVertex(vid, Frame.ToFramePoint(Position));
			}
		});

		if (Mesh.HasNormals())
		{
			int NumNormals = Mesh.MaxNormalID();
			ParallelFor(NumNormals, [&](int elemid)
			{
				if (Mesh.IsNormal(elemid))
				{
					FVector3f Normal = Mesh.GetNormal(elemid);
					Mesh.SetNormal(elemid, (FVector3f)Frame.ToFrameVector((FVector3d)Normal));
				}
			});
		}
	}

	/**
	* Transform Mesh out of local coordinates of Frame
	*/
	template<class TriangleMeshType>
	void FrameCoordsToWorld(TriangleMeshType& Mesh, const FFrame3d& Frame)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid)
		{
			if (Mesh.IsVertex(vid))
			{
				FVector3d Position = Mesh.GetVertex(vid);
				Mesh.SetVertex(vid, Frame.FromFramePoint(Position));
			}
		});

		if (Mesh.HasNormals())
		{
			int NumNormals = Mesh.MaxNormalID();
			ParallelFor(NumNormals, [&](int elemid)
			{
				if (Mesh.IsNormal(elemid))
				{
					FVector3f Normal = Mesh.GetNormal(elemid);
					Mesh.SetNormal(elemid, (FVector3f)Frame.FromFrameVector((FVector3d)Normal));
				}
			});
		}
	}


	/**
	* Apply given Transform to a Mesh.
	* Modifies Vertex Positions and Normals, and any Per-Triangle Normal Overlays
	*/
	template<class TriangleMeshType>
	void ApplyTransform(TriangleMeshType& Mesh, const FTransformSRT3d& Transform)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid)
		{
			if (Mesh.IsVertex(vid))
			{
				FVector3d Position = Mesh.GetVertex(vid);
				Position = Transform.TransformPosition(Position);
				Mesh.SetVertex(vid, Position);
			}
		});

		if (Mesh.HasNormals())
		{
			int NumNormals = Mesh.MaxNormalID();
			ParallelFor(NumNormals, [&](int elemid)
			{
				if (Mesh.IsNormal(elemid))
				{
					FVector3f Normal = Mesh.GetNormal(elemid);
					Mesh.SetNormal(elemid, (FVector3f)Transform.TransformNormal((FVector3d)Normal));
				}
			});
		}
	}


	/**
	* Apply inverse of given Transform to a Mesh.
	* Modifies Vertex Positions and Normals, and any Per-Triangle Normal Overlays
	*/
	template<class TriangleMeshType>
	void ApplyTransformInverse(TriangleMeshType& Mesh, const FTransformSRT3d& Transform)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid)
		{
			if (Mesh.IsVertex(vid)) 
			{
				FVector3d Position = Mesh.GetVertex(vid);
				Position = Transform.InverseTransformPosition(Position);
				Mesh.SetVertex(vid, Position);
			}
		});

		if (Mesh.HasNormals())
		{
			int NumNormals = Mesh.MaxNormalID();
			ParallelFor(NumNormals, [&](int elemid)
			{
				if (Mesh.IsNormal(elemid))
				{
					FVector3f Normal = Mesh.GetNormal(elemid);
					Mesh.SetNormal(elemid, (FVector3f)Transform.InverseTransformNormal((FVector3d)Normal));
				}
			});
		}
	}


	/**
	* Apply given Transform to a Mesh.
	* Modifies Vertex Positions and Normals, and any Per-Triangle Normal Overlays
	*/
	template<class TriangleMeshType>
	void ApplyTransform(TriangleMeshType& Mesh,
		TFunctionRef<FVector3d(const FVector3d&)> PositionTransform,
		TFunctionRef<FVector3f(const FVector3f&)> NormalTransform)
	{
		int NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int vid)
		{
			if (Mesh.IsVertex(vid))
			{
				FVector3d Position = Mesh.GetVertex(vid);
				Position = PositionTransform(Position);
				Mesh.SetVertex(vid, Position);
			}
		});

		if (Mesh.HasNormals())
		{
			int NumNormals = Mesh.MaxNormalID();
			ParallelFor(NumNormals, [&](int elemid)
			{
				if (Mesh.IsNormal(elemid))
				{
					FVector3f Normal = Mesh.GetNormal(elemid);
					Normal = NormalTransform(Normal);
					Mesh.SetNormal(elemid, UE::Geometry::Normalized(Normal));
				}
			});
		}
	}

};
