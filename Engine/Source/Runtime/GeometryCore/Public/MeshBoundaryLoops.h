// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshBoundaryLoops

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "EdgeLoop.h"
#include "EdgeSpan.h"
#include "VectorUtil.h"
#include "Util/BufferUtil.h"

namespace UE
{
namespace Geometry
{

class FMeshBoundaryLoops
{
public:
	/** Mesh we are finding loops on */
	const FDynamicMesh3* Mesh = nullptr;

	/** Resulting set of loops filled by Compute() */
	TArray<FEdgeLoop> Loops;

	/** Resulting set of spans filled by Compute(), if SpanBehavior == Compute */
	TArray<FEdgeSpan> Spans;

	/** If true, we aborted computation due to unrecoverable errors */
	bool bAborted = false;
	/** If true, we found at least one open span during Compute(). This can occur in failure cases or if the search is restricted to a subset using EdgeFilterFunc */
	bool bSawOpenSpans = false;
	/** If true, we had to call back to spans because of failures during Compute(). This happens if we cannot extract simple loops from a loop with bowties. */
	bool bFellBackToSpansOnFailure = false;
	
	enum class ESpanBehaviors
	{
		Ignore, Abort, Compute
	};
	/** What Compute() will do if it encounter open spans */
	ESpanBehaviors SpanBehavior = ESpanBehaviors::Compute;


	enum class EFailureBehaviors
	{
		Abort,       // die, and you clean up
		ConvertToOpenSpan     // keep un-closed loop as a span
	};
	/** What Compute() will do if it encounters unrecoverable errors while walking around a loop */
	EFailureBehaviors FailureBehavior = EFailureBehaviors::ConvertToOpenSpan;

	/** If non-null, then only edges that pass this filter are considered. This may result in open spans. */
	TFunction<bool(int)> EdgeFilterFunc = nullptr;

	/** If we encountered unrecoverable errors, it is generally due to bowtie vertices. The problematic bowties will be returned here so that the client can try repairing these vertices. */
	TArray<int> FailureBowties;

	/** After finding spans of connected edges, don't convert them to loops -- just leave them all as spans */
	bool bOnlyComputeSpans = false;

public:
	FMeshBoundaryLoops() {}

	FMeshBoundaryLoops(const FDynamicMesh3* MeshIn, bool bAutoCompute = true)
	{
		Mesh = MeshIn;
		if (bAutoCompute)
		{
			Compute();
		}
	}

	void SetMesh(const FDynamicMesh3* MeshIn) { Mesh = MeshIn; }

	/**
	 * Find the set of boundary EdgeLoops and EdgeSpans.
	 * .SpanBehavior and .FailureBehavior control what happens if we run into problem cases
	 * @return false if errors occurred, in this case output set is incomplete
	 */
	GEOMETRYCORE_API bool Compute();



	/** @return number of loops found by Compute() */
	int GetLoopCount() const
	{
		return Loops.Num();
	}

	/** @return number of spans found by Compute() */
	int GetSpanCount() const
	{
		return Spans.Num();
	}

	/** @return Loop at the given index */
	const FEdgeLoop& operator[](int Index) const
	{
		return Loops[Index];
	}


	/** @return index of loop with maximum number of vertices */
	GEOMETRYCORE_API int GetMaxVerticesLoopIndex() const;

	/** @return index of loop with longest arc length, or -1 if no loops */
	GEOMETRYCORE_API int GetLongestLoopIndex() const;

	/**
	 * @return pair (LoopIndex,VertexIndexInLoop) of VertexID in EdgeLoops, or FIndex2i::Invalid if not found
	 */
	GEOMETRYCORE_API FIndex2i FindVertexInLoop(int VertexID) const;

	/**
	 * @return index of loop that contains vertex, or -1 if not found
	 */
	GEOMETRYCORE_API int FindLoopContainingVertex(int VertexID) const;

	/**
	 * @return index of loop that contains edge, or -1 if not found
	 */
	GEOMETRYCORE_API int FindLoopContainingEdge(int EdgeID) const;

	/**
	 * @return index of loop that best matches input triangles, or -1 if not found
	 */
	GEOMETRYCORE_API int FindLoopTrianglesHint(const TArray<int>& BorderHintTris) const;

	/**
	 * @return index of loop that best matches input edges, or -1 if not found
	 */
	GEOMETRYCORE_API int FindLoopEdgesHint(const TSet<int>& BorderHintEdges) const;

	/**
	 * @return index of span that contains edge, or -1 if not found
	*/
	GEOMETRYCORE_API int FindSpanContainingEdge(int EdgeID) const;


protected:

	TArray<int> VerticesTemp;

	// [TODO] cache this : a dictionary? we will not need very many, but we will
	//   need each multiple times!
	GEOMETRYCORE_API FVector3d GetVertexNormal(int vid);

	// ok, bdry_edges[0...bdry_edges_count] contains the boundary edges coming out of bowtie_v.
	// We want to pick the best one to continue the loop that came in to bowtie_v on incoming_e.
	// If the loops are all sane, then we will get the smallest loops by "turning left" at bowtie_v.
	GEOMETRYCORE_API int FindLeftTurnEdge(int incoming_e, int bowtie_v, TArray<int>& bdry_edges, int bdry_edges_count, TArray<bool>& used_edges);
	
	struct Subloops
	{
		TArray<FEdgeLoop> Loops;
		TArray<FEdgeSpan> Spans;
	};


	// This is called when loopV contains one or more "bowtie" vertices.
	// These vertices *might* be duplicated in loopV (but not necessarily)
	// If they are, we have to break loopV into subloops that don't contain duplicates.
	GEOMETRYCORE_API bool ExtractSubloops(TArray<int>& loopV, TArray<int>& loopE, TArray<int>& bowties, Subloops& SubloopsOut);



protected:

	friend class FMeshRegionBoundaryLoops;

	/*
	 * static Utility functions that can be re-used in MeshRegionBoundaryLoops
	 * In all the functions below, the list loopV is assumed to possibly
	 * contain "removed" vertices indicated by -1. These are ignored.
	 */

	// Check if the loop from bowtieV to bowtieV inside loopV contains any other bowtie verts.
	// Also returns start and end indices in loopV of "clean" loop
	// Note that start may be < end, if the "clean" loop wraps around the end
	static GEOMETRYCORE_API bool IsSimpleBowtieLoop(const TArray<int>& LoopVerts, const TArray<int>& BowtieVerts, int BowtieVertex, int& start_i, int& end_i);

	// check if forward path from loopV[i1] to loopV[i2] contains any bowtie verts other than bowtieV
	static GEOMETRYCORE_API bool IsSimplePath(const TArray<int>& LoopVerts, const TArray<int>& BowtieVerts, int BowtieVertex, int i1, int i2);


	// Read out the span from loop[i0] to loop [i1-1] into an array.
	// If bMarkInvalid, then these values are set to -1 in loop
	static GEOMETRYCORE_API void ExtractSpan(TArray<int>& Loop, int i0, int i1, bool bMarkInvalid, TArray<int>& OutSpan);

	// count number of valid vertices in l between loop[i0] and loop[i1-1]
	static GEOMETRYCORE_API int CountSpan(const TArray<int>& Loop, int i0, int i1);

	// find the index of item in loop, starting at start index
	static GEOMETRYCORE_API int FindIndex(const TArray<int>& Loop, int Start, int Item);

	// count number of times item appears in loop
	static GEOMETRYCORE_API int CountInList(const TArray<int>& Loop, int Item);


};


} // end namespace UE::Geometry
} // end namespace UE
