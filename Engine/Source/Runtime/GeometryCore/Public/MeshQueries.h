// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Distance/DistPoint3Triangle3.h"
#include "Intersection/IntrRay3Triangle3.h"
#include "Intersection/IntrTriangle3Triangle3.h"
#include "BoxTypes.h"
#include "IndexTypes.h"
#include "Algo/Accumulate.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{


template <class TriangleMeshType>
class TMeshQueries
{
public:
	TMeshQueries() = delete;

	/**
	 * construct a DistPoint3Triangle3 object for a Mesh triangle
	 */
	static FDistPoint3Triangle3d TriangleDistance(const TriangleMeshType& Mesh, int TriIdx, FVector3d Point)
	{
		check(Mesh.IsTriangle(TriIdx));
		FTriangle3d tri;
		Mesh.GetTriVertices(TriIdx, tri.V[0], tri.V[1], tri.V[2]);
		FDistPoint3Triangle3d q(Point, tri);
		q.GetSquared();
		return q;
	}

	/**
	 * convenience function to construct a IntrRay3Triangle3 object for a Mesh triangle
	 */
	static FIntrRay3Triangle3d TriangleIntersection(const TriangleMeshType& Mesh, int TriIdx, const FRay3d& Ray)
	{
		check(Mesh.IsTriangle(TriIdx));
		FTriangle3d tri;
		Mesh.GetTriVertices(TriIdx, tri.V[0], tri.V[1], tri.V[2]);
		FIntrRay3Triangle3d q(Ray, tri);
		q.Find();
		return q;
	}

	/**
	 * Compute triangle centroid
	 * @param Mesh Mesh with triangle
	 * @param TriIdx Index of triangle
	 * @return Computed centroid
	 */
	static FVector3d GetTriCentroid(const TriangleMeshType& Mesh, int TriIdx)
	{
		FTriangle3d Triangle;
		Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		return Triangle.Centroid();
	}

	/**
	 * Compute the normal, area, and centroid of a triangle all together
	 * @param Mesh Mesh w/ triangle
	 * @param TriIdx Index of triangle
	 * @param Normal Computed normal (returned by reference)
	 * @param Area Computed area (returned by reference)
	 * @param Centroid Computed centroid (returned by reference)
	 */
	static void GetTriNormalAreaCentroid(const TriangleMeshType& Mesh, int TriIdx, FVector3d& Normal, double& Area, FVector3d& Centroid)
	{
		FTriangle3d Triangle;
		Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		Centroid = Triangle.Centroid();
		Normal = VectorUtil::NormalArea(Triangle.V[0], Triangle.V[1], Triangle.V[2], Area);
	}

	/**
	 * Get the average of the mesh vertices.
	 */
	static FVector3d GetMeshVerticesCentroid(const TriangleMeshType& Mesh)
	{
		FVector3d Centroid(0, 0, 0);
		for (int VertIdx = 0; VertIdx < Mesh.MaxVertexID(); VertIdx++)
		{
			if (Mesh.IsVertex(VertIdx))
			{
				Centroid += Mesh.GetVertex(VertIdx);
			}
		}
		int NumVertices = Mesh.VertexCount();
		if (NumVertices > 0)
		{
			Centroid /= (double)NumVertices;
		}
		return Centroid;
	}

	/**
	 * Get the volume of a mesh using a method that is more robust to inputs with holes
	 * @param Mesh Input shape
	 * @param DimScaleFactor Scale factor to apply to each dimension; useful for keeping volume calculation in a good range for floating point precision
	 * @return Mesh volume (scaled by DimScaleFactor^3)
	 */
	static double GetVolumeNonWatertight(const TriangleMeshType& Mesh, double DimScaleFactor = 1)
	{
		double Volume = 0.0;
		// computing wrt a centroid is enough to ensure that e.g. a plane will have volume 0 instead of arbitrary volume
		FVector3d RefPt = GetMeshVerticesCentroid(Mesh);
		for (int TriIdx = 0; TriIdx < Mesh.MaxTriangleID(); TriIdx++)
		{
			if (!Mesh.IsTriangle(TriIdx))
			{
				continue;
			}

			FVector3d V0, V1, V2;
			Mesh.GetTriVertices(TriIdx, V0, V1, V2);

			// (6x) volume of the tetrahedron formed by the triangles and the reference point
			FVector3d V1mRef = (V1 - RefPt) * DimScaleFactor;
			FVector3d V2mRef = (V2 - RefPt) * DimScaleFactor;
			FVector3d N = V2mRef.Cross(V1mRef);

			Volume += ((V0-RefPt) * DimScaleFactor).Dot(N);
		}

		return Volume * (1.0 / 6.0);
	}


	static FVector2d GetVolumeArea(const TriangleMeshType& Mesh)
	{
		double Volume = 0.0;
		double Area = 0;
		for (int TriIdx = 0; TriIdx < Mesh.MaxTriangleID(); TriIdx++)
		{
			if (!Mesh.IsTriangle(TriIdx))
			{
				continue;
			}

			FVector3d V0, V1, V2;
			Mesh.GetTriVertices(TriIdx, V0, V1, V2);

			// Get cross product of edges and (un-normalized) normal vector.
			FVector3d V1mV0 = V1 - V0;
			FVector3d V2mV0 = V2 - V0;
			FVector3d N = V2mV0.Cross(V1mV0);

			Area += N.Length();

			double tmp0 = V0.X + V1.X;
			double f1x = tmp0 + V2.X;
			Volume += N.X * f1x;
		}

		return FVector2d(Volume * (1.0/6.0), Area * .5f);
	}

	static FVector2d GetVolumeAreaCenter(const TriangleMeshType& Mesh, FVector3d& OutCenterOfMass)
	{
		double Volume = 0.0;
		double Area = 0;
		OutCenterOfMass = FVector3d::ZeroVector;

		// Compute quantities relative to the first vertex for more stable computation
		FVector3d RefVert = FVector3d::ZeroVector;
		for (int VertIdx = 0; VertIdx < Mesh.MaxVertexID(); ++VertIdx)
		{
			if (Mesh.IsVertex(VertIdx))
			{
				RefVert = Mesh.GetVertex(VertIdx);
				break;
			}
		}
		for (int TriIdx = 0; TriIdx < Mesh.MaxTriangleID(); TriIdx++)
		{
			if (!Mesh.IsTriangle(TriIdx))
			{
				continue;
			}

			FVector3d V0, V1, V2;
			Mesh.GetTriVertices(TriIdx, V0, V1, V2);
			// Subtract reference vertex for stability
			V0 -= RefVert;
			V1 -= RefVert;
			V2 -= RefVert;

			// Get cross product of edges and (un-normalized) normal vector.
			FVector3d V1mV0 = V1 - V0;
			FVector3d V2mV0 = V2 - V0;
			FVector3d N = V2mV0.Cross(V1mV0);
			Area += N.Length();

			FVector3d F1 = V0 + V1 + V2;
			FVector3d F2(
				V0.X * V0.X + V1.X * (V0.X + V1.X) + V2.X * F1.X,
				V0.Y * V0.Y + V1.Y * (V0.Y + V1.Y) + V2.Y * F1.Y,
				V0.Z * V0.Z + V1.Z * (V0.Z + V1.Z) + V2.Z * F1.Z);

			Volume += N.X * F1.X;
			OutCenterOfMass += N * F2;
		}

		if (Volume != 0.0)
		{
			OutCenterOfMass /= (Volume * 4.0);
		}

		OutCenterOfMass += RefVert;

		return FVector2d(Volume * (1.0 / 6.0), Area * .5);
	}


	static FVector2d GetVolumeArea(const TriangleMeshType& Mesh, const TArray<int>& TriIndices)
	{
		double Volume = 0.0;
		double Area = 0;
		for (int TriIdx : TriIndices)
		{
			if (!Mesh.IsTriangle(TriIdx))
			{
				continue;
			}

			FVector3d V0, V1, V2;
			Mesh.GetTriVertices(TriIdx, V0, V1, V2);

			// Get cross product of edges and (un-normalized) normal vector.
			FVector3d V1mV0 = V1 - V0;
			FVector3d V2mV0 = V2 - V0;
			FVector3d N = V2mV0.Cross(V1mV0);

			Area += N.Length();

			double tmp0 = V0.X + V1.X;
			double f1x = tmp0 + V2.X;
			Volume += N.X * f1x;
		}

		return FVector2d(Volume * (1.0 / 6.0), Area * .5f);
	}

	static FAxisAlignedBox3d GetTriBounds(const TriangleMeshType& Mesh, int TID)
	{
		FIndex3i TriInds = Mesh.GetTriangle(TID);
		FVector3d MinV, MaxV, V = (FVector3d)Mesh.GetVertex(TriInds.A);
		MinV = MaxV = V;
		for (int i = 1; i < 3; ++i)
		{
			V = (FVector3d)Mesh.GetVertex(TriInds[i]);
			if (V.X < MinV.X)				MinV.X = V.X;
			else if (V.X > MaxV.X)			MaxV.X = V.X;
			if (V.Y < MinV.Y)				MinV.Y = V.Y;
			else if (V.Y > MaxV.Y)			MaxV.Y = V.Y;
			if (V.Z < MinV.Z)				MinV.Z = V.Z;
			else if (V.Z > MaxV.Z)			MaxV.Z = V.Z;
		}
		return FAxisAlignedBox3d(MinV, MaxV);
	}

	static FAxisAlignedBox3d GetBounds(const TriangleMeshType& Mesh)
	{
		FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
		for (int i = 0; i < Mesh.MaxVertexID(); ++i)
		{
			if (Mesh.IsVertex(i))
			{
				Bounds.Contain(Mesh.GetVertex(i));
			}
		}
		return Bounds;
	}

	/** @return bounding box of subset of triangles of Mesh */
	template<typename EnumerableTriListType>
	static FAxisAlignedBox3d GetTrianglesBounds(
		const TriangleMeshType& Mesh, 
		const EnumerableTriListType& Triangles, 
		const FTransform& Transform = FTransform::Identity)
	{
		FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
		for (int32 tid : Triangles)
		{
			if (Mesh.IsTriangle(tid))
			{
				FVector3d A,B,C;
				Mesh.GetTriVertices(tid, A,B,C);		// cannot use GetTriBounds here unless it is a FDynamicMesh3!
				Bounds.Contain(Transform.TransformPosition(A));
				Bounds.Contain(Transform.TransformPosition(B));
				Bounds.Contain(Transform.TransformPosition(C));
			}
		}
		return Bounds;
	}

	/** @return bounding box of subset of vertices of Mesh */
	template<typename EnumerableTriListType>
	static FAxisAlignedBox3d GetVerticesBounds(
		const TriangleMeshType& Mesh, 
		const EnumerableTriListType& Vertices,
		const FTransform& Transform = FTransform::Identity)
	{
		FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
		for (int32 vid : Vertices)
		{
			if (Mesh.IsVertex(vid))
			{
				Bounds.Contain( Transform.TransformPosition(Mesh.GetVertex(vid)) );
			}
		}
		return Bounds;
	}

	// brute force search for nearest triangle to Point
	static int FindNearestTriangle_LinearSearch(const TriangleMeshType& Mesh, const FVector3d& P)
	{
		int tNearest = IndexConstants::InvalidID;
		double fNearestSqr = TNumericLimits<double>::Max();
		for (int TriIdx : Mesh.TriangleIndicesItr())
		{
			double distSqr = TriDistanceSqr(Mesh, TriIdx, P);
			if (distSqr < fNearestSqr)
			{
				fNearestSqr = distSqr;
				tNearest = TriIdx;
			}
		}
		return tNearest;
	}

	/**
	 * @return nearest point on Mesh to P, or P if nearest point was not found
	 */
	static FVector3d FindNearestPoint_LinearSearch(const TriangleMeshType& Mesh, const FVector3d& P)
	{
		FVector3d NearestPoint = P;
		double NearestSqr = TNumericLimits<double>::Max();
		for (int TriIdx : Mesh.TriangleIndicesItr())
		{
			FDistPoint3Triangle3d Query = TriangleDistance(Mesh, TriIdx, P);
			if (Query.GetSquared() < NearestSqr)
			{
				NearestSqr = Query.GetSquared();
				NearestPoint = Query.ClosestTrianglePoint;
			}
		}
		return NearestPoint;
	}



	/**
	 * Compute distance from Point to triangle in Mesh, with minimal extra objects/etc
	 */
	static double TriDistanceSqr(const TriangleMeshType& Mesh, int TriIdx, const FVector3d& Point)
	{
		FTriangle3d Triangle;
		Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

		FDistPoint3Triangle3d Distance(Point, Triangle);
		return Distance.GetSquared();
	}

	// brute force search for nearest triangle intersection
	static int FindHitTriangle_LinearSearch(const TriangleMeshType& Mesh, const FRay3d& Ray)
	{
		int tNearestID = IndexConstants::InvalidID;
		double fNearestT = TNumericLimits<double>::Max();
		FTriangle3d Triangle;

		for (int TriIdx = 0; TriIdx < Mesh.MaxTriangleID(); TriIdx++)
		{
			if (!Mesh.IsTriangle(TriIdx))
			{
				continue;
			}
			Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
			FIntrRay3Triangle3d Query(Ray, Triangle);
			if (Query.Find())
			{
				if (Query.RayParameter < fNearestT)
				{
					fNearestT = Query.RayParameter;
					tNearestID = TriIdx;
				}
			}
		}

		return tNearestID;
	}

	// brute force search for all triangle intersections, sorted
	static void FindHitTriangles_LinearSearch(const TriangleMeshType& Mesh, const FRay3d& Ray, TArray<TPair<float, int>>& SortedHitTriangles)
	{
		FTriangle3d Triangle;
		SortedHitTriangles.Empty();

		for (int TriIdx : Mesh.TriangleIndicesItr())
		{
			Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
			FIntrRay3Triangle3d Query(Ray, Triangle);
			if (Query.Find())
			{
				SortedHitTriangles.Emplace(Query.RayParameter, TriIdx);
			}
		}

		SortedHitTriangles.Sort([](const TPair<float, int>& A, const TPair<float, int>& B)
		{
			return A.Key < B.Key;
		});
	}

	/**
	 * brute force search for any intersecting triangles on two meshes
	 * @return Index pair of IDs of first intersecting triangles found, or InvalidID if no intersection found
	 */
	static FIndex2i FindIntersectingTriangles_LinearSearch(const TriangleMeshType& Mesh1, const TriangleMeshType& Mesh2)
	{
		for (int TI = 0; TI < Mesh1.MaxTriangleID(); TI++)
		{
			if (!Mesh1.IsTriangle(TI))
			{
				continue;
			}
			FTriangle3d Tri1;
			Mesh1.GetTriVertices(TI, Tri1.V[0], Tri1.V[1], Tri1.V[2]);
			for (int TJ = 0; TJ < Mesh2.MaxTriangleID(); TJ++)
			{
				if (!Mesh2.IsTriangle(TJ))
				{
					continue;
				}
				FTriangle3d Tri2;
				Mesh2.GetTriVertices(TJ, Tri2.V[0], Tri2.V[1], Tri2.V[2]);
				if (FIntrTriangle3Triangle3d::Intersects(Tri1, Tri2))
				{
					return FIndex2i(TI, TJ);
				}
			}
		}
		return FIndex2i::Invalid();
	}

	/**
	 * convenience function to construct a IntrRay3Triangle3 object for a Mesh triangle
	 */
	template<typename RayType = FRay3d>
	static FIntrRay3Triangle3d RayTriangleIntersection(const TriangleMeshType& Mesh, int TriIdx, const RayType& Ray)
	{
		FTriangle3d Triangle;
		Mesh.GetTriVertices(TriIdx, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

		TIntrRay3Triangle3<double, RayType> Query(Ray, Triangle);
		Query.Find();
		return Query;
	}

	/** 
	 * Compute the length for each edge and return the result as an array of Mesh.MaxEdgeID() size.
	 * When mesh is not compact, the length of the missing edges is set to -1.0
	 */
	static void GetAllEdgeLengths(const TriangleMeshType& Mesh, TArray<double>& Lengths, double& TotalLength)
	{
		Lengths.Init(-1.0, Mesh.MaxEdgeID());
		TotalLength = 0.0;
		for (const int32 EdgeID : Mesh.EdgeIndicesItr())
		{
			FVector3d vA, vB;
			Mesh.GetEdgeV(EdgeID, vA, vB);
			Lengths[EdgeID] = (vA - vB).Length();
			TotalLength += Lengths[EdgeID];
		}
	}

	/// Compute the mean edge length for the given mesh
	static double AverageEdgeLength(const TriangleMeshType& Mesh)
	{
		if (Mesh.EdgeCount() == 0) 
		{ 
			return 0.0; 
		}

		double SumLengths = Algo::TransformAccumulate(Mesh.EdgeIndicesItr(), [&Mesh](int EdgeIndex) -> double
		{
			FVector3d vA, vB;
			Mesh.GetEdgeV(EdgeIndex, vA, vB);
			return (vA - vB).Length();
		}, 0.0);

		return SumLengths / Mesh.EdgeCount();
	}

	/// Compute the longest edge length for the given mesh
	static double MaxEdgeLength(const TriangleMeshType& Mesh)
	{
		double MaxLength = 0.0;
		for (auto EdgeID : Mesh.EdgeIndicesItr())
		{
			FVector3d vA, vB;
			Mesh.GetEdgeV(EdgeID, vA, vB);
			MaxLength = FMath::Max(MaxLength, (vA - vB).Length());
		}

		return MaxLength;
	}

	/// Compute the shortest edge length for the given mesh
	static double MinEdgeLength(const TriangleMeshType& Mesh)
	{
		if (Mesh.EdgeCount() == 0)
		{
			return 0.0;
		}

		double MinLength = FMathd::MaxReal;
		for (auto EdgeID : Mesh.EdgeIndicesItr())
		{
			FVector3d vA, vB;
			Mesh.GetEdgeV(EdgeID, vA, vB);
			MinLength = FMath::Min(MinLength, (vA - vB).Length());
		}

		return MinLength;
	}

	/// Given a mesh and a subset of mesh edges, compute the total length of all the edges
	static double TotalEdgeLength(const TriangleMeshType& Mesh, const TArray<int>& Edges)
	{
		double AccumulatedLength = 0;
		for (int EdgeID : Edges)
		{
			if (Mesh.IsEdge(EdgeID))
			{
				FVector3d A, B;
				Mesh.GetEdgeV(EdgeID, A, B);
				AccumulatedLength += Distance(A, B);
			}
		}
		return AccumulatedLength;
	}


	/// Given a mesh and a subset of mesh edges, compute the min, max, and mean edge lengths
	static void EdgeLengthStatsFromEdges(const TriangleMeshType& Mesh, const TArray<int>& Edges, double& MinEdgeLength,
		double& MaxEdgeLength, double& AverageEdgeLength)
	{
		if (Mesh.EdgeCount() == 0)
		{
			MinEdgeLength = 0.0;
			MaxEdgeLength = 0.0;
			AverageEdgeLength = 0.0;
			return;
		}

		MinEdgeLength = BIG_NUMBER;
		MaxEdgeLength = -BIG_NUMBER;
		AverageEdgeLength = 0;
		int EdgeCount = 0;

		for (int EdgeID : Edges)
		{
			if (Mesh.IsEdge(EdgeID))
			{
				FVector3d A, B;
				Mesh.GetEdgeV(EdgeID, A, B);
				double Length = Distance(A, B);
				if (Length < MinEdgeLength) { MinEdgeLength = Length; }
				if (Length > MaxEdgeLength) { MaxEdgeLength = Length; }
				AverageEdgeLength += Length;
				++EdgeCount;
			}
		}

		AverageEdgeLength /= (double)EdgeCount;
	}

	/// Compute the min, max, and mean edge lengths for the given mesh. Optionally, choose a subest of size NumSamples
	/// and compute stats of that subset.
	static void EdgeLengthStats(const TriangleMeshType& Mesh, double& MinEdgeLength, double& MaxEdgeLength,
		double& AverageEdgeLength, int NumSamples = 0)
	{
		if (Mesh.EdgeCount() == 0)
		{
			MinEdgeLength = 0.0;
			MaxEdgeLength = 0.0;
			AverageEdgeLength = 0.0;
			return;
		}

		MinEdgeLength = BIG_NUMBER;
		MaxEdgeLength = -BIG_NUMBER;
		AverageEdgeLength = 0;
		int MaxID = Mesh.MaxEdgeID();

		// if we are only taking some samples, use a prime-modulo-loop instead of random
		int PrimeNumber = (NumSamples == 0) ? 1 : 31337;
		int MaxCount = (NumSamples == 0) ? MaxID : NumSamples;

		FVector3d A, B;
		int EdgeID = 0;
		int EdgeCount = 0;
		do
		{
			if (Mesh.IsEdge(EdgeID))
			{
				Mesh.GetEdgeV(EdgeID, A, B);
				double Length = Distance(A, B);
				if (Length < MinEdgeLength) MinEdgeLength = Length;
				if (Length > MaxEdgeLength) MaxEdgeLength = Length;
				AverageEdgeLength += Length;
				++EdgeCount;
			}
			EdgeID = (EdgeID + PrimeNumber) % MaxID;
		} while (EdgeID != 0 && EdgeCount < MaxCount);

		AverageEdgeLength /= (double)EdgeCount;
	}

	/// For each vertex on MeshA, compute the distance to the nearest point on the surface contained in SpatialB.
	/// @param MeshA The mesh whose vertex distances should be computed.
	/// @param SpatialB The target surface's acceleration structure.
	/// @param Distances For each vertex in MeshA, the distance to the closest point in SpatialB.
	template<typename MeshSpatialType>
	static void VertexToSurfaceDistances(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialB, TArray<double>& Distances)
	{
		check(SpatialB.SupportsNearestTriangle());
		Distances.SetNumZeroed(MeshA.MaxVertexID());

		ParallelFor(MeshA.MaxVertexID(), [&MeshA, &SpatialB, &Distances](int VertexID)
		{
			if (!MeshA.IsVertex(VertexID)) { return; }

			FVector3d VertexPosition = MeshA.GetVertex(VertexID);
			double DistSqr;
			SpatialB.FindNearestTriangle(VertexPosition, DistSqr);
			Distances[VertexID] = sqrt(DistSqr);
		});
	}

	/// Compute all vertex-to-surface distances in parallel. Serial raw loop to find max element (using Algo::MaxElement 
	/// was never faster in initial benchmarking.)
	/// @param MeshA The mesh whose maximum vertex distance should be computed.
	/// @param SpatialB The target surface's acceleration structure.
	/// @return The maximum distance to the surface contained in SpatialB over all vertices in MeshA.
	template<typename MeshSpatialType>
	static double HausdorffDistance(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialB)
	{
		TArray<double> Distances;
		VertexToSurfaceDistances(MeshA, SpatialB, Distances);

		double MaxDist = -BIG_NUMBER;
		for (auto& Dist : Distances)
		{
			MaxDist = FMath::Max(Dist, MaxDist);
		}

		return MaxDist;
	}

	/// Because Hausdorff distance is not symmetric, we compute the maximum of the distances between two surfaces.
	template<typename MeshSpatialType>
	static double TwoSidedHausdorffDistance(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialA,
											const TriangleMeshType& MeshB, const MeshSpatialType& SpatialB)
	{
		return FMath::Max(HausdorffDistance(MeshA, SpatialB), HausdorffDistance(MeshB, SpatialA));
	}

	/// Approximately compute the surface area of MeshA that is within the threshold distance of a target mesh
	/// 
	/// @param MeshA The mesh whose contact area should be computed.
	/// @param SpatialB The target surface's acceleration structure.
	/// @param ThresholdDistance The distance below which a point on MeshA is considered to be in contact with the target mesh
	/// @param SamplesPerArea Requested minimum samples per unit of area. Note all triangles will at least be sampled once regardless.
	/// @param MaxSamplesPerTri Optional maximum samples per triangle, if greater than 0.
	template<typename MeshSpatialType, typename MeshRealType = double>
	static double ApproximateContactArea(const TriangleMeshType& MeshA, const TTransform<MeshRealType>& TransformA, 
		const MeshSpatialType& SpatialB, const TTransform<MeshRealType>& TransformB, double ThresholdDistance, double SamplesPerArea, int32 MaxSamplesPerTri = -1)
	{
		TArray<float> ContactAreas;
		ContactAreas.SetNumZeroed(MeshA.MaxTriangleID());

		ParallelFor(MeshA.MaxTriangleID(),
			[&MeshA, &TransformA, &SpatialB, &TransformB, &ContactAreas, ThresholdDistanceSq = ThresholdDistance*ThresholdDistance, SamplesPerArea, MaxSamplesPerTri](int TriangleID)
		{
			if (!MeshA.IsTriangle(TriangleID))
			{
				return;
			}

			TVector<MeshRealType> A, B, C;
			MeshA.GetTriVertices(TriangleID, A, B, C);
			A = TransformA.TransformPosition(A);
			B = TransformA.TransformPosition(B);
			C = TransformA.TransformPosition(C);
			float TriangleArea = (float)VectorUtil::Area(A, B, C);
			if (TriangleArea <= 0) // no contact area from degenerate triangles
			{
				return;
			}
			FRandomStream RandStream(TriangleID);
			int32 NumSamples = FMath::Max(1, FMath::CeilToInt32(SamplesPerArea * TriangleArea));
			if (MaxSamplesPerTri > 0 && NumSamples > MaxSamplesPerTri)
			{
				NumSamples = MaxSamplesPerTri;
			}
			int32 ContactSamples = 0;
			for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
			{
				TVector<MeshRealType> P = VectorUtil::UniformSampleTrianglePoint<MeshRealType>((MeshRealType)RandStream.GetFraction(), (MeshRealType)RandStream.GetFraction(), A, B, C);
				int32 TriB_Unused;
				ContactSamples += (int32)SpatialB.IsWithinDistanceSquared((FVector3d)TransformB.InverseTransformPosition(P), ThresholdDistanceSq, TriB_Unused);
			}
			ContactAreas[TriangleID] = TriangleArea * ((float)ContactSamples / (float)NumSamples);
		});

		double TotalArea = 0;
		for (float Area : ContactAreas)
		{
			TotalArea += (double)Area;
		}
		return TotalArea;
	}

	/// Version of ApproximateContactArea assuming MeshA and SpatialB are in a common coordinate space
	template<typename MeshSpatialType, typename MeshRealType = double>
	static double ApproximateContactArea(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialB, double ThresholdDistance, double SamplesPerArea)
	{
		return ApproximateContactArea(MeshA, FTransform3d::Identity, SpatialB, FTransform3d::Identity, ThresholdDistance, SamplesPerArea);
	}


	/// Compute all vertex-to-surface distances in serial. Should only be used for debugging the parallel version above!
	template<typename MeshSpatialType>
	static void VertexToSurfaceDistancesSerial(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialB, TArray<double>& Distances)
	{
		check(SpatialB.SupportsNearestTriangle());
		Distances.SetNumZeroed(MeshA.MaxVertexID());

		for (int VertexID = 0; VertexID < MeshA.MaxVertexID(); ++VertexID)
		{
			if (!MeshA.IsVertex(VertexID)) { continue; }

			FVector3d VertexPosition = MeshA.GetVertex(VertexID);
			double DistSqr;
			SpatialB.FindNearestTriangle(VertexPosition, DistSqr);
			Distances[VertexID] = sqrt(DistSqr);
		}
	}

	/// Compute all distances in serial, then a serial raw loop to find max. Should only be used for debugging the 
	/// parallel version above!
	template<typename MeshSpatialType>
	static double HausdorffDistanceSerial(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialB)
	{
		TArray<double> Distances;
		VertexToSurfaceDistancesSerial(MeshA, SpatialB, Distances);

		double MaxDist = -BIG_NUMBER;
		for (auto& Dist : Distances)
		{
			MaxDist = FMath::Max(Dist, MaxDist);
		}

		return MaxDist;
	}

	template<typename MeshSpatialType>
	static double TwoSidedHausdorffDistanceSerial(const TriangleMeshType& MeshA, const MeshSpatialType& SpatialA,
												  const TriangleMeshType& MeshB, const MeshSpatialType& SpatialB)
	{
		return FMath::Max(HausdorffDistanceSerial(MeshA, SpatialB), HausdorffDistanceSerial(MeshB, SpatialA));
	}

	/**
	 * Compute various statistics on distances between two meshes
	 */
	template<typename MeshSpatialType>
	static void MeshDistanceStatistics(
		const TriangleMeshType& MeshA, 
		const MeshSpatialType& SpatialB,
		const TriangleMeshType* MeshB, 
		const MeshSpatialType* SpatialA,
		bool bSymmetric,
		double& MaxDistance,
		double& MinDistance,
		double& AverageDistance,
		double& RootMeanSqrDeviation
		)
	{
		TArray<double> Distances;
		VertexToSurfaceDistances(MeshA, SpatialB, Distances);

		if (bSymmetric && MeshB != nullptr && SpatialA != nullptr)
		{
			TArray<double> Distances2;
			VertexToSurfaceDistances(*MeshB, *SpatialA, Distances2);
			Distances.Append(Distances2);
		}

		double NumDistances = (double)Distances.Num();

		MaxDistance = -BIG_NUMBER;
		MinDistance = BIG_NUMBER;
		AverageDistance = 0;
		for (double& Dist : Distances)
		{
			MaxDistance = FMath::Max(Dist, MaxDistance);
			MinDistance = FMath::Min(Dist, MinDistance);
			AverageDistance += Dist / NumDistances;
			RootMeanSqrDeviation += (Dist * Dist);
		}
		RootMeanSqrDeviation = FMathd::Sqrt(RootMeanSqrDeviation / NumDistances);
	}

	/**
     * Retrieve the area and/or angle weights for each vertex of a triangle.
	 * 
     * @param Mesh the mesh to query
     * @param TriID the triangle index of the mesh to query
     * @param TriArea the area of the triangle
     * @param bWeightByArea if true, include weighting by the area of the triangle
     * @param bWeightByAngle if true, include weighting by the interior angles of the triangle
     */
    static FVector3d GetVertexWeightsOnTriangle(const TriangleMeshType& Mesh, int TriID, double TriArea, bool bWeightByArea, bool bWeightByAngle)
    {
        FVector3d TriNormalWeights = FVector3d::One();
        if (bWeightByAngle)
        {
            TriNormalWeights = Mesh.GetTriInternalAnglesR(TriID); // component-wise multiply by per-vertex internal angles
        }
        if (bWeightByArea)
        {
            TriNormalWeights *= TriArea;
        }
        return TriNormalWeights;
    }

	/**
	 * Get triangles that contain at least on vertex in the Vertices array.
	 * 
	 * @param Vertices Array of mesh vertex IDs.
	 * @param Triangles Triangle IDs containing at least one vertex ID in Vertices.
	 */
	static TArray<int32> GetVertexSelectedTriangles(const TriangleMeshType& Mesh, const TArray<int32>& Vertices)
	{
		TSet<int32> TriangleSet; // All triangles shared by vertices in Vertices array
		TriangleSet.Reserve(Vertices.Num());
		for (const int32 VID : Vertices)
		{
			for (const int32 TID : Mesh.VtxTrianglesItr(VID))
			{
				TriangleSet.Add(TID);
			}
		}

		return TriangleSet.Array();
	}

	/**
	 * Expand selection of vertices with one-ring neighbors.
	 * 
	 * @param Selection Array of Mesh vertex IDs.
     * @param ExpandedSelection All vertices in Selection plus one-ring neighbors for each vertex.
     * @param VIDToExpandedSelectionIdx Maps mesh Vertex ID to ExpandedSelection array index.
     */
    static void ExpandVertexSelectionToNeighbors(const TriangleMeshType& Mesh, 
                                    	   		 const TArray<int32>& Selection, 
                                    	   		 TArray<int32>& ExpandedSelection, 
                                    	   		 TMap<int32,int32>& VIDToExpandedSelectionIdx)
    {
        VIDToExpandedSelectionIdx.Reserve(Selection.Num());
        ExpandedSelection.Reserve(Selection.Num());

        int32 Idx = 0;
        for (const int32 SelectedVID : Selection)
        {
            if (!VIDToExpandedSelectionIdx.Contains(SelectedVID))
            {
                VIDToExpandedSelectionIdx.Add(SelectedVID, Idx++);
                ExpandedSelection.Add(SelectedVID);
            }

            for (const int32 NeighborVID : Mesh.VtxVerticesItr(SelectedVID))
            {
                if (!VIDToExpandedSelectionIdx.Contains(NeighborVID))
                {
                    VIDToExpandedSelectionIdx.Add(NeighborVID, Idx++);
                    ExpandedSelection.Add(NeighborVID);
                }
            }
        }
    }
};


} // end namespace UE::Geometry
} // end namespace UE