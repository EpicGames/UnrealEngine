// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshRegionBoundaryLoops

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "EdgeLoop.h"
#include "Util/SparseIndexCollectionTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * Extract FEdgeLoops on the boundary of a set of triangles of a mesh.
 * @todo this was an early port and possibly can be optimized
 */
class FMeshRegionBoundaryLoops
{
public:

	// INPUTS

	/** Mesh we are finding loops on */
	const FDynamicMesh3* Mesh = nullptr;
	/** Resulting set of loops filled by Compute() */
	TArray<FEdgeLoop> Loops;

	// OUTPUTS

	/** If true, we did not completely succeed in extracting all loops */
	bool bFailed = false;

public:
	FMeshRegionBoundaryLoops() {}

	GEOMETRYCORE_API FMeshRegionBoundaryLoops(const FDynamicMesh3* MeshIn, const TArray<int>& RegionTris, bool bAutoCompute = true);

	GEOMETRYCORE_API void SetMesh(const FDynamicMesh3* MeshIn, const TArray<int>& RegionTris);

	/**
	 * Find set of FEdgeLoops on the border of the input triangle set
	 * @return false if errors occurred, in this case output set is incomplete
	 */
	GEOMETRYCORE_API bool Compute();


	/** @return number of loops found by Compute() */
	int32 GetLoopCount() const
	{
		return Num();
	}

	/** @return number of loops found by Compute() */
	int32 Num() const
	{
		return Loops.Num();
	}

	/** @return Loop at the given index */
	const FEdgeLoop& operator[](int Index) const
	{
		return Loops[Index];
	}

	const TArray<FEdgeLoop>& GetLoops() const
	{
		return Loops;
	}

	/** @return index of loop with maximum number of vertices */
	GEOMETRYCORE_API int GetMaxVerticesLoopIndex() const;
	
	template<typename ElementType>
	using ElementIDAndValue = TPair<int32, ElementType>;
	template<typename ElementType>
	using VidOverlayMap = TMap<int32, ElementIDAndValue<ElementType>>;

	/**
	 * Generates a map from vertex ID's of the loop vertices to corresponding overlay element ID's 
	 * and values. The elements chosen are those associated with the internal triangle whose
	 * edge goes out of that vertex in the loop direction.
	 * The function is useful in cases where the triangles inside the loop will be deleted and replaced,
	 * and we want to keep the same UV's on the border vertices.
	 * Does not clear LoopVidsToOverlayElementsOut before use (simply adds to it), so could 
	 * be used for multiple loops.
	 *
	 * @returns false if there is an error (if the overlay does not have an element for one of the vids)
	 */
	template<typename StorageType, int ElementSize, typename ElementType>
	GEOMETRYCORE_API bool GetLoopOverlayMap(const FEdgeLoop& LoopIn,
		const TDynamicMeshOverlay<StorageType, ElementSize>& Overlay,
		VidOverlayMap<ElementType>& LoopVidsToOverlayElementsOut);

	/**
	 * Given a map generated by GetLoopOverlayMap(), checks that the overlay elements pointed to by the 
	 * map still exist in the overlay, and sets the ID to be invalid if not.
	 * This can be called after deleting the triangles inside a loop to update the generated map
	 * and remove any UV elements that no longer exist (because the vertex was on a UV seam).
	 */
	template<typename StorageType, int ElementSize, typename ElementType>
	GEOMETRYCORE_API void UpdateLoopOverlayMapValidity(VidOverlayMap<ElementType>& LoopVidsToOverlayElements,
		const TDynamicMeshOverlay<StorageType, ElementSize>& Overlay);

	/**
	* Find the edge loop border around a set of triangles of a Mesh.
	* This is computed via local walk and so does not create any full-mesh data structures.
	* However current implementation may not be efficient for large triangle sets.
	* Algorithm terminates if a non-manifold boundary is detected, and returns false if some triangles are unused.
	* @param Loop output loop will be stored here. This value is garbage if false is returned.
	* @return true if a single well-formed loop was found, false if non-manifold or failure case encountered
	*/
	static GEOMETRYCORE_API bool GetTriangleSetBoundaryLoop(const FDynamicMesh3& Mesh, const TArray<int32>& Tris, FEdgeLoop& Loop);

protected:

	// sets of included triangles and edges
	FIndexFlagSet Triangles;
	FIndexFlagSet Edges;
	TArray<int> edges_roi;

	bool IsEdgeOnBoundary(int eid) const { return Edges.Contains(eid); }

	// returns true for both internal and mesh boundary edges
	// tid_in and tid_out are triangles 'in' and 'out' of set, respectively
	GEOMETRYCORE_API bool IsEdgeOnBoundary(int eid, int& tid_in, int& tid_out) const;

	// return same indices as GetEdgeV, but oriented based on attached triangle
	GEOMETRYCORE_API FIndex2i GetOrientedEdgeVerts(int eID, int tid_in);

	// returns first two boundary edges, and count of total boundary edges
	GEOMETRYCORE_API int GetVertexBoundaryEdges(int vID, int& e0, int& e1);

	// e needs to be large enough (ie call GetVtxBoundaryEdges, or as large as max one-ring)
	// returns count, ie number of elements of e that were filled
	GEOMETRYCORE_API int GetAllVertexBoundaryEdges(int vID, TArray<int>& e);


	// [TODO] cache this : a dictionary? we will not need very many, but we will
	//   need each multiple times!
	GEOMETRYCORE_API FVector3d GetVertexNormal(int vid);

	// ok, bdry_edges[0...bdry_edges_count] contains the boundary edges coming out of bowtie_v.
	// We want to pick the best one to continue the loop that came : to bowtie_v on incoming_e.
	// If the loops are all sane, then we will get the smallest loops by "turning left" at bowtie_v.
	GEOMETRYCORE_API int FindLeftTurnEdge(int incoming_e, int bowtie_v, TArray<int>& bdry_edges, int bdry_edges_count, const FIndexFlagSet& used_edges);


	// This is called when loopV contains one or more "bowtie" vertices.
	// These vertices *might* be duplicated : loopV (but not necessarily)
	// If they are, we have to break loopV into subloops that don't contain duplicates.
	GEOMETRYCORE_API bool TryExtractSubloops(TArray<int>& loopV, const TArray<int>& loopE, const TArray<int>& bowties, TArray<FEdgeLoop>& SubLoopsOut);


};


} // end namespace UE::Geometry
} // end namespace UE
