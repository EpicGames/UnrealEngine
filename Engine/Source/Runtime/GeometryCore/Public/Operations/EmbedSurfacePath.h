// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshPlaneCut

#pragma once
#include "CoreMinimal.h"

#include "MathUtil.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "GeometryTypes.h"
#include "Curve/GeneralPolygon2.h"
#include "Selections/MeshFaceSelection.h"


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

enum class ESurfacePointType
{
	Vertex = 0,
	Edge = 1,
	Triangle = 2
};

struct FMeshSurfacePoint
{
	// The vertex, edge or triangle ID depending on PointType
	int ElementID;

	// Weights for each element in order; i.e. for an edge the surface point is at Edge.V[0] * BaryCoord[0] + Edge.V[1] * BaryCoord[1]. For a vertex, always (1,0,0)
	FVector3d BaryCoord;
	ESurfacePointType PointType;

	FMeshSurfacePoint() : ElementID(-1)
	{
	}
	FMeshSurfacePoint(int TriangleID, const FVector3d& BaryCoord) : ElementID(TriangleID), BaryCoord(BaryCoord), PointType(ESurfacePointType::Triangle)
	{
	}
	UE_DEPRECATED(5.8, "Use the static MakeEdgePoint method; note that it takes LerpParam instead of FirstCoordWt, which is 1-FirstCoordWt")
	FMeshSurfacePoint(int EdgeID, double FirstCoordWt) : ElementID(EdgeID), BaryCoord(FirstCoordWt, 1-FirstCoordWt, 0), PointType(ESurfacePointType::Edge)
	{
	}
	FMeshSurfacePoint(int VertexID) : ElementID(VertexID), BaryCoord(1,0,0), PointType(ESurfacePointType::Vertex)
	{
	}
	FMeshSurfacePoint(int InElementID, FVector3d InBaryCoord, ESurfacePointType InPointType) : ElementID(InElementID), BaryCoord(InBaryCoord), PointType(InPointType) 
	{
	}

	static FMeshSurfacePoint MakeEdgePoint(int32 EdgeID, double LerpParam)
	{
		return FMeshSurfacePoint(EdgeID, FVector3d(1 - LerpParam, LerpParam, 0.), ESurfacePointType::Edge);
	}

	// Get the parameter which should be passed to a SplitEdge() call to insert this surface point on its edge. Only meaningful for edge points.
	double GetEdgeSplitParam() const
	{
		checkSlow(PointType == ESurfacePointType::Edge);
		return BaryCoord[1];
	}

	GEOMETRYCORE_API FVector3d Pos(const FDynamicMesh3 *Mesh) const;
};

/**
 * Walk the surface of an FDynamicMesh to try find a planar path connecting two points.  Paths include every vertex and edge they need to cross.  Greedy algorithm will only return one path if there are multiple.
 */
//bool GEOMETRYCORE_API WalkMeshPlanar(
//	const FDynamicMesh3* Mesh, int StartTri, int EndVertID, FVector3d StartPt, int EndTri, FVector3d EndPt, FVector3d WalkPlaneNormal, 
//	TFunction<FVector3d(const FDynamicMesh3*, int)> VertexToPosnFn, bool bAllowBackwardsSearch, double AcceptEndPtOutsideDist,
//	double PtOnPlaneThreshold, TArray<TPair<FMeshSurfacePoint, int>>& WalkedPath, double BackwardsTolerance = FMathd::ZeroTolerance * 10);


/**
 * Represent a path on the surface of a mesh via barycentric coordinates and triangle references
 */
class FMeshSurfacePath
{
public:
	FDynamicMesh3 *Mesh;
	TArray<TPair<FMeshSurfacePoint, int>> Path; // Surface points paired with triangle to walk to get to next surface point
	bool bIsClosed;

public:

	/**
	 * Cut mesh with plane. Assumption is that plane normal is Z value.
	 */
	FMeshSurfacePath(FDynamicMesh3* Mesh) : Mesh(Mesh), bIsClosed(false)
	{
	}
	virtual ~FMeshSurfacePath() = default;

	// TODO: not clear if we need this -- helper to find the shared triangle (if it exists) that we could traverse to connect two surface points.  this gets a bit hairy and it's easier to keep track of the traversed triangles during the mesh walk that creates any path in the first place
	//static int FindSharedTriangle(const FDynamicMesh3* Mesh, const FMeshSurfacePoint& A, const FMeshSurfacePoint& B);
	
	/**
	 * @return True if the Path exactly sticks to the mesh surface, and never jumps to disconnected elements
	 */
	GEOMETRYCORE_API bool IsConnected() const;

	bool IsClosed() const
	{
		return bIsClosed;
	}

	void Reset()
	{
		Path.Reset();
		bIsClosed = false;
	}

	GEOMETRYCORE_API bool AddViaPlanarWalk(int StartTri, int StartVID, FVector3d StartPt, int EndTri, int EndVertID, FVector3d EndPt,
		FVector3d WalkPlaneNormal, TFunction<FVector3d(const FDynamicMesh3*, int)> VertexToPosnFn = nullptr,
		bool bAllowBackwardsSearch = true, double AcceptEndPtOutsideDist = FMathd::ZeroTolerance, 
		double PtOnPlaneThresholdSq = FMathf::ZeroTolerance*100, double BackwardsTolerance = FMathd::ZeroTolerance*10);
	// TODO: Also support geodesic walks, other alternatives?

	/**
	 * Make the embedded path into a closed loop.  Only succeeds if path start and end already share a triangle!
	 *
	 * @return true if succeeded
	 */
	GEOMETRYCORE_API bool ClosePath();

	/**
	* @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	*/
	virtual EOperationValidationResult Validate()
	{
		if (!IsConnected())
		{
			return EOperationValidationResult::Failed_UnknownReason;
		}

		return EOperationValidationResult::Ok;
	}

	// TODO: support for embedding arbitrary paths in mesh using a graph remesher
	///**
	// * Embed surface path in mesh, such that each point in path is a mesh vertex with a connecting edge.  Note that IsConnected must be true for this to succeed.
	// *
	// * @param bUpdatePath Updating the Path array with the new vertices (if false, the path will no longer be valid after running this function)
	// * @param PathVertices Indices of the vertices on the path after embedding succeeds; NOTE these will not be 1:1 with the input Path
	// * @param MeshGraphFn Function to create a mesh from an input 2D graph of labeled edges.
	// * @return true if embedding succeeded.
	// */
	//bool EmbedPath(bool bUpdatePath, TArray<int>& PathVertices, TFunction<bool(const TArray<FVector2d>& Vertices, const TArray<FIndex3i>& LabelledEdges, TArray<FVector2d>& OutVertices, TArray<int32>& OutVertexMap, TArray<FIndex3i>& OutTriangles)> MeshGraphFn);

	/**
	 * Additional options for controlling simple path embedding
	 */
	struct FEmbedSimplePathSettings
	{
		// Whether to snap intermediate triange/edge path points to vertices, using the SnapElementThresholdSq
		bool bSimplifyPathBySnapping = false;

		// Especially after simplification by snapping, we could create paths that revisit the same vertex multiple times --
		// this option will remove such loops by removing the path between the repeated vertex; i.e. Path A B C D B E becomes just A B E
		// Note this option should not be used if embedding curved paths, as the loop may be an intentional feature in this case
		bool bRemovePathLoops = false;

		// If an edge split would create an edge smaller than the snap threshold, allow an edge flip instead.
		// Note this is equivalent to splitting then immediately collapsing. Flip will not be performed if it would introduce a fold-over.
		bool bAllowEdgeFlipToCollapseTinyEdges = false;

		FEmbedSimplePathSettings() {}

		// @return Settings with all path simplification options enabled
		static FEmbedSimplePathSettings WithSimplification()
		{
			FEmbedSimplePathSettings Settings;
			Settings.bSimplifyPathBySnapping = true;
			Settings.bRemovePathLoops = true;
			Settings.bAllowEdgeFlipToCollapseTinyEdges = true;
			return Settings;
		}
	};

	/**
	 * Embed a surface path in mesh provided that the path only crosses vertices and edges except at the start and end, so we can add the path easily with local edge splits and possibly two triangle pokes (rather than needing general remeshing machinery)
	 *
	 * @param bUpdatePath Updating the Path array with the new vertices (if false, the path will no longer be valid after running this function)
	 * @param PathVertices Indices of the vertices on the path after embedding succeeds; NOTE these will not be 1:1 with the input Path
	 * @param bDoNotDuplicateFirstVertexID Useful if repeatedly calling EmbedSimplePath to extend a path. If true, will not add the first path vertex if it matches the last vertex of the initial, passed-in PathVertices.
	 * @param SnapElementThresholdSq Squared distance threshold below which path vertices can be snapped to existing elements
	 * @param Settings Additional options controlling how the path is embedded (e.g. to enable more aggressive snapping)
	 * @return true if embedding succeeded.
	 */
	GEOMETRYCORE_API bool EmbedSimplePath(bool bUpdatePath, TArray<int>& PathVertices, bool bDoNotDuplicateFirstVertexID = true, 
		double SnapElementThresholdSq = FMathf::ZeroTolerance*100, const FEmbedSimplePathSettings& Settings = FEmbedSimplePathSettings());

	// TODO: add functionality to delete 'inside' of path -- but how do we determine what is inside?  ref MeshFacesFromLoop.cs in geometry3sharp
};

/**
 * Embed a 2D path into a mesh by projection, starting the walk from a given triangle.  Optionally select the triangles inside the path.
 */
bool GEOMETRYCORE_API EmbedProjectedPath(FDynamicMesh3* Mesh, int StartTriID, FFrame3d Frame, const TArray<FVector2d>& Path2D, TArray<int>& OutPathVertices, TArray<int>& OutVertexCorrespondence, bool bClosePath, FMeshFaceSelection *EnclosedFaces = nullptr, double PtSnapVertexOrEdgeThresholdSq = FMathf::ZeroTolerance*100);

/**
 * Embed multiple 2D paths into a mesh by projection, starting the walks from the given triangles.  Optionally select the triangles inside the paths.
 */
bool GEOMETRYCORE_API EmbedProjectedPaths(FDynamicMesh3* Mesh, const TArrayView<const int> StartTriIDs, FFrame3d Frame, const TArrayView<const TArray<FVector2d>> AllPaths, TArray<TArray<int>>& OutAllPathVertices, TArray<TArray<int>>& OutAllVertexCorrespondence, bool bClosePaths, FMeshFaceSelection* EnclosedFaces, double PtSnapVertexOrEdgeThresholdSq = FMathf::ZeroTolerance*100);


} // end namespace UE::Geometry
} // end namespace UE
