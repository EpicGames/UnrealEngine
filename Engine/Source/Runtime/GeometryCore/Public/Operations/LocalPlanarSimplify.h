// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshPlaneCut

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "DynamicMesh/InfoTypes.h"


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;


//
// Class to support local planar simplification, to reduce excess edges created by plane cuts, edge loop insertions, and similar operations
// (Note a custom version of this logic also exists on the FMeshBoolean class)
//
class FLocalPlanarSimplify
{
public:
	// Simplification settings

	/** Degrees of deviation from coplanar that we will still simplify */
	double SimplificationAngleTolerance = .1;
	/**
	 * If triangle quality (aspect ratio) is worse than this threshold, only simplify in ways that improve quality.  If <= 0, triangle quality is ignored.
	 *  Note: For aspect ratio we use definition: 4*TriArea / (sqrt(3)*MaxEdgeLen^2), ref: https://people.eecs.berkeley.edu/~jrs/papers/elemj.pdf p.53
	 *  Equilateral triangles have value 1; Smaller values -> lower quality
	 */
	double TryToImproveTriQualityThreshold = .25;
	/** Prevent simplification from distorting triangle groups */
	bool bPreserveTriangleGroups = true;
	/** Prevent simplification from distorting vertex UVs */
	bool bPreserveVertexUVs = true;
	/** Prevent simplification from distorting overlay UVs */
	bool bPreserveOverlayUVs = true;
	/** When preserving UVs, sets maximum allowed change in UV coordinates from collapsing an edge, measured at the removed vertex */
	float UVDistortTolerance = FMathf::ZeroTolerance;
	/** Prevent simplification from distorting vertex normals */
	bool bPreserveVertexNormals = true;
	/** When preserving normals, sets maximum allowed change in normals from collapsing an edge, measured at the removed vertex in degrees */
	float NormalDistortTolerance = .01f;


	/**
	 * Simplify Mesh along the given edges, and update the edge set accordingly
	 * @param Mesh				Mesh to simplify
	 * @param InOutEdges		Edges to simplify along -- will be updated to remove any edges that have been collapsed by simplification
	 * @param ProcessCollapse	Optional function to be called whenever the simplification collapses an edge
	 */
	GEOMETRYCORE_API void SimplifyAlongEdges(FDynamicMesh3& Mesh, TSet<int32>& InOutEdges, TUniqueFunction<void(const DynamicMeshInfo::FEdgeCollapseInfo&)> ProcessCollapse = nullptr) const;

	/**
	 * Collapse degenerate edges. Note that SimplifyAlongEdges currently assumes that this (or a similar pass) is run beforehand
	 * @param Mesh Mesh to repair
	 * @param InOutEdges Set of edges to consider for collapse; optionally updated by removing edges at they are collapsed
	 * @param bRemoveAllDegenerateFromInputSet Whether we should also check whether the neighbor edges removed by collapse were also in the set. Note: Often not needed, and has additional processing cost.
	 * @param DegenerateEdgeLenTol Tolerance length at which we collapse edges
	 * @param TriangleSelection	Optional set tracking an active selection. Any triangles removed by collapse will also be removed from the set.*/
	GEOMETRYCORE_API static void CollapseDegenerateEdges(FDynamicMesh3& Mesh, TSet<int32>& InOutEdges, bool bRemoveAllDegenerateFromInputSet = false,
		double DegenerateEdgeLenTol = UE_DOUBLE_SMALL_NUMBER, TSet<int>* TriangleSelection = nullptr);


	//
	// The below helper functions are used by SimplifyAlongEdges, but also the similar customized versions of the same algorithm in FMeshBoolean and FMeshSelfUnion as well
	//

	/**
	 * Test if the triangles connected to a vertex are all coplanar
	 * @param Mesh The mesh to query
	 * @param VID The vertex to query
	 * @param DotTolerance If the dot product of two normals are >= this tolerance, the normals are considered equivalent
	 * @param The normal of the first triangle attached to the vertex.
	 * @return Whether all the triangles were coplanar
	 */
	static GEOMETRYCORE_API bool IsFlat(const FDynamicMesh3& Mesh, int VID, double DotTolerance, FVector3d& OutFirstNormal);

	/**
	 * Test if the triangles connected to a vertex could be flattened by folding along the given edge
	 * @param Mesh The mesh to query
	 * @param EID The edge around which the triangles are allowed to fold
	 * @param VID The vertex to query
	 * @param DotTolerance If the dot product of two normals are >= this tolerance, the normals are considered equivalent
	 * @param The normal of the first triangle attached to the vertex.
	 * @return Whether all the triangles were coplanar
	 */
	static GEOMETRYCORE_API bool IsDevelopableAlongEdge(const FDynamicMesh3& Mesh, int EID, int VID, double DotTolerance, FVector3d& NormalA, bool& bIsFlat);

	/**
	 * Test if a given edge collapse would cause a triangle flip or other unacceptable decrease in mesh quality
	 * Specialized for collapsing at flat triangles
	 * @param Mesh The mesh to query
	 * @param RemoveVNormal Only used if bHasMultipleNormals==false, i.e. if the surface is locally flat, to avoid recomputing the normal. The normal for all triangles surrounding RemoveV.
	 * @param RemoveV The vertex to consider removing by an edge collapse
	 * @param RemoveVPos The position of the vertex RemoveV
	 * @param KeepV The vertex that is not being removed in the edge collapse
	 * @param KeepVPos The position of vertex KeepV
	 * @param TryToImproveTriQualityThreshold Threshold for triangle quality (see comment for class member, above)
	 * @param bHasMultipleNormals If false, assume the triangulation is locally flat, so we can use ExpectNormal instead of recomputing the expected normal
	 */
	static GEOMETRYCORE_API bool CollapseWouldHurtTriangleQuality(
		const FDynamicMesh3& Mesh, const FVector3d& RemoveVNormal,
		int32 RemoveV, const FVector3d& RemoveVPos, int32 KeepV, const FVector3d& KeepVPos,
		double TryToImproveTriQualityThreshold, bool bHasMultipleNormals = false
	);

	/**
	 * Test if a given edge collapse would change the mesh shape, mesh triangle group shape, or UVs unacceptably
	 */
	static GEOMETRYCORE_API bool CollapseWouldChangeShapeOrUVs(
		const FDynamicMesh3& Mesh, const TSet<int>& PathEdgeSet, double DotTolerance, int SourceEID,
		int32 RemoveV, const FVector3d& RemoveVPos, int32 KeepV, const FVector3d& KeepVPos,
		const FVector3d& EdgeDir, bool bPreserveTriangleGroups, bool bPreserveUVsForMesh,
		bool bPreserveVertexUVs, bool bPreserveOverlayUVs, float UVToleranceSq,
		bool bPreserveVertexNormals, float NormalEqualCosThreshold);

};

} // end namespace UE::Geometry
} // end namespace UE
