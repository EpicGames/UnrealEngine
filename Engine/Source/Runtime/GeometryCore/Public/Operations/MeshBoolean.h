// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshBoolean

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"

#include "DynamicMeshEditor.h"

#include "Spatial/FastWinding.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Util/ProgressCancel.h"


namespace UE
{
namespace Geometry
{



/**
 * MeshBoolean -- perform a boolean operation on two input meshes.
 */
class FMeshBoolean
{
public:

	//
	// Inputs
	//
	const FDynamicMesh3* Meshes[2];
	const FTransformSRT3d Transforms[2];

	enum class EBooleanOp
	{
		Union,
		Difference,
		Intersect,
		TrimInside,
		TrimOutside,
		NewGroupInside,
		NewGroupOutside
	};
	EBooleanOp Operation;
	
	

	/** Tolerance distance for considering a point to be on a vertex or edge, especially during mesh-mesh cutting */
	double SnapTolerance = FMathf::ZeroTolerance * 1.0;

	/** Whether to do additional processing to try to remove degenerate edges */
	bool bCollapseDegenerateEdgesOnCut = true;
	/** Tolerance factor (multiplied by SnapTolerance) for removing short edges created by the cutting process; should be no more than 2 */
	double DegenerateEdgeTolFactor = 1.5;

	/** Threshold to determine whether triangle in one mesh is inside or outside of the other */
	double WindingThreshold = .5;

	/** Put the Result mesh in the same space as the input.  If true, ResultTransform will be the identity transform. */
	bool bPutResultInInputSpace = true;

	/** Weld newly-created cut edges where the input meshes meet.  If false, the input meshes will remain topologically disconnected. */
	bool bWeldSharedEdges = true;

	/** Control whether new edges should be tracked */
	bool bTrackAllNewEdges = false;

	/** Set this to be able to cancel running operation */
	FProgressCancel* Progress = nullptr;

	/** Control whether we attempt to auto-simplify the small planar triangles that the boolean operation tends to generate */
	bool bSimplifyAlongNewEdges = false;
	//
    // Simplification-specific settings (only relevant if bSimplifyAlongNewEdges==true):
	//
	/** Degrees of deviation from coplanar that we will still simplify */
	double SimplificationAngleTolerance = .1;
	/**
	 * If triangle quality (aspect ratio) is worse than this threshold, only simplify in ways that improve quality.  If <= 0, triangle quality is ignored.
	 *  Note: For aspect ratio we use definition: 4*TriArea / (sqrt(3)*MaxEdgeLen^2), ref: https://people.eecs.berkeley.edu/~jrs/papers/elemj.pdf p.53
	 *  Equilateral triangles have value 1; Smaller values -> lower quality
	 */
	double TryToImproveTriQualityThreshold = .5;
	/** Prevent simplification from distorting triangle groups */
	bool bPreserveTriangleGroups = true;
	/** Prevent simplification from distorting vertex UVs */
	bool bPreserveVertexUVs = true;
	/** Prevent simplification from distorting overlay UVs */
	bool bPreserveOverlayUVs = true;
	/** When preserving UVs, sets maximum allowed change in UV coordinates from collapsing an edge, measured at the removed vertex */
	float UVDistortTolerance = FMathf::ZeroTolerance;
	/** Prevent simplification from distorting vertex normals */
	bool bPreserveVertexNormals = true;
	/** When preserving normals, sets maximum allowed change in normals from collapsing an edge, measured at the removed vertex in degrees */
	float NormalDistortTolerance = .01f;
	/** If > -1, then only preserve the UVs of one of the input meshes.  Useful when cutting an artist-created asset w/ procedural geometry. */
	int PreserveUVsOnlyForMesh = -1;



	//
	// Input & Output (to be modified by algorithm)
	//

	// An existing mesh, to be filled with the boolean result
	FDynamicMesh3* Result;

	//
	// Output
	//

	/** Transform taking the result mesh back to the original space of the inputs */
	FTransformSRT3d ResultTransform;

	/** Boundary edges created by the mesh boolean algorithm failing to cleanly weld (doesn't include boundaries that already existed in source meshes) */
	TArray<int> CreatedBoundaryEdges;

	/** All edges created by mesh boolean algorithm. Only populated if bTrackAllNewEdges = true */
	TSet<int32> AllNewEdges;

	/**
	 * When doing an operation that merges two meshes (Union, Difference, Intersect)
	 * and bPopulateSecondMeshGroupMap is true, we populate a map from group ID's in the
	 * second mesh to group ID's in the result. Useful when the boolean is part of
	 * another operation that might want to track selection into the result.
	 * 
	 * TODO: It would be nice to have a similar triangle map, but it's more tedious to
	 * implement due to the mesh cuts that happen. We might also someday want something
	 * similar for the first mesh, where the group ID's will usually stay the same except
	 * when all triangles of a particular group are removed and the group ID is repurposed
	 * for the second mesh...
	 */
	bool bPopulateSecondMeshGroupMap = false;
	FIndexMapi SecondMeshGroupMap;

	// If set, will be populated with an array of source mesh indices per TriangleID in the result mesh,
	// indicating if that triangle was from the Meshes[0] or Meshes[1] source.
	// Note: For non-compact meshes, invalid triangle IDs will have arbitrary indices.
	// Note: Only filled for operations that combine both meshes (i.e., when OperatesOnSingleMesh(Operation) == false)
	TOptional<TArray<int8>> TrackPerTriangleSourceMesh;

public:

	/**
	 * Perform a boolean operation to combine two meshes into a provided output mesh.
	 * @param MeshA First mesh to combine
	 * @param TransformA Transform of MeshA
	 * @param MeshB Second mesh to combine
	 * @param TransformB Transform of MeshB
	 * @param OutputMesh Mesh to store output
	 * @param Operation How to combine meshes
	 */
	FMeshBoolean(const FDynamicMesh3* MeshA, const FTransformSRT3d& TransformA, const FDynamicMesh3* MeshB, const FTransformSRT3d& TransformB,
				 FDynamicMesh3* OutputMesh, EBooleanOp Operation)
		: Meshes{ MeshA, MeshB }, Transforms{ TransformA, TransformB }, Operation(Operation), Result(OutputMesh)
	{
		check(MeshA != nullptr && MeshB != nullptr && OutputMesh != nullptr);
	}

	FMeshBoolean(const FDynamicMesh3* MeshA, const FDynamicMesh3* MeshB, FDynamicMesh3* OutputMesh, EBooleanOp Operation)
		: Meshes{ MeshA, MeshB }, Transforms{ FTransformSRT3d::Identity(), FTransformSRT3d::Identity() }, Operation(Operation), Result(OutputMesh)
	{
		check(MeshA != nullptr && MeshB != nullptr && OutputMesh != nullptr);
	}

	virtual ~FMeshBoolean() = default;
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Compute the Boolean operation, using winding number for inside/outside tests
	 * @return true if operation succeeds
	 */
	GEOMETRYCORE_API bool Compute();

	// Helper to test if an operation will incorporate/edit both meshes, or just one
	// Note: If the operation works on a single mesh, then TrackPerTriangleSourceMesh will not be filled
	static bool OperatesOnSingleMesh(EBooleanOp Operation)
	{
		return Operation == EBooleanOp::TrimInside || Operation == EBooleanOp::TrimOutside || Operation == EBooleanOp::NewGroupInside || Operation == EBooleanOp::NewGroupOutside;
	}

	// Data passed to the FCustomInsideMeshTest::IsPointInsideFn
	struct FCustomInsideTestContext
	{
		// AABB Tree for the test mesh; always non-null
		FDynamicMeshAABBTree3* Spatial;
		// Note: OptionalWindingTree will be null when FCustomInsideMeshTest has bRequiresWinding == false
		TFastWindingTree<FDynamicMesh3>* OptionalWindingTree;
	};

	// Function type for custom inside/outside query functions
	// Note: If implementing a custom function that needs to access the underlying mesh, it is accessible via Context.Spatial->GetMesh()
	using TestInsideFunctionType = TFunction<bool(const FVector3d& TestPosition, const FCustomInsideTestContext& Context)>;
	struct FCustomInsideMeshTest
	{
		TestInsideFunctionType IsPointInsideFn;
		bool bRequiresWinding = true;

		/**
		 * Construct a custom class for testing whether a point is inside a mesh.
		 * Note: This will be called in the local space of the boolean, which is not the initial / world-space of the input meshes
		 * 
		 * @param IsPointInsideFn Custom function reference to test if a given point is inside a mesh.
		 * @param bRequiresWinding Whether the winding tree needs to be computed for the custom function. If false, the OptionalWindingTree passed to IsPointInsideFn will be null.
		 */
		FCustomInsideMeshTest(TestInsideFunctionType IsPointInsideFn, bool bRequiresWinding = true)
			: IsPointInsideFn(IsPointInsideFn), bRequiresWinding(bRequiresWinding)
		{}
	};

	/**
	 * Compute the Boolean operation with a custom function determining if points are inside the given mesh
	 */
	GEOMETRYCORE_API bool ComputeWithCustomInside(FCustomInsideMeshTest TestMeshA, FCustomInsideMeshTest TestMeshB);

protected:
	/** If this returns true, abort computation.  */
	virtual bool Cancelled()
	{
		return (Progress == nullptr) ? false : Progress->Cancelled();
	}

private:

	GEOMETRYCORE_API int FindNearestEdge(const FDynamicMesh3& OnMesh, const TArray<int>& EIDs, FVector3d Pos);

	GEOMETRYCORE_API bool MergeEdges(const FMeshIndexMappings& IndexMaps, FDynamicMesh3* CutMesh[2], const TArray<int> CutBoundaryEdges[2], const TMap<int, int>& AllVIDMatches);

	GEOMETRYCORE_API void SimplifyAlongNewEdges(int NumMeshesToProcess, FDynamicMesh3* CutMesh[2], TArray<int> CutBoundaryEdges[2], TMap<int, int>& AllVIDMatches);

};


} // end namespace UE::Geometry
} // end namespace UE
