// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshMeshCut

// Functionality to cut a mesh with itself (FMeshSelfCut) or with another mesh (FMeshMeshCut)

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "Spatial/PointHashGrid3.h"
#include "Spatial/MeshAABBTree3.h" // for MeshIntersection::FIntersectionsQueryResult

#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
namespace Geometry
{

class FMeshSelfCut
{
public:
	//
	// Inputs
	//

	/** Mesh to cut -- note that FMeshSelfCut::Cut is destructive, so this are an output */
	FDynamicMesh3* Mesh;

	/** Tolerance distance for considering a point to be on a vertex, edge or plane */
	double SnapTolerance = FMathf::ZeroTolerance * 100.0;

	/** If true, detect coplanar faces and re-triangulate so that the triangulations match 1:1 (TODO: not implemented!) */
	bool bCutCoplanar = false;

	/** If true, record vertex insertions */
	bool bTrackInsertedVertices = false;

	/**
	 * Outputs
	 */

	/**
	 * Packed chains of vertex IDs, representing the vertices for each segment in the mesh
	 * Packed as the number of vertices in that chain, followed by that many vertex ids, per segment
	 * NOT 1:1 w/ segments; some segments may have failed to insert
	 *
	 * Empty if bTrackInsertedVertices == false
	 */
	TArray<int> VertexChains;

	FMeshSelfCut(FDynamicMesh3* Mesh) : Mesh(Mesh)
	{
	}

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Split mesh along the provided intersections
	 * @param Intersections A set of mesh intersections, for example as returned by TMeshAABBTree3::FindAllSelfIntersections
	 */
	GEOMETRYCORE_API bool Cut(const MeshIntersection::FIntersectionsQueryResult& Intersections);

	void ResetOutputs()
	{
		VertexChains.Reset();
	}

	// TODO: pull out all internal functionality from the private section of FMeshMeshCut, put as much of it as possible on FCutWorkingInfo, move all that to the cpp file!
	// and then re-use the functionality for self-cutting
};

/**
 * Cut a mesh where it crosses a second mesh -- resolving all intersections, but not deleting geometry.  Optionally resolve intersections mutually for both meshes.
 */
class FMeshMeshCut
{
public:

	//
	// Inputs
	//

	/** Meshes to cut -- note that FMeshMeshCut::Cut is destructive, so these are also outputs */
	FDynamicMesh3* Mesh[2];

	/** Tolerance distance for considering a point to be on a vertex, edge or plane */
	double SnapTolerance = FMathf::ZeroTolerance * 100.0;

	/** If true, modify both meshes to split at crossing points; otherwise only modify MeshA */
	bool bMutuallyCut = true;

	/** If true, detect coplanar faces and re-triangulate so that the triangulations match 1:1 (TODO: not implemented!) */
	bool bCutCoplanar = false;

	/** If true, record vertex insertions */
	bool bTrackInsertedVertices = false;

	/**
	 * Outputs
	 */

	 /**
	  * Packed chains of vertex IDs, representing the vertices for each segment in the mesh
	  * Packed as the number of vertices in that chain, followed by that many vertex ids, per segment
	  * NOT 1:1 w/ segments; some segments may have failed to insert
	  *
	  * Empty if bTrackInsertedVertices == false
	  */
	TArray<int> VertexChains[2];

	/**
	 * For each intersection segment, where the corresponding elements start in VertexChains.
	 * May contain IndexConstants::InvalidIndex for some segments, in cases where cutting a segment failed
	 *
	 * Empty if bTrackInsertedVertices == false
	 */
	TArray<int> SegmentToChain[2];


	FMeshMeshCut(FDynamicMesh3* MeshA, FDynamicMesh3* MeshB) : Mesh{ MeshA, MeshB }
	{
	}

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Split mesh(es) along the provided intersections
	 * @param Intersections A set of mesh-mesh intersections, for example as returned by TMeshAABBTree3::FindAllIntersections
	 */
	GEOMETRYCORE_API bool Cut(const MeshIntersection::FIntersectionsQueryResult& Intersections);


	void ResetOutputs()
	{
		for (int MeshIdx = 0; MeshIdx < 2; MeshIdx++) // reset both regardless of how many will be processed next
		{
			VertexChains[MeshIdx].Reset();
			SegmentToChain[MeshIdx].Reset();
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE
