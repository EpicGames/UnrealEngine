// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"

#include "DynamicMeshEditor.h"

#include "Spatial/FastWinding.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Util/ProgressCancel.h"

#include "Operations/MeshBoolean.h" // for shared utility functions


namespace UE
{
namespace Geometry
{

/**
 * MeshSelfUnion -- perform a "Mesh Boolean" style union of a mesh on itself, resolving any self intersections and welding the new boundaries as needed
 */
class FMeshSelfUnion
{
public:

	//
	// Inputs
	//
	
	/** Whether to do additional processing to try to remove degenerate edges */
	bool bCollapseDegenerateEdgesOnCut = true;
	/** Tolerance factor (multiplied by SnapTolerance) for removing short edges created by the cutting process; should be no more than 2 */
	double DegenerateEdgeTolFactor = 1.5;

	/** Tolerance distance for considering a point to be on a vertex or edge, especially during mesh-mesh cutting */
	double SnapTolerance = FMathf::ZeroTolerance * 1.0;

	/** Amount we nudge samples off the surface when evaluating winding number, to avoid numerical issues */
	double NormalOffset = FMathf::ZeroTolerance * 1.0;

	/** Threshold to determine whether triangle in one mesh is inside or outside of the other */
	double WindingThreshold = .5;

	/** Whether to remove visible "open" geometry */
	bool bTrimFlaps = false;

	/** Weld newly-created cut edges where the mesh is unioned with itself.  If false, newly joined surfaces remain topologically disconnected. */
	bool bWeldSharedEdges = true;

	/** Control whether new edges should be tracked */
	bool bTrackAllNewEdges = false;

	/** Set this to be able to cancel running operation */
	FProgressCancel* Progress = nullptr;

	/** Control whether we attempt to auto-simplify the small planar triangles that the boolean operation tends to generate */
	bool bSimplifyAlongNewEdges = false;
	//
	// Simplification-specific settings (only relevant if bSimplifyAlongNewEdges==true):
	//
	/** Degrees of deviation from coplanar that we will still simplify */
	double SimplificationAngleTolerance = .1;
	/**
	 * If triangle quality (aspect ratio) is worse than this threshold, only simplify in ways that improve quality.  If <= 0, triangle quality is ignored.
	 *  Note: For aspect ratio we use definition: 4*TriArea / (sqrt(3)*MaxEdgeLen^2), ref: https://people.eecs.berkeley.edu/~jrs/papers/elemj.pdf p.53
	 *  Equilateral triangles have value 1; Smaller values -> lower quality
	 */
	double TryToImproveTriQualityThreshold = .25;
	/** Prevent simplification from distorting triangle groups */
	bool bPreserveTriangleGroups = true;
	/** Prevent simplification from distorting vertex UVs */
	bool bPreserveVertexUVs = true;
	/** Prevent simplification from distorting overlay UVs */
	bool bPreserveOverlayUVs = true;
	/** When preserving UVs, sets maximum allowed change in UV coordinates from collapsing an edge, measured at the removed vertex */
	float UVDistortTolerance = FMathf::ZeroTolerance;
	/** Prevent simplification from distorting vertex normals */
	bool bPreserveVertexNormals = true;
	/** When preserving normals, sets maximum allowed change in normals from collapsing an edge, measured at the removed vertex in degrees */
	float NormalDistortTolerance = .01f;


	//
	// Input & Output (to be modified by algorithm)
	//

	// The input mesh, to be modified
	FDynamicMesh3* Mesh;

	//
	// Output
	//

	/** Boundary edges created by the mesh boolean algorithm failing to cleanly weld (doesn't include boundaries that already existed in source mesh) */
	TArray<int> CreatedBoundaryEdges;

	/** All edges created by mesh boolean algorithm. Only populated if bTrackAllNewEdges = true */
	TSet<int32> AllNewEdges;

public:

	FMeshSelfUnion(FDynamicMesh3* MeshIn)
		: Mesh(MeshIn)
	{
		check(MeshIn != nullptr);
	}

	virtual ~FMeshSelfUnion() = default;
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Compute the plane cut by splitting mesh edges that cross the cut plane, and then deleting any triangles
	 * on the positive side of the cutting plane.
	 * @return true if operation succeeds
	 */
	GEOMETRYCORE_API bool Compute();

protected:
	/** If this returns true, abort computation.  */
	virtual bool Cancelled()
	{
		return (Progress == nullptr) ? false : Progress->Cancelled();
	}

private:

	GEOMETRYCORE_API int FindNearestEdge(const TArray<int>& EIDs, const TArray<int>& BoundaryNbrEdges, FVector3d Pos);

	GEOMETRYCORE_API bool MergeEdges(const TArray<int>& CutBoundaryEdges, const TMap<int, int>& AllVIDMatches);

	GEOMETRYCORE_API void SimplifyAlongNewEdges(TArray<int>& CutBoundaryEdges, TMap<int, int>& FoundMatches);

};


} // end namespace UE::Geometry
} // end namespace UE
