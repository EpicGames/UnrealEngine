// Copyright Epic Games, Inc. All Rights Reserved.

// ported from geometry3Sharp Box3

#pragma once

#include "Math/OrientedBox.h"
#include "VectorTypes.h"
#include "BoxTypes.h"
#include "FrameTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TOrientedBox3 is a non-axis-aligned 3D box defined by a 3D frame and extents along the axes of that frame
 * The frame is at the center of the box.
 */
template<typename RealType>
struct TOrientedBox3
{
	// available for porting: ContainPoint


	/** 3D position (center) and orientation (axes) of the box */
	TFrame3<RealType> Frame;
	/** Half-dimensions of box measured along the three axes */
	TVector<RealType> Extents;

	TOrientedBox3() : Extents(1,1,1) {}

	/**
	 * Create axis-aligned box with given Origin and Extents
	 */
	TOrientedBox3(const TVector<RealType>& Origin, const TVector<RealType> & ExtentsIn)
		: Frame(Origin), Extents(ExtentsIn)
	{
	}

	/**
	 * Create oriented box with given Frame and Extents
	 */
	TOrientedBox3(const TFrame3<RealType>& FrameIn, const TVector<RealType> & ExtentsIn)
		: Frame(FrameIn), Extents(ExtentsIn)
	{
	}

	/**
	 * Create oriented box from axis-aligned box
	 */
	TOrientedBox3(const TAxisAlignedBox3<RealType>& AxisBox)
		: Frame(AxisBox.Center()), Extents((RealType)0.5 * AxisBox.Diagonal())
	{
	}

	// Convert to Core Math's FOrientedBox
	explicit operator FOrientedBox() const
	{
		FOrientedBox ToRet;
		ToRet.Center = (FVector)Frame.Origin;
		ToRet.AxisX = (FVector)Frame.X();
		ToRet.AxisY = (FVector)Frame.Y();
		ToRet.AxisZ = (FVector)Frame.Z();
		ToRet.ExtentX = (double)Extents.X;
		ToRet.ExtentY = (double)Extents.Y;
		ToRet.ExtentZ = (double)Extents.Z;
		return ToRet;
	}
	
	explicit operator TBox<RealType>() const
	{
		TBox<RealType> ToRet;
		EnumerateCorners([&](const TVector<RealType>& Corner) { ToRet += Corner; });
		return ToRet;
	}

	// Convert from Core Math's FOrientedBox
	TOrientedBox3(const FOrientedBox& Box) :
		Frame(TVector<RealType>(Box.Center),
			TVector<RealType>(Box.AxisX),
			TVector<RealType>(Box.AxisY),
			TVector<RealType>(Box.AxisZ)),
		Extents((RealType)Box.ExtentX, (RealType)Box.ExtentY, (RealType)Box.ExtentZ)
	{
	}

	/** @return box with unit dimensions centered at origin */
	static TOrientedBox3<RealType> UnitZeroCentered() { return TOrientedBox3<RealType>(TVector<RealType>::Zero(), (RealType)0.5*TVector<RealType>::One()); }

	/** @return box with unit dimensions where minimum corner is at origin */
	static TOrientedBox3<RealType> UnitPositive() { return TOrientedBox3<RealType>((RealType)0.5*TVector<RealType>::One(), (RealType)0.5*TVector<RealType>::One()); }


	/** @return center of the box */
	TVector<RealType> Center() const { return Frame.Origin; }

	/** @return X axis of the box */
	TVector<RealType> AxisX() const { return Frame.X(); }

	/** @return Y axis of the box */
	TVector<RealType> AxisY() const { return Frame.Y(); }

	/** @return Z axis of the box */
	TVector<RealType> AxisZ() const { return Frame.Z(); }

	/** @return an axis of the box */
	TVector<RealType> GetAxis(int AxisIndex) const { return Frame.GetAxis(AxisIndex); }

	/** @return vector from minimum-corner to maximum-corner of box */
	inline TVector<RealType> Diagonal() const
	{
		return Frame.PointAt(Extents.X, Extents.Y, Extents.Z) - Frame.PointAt(-Extents.X, -Extents.Y, -Extents.Z);
	}

	/** @return volume of box */
	inline RealType Volume() const
	{
		return (RealType)8 * FMath::Max(0, Extents.X) * FMath::Max(0, Extents.Y) * FMath::Max(0, Extents.Z);
	}

	/** @return surface area of box */
	inline RealType SurfaceArea() const
	{
		TVector<RealType> ClampExtents(FMath::Max(0, Extents.X), FMath::Max(0, Extents.Y), FMath::Max(0, Extents.Z));
		return (RealType)8 * (ClampExtents.X * ClampExtents.Y + ClampExtents.X * ClampExtents.Z + ClampExtents.Y * ClampExtents.Z);
	}

	/** @return true if box contains point */
	inline bool Contains(const TVector<RealType>& Point) const
	{
		TVector<RealType> InFramePoint = Frame.ToFramePoint(Point);
		return (TMathUtil<RealType>::Abs(InFramePoint.X) <= Extents.X) &&
			(TMathUtil<RealType>::Abs(InFramePoint.Y) <= Extents.Y) &&
			(TMathUtil<RealType>::Abs(InFramePoint.Z) <= Extents.Z);
	}

	/** @return true if box is valid (has non-negative extents) */
	bool IsValid() const
	{
		return Extents.X >= 0 && Extents.Y >= 0 && Extents.Z >= 0;
	}

	// corners [ (-x,-y), (x,-y), (x,y), (-x,y) ], -z, then +z
	//
	//   7---6     +z       or        3---2     -z
	//   |\  |\                       |\  |\
    //   4-\-5 \                      0-\-1 \
    //    \ 3---2                      \ 7---6   
	//     \|   |                       \|   |
	//      0---1  -z                    4---5  +z
	//
	// @todo does this ordering make sense for UE? we are in LHS instead of RHS here
	// if this is modified, likely need to update IndexUtil::BoxFaces and BoxFaceNormals

	/**
	 * @param Index corner index in range 0-7
	 * @return Corner point on the box identified by the given index. See diagram in OrientedBoxTypes.h for index/corner mapping.
	 */
	TVector<RealType> GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 7);
		RealType dx = (((Index & 1) != 0) ^ ((Index & 2) != 0)) ? (Extents.X) : (-Extents.X);
		RealType dy = ((Index / 2) % 2 == 0) ? (-Extents.Y) : (Extents.Y);
		RealType dz = (Index < 4) ? (-Extents.Z) : (Extents.Z);
		return Frame.PointAt(dx, dy, dz);
	}

	/**
	 * Call CornerPointFunc(FVector3) for each of the 8 box corners. Order is the same as GetCorner(X).
	 * This is more efficient than calling GetCorner(X) because the Rotation matrix is only computed once.
	 */
	template<typename PointFuncType>
	void EnumerateCorners(PointFuncType CornerPointFunc) const
	{
		TMatrix3<RealType> RotMatrix = Frame.Rotation.ToRotationMatrix();
		RealType X = Extents.X, Y = Extents.Y, Z = Extents.Z;
		CornerPointFunc( RotMatrix*TVector<RealType>(-X,-Y,-Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>( X,-Y,-Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>( X, Y,-Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>(-X, Y,-Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>(-X,-Y, Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>( X,-Y, Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>( X, Y, Z) + Frame.Origin );
		CornerPointFunc( RotMatrix*TVector<RealType>(-X, Y, Z) + Frame.Origin );
	}


	/**
	 * Call CornerPointPredicate(FVector3) for each of the 8 box corners, with early-out if any call returns false
	 * @return true if all tests pass
	 */
	template<typename PointPredicateType>
	bool TestCorners(PointPredicateType CornerPointPredicate) const
	{
		TMatrix3<RealType> RotMatrix = Frame.Rotation.ToRotationMatrix();
		RealType X = Extents.X, Y = Extents.Y, Z = Extents.Z;
		return CornerPointPredicate(RotMatrix * TVector<RealType>(-X, -Y, -Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(X, -Y, -Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(X, Y, -Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(-X, Y, -Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(-X, -Y, Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(X, -Y, Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(X, Y, Z) + Frame.Origin) &&
			CornerPointPredicate(RotMatrix * TVector<RealType>(-X, Y, Z) + Frame.Origin);
	}



	/**
	 * Get whether the corner at Index (see diagram in GetCorner documentation comment) is in the negative or positive direction for each axis
	 * @param Index corner index in range 0-7
	 * @return Index3i with 0 or 1 for each axis, 0 if corner is in the negative direction for that axis, 1 if in the positive direction
	 */
	static FIndex3i GetCornerSide(int Index)
	{
		check(Index >= 0 && Index <= 7);
		return FIndex3i(
			(((Index & 1) != 0) ^ ((Index & 2) != 0)) ? 1 : 0,
			((Index / 2) % 2 == 0) ? 0 : 1,
			(Index < 4) ? 0 : 1
		);
	}



	/**
	 * Find squared distance to box.
	 * @param Point input point
	 * @return Squared distance from point to box, or 0 if point is inside box
	 */
	RealType DistanceSquared(TVector<RealType> Point) const
	{
		TVector<RealType> Local = Frame.ToFramePoint(Point);
		TVector<RealType> BeyondExtents = Local.GetAbs() - Extents;
		TVector<RealType> ClampedBeyond( // clamp negative (inside) to zero
			FMath::Max((RealType)0, BeyondExtents.X),
			FMath::Max((RealType)0, BeyondExtents.Y),
			FMath::Max((RealType)0, BeyondExtents.Z));
		return ClampedBeyond.SquaredLength();
	}

	/**
	 * Find signed distance to box surface
	 * @param Point input point
	 * @return Signed distance from point to box surface; negative if point is inside box
	 */
	RealType SignedDistance(TVector<RealType> Point) const
	{
		TVector<RealType> Local = Frame.ToFramePoint(Point);
		TVector<RealType> VsExtents = Local.GetAbs() - Extents;
		RealType MaxComponent = VsExtents.GetMax();
		if (MaxComponent < 0) // Inside the box on every dimension
		{
			// The least-inside dimension gives the distance to the box surface
			return MaxComponent;
		}
		else // Outside the box along at least one dimension
		{
			// clamp negative (inside) to zero
			TVector<RealType> ClampedBeyond(
				FMath::Max((RealType)0, VsExtents.X),
				FMath::Max((RealType)0, VsExtents.Y),
				FMath::Max((RealType)0, VsExtents.Z)
			);
			return ClampedBeyond.Length();
		}
	}

	 /**
	  * Find closest point on box
	  * @param Point input point
	  * @return Closest point on box. Input point is returned if it is inside box.
	  */
	 TVector<RealType> ClosestPoint(const TVector<RealType>& Point) const
	 {
		 TMatrix3<RealType> RotMatrix = Frame.Rotation.ToRotationMatrix();
		 TVector<RealType> FromOrigin = Point - Frame.Origin;
		 // Transform to local space
		 TVector<RealType> Local(RotMatrix.TransformByTranspose(FromOrigin));
		 // Clamp to box
		 Local.X = FMath::Clamp(Local.X, -Extents.X, Extents.X);
		 Local.Y = FMath::Clamp(Local.Y, -Extents.Y, Extents.Y);
		 Local.Z = FMath::Clamp(Local.Z, -Extents.Z, Extents.Z);
		 // Transform back
		 return Frame.Origin + RotMatrix * Local;
	 }


	 // Create a merged TOrientedBox, encompassing this box and one other.
	 // Uses heuristics to test a few possible orientations and pick the smallest-volume result. Not an optimal fit.
	 // @param Other						The box to merge with
	 // @param bOnlyConsiderExistingAxes	If true, the merged box will always use the Rotation of one of the input boxes
	 GEOMETRYCORE_API TOrientedBox3<RealType> Merge(const TOrientedBox3<RealType>& Other, bool bOnlyConsiderExistingAxes = false) const;

	 /**
	  * Transform the frame of the oriented box so its local axes are closer to the corresponding world axes
	  * (Does not change the shape of the box, just re-orders the axes if needed)
	  */
	 [[nodiscard]] GEOMETRYCORE_API TOrientedBox3<RealType> ReparameterizedCloserToWorldFrame() const;

	 /**
	  * @param OtherBox An oriented box to test for intersection against this box
	  * @param ParallelAxesTolerance Tolerance for handling the special case where the box axes are nearly parallel
	  * @return Whether the boxes overlap
	  */
	 [[nodiscard]] GEOMETRYCORE_API bool Intersects(const TOrientedBox3& OtherBox, RealType ParallelAxesTolerance = TMathUtil<RealType>::ZeroTolerance) const;
	 
	 /**
	  * @param OtherBox An axis-aligned box to test for intersection against this box
	  * @param ParallelAxesTolerance Tolerance for handling the special case where the box axes are nearly parallel
	  * @return Whether the boxes overlap
	  */
	 [[nodiscard]] GEOMETRYCORE_API bool Intersects(const TAxisAlignedBox3<RealType>& OtherBox, RealType ParallelAxesTolerance = TMathUtil<RealType>::ZeroTolerance) const;
};

template<typename RealType>
struct TOrientedBox2
{
	// Center of the box
	TVector2<RealType> Origin;
	// X Axis of the box -- must be normalized
	TVector2<RealType> UnitAxisX;
	/** Half-dimensions of box measured along the two axes */
	TVector2<RealType> Extents;

	TOrientedBox2() : Origin(0, 0), UnitAxisX(1, 0), Extents(1, 1) {}

	/**
	 * Create axis-aligned box with given Origin and Extents
	 */
	TOrientedBox2(const TVector2<RealType>& OriginIn, const TVector2<RealType>& ExtentsIn)
		: Origin(OriginIn), UnitAxisX(1, 0), Extents(ExtentsIn)
	{
	}

	/**
	 * Create oriented box with given Origin, X Axis and Extents
	 */
	TOrientedBox2(const TVector2<RealType>& OriginIn, const TVector2<RealType>& XAxisIn, const TVector2<RealType>& ExtentsIn)
		: Origin(OriginIn), UnitAxisX(XAxisIn), Extents(ExtentsIn)
	{
		if (!UnitAxisX.Normalize())
		{
			UnitAxisX = TVector2<RealType>(1, 0);
		}
	}

	/**
	 * Create oriented box with given Origin, Angle (in radians) and Extents
	 */
	TOrientedBox2(const TVector2<RealType>& OriginIn, RealType AngleRad, const TVector2<RealType>& ExtentsIn)
		: Origin(OriginIn), UnitAxisX(FMath::Cos(AngleRad), FMath::Sin(AngleRad)), Extents(ExtentsIn)
	{
	}

	/**
	 * Create oriented box from axis-aligned box
	 */
	TOrientedBox2(const TAxisAlignedBox2<RealType>& AxisBox)
		: Origin(AxisBox.Center()), UnitAxisX(1,0), Extents(AxisBox.Extents())
	{
	}

	void SetAngleRadians(RealType AngleRad)
	{
		UnitAxisX = TVector2<RealType>(FMath::Cos(AngleRad), FMath::Sin(AngleRad));
	}

	/** @return box with unit dimensions centered at origin */
	static TOrientedBox2<RealType> UnitZeroCentered() { return TOrientedBox2<RealType>(TVector2<RealType>::Zero(), (RealType)0.5 * TVector2<RealType>::One()); }

	/** @return box with unit dimensions where minimum corner is at origin */
	static TOrientedBox2<RealType> UnitPositive() { return TOrientedBox2<RealType>((RealType)0.5 * TVector2<RealType>::One(), (RealType)0.5 * TVector2<RealType>::One()); }


	/** @return center of the box */
	inline TVector2<RealType> Center() const { return Origin; }

	/** @return X axis of the box */
	inline TVector2<RealType> AxisX() const { return UnitAxisX; }

	/** @return Y axis of the box  (the X axis rotated 90 degrees counter-clockwise) */
	inline TVector2<RealType> AxisY() const { return TVector2<RealType>(-UnitAxisX.Y, UnitAxisX.X); }

	/** @return an axis of the box */
	inline TVector2<RealType> GetAxis(int AxisIndex) const { return AxisIndex == 0 ? AxisX() : AxisY(); }

	/** @return vector from minimum-corner to maximum-corner of box */
	inline TVector2<RealType> Diagonal() const
	{
		return (AxisX() * Extents.X + AxisY() * Extents.Y) * (RealType)2;
	}

	/** @return area of box */
	inline RealType Area() const
	{
		return (RealType)4 * FMath::Max(0, Extents.X) * FMath::Max(0, Extents.Y);
	}

	/** @return perimeter of box */
	inline RealType Perimeter() const
	{
		return (RealType)4 * (FMath::Max(0, Extents.X) + FMath::Max(0, Extents.Y));
	}

	/** @return Point transformed to the local space of the box (Origin at box center, Axes aligned to box) */
	inline TVector2<RealType> ToLocalSpace(const TVector2<RealType>& Point) const
	{
		TVector2<RealType> FromOrigin = Point - Origin;
		return TVector2<RealType>(UnitAxisX.Dot(FromOrigin), AxisY().Dot(FromOrigin));
	}

	/** @return Point transformed from the local space of the box (Origin at box center, Axes aligned to box) */
	inline TVector2<RealType> FromLocalSpace(const TVector2<RealType>& Point) const
	{
		return Origin + Point.X * UnitAxisX + Point.Y * AxisY();
	}

	/** @return true if box contains point */
	inline bool Contains(const TVector2<RealType>& Point) const
	{
		TVector2<RealType> LocalPoint = ToLocalSpace(Point);
		return (TMathUtil<RealType>::Abs(LocalPoint.X) <= Extents.X) &&
			(TMathUtil<RealType>::Abs(LocalPoint.Y) <= Extents.Y);
	}


	//// corners 

	/**
	 * @param Index corner index in range 0-3
	 * @return Corner point on the box identified by the given index.
	 * Ordering is: [ (-x,-y), (x,-y), (x,y), (-x,y) ]
	 */
	TVector2<RealType> GetCorner(int Index) const
	{
		check(Index >= 0 && Index <= 3);
		RealType DX = RealType(int((Index == 1) | (Index == 2)) * 2 - 1) * Extents.X; // X positive at indices 1, 2
		RealType DY = RealType((Index & 2) - 1) * Extents.Y; // Y positive at indices 2, 3
		return Origin + DX * UnitAxisX + DY * AxisY();
	}

	/**
	 * Call CornerPointFunc(TVector2<RealType>) for each of the 4 box corners. Order is the same as GetCorner(X).
	 */
	template<typename PointFuncType>
	void EnumerateCorners(PointFuncType CornerPointFunc) const
	{
		TVector2<RealType> X = AxisX() * Extents.X, Y = AxisY() * Extents.Y;

		CornerPointFunc(Origin - X - Y);
		CornerPointFunc(Origin + X - Y);
		CornerPointFunc(Origin + X + Y);
		CornerPointFunc(Origin - X + Y);
	}

	/**
	 * Call CornerPointPredicate(TVector2<RealType>) for each of the 4 box corners, with early-out if any call returns false
	 * @return true if all tests pass
	 */
	template<typename PointPredicateType>
	bool TestCorners(PointPredicateType CornerPointPredicate) const
	{
		TVector2<RealType> X = AxisX() * Extents.X, Y = AxisY() * Extents.Y;

		return
			CornerPointFunc(Origin - X - Y) &&
			CornerPointFunc(Origin + X - Y) &&
			CornerPointFunc(Origin + X + Y) &&
			CornerPointFunc(Origin - X + Y);
	}

	/**
	 * Get whether the corner at Index (see GetCorner documentation comment) is in the negative or positive direction for each axis
	 * @param Index corner index in range 0-4
	 * @return Index2i with 0 or 1 for each axis, 0 if corner is in the negative direction for that axis, 1 if in the positive direction
	 */
	static FIndex2i GetCornerSide(int Index)
	{
		check(Index >= 0 && Index <= 3);
		return FIndex2i(
			int((Index == 1) | (Index == 2)),
			(Index & 2) >> 1
		);
	}



	/**
	 * Find squared distance to box.
	 * @param Point input point
	 * @return Squared distance from point to box, or 0 if point is inside box
	 */
	RealType DistanceSquared(const TVector2<RealType>& Point) const
	{
		TVector2<RealType> BeyondExtents = ToLocalSpace(Point).GetAbs() - Extents;
		TVector2<RealType> ClampedBeyond( // clamp negative (inside) to zero
			FMath::Max((RealType)0, BeyondExtents.X),
			FMath::Max((RealType)0, BeyondExtents.Y));
		return ClampedBeyond.SquaredLength();
	}

	/**
	 * Find closest point on box
	 * @param Point input point
	 * @return Closest point on box. Input point is returned if it is inside box.
	 */
	TVector2<RealType> ClosestPoint(const TVector2<RealType>& Point) const
	{
		TVector2<RealType> Local = ToLocalSpace(Point);
		Local.X = FMath::Clamp(Local.X, -Extents.X, Extents.X);
		Local.Y = FMath::Clamp(Local.Y, -Extents.Y, Extents.Y);
		return FromLocalSpace(Local);
	}

};

typedef TOrientedBox2<float> FOrientedBox2f;
typedef TOrientedBox2<double> FOrientedBox2d;
typedef TOrientedBox3<float> FOrientedBox3f;
typedef TOrientedBox3<double> FOrientedBox3d;

} // end namespace UE::Geometry
} // end namespace UE
