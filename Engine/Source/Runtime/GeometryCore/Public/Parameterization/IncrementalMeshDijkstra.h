// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "BoxTypes.h"
#include "FrameTypes.h"
#include "Util/IndexPriorityQueue.h"
#include "Util/DynamicVector.h"

namespace UE
{
namespace Geometry
{

/**
 * TIncrementalMeshDijkstra computes graph distances on a mesh from seed point(s) using Dijkstra's algorithm.
 * Derived from TMeshDijkstra. This Incremental variant allows adding new Seed points to an already-computed
 * solution.
 */
template<class PointSetType>
class TIncrementalMeshDijkstra
{
public:
	/** PointSet we are calculating on */
	const PointSetType* PointSet;

	/**
	 * Return the 3D Position of a given Point in the PointSet. 
	 * This function is set to PointSet->GetVertex() in the constructor below, but can be 
	 * replaced with an external function if necessary (eg to provide deformed mesh positions, etc)
	 */
	TUniqueFunction<FVector3d(int32)> GetPositionFunc;


	/**
	 * Construct TMeshDijkstra for the given PointSet. We will hold a reference to this
	 * PointSet for the lifetime of the class.
	 */
	TIncrementalMeshDijkstra(const PointSetType* PointSetIn)
	{
		PointSet = PointSetIn;

		int32 MaxID = PointSet->MaxVertexID();
		Queue.Initialize(MaxID);

		GetPositionFunc = [this](int32 PointID) { return  PointSet->GetVertex(PointID); };

		ForceInitializeAllNodes();
	}

	static double InvalidDistance() { return TNumericLimits<double>::Max(); };

	/**
	 *  FSeedPoint defines a seed point passed to the various compute methods below
	 */
	struct FSeedPoint
	{
		/** Client-defined integer ID for this seed point (not used internally) */
		int32 ExternalID = -1;
		/** Point ID for this seed point, must be a valid point ID in the input PointSet */
		int32 PointID = 0;
		/** Initial distance for this seed point */
		double StartDistance = 0;
	};


	/**
	 * Add new SeedPoints to the current solution, and then propagate updated
	 * graph distances to any points closer to these new seed points than to
	 * existing seed points.
	 */
	void AddSeedPoints(const TArray<FSeedPoint>& SeedPointsIn)
	{
		++CurrentSeedTimestamp;

		for (const FSeedPoint& SeedPoint : SeedPointsIn)
		{
			check(SeedPointExternalIDMap.Contains(SeedPoint.ExternalID) == false);
			int32 NewIndex = SeedPoints.Num();
			SeedPointExternalIDMap.Add(SeedPoint.ExternalID, NewIndex);
			SeedPoints.Add(SeedPoint);

			int32 PointID = SeedPoint.PointID;
			if (ensure(Queue.Contains(PointID) == false))
			{
				FGraphNode* Node = GetNodeForPointSetID(PointID);
				Node->GraphDistance = SeedPoint.StartDistance;
				Node->FrozenTimestamp = CurrentSeedTimestamp;
				Node->SeedID = NewIndex;
				Queue.Insert(PointID, float(Node->GraphDistance));
			}
		}


		while (Queue.GetCount() > 0)
		{
			int32 NextID = Queue.Dequeue();
			FGraphNode* Node = GetNodeForPointSetID(NextID);
			check(Node != nullptr);

			Node->FrozenTimestamp = CurrentSeedTimestamp;
			UpdateNeighboursSparse(Node);
		}
	}

	/**
	 * @return the ExternalID of the SeedPoint that is closest to PointID, or -1 if PointID does not have a valid graph distance
	 */
	int32 GetSeedExternalIDForPointSetID(int32 PointID)
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		if (Node == nullptr || Node->FrozenTimestamp == InvalidFrozenTimestamp)
		{
			return -1;
		}
		int32 SeedIndex = Node->SeedID;
		return SeedPoints[SeedIndex].ExternalID;
	}


	/**
	 * @return vertex id associated with the maximum graph distance returned by GetMaxGraphDistance()
	 */
	int32 FindMaxGraphDistancePointID() const
	{
		int32 MaxID = PointSet->MaxVertexID();
		double MaxDistance = 0;
		int32 MaxDistancePointID = -1;
		for (int32 PointID = 0; PointID < MaxID; PointID++)
		{
			if (PointSet->IsVertex(PointID))
			{
				double Distance = GetDistance(PointID);
				if (Distance != InvalidDistance() && Distance > MaxDistance)
				{
					MaxDistance = Distance;
					MaxDistancePointID = PointID;
				}
			}
		}
		return MaxDistancePointID;
	}


	/**
	 * @return true if the distance for index PointID was calculated
	 */
	bool HasDistance(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->FrozenTimestamp != InvalidFrozenTimestamp);
	}


	/**
	 * @return the distance calculated for index PointID
	 */
	double GetDistance(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->FrozenTimestamp != InvalidFrozenTimestamp) ? Node->GraphDistance : InvalidDistance();
	}

	/**
	 * Find path from a point to the nearest seed point
	 * @param PointID starting point, assumption is that we have computed dijkstra to this point
	 * @param PathToSeedOut path is returned here, includes PointID and seed point as last element
	 * @param MaxLength if PathToSeedOut grows beyond this length, we abort the search
	 * @return true if valid path was found
	 */
	bool FindPathToNearestSeed(int32 PointID, TArray<int32>& PathToSeedOut, int32 MaxLength = 100000)
	{
		const FGraphNode* CurNode = GetNodeForPointSetID(PointID);
		if (CurNode == nullptr || CurNode->FrozenTimestamp == InvalidFrozenTimestamp)
		{
			return false;
		}

		PathToSeedOut.Reset();
		PathToSeedOut.Add(PointID);

		int32 IterCount = 0;
		while (IterCount++ < MaxLength)
		{
			if (CurNode->ParentPointID == -1)
			{
				return true;
			}

			PathToSeedOut.Add(CurNode->ParentPointID);

			CurNode = GetNodeForPointSetID(CurNode->ParentPointID);
			if (CurNode == nullptr || CurNode->FrozenTimestamp == InvalidFrozenTimestamp)
			{
				PathToSeedOut.Reset();
				return false;
			}
		}
		return true;
	}

private:

	// information about each active/computed point
	struct FGraphNode
	{
		int32 PointID;
		int32 ParentPointID;
		int32 SeedID;
		double GraphDistance;
		int32 FrozenTimestamp;
	};

	// currently all nodes are allocated on initialization
	TArray<FGraphNode> AllocatedNodes;

	// queue of nodes to process (for dijkstra front propagation)
	FIndexPriorityQueue Queue;

	TArray<FSeedPoint> SeedPoints;
	TMap<int32, int32> SeedPointExternalIDMap;

	const int32 InvalidFrozenTimestamp = -1;
	int32 CurrentSeedTimestamp = 0;


	void ForceInitializeAllNodes()
	{
		int32 MaxID = PointSet->MaxVertexID();
		AllocatedNodes.SetNum(MaxID);
		for (int32 k = 0; k < MaxID; ++k)
		{
			FGraphNode NewNode{ k, -1, -1, InvalidDistance(), InvalidFrozenTimestamp };
			AllocatedNodes[k] = NewNode;
		}
	}


	FGraphNode* GetNodeForPointSetID(int32 PointSetID)
	{
		return &AllocatedNodes[PointSetID];
	}

	const FGraphNode* GetNodeForPointSetID(int32 PointSetID) const
	{
		return &AllocatedNodes[PointSetID];
	}


	// given new Distance/UV at Parent, check if any of its neighbours are in the queue,
	// and if they are, and the new graph distance is shorter, update their queue position
	// (this is basically the update step of Disjktras algorithm)
	void UpdateNeighboursSparse(FGraphNode* Parent)
	{
		FVector3d ParentPos(GetPositionFunc(Parent->PointID));
		double ParentDist = Parent->GraphDistance;

		for (int32 NbrPointID : PointSet->VtxVerticesItr(Parent->PointID))
		{
			FGraphNode* NbrNode = GetNodeForPointSetID(NbrPointID);
			if (NbrNode->FrozenTimestamp == CurrentSeedTimestamp)
			{
				continue;
			}

			double NbrDist = ParentDist + Distance(ParentPos, GetPositionFunc(NbrPointID));
			if (Queue.Contains(NbrPointID))
			{
				if (NbrDist < NbrNode->GraphDistance)
				{
					NbrNode->ParentPointID = Parent->PointID;
					NbrNode->SeedID = Parent->SeedID;
					NbrNode->GraphDistance = NbrDist;
					Queue.Update(NbrPointID, float(NbrNode->GraphDistance));
				}
			}
			else 
			{
				if (NbrDist < NbrNode->GraphDistance)
				{
					NbrNode->ParentPointID = Parent->PointID;
					NbrNode->SeedID = Parent->SeedID;
					NbrNode->GraphDistance = NbrDist;
					Queue.Insert(NbrPointID, float(NbrNode->GraphDistance));
				}
			}
		}
	}

};


} // end namespace UE::Geometry
} // end namespace UE