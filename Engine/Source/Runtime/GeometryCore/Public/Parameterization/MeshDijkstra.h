// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "BoxTypes.h"
#include "FrameTypes.h"
#include "Util/IndexPriorityQueue.h"
#include "Util/DynamicVector.h"

namespace UE
{
namespace Geometry
{

/**
 * TMeshDijkstra computes graph distances on a mesh from seed point(s) using Dijkstra's algorithm.
 *
 * Templated on the point set type, which must provide positions, normals, and neighbours.
 * Currently will only work for FDynamicMesh3 and FDynamicPointSet3 because of call to PointSetType->VtxVerticesItr()
 */
template<class PointSetType>
class TMeshDijkstra
{
public:
	/** PointSet we are calculating on */
	const PointSetType* PointSet;

	/**
	 * Return the 3D Position of a given Point in the PointSet. 
	 * This function is set to PointSet->GetVertex() in the constructor below, but can be 
	 * replaced with an external function if necessary (eg to provide deformed mesh positions, etc)
	 */
	TUniqueFunction<FVector3d(int32)> GetPositionFunc;

	/**
	 * If enabled, when computing local point-pair distances, GetWeightedDistanceFunc() will be called with the 
	 * Euclidean distance to allow for alternative metrics
	 */
	bool bEnableDistanceWeighting = false;

	/**
	 * Called when computing pairwise point distances between neighbours FromVID and ToVID, to allow for alternative distance metrics.
	 * SeedVID is the seed point that FromVID's value was propagated from (ie point reached by gradient walk)
	 * EuclideanDistance is the distance between the two input points
	 */
	TUniqueFunction<double(int32 FromVID, int32 ToVID, int32 SeedVID, double EuclideanDistance)> GetWeightedDistanceFunc;


	/**
	 * Construct TMeshDijkstra for the given PointSet. We will hold a reference to this
	 * PointSet for the lifetime of the class.
	 */
	TMeshDijkstra(const PointSetType* PointSetIn)
	{
		PointSet = PointSetIn;

		int32 MaxID = PointSet->MaxVertexID();
		Queue.Initialize(MaxID);

		MaxGraphDistance = 0.0;
		MaxGraphDistancePointID = -1;

		GetPositionFunc = [this](int32 PointID) { return  PointSet->GetVertex(PointID); };
		bEnableDistanceWeighting = false;
		GetWeightedDistanceFunc = [](int32, int32, int32, double Distance) { return Distance; };
	}

	/** @return distance value that indicates invalid or uncomputed distance */
	static double InvalidDistance() { return TNumericLimits<double>::Max(); };

	/**
	 * Reset internal data structures but keep allocated memory
	 */
	void Reset()
	{
		IDToNodeIndexMap.Reset();
		AllocatedNodes.Clear();
		Queue.Clear(false);
		MaxGraphDistance = 0.0;
		MaxGraphDistancePointID = -1;
	}


	/**
	 *  FSeedPoint defines a seed point passed to the various compute methods below
	 */
	struct FSeedPoint
	{
		/** Client-defined integer ID for this seed point (not used internally) */
		int32 ExternalID = -1;
		/** Point ID for this seed point, must be a valid point ID in the input PointSet */
		int32 PointID = 0;
		/** Initial distance for this seed point */
		double StartDistance = 0;
	};


	/**
	 * Computes graph distances outwards from seed points to all points that are less/equal to ComputeToMaxDistance from the seed.
	 * @param SeedPointsIn seed points used to initialize computation, ie geodesics propagate out from this point set
	 * @param ComputeToMaxDistance target graph-distance radius, will not compute/set distances on points with graph-distance larger than this
	 */
	void ComputeToMaxDistance(const TArray<FSeedPoint>& SeedPointsIn, double ComputeToMaxDistanceIn)
	{
		MaxGraphDistance = 0.0f;
		MaxGraphDistancePointID = -1;

		SeedPoints.Reset();
		for (const FSeedPoint& SeedPoint : SeedPointsIn)
		{
			int32 NewIndex = SeedPoints.Num();
			SeedPoints.Add(SeedPoint);

			int32 PointID = SeedPoint.PointID;
			if (ensure(Queue.Contains(PointID) == false))
			{
				FGraphNode* Node = GetNodeForPointSetID(PointID, true);
				Node->GraphDistance = SeedPoint.StartDistance;
				Node->bFrozen = true;
				Node->SeedPointID = NewIndex;
				Queue.Insert(PointID, float(Node->GraphDistance));
			}
		}

		while (Queue.GetCount() > 0)
		{
			int32 NextID = Queue.Dequeue();
			FGraphNode* Node = GetNodeForPointSetID(NextID, false);
			check(Node != nullptr);

			MaxGraphDistance = TMathUtil<double>::Max(Node->GraphDistance, MaxGraphDistance);
			if (MaxGraphDistance > ComputeToMaxDistanceIn)
			{
				return;
			}

			Node->bFrozen = true;
			MaxGraphDistancePointID = Node->PointID;
			UpdateNeighboursSparse(Node);
		}
	}



	/**
	 * Computes graph distances outwards from seed points to all points that are less/equal to ComputeToMaxDistance from the seed.
	 * @param SeedPointsIn 2D tuples that define seed points as (PointID, InitialDistance) pairs
	 * @param ComputeToMaxDistance target graph-distance radius, will not compute/set distances on points with graph-distance larger than this
	 */
	void ComputeToMaxDistance(const TArray<FVector2d>& SeedPointsIn, double ComputeToMaxDistanceIn)
	{
		TArray<FSeedPoint> ConvertedSeedPoints;
		int32 N = SeedPointsIn.Num();
		ConvertedSeedPoints.SetNum(SeedPointsIn.Num());
		for (int32 k = 0; k < N; ++k)
		{
			ConvertedSeedPoints[k] = FSeedPoint{ -1, (int)SeedPointsIn[k].X, SeedPointsIn[k].Y };
		}
		ComputeToMaxDistance(ConvertedSeedPoints, ComputeToMaxDistanceIn);
	}



	 /**
	  * Computes graph distances outwards from seed points to all points that are less/equal to ComputeToMaxDistance from the seed, or until TargetPointID is reached.
	  * This is useful for finding shortest paths between two points
	  * @param SeedPointsIn seed points used to initialize computation, ie geodesics propagate out from this point set
	  * @param TargetPointID the target point, computation stops when this point is reached
	  * @param ComputeToMaxDistance target graph-distance radius, will not compute/set distances on points with graph-distance larger than this
	  */
	bool ComputeToTargetPoint(const TArray<FSeedPoint>& SeedPointsIn, int32 TargetPointID, double ComputeToMaxDistanceIn = TNumericLimits<double>::Max())
	{
		MaxGraphDistance = 0.0f;
		MaxGraphDistancePointID = -1;

		SeedPoints.Reset();
		for (const FSeedPoint& SeedPoint : SeedPointsIn)
		{
			int32 NewIndex = SeedPoints.Num();
			SeedPoints.Add(SeedPoint);

			int32 PointID = SeedPoint.PointID;
			if (ensure(Queue.Contains(PointID) == false))
			{
				FGraphNode* Node = GetNodeForPointSetID(PointID, true);
				Node->GraphDistance = SeedPoint.StartDistance;
				Node->bFrozen = true;
				Node->SeedPointID = NewIndex;
				Queue.Insert(PointID, (float)Node->GraphDistance);
			}
		}

		while (Queue.GetCount() > 0)
		{
			int32 NextID = Queue.Dequeue();
			FGraphNode* Node = GetNodeForPointSetID(NextID, false);
			check(Node != nullptr);

			MaxGraphDistance = TMathUtil<double>::Max(Node->GraphDistance, MaxGraphDistance);
			if (MaxGraphDistance > ComputeToMaxDistanceIn)
			{
				return false;
			}

			Node->bFrozen = true;
			MaxGraphDistancePointID = Node->PointID;

			if (Node->PointID == TargetPointID)
			{
				return true;
			}

			UpdateNeighboursSparse(Node);
		}

		return false;
	}




	/**
	 * @return the maximum graph distance encountered during the computation
	 */
	double GetMaxGraphDistance() const 
	{
		return MaxGraphDistance;
	}


	/**
	 * @return vertex id associated with the maximum graph distance returned by GetMaxGraphDistance()
	 */
	int32 GetMaxGraphDistancePointID() const
	{
		return MaxGraphDistancePointID;
	}


	/**
	 * @return the ExternalID of the SeedPoint that is closest to PointID, or -1 if PointID does not have a valid graph distance
	 */
	int32 GetSeedExternalIDForPointSetID(int32 PointID)
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		if (Node == nullptr || Node->bFrozen == false)
		{
			return -1;
		}
		int32 SeedIndex = Node->SeedPointID;
		return SeedPoints[SeedIndex].ExternalID;
	}


	/**
	 * @return true if the distance for index PointID was calculated
	 */
	bool HasDistance(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->bFrozen);
	}


	/**
	 * @return the distance calculated for index PointID
	 */
	double GetDistance(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->bFrozen) ? Node->GraphDistance : InvalidDistance();
	}

	/**
	 * Find path from a point to the nearest seed point
	 * @param PointID starting point, assumption is that we have computed dijkstra to this point
	 * @param PathToSeedOut path is returned here, includes PointID and seed point as last element
	 * @param MaxLength if PathToSeedOut grows beyond this length, we abort the search
	 * @return true if valid path was found
	 */
	bool FindPathToNearestSeed(int32 PointID, TArray<int32>& PathToSeedOut, int32 MaxLength = 100000)
	{
		const FGraphNode* CurNode = GetNodeForPointSetID(PointID);
		if (CurNode == nullptr || CurNode->bFrozen == false)
		{
			return false;
		}

		PathToSeedOut.Reset();
		PathToSeedOut.Add(PointID);

		int32 IterCount = 0;
		while (IterCount++ < MaxLength)
		{
			if (CurNode->ParentPointID == -1)
			{
				return true;
			}

			PathToSeedOut.Add(CurNode->ParentPointID);

			CurNode = GetNodeForPointSetID(CurNode->ParentPointID);
			if (CurNode == nullptr || CurNode->bFrozen == false)
			{
				PathToSeedOut.Reset();
				return false;
			}
		}
		return true;
	}

	struct FComputedPoint
	{
		int32 PointID = INDEX_NONE;
		int32 SeedIndex = INDEX_NONE;
		double GraphDistance = 0;
		bool bSeedPoint = false;
	};

	
	void EnumerateComputedPoint(TFunction<void(FComputedPoint)> InFunction)
	{
		for (const FGraphNode& Node : AllocatedNodes)
		{
			if (Node.bFrozen)
			{
				FComputedPoint PointInfo;
				PointInfo.PointID = Node.PointID;
				PointInfo.SeedIndex = Node.SeedPointID;
				PointInfo.GraphDistance = Node.GraphDistance;
				PointInfo.bSeedPoint = Node.ParentPointID == INDEX_NONE;
				
				InFunction(MoveTemp(PointInfo));
			}
		}	
	}
	
private:

	// information about each active/computed point
	struct FGraphNode
	{
		int32 PointID;
		int32 ParentPointID;
		int32 SeedPointID;
		double GraphDistance;
		bool bFrozen;
	};

	// To avoid constructing FGraphNode for all input points (because we are computing a "local" param),
	// we only allocate on demand, and then store a sparse mapping in IDToNodeIndexMap
	TMap<int32, int32> IDToNodeIndexMap;
	TDynamicVector<FGraphNode> AllocatedNodes;

	// queue of nodes to process (for dijkstra front propagation)
	FIndexPriorityQueue Queue;

	TArray<FSeedPoint> SeedPoints;

	// max distances encountered during last compute
	double MaxGraphDistance;

	int32 MaxGraphDistancePointID;

	FGraphNode* GetNodeForPointSetID(int32 PointSetID, bool bCreateIfMissing)
	{
		const int32* AllocatedIndex = IDToNodeIndexMap.Find(PointSetID);
		if (AllocatedIndex == nullptr)
		{
			if (bCreateIfMissing)
			{
				FGraphNode NewNode{ PointSetID, -1, 0, false };
				int32 NewIndex = AllocatedNodes.Num();
				AllocatedNodes.Add(NewNode);
				IDToNodeIndexMap.Add(PointSetID, NewIndex);
				return &AllocatedNodes[NewIndex];
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return &AllocatedNodes[*AllocatedIndex];
		}
	}


	const FGraphNode* GetNodeForPointSetID(int32 PointSetID) const
	{
		const int32* AllocatedIndex = IDToNodeIndexMap.Find(PointSetID);
		return (AllocatedIndex != nullptr) ? &AllocatedNodes[*AllocatedIndex] : nullptr;
	}


	// given new Distance/UV at Parent, check if any of its neighbours are in the queue,
	// and if they are, and the new graph distance is shorter, update their queue position
	// (this is basically the update step of Disjktras algorithm)
	void UpdateNeighboursSparse(FGraphNode* Parent)
	{
		FVector3d ParentPos(GetPositionFunc(Parent->PointID));
		double ParentDist = Parent->GraphDistance;

		for (int32 NbrPointID : PointSet->VtxVerticesItr(Parent->PointID))
		{
			FGraphNode* NbrNode = GetNodeForPointSetID(NbrPointID, true);
			if (NbrNode->bFrozen)
			{
				continue;
			}

			double LocalDist = Distance(ParentPos, GetPositionFunc(NbrPointID));
			if (bEnableDistanceWeighting)
			{
				int SeedPointID = SeedPoints[Parent->SeedPointID].PointID;
				LocalDist = GetWeightedDistanceFunc(Parent->PointID, NbrPointID, SeedPointID, LocalDist);
			}
			double NbrDist = ParentDist + LocalDist;

			if (Queue.Contains(NbrPointID))
			{
				if (NbrDist < NbrNode->GraphDistance)
				{
					NbrNode->ParentPointID = Parent->PointID;
					NbrNode->GraphDistance = NbrDist;
					NbrNode->SeedPointID = Parent->SeedPointID;
					Queue.Update(NbrPointID, float(NbrNode->GraphDistance));
				}
			}
			else 
			{
				NbrNode->ParentPointID = Parent->PointID;
				NbrNode->GraphDistance = NbrDist;
				NbrNode->SeedPointID = Parent->SeedPointID;
				Queue.Insert(NbrPointID, float(NbrNode->GraphDistance));
			}
		}
	}

};


} // end namespace UE::Geometry
} // end namespace UE