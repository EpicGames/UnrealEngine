// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "BoxTypes.h"
#include "FrameTypes.h"
#include "Util/IndexPriorityQueue.h"
#include "Util/DynamicVector.h"
#include "DynamicMesh/MeshNormals.h"

namespace UE
{
namespace Geometry
{

/**
 * Type of local parameterization
 */
enum class ELocalParamTypes : uint8
{
	/** Calculate a planar projection */
	PlanarProjection = 1,
	/** Calculate Discrete Exponential Map */
	ExponentialMap = 2,
	/** Calculate Discrete Exponential Map with Upwind-Averaging */
	ExponentialMapUpwindAvg = 3
};

/**
 * TMeshLocalParam computes a local UV parameterization of a set of connected PointsWithNormals,
 * where "local" means "in a geodesic disc around a starting point". 
 *
 * The computation is based on region-growing, and geodesic distances are actually graph distances,
 * measured with Dijkstras algorithm.
 *
 * Templated on the point set type, which must provide positions, normals, and neighbours.
 * Currently will only work for FDynamicMesh3 and FDynamicPointSet3 because of call to PointSetType->VtxVerticesItr()
 */
template<class PointSetType>
class TMeshLocalParam
{
public:
	const PointSetType* PointSet;

	/** Type of local parameterization to compute */
	ELocalParamTypes ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;

	/** 
	 * If true, ExternalNormalFunc will be used to fetch normals instead of requesting from PointSet->GetVertexNormal.
	 * This allows for (1) vertex normals to be computed on the fly, if only a subset of the point set is being parameterizedx
	 * and (2) alternate normals (eg smoothed, etc) to be provided w/o having to define a separate class.
	 */
	bool bEnableExternalNormals = false;

	/** Normals will be requested from this function if bEnableExternalNormals == true */
	TFunction<FVector3d(int32)> ExternalNormalFunc;


	TMeshLocalParam(const PointSetType* PointSetIn)
	{
		PointSet = PointSetIn;

		int32 MaxID = PointSet->MaxVertexID();
		Queue.Initialize(MaxID);

		MaxGraphDistance = 0.0;
		MaxUVDistance = 0.0;
	}

	/**
	 * Reset internal data structures but keep allocated memory
	 */
	void Reset()
	{
		IDToNodeIndexMap.Reset();
		AllocatedNodes.Clear();
		Queue.Clear(false);
		MaxGraphDistance = 0.0;
		MaxUVDistance = 0.0;
	}



	/**
	 * Computes UVs outwards from seed frame/nbrs to all points that are less/equal to ComputeToMaxDistance from the seed.
	 * @param SeedFrameIn 3D frame on surface of point set, parameterization is computed "in" this frame (eg will align u/v to x/y axes, at origin)
	 * @param SeedNbrs 3 points that will be planar-projected into the SeedFrame, to initialize the region-growing (kind of triangle-mesh-specific)
	 * @param ComputeToMaxDistanceIn target radius for parameterization, will not set UVs on points with graph-distance larger than this
	 */
	void ComputeToMaxDistance(const FFrame3d& SeedFrameIn, const FIndex3i& SeedNbrs, double ComputeToMaxDistanceIn)
	{
		SeedFrame = SeedFrameIn;
		MaxGraphDistance = 0.0f;
		MaxUVDistance = 0.0f;

		for (int32 j = 0; j < 3; ++j) 
		{
			int32 NbrPointID = SeedNbrs[j];
			FGraphNode* Node = GetNodeForPointSetID(NbrPointID, true);
			Node->UV = ComputeLocalUV(SeedFrame, GetPosition(NbrPointID));
			Node->GraphDistance = Node->UV.Length();
			Node->bFrozen = true;
			check(Queue.Contains(NbrPointID) == false);
			Queue.Insert(NbrPointID, float(Node->GraphDistance));
		}

		ProcessQueueUntilTermination(ComputeToMaxDistanceIn);
	}


	/**
	 * Computes UVs outwards from seed vertex to all points that are less/equal to ComputeToMaxDistance from the seed.
	 * @param CenterPointVtxID ID of seed vertex
	 * @param CenterPointFrame 3D frame on surface of point set, parameterization is computed "in" this frame (eg will align u/v to x/y axes, at origin)
	 * @param ComputeToMaxDistanceIn target radius for parameterization, will not set UVs on points with graph-distance larger than this
	 */
	void ComputeToMaxDistance(int32 CenterPointVtxID, const FFrame3d& CenterPointFrame, double ComputeToMaxDistanceIn)
	{
		SeedFrame = CenterPointFrame;
		MaxGraphDistance = 0.0f;
		MaxUVDistance = 0.0f;

		// center point 
		FGraphNode* CenterNode = GetNodeForPointSetID(CenterPointVtxID, true);
		CenterNode->UV = FVector2d::Zero();
		CenterNode->GraphDistance = 0;
		CenterNode->bFrozen = true;

		Queue.Insert(CenterPointVtxID, 0);

		ProcessQueueUntilTermination(ComputeToMaxDistanceIn);
	}



	/**
	 * Scale and then Translate all calculated UV values
	 */
	void TransformUV(double Scale, FVector2d Translate)
	{
		for (FGraphNode& Node : AllocatedNodes)
		{
			if (Node.bFrozen)
			{
				Node.UV = (Node.UV * Scale) + Translate;
			}
		}
	}



	/**
	 * @return the maximum graph distance/radius encountered during the computation
	 */
	double GetMaxGraphDistance() const 
	{
		return MaxGraphDistance;
	}

	/**
	 * @return the maximum UV distance/radius encountered during the computation
	 */
	double GetMaxUVDistance() const
	{
		return MaxUVDistance;
	}

	/**
	 * @return true if the UV for index PointID was calculated
	 */
	bool HasUV(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->bFrozen);
	}

	/**
	 * @return Array of points with set UVs. 
	 */
	void GetPointsWithUV(TArray<int32>& Points) const 
	{
		Points.Empty(AllocatedNodes.Num());
		for (const FGraphNode& Node : AllocatedNodes)
		{
			if (Node.bFrozen)
			{
				Points.Add(Node.PointID);
			}
		}
	}

	/**
	 * @return the UV calculated for index PointID
	 */
	FVector2d GetUV(int32 PointID) const
	{
		const FGraphNode* Node = GetNodeForPointSetID(PointID);
		return (Node != nullptr && Node->bFrozen) ? Node->UV : InvalidUV();
	}

	/**
	 * Find all computed UVs within the specified distances
	 * @param PointIDsOut PointID corresponding to each computed UV is returned here
	 * @param PointUVsOut UV value corresponding to each PointID is returned here
	 * @param MaxUVMagnitude computed UVs will only be included if their magnitude is smaller than this value (eg, this is the UV-space radius)
	 * @param MaxGraphDistance computed UVs will only be included if their graph distance is smaller than this value. Graph distance is generally always be larger than UV distance. This can be used to avoid problematic UVs that might result from algorithm failures.
	 */
	void GetAllComputedUVs(
		TArray<int32>& PointIDsOut, 
		TArray<FVector2d>& PointUVsOut,
		double MaxUVMagnitude = TNumericLimits<float>::Max(),
		double MaxGraphDist = TNumericLimits<float>::Max() ) const
	{
		double MaxUVMagSqr = MaxUVMagnitude * MaxUVMagnitude;
		for (const FGraphNode& Node : AllocatedNodes)
		{
			if (Node.bFrozen && Node.GraphDistance < MaxGraphDist && Node.UV.SquaredLength() < MaxUVMagSqr)
			{
				PointIDsOut.Add(Node.PointID);
				PointUVsOut.Add(Node.UV);
			}
		}
	}



	/**
	 * Apply a function to each calculated UV
	 */
	void ApplyUVs( TFunctionRef<void(int32 PointID, const FVector2d& UV)> ApplyFunc ) const
	{
		for ( const FGraphNode& Node : AllocatedNodes )
		{
			if (Node.bFrozen)
			{
				ApplyFunc(Node.PointID, Node.UV);
			}
		}
	}


	/**
	 * @return 2D Axis-Aligned Bounding Box of the calculated UVs
	 */
	FAxisAlignedBox2d GetUVBounds() const
	{
		FAxisAlignedBox2d Bounds = FAxisAlignedBox2d::Empty();
		for (const FGraphNode& Node : AllocatedNodes)
		{
			if (Node.bFrozen)
			{
				Bounds.Contain(Node.UV);
			}
		}
		return Bounds;
	}


protected:

	// wrap the calls to PointSet Vertex/Normal. Should make these more general.

	FVector3d GetPosition(const int32 PointID) const
	{
		return PointSet->GetVertex(PointID);
	}

	FVector3d GetNormal(const int32 PointID) const
	{
		if (PointSet->HasVertexNormals())
		{
			return FVector3d(PointSet->GetVertexNormal(PointID));
		}
		// Note: Currently only have a standard vertex normal calculation for FDynamicMesh3
		else if constexpr (std::is_same_v<PointSetType, FDynamicMesh3>)
		{
			return FMeshNormals::ComputeVertexNormal(*PointSet, PointID);
		}
		else
		{
			ensureMsgf(false, TEXT("Local parameterization requires vertex normals!"));
			return FVector3d::UnitY();
		}
	}

protected:

	// information about each active/computed point
	struct FGraphNode
	{
		int32 PointID;
		int32 ParentPointID;
		double GraphDistance;
		FVector2d UV;
		bool bFrozen;

		FVector3d CachedNormal;
	};


	FFrame3d GetFrame(const FGraphNode& Node) const
	{
		return FFrame3d(GetPosition(Node.PointID), Node.CachedNormal);
	}


	// To avoid constructing FGraphNode for all input points (because we are computing a "local" param),
	// we only allocate on demand, and then store a sparse mapping in IDToNodeIndexMap
	TMap<int32, int32> IDToNodeIndexMap;
	TDynamicVector<FGraphNode> AllocatedNodes;

	// queue of nodes to process (for dijkstra front propagation)
	FIndexPriorityQueue Queue;

	static FVector2d InvalidUV() { return FVector2d(TNumericLimits<double>::Max(), TNumericLimits<double>::Max()); };

	// seed frame, unwrap is centered around this position/axes
	FFrame3d SeedFrame;

	// max distances encountered during last compute
	double MaxGraphDistance;
	double MaxUVDistance;


	void ProcessQueueUntilTermination(double MaxDistance)
	{
		while (Queue.GetCount() > 0)
		{
			int32 NextID = Queue.Dequeue();
			FGraphNode* Node = GetNodeForPointSetID(NextID, false);
			check(Node != nullptr);

			MaxGraphDistance = TMathUtil<double>::Max(Node->GraphDistance, MaxGraphDistance);
			if (MaxGraphDistance > MaxDistance)
			{
				return;
			}

			if (Node->ParentPointID >= 0)
			{
				switch (ParamMode)
				{
				case ELocalParamTypes::ExponentialMap:
					UpdateUVExpmap(*Node);
					break;
				case ELocalParamTypes::ExponentialMapUpwindAvg:
					UpdateUVExpmapUpwind(*Node);
					break;
				case ELocalParamTypes::PlanarProjection:
					UpdateUVPlanar(*Node);
					break;
				}
			}

			double UVDistSqr = Node->UV.SquaredLength();
			if (UVDistSqr > MaxUVDistance)
			{
				MaxUVDistance = UVDistSqr;
			}

			Node->bFrozen = true;
			UpdateNeighboursSparse(Node);
		}

		MaxUVDistance = TMathUtil<double>::Sqrt(MaxUVDistance);
	}



	FVector2d ComputeLocalUV(const FFrame3d& Frame, FVector3d Position) const
	{
		Position -= Frame.Origin;
		FVector2d UV(Position.Dot(Frame.X()), Position.Dot(Frame.Y()));
		return UV;
	}

	// calculate the UV value at Position based on the existing NbrUV, using the frame at NbrFrame, and the original SeedFrame
	FVector2d PropagateUV(const FVector3d& Position, const FVector2d& NbrUV, const FFrame3d& NbrFrame, const FFrame3d& SeedFrameIn) const
	{
		// project Position into local space of NbrFrame
		FVector2d LocalUV = ComputeLocalUV(NbrFrame, Position);

		FFrame3d SeedToLocal(SeedFrameIn);
		SeedToLocal.AlignAxis(2, NbrFrame.Z());

		FVector3d vAlignedSeedX = SeedToLocal.X();
		FVector3d vLocalX = NbrFrame.X();

		double CosTheta = vLocalX.Dot(vAlignedSeedX);

		// compute rotated min-dist vector for this particle
		double Temp = 1.0 - CosTheta * CosTheta;
		if (Temp < 0)
		{
			Temp = 0;     // need to clamp so that sqrt works...
		}
		double SinTheta = (double)TMathUtil<double>::Sqrt(Temp);
		FVector3d vCross = vLocalX.Cross(vAlignedSeedX);
		if ( vCross.Dot(NbrFrame.Z()) < 0)  // get the right sign...
		{
			SinTheta = -SinTheta;
		}
		FMatrix2d FrameRotateMat(CosTheta, SinTheta, -SinTheta, CosTheta);
		return NbrUV + FrameRotateMat * LocalUV;
	}


	void UpdateUVExpmap(FGraphNode& Node)
	{
		FGraphNode* ParentNode = GetNodeForPointSetID(Node.ParentPointID, false);
		check(ParentNode != nullptr);

		FFrame3d ParentFrame = GetFrame(*ParentNode);

		Node.UV = PropagateUV(GetPosition(Node.PointID), ParentNode->UV, ParentFrame, SeedFrame);
	}



	void UpdateUVExpmapUpwind(FGraphNode& Node)
	{
		FGraphNode* ParentNode = GetNodeForPointSetID(Node.ParentPointID, false);
		check(ParentNode != nullptr);

		FVector3d NodePos = GetPosition(Node.PointID);

		FVector2d AverageUV = FVector2d::Zero();
		double WeightSum = 0;
		int32 NbrCount = 0;
		for (int32 NbrPointID : PointSet->VtxVerticesItr(Node.PointID))
		{
			FGraphNode* NbrNode = GetNodeForPointSetID(NbrPointID, false);
			if (NbrNode != nullptr && NbrNode->bFrozen)
			{
				FFrame3d NbrFrame(GetFrame(*NbrNode));
				FVector2d NbrUV = PropagateUV(NodePos, NbrNode->UV, NbrFrame, SeedFrame);
				double Weight = 1.0 / (DistanceSquared(NodePos,NbrFrame.Origin) + TMathUtil<double>::ZeroTolerance);
				AverageUV += Weight * NbrUV;
				WeightSum += Weight;
				NbrCount++;
			}
		}
		check(NbrCount > 0);

		Node.UV = AverageUV / WeightSum;
	}



	void UpdateUVPlanar(FGraphNode& Node)
	{
		Node.UV = ComputeLocalUV(SeedFrame, GetPosition(Node.PointID));
	}




	FGraphNode* GetNodeForPointSetID(int32 PointSetID, bool bCreateIfMissing)
	{
		const int32* AllocatedIndex = IDToNodeIndexMap.Find(PointSetID);
		if (AllocatedIndex == nullptr)
		{
			if (bCreateIfMissing)
			{
				FGraphNode NewNode{ PointSetID, -1, 0, FVector2d::Zero(), false, FVector3d::UnitZ() };

				if (bEnableExternalNormals)
				{
					NewNode.CachedNormal = ExternalNormalFunc(PointSetID);
				}
				else
				{
					NewNode.CachedNormal = GetNormal(PointSetID);
				}

				int32 NewIndex = AllocatedNodes.Num();
				AllocatedNodes.Add(NewNode);
				IDToNodeIndexMap.Add(PointSetID, NewIndex);
				return &AllocatedNodes[NewIndex];
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return &AllocatedNodes[*AllocatedIndex];
		}
	}


	const FGraphNode* GetNodeForPointSetID(int32 PointSetID) const
	{
		const int32* AllocatedIndex = IDToNodeIndexMap.Find(PointSetID);
		return (AllocatedIndex != nullptr) ? &AllocatedNodes[*AllocatedIndex] : nullptr;
	}


	// given new Distance/UV at Parent, check if any of its neighbours are in the queue,
	// and if they are, and the new graph distance is shorter, update their queue position
	// (this is basically the update step of Disjktras algorithm)
	void UpdateNeighboursSparse(FGraphNode* Parent)
	{
		FVector3d ParentPos(GetPosition(Parent->PointID));
		double ParentDist = Parent->GraphDistance;

		for (int32 NbrPointID : PointSet->VtxVerticesItr(Parent->PointID))
		{
			FGraphNode* NbrNode = GetNodeForPointSetID(NbrPointID, true);
			if (NbrNode->bFrozen)
			{
				continue;
			}

			double NbrDist = ParentDist + Distance(ParentPos, GetPosition(NbrPointID));
			if (Queue.Contains(NbrPointID))
			{
				if (NbrDist < NbrNode->GraphDistance)
				{
					NbrNode->ParentPointID = Parent->PointID;
					NbrNode->GraphDistance = NbrDist;
					Queue.Update(NbrPointID, float(NbrNode->GraphDistance));
				}
			}
			else 
			{
				NbrNode->ParentPointID = Parent->PointID;
				NbrNode->GraphDistance = NbrDist;
				Queue.Insert(NbrPointID, float(NbrNode->GraphDistance));
			}
		}
	}

};


} // end namespace UE::Geometry
} // end namespace UE