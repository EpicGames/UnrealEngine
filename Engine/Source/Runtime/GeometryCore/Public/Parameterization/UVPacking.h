// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Containers/Array.h"
#include "IndexTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "VectorTypes.h"

template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Geometry
{

/**
 * FUVPacker implements various strategies for packing UV islands in a
 * generic mesh class.  The UV islands must already be identified, this
 * class simply scales/rotates/translates the islands to fit.
 */
class FUVPacker
{
public:

	// abstract interface to apply UV packer a mesh
	struct IUVMeshView
	{
		virtual ~IUVMeshView() = default;
		virtual FIndex3i GetTriangle(int32 TID) const = 0;
		virtual FIndex3i GetUVTriangle(int32 TID) const = 0;
		virtual FVector3d GetVertex(int32 VID) const = 0;
		virtual FVector2f GetUV(int32 EID) const = 0;
		virtual void SetUV(int32 EID, FVector2f UV) = 0;
	};
	

	/** Resolution of the target texture. This is used to convert pixel gutter/border thickness to UV space */
	int32 TextureResolution = 512;

	/** Thickness of gutter/border in pixel dimensions. Not supported by all packing methods  */
	float GutterSize = 1.0;

	/** If true, original island scale is maintained during any packing process. */
	bool bPreserveScale = false;

	/** If true, original island rotation is maintained during any packing process. Automatically prevents bAllowFlips from applying, if set. */
	bool bPreserveRotation = false;

	/** If true, islands can be flipped in addition to rotate/translate/scale */
	bool bAllowFlips = false;

	/** Attempt to rescale islands to match texel-to-world-space ratio across islands, based on ratio of World- and UV-space edge lengths */
	bool bScaleIslandsByWorldSpaceTexelRatio = false;

	/**
	 * Standard UE UV layout, similar to that used for Lightmap UVs. 
	 * All UV islands are packed into standard positive-unit-square.
	 * Only supports single-pixel border size.
	 */
	GEOMETRYCORE_API bool StandardPack(IUVMeshView* Mesh, int NumIslands, TFunctionRef<void(int, TArray<int32>&)> CopyIsland);

	/// Version of StandardPack that takes an array of arrays instead of a TFunctionRef, for convenience
	bool StandardPack(IUVMeshView* Mesh, const TArray<TArray<int>>& UVIslands)
	{
		return StandardPack(Mesh, UVIslands.Num(), [&UVIslands](int Idx, TArray<int32>& IslandOut)
			{
				IslandOut = UVIslands[Idx];
			});
	}

	/**
	 * Uniformly scale all UV islands so that the largest fits in positive-unit-square,
	 * and translate each islands separately so that it's bbox-min is at the origin.
	 * So the islands are "stacked" and all fit in the unit box.
	 */
	GEOMETRYCORE_API bool StackPack(IUVMeshView* Mesh, int NumIslands, TFunctionRef<void(int, TArray<int32>&)> CopyIsland);

	/// Version of StackPack that takes an array of arrays instead of a TFunctionRef, for convenience
	bool StackPack(IUVMeshView* Mesh, const TArray<TArray<int>>& UVIslands)
	{
		return StackPack(Mesh, UVIslands.Num(), [&UVIslands](int Idx, TArray<int32>& IslandOut)
			{
				IslandOut = UVIslands[Idx];
			});
	}

protected:

	/**
	 * Compute common stats used by the packing algorithms to transform UV islands
	 */
	GEOMETRYCORE_API void GetIslandStats(IUVMeshView* Mesh, const TArray<int32>& Island, FAxisAlignedBox2d& IslandBoundsOut, double& IslandScaleFactorOut, double& UVAreaOut);

};


} // end namespace UE::Geometry
} // end namespace UE

