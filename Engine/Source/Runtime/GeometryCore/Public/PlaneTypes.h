// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3sharp Plane3

#pragma once

#include "VectorTypes.h"
#include "VectorUtil.h"
#include "Math/Plane.h"
#include "Math/Transform.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template<typename RealType>
struct TPlane3
{
	TVector<RealType> Normal;
	RealType Constant;

	TPlane3() {}

	TPlane3(const UE::Math::TVector<RealType>& Normal, double Constant) : Normal(Normal), Constant(Constant)
	{
	}

	/**
	 * N is specified, c = Dot(N,P) where P is a point on the plane.
	 */
	TPlane3(const UE::Math::TVector<RealType>& Normal, const UE::Math::TVector<RealType>& Point) : Normal(Normal), Constant(Normal.Dot(Point))
	{
	}

	/**
	 * N = Cross(P1-P0,P2-P0)/Length(Cross(P1-P0,P2-P0)), c = Dot(N,P0) where
	 * P0, P1, P2 are points on the plane.
	 */
	TPlane3(const UE::Math::TVector<RealType>& P0, const UE::Math::TVector<RealType>& P1, const UE::Math::TVector<RealType>& P2)
	{
		Normal = VectorUtil::Normal(P0, P1, P2);
		Constant = Normal.Dot(P0);
	}
	
	explicit TPlane3(const FPlane& Plane) : TPlane3(Plane.GetNormal(), Plane.W)
	{
	}
	
	explicit operator FPlane() const
	{
		return FPlane(Normal.X, Normal.Y, Normal.Z, Constant);
	}

	/**
	 * Transform the plane by the given transform
	 */
	void Transform(const TTransform<RealType>& Tr)
	{
		TVector<RealType> TransformedOrigin = Tr.TransformPosition(Normal * Constant);
		// Transform the normal
		Normal = VectorUtil::TransformNormal(Tr, Normal);
		// Update the plane constant using the transformed normal and origin
		Constant = Normal.Dot(TransformedOrigin);
	}

	/**
	 * Transform the plane by the inverse of the given transform
	 */
	void InverseTransform(const TTransform<RealType>& Tr)
	{
		TVector<RealType> TransformedOrigin = Tr.InverseTransformPosition(Normal * Constant);
		// Inverse transform the normal
		Normal = VectorUtil::InverseTransformNormal(Tr, Normal);
		// Update the plane constant using the transformed normal and origin
		Constant = Normal.Dot(TransformedOrigin);
	}

	/**
	 * Compute d = Dot(N,P)-c where N is the plane normal and c is the plane
	 * constant.  This is a signed distance.  The sign of the return value is
	 * positive if the point is on the positive side of the plane, negative if
	 * the point is on the negative side, and zero if the point is on the
	 * plane.
	 */
	double DistanceTo(const UE::Math::TVector<RealType>& P) const
	{
		return Normal.Dot(P) - Constant;
	}

	/**
	 * The "positive side" of the plane is the half space to which the plane
	 * normal points.  The "negative side" is the other half space.  The
	 * function returns +1 when P is on the positive side, -1 when P is on the
	 * the negative side, or 0 when P is on the plane.
	 */
	int WhichSide(const UE::Math::TVector<RealType>& P) const
	{
		double Distance = DistanceTo(P);
		if (Distance < 0)
		{
			return -1;
		}
		else if (Distance > 0)
		{
			return +1;
		}
		else
		{
			return 0;
		}
	}




	/**
	 * Compute intersection of Line with Plane
	 * @param LineOrigin origin of line
	 * @param LineDirection direction of line
	 * @param HitPointOut intersection point, or invalid point if line is parallel to plane
	 * @return true if Line intersects Plane and IntersectionPointOut is valid
	 */
	bool FindLineIntersection(
		const UE::Math::TVector<RealType>& LineOrigin, 
		const UE::Math::TVector<RealType>& LineDirection, 
		UE::Math::TVector<RealType>& IntersectionPointOut) const
	{
		RealType NormalDot = LineDirection.Dot(Normal);
		if ( TMathUtil<RealType>::Abs(NormalDot) < TMathUtil<RealType>::ZeroTolerance )
		{
			IntersectionPointOut = TVector<RealType>(TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max(), TNumericLimits<RealType>::Max());
			return false;
		}
		RealType t = -(LineOrigin.Dot(Normal) - Constant) / NormalDot;
		IntersectionPointOut = LineOrigin + t * LineDirection;
		return true;
	}


	enum EClipSegmentType
	{
		FullyClipped,
		FirstClipped,
		SecondClipped,
		NotClipped
	};

	/**
	 * Clip line segment defined by two points against plane. Region of Segment on positive side of Plane is kept.
	 * 
	 * @param Point0 first point of segment
	 * @param Point1 second point of segment
	 * @return FullyClipped if the segment lies fully behind or exactly in the plane
	 *         FirstClipped if Point0 became the intersection point on the plane
	 *         SecondClipped if Point1 became the intersection point on the plane
	 *         NotClipped if the segment lies fully in front of the plane
	 */
	EClipSegmentType ClipSegment(UE::Math::TVector<RealType>& Point0, UE::Math::TVector<RealType>& Point1) const
	{
		RealType Dist0 = DistanceTo(Point0);
		RealType Dist1 = DistanceTo(Point1);
		if (Dist0 <= 0 && Dist1 <= 0)
		{
			return EClipSegmentType::FullyClipped;
		}
		else if (Dist0 * Dist1 > 0)
		{
			return EClipSegmentType::NotClipped;
		}

		TVector<RealType> DirectionVec = Point1 - Point0;
		TVector<RealType> Direction = Normalized(DirectionVec);
		RealType Length = DirectionVec.Dot(Direction);

		// test if segment is parallel to plane, if so, no intersection
		RealType NormalDot = Direction.Dot(Normal);
		if ( TMathUtil<RealType>::Abs(NormalDot) < TMathUtil<RealType>::ZeroTolerance )
		{
			return EClipSegmentType::NotClipped;
		}

		RealType LineT = -Dist0 / NormalDot;  // calculate line parameter for line/plane intersection
		if (LineT > 0 && LineT < Length)   // verify segment intersection  (should always be true...)
		{
			if (NormalDot < 0)
			{
				Point1 = Point0 + LineT * Direction;
				return EClipSegmentType::SecondClipped;
			}
			else
			{
				Point0 += LineT * Direction;
				return EClipSegmentType::FirstClipped;
			}
		}
		return EClipSegmentType::NotClipped;
	}

};

typedef TPlane3<float> FPlane3f;
typedef TPlane3<double> FPlane3d;

} // end namespace UE::Geometry
} // end namespace UE

