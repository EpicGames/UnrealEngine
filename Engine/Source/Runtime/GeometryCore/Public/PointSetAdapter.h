// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TPointSetAdapter provides a very generic interface to an indexable list of points.
 * The list may be sparse, ie some indices may be invalid.
 */
template<typename RealType>
struct TPointSetAdapter
{
	/** Maximum point index */
	TFunction<int32()> MaxPointID;
	/** Number of points. If PointCount == MaxPointID, then there are no gaps in the index list */
	TFunction<int32()> PointCount;
	/** Returns true if this index valid */
	TFunction<bool(int32)> IsPoint;
	/** Get point at this index */
	TFunction<TVector<RealType>(int32)> GetPoint;

	/** Returns true if this point set has per-point normals */
	TFunction<bool()> HasNormals;
	/** Get the normal at a point index */
	TFunction<FVector3f(int32)> GetPointNormal;
};

typedef TPointSetAdapter<double> FPointSetAdapterd;
typedef TPointSetAdapter<float> FPointSetAdapterf;


} // end namespace UE::Geometry
} // end namespace UE