// Copyright Epic Games, Inc. All Rights Reserved.

// port of geometry3Sharp Polygon

#pragma once

#include "Templates/UnrealTemplate.h"
#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "BoxTypes.h"
#include "SegmentTypes.h"
#include "LineTypes.h"
#include "MathUtil.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "Curve/CurveUtil.h"
#include "Algo/Reverse.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TPolygon2 is a 2D polygon represented as a list of Vertices.
 */
template<typename T>
class TPolygon2
{

protected:
	/** The list of vertices/corners of the polygon */
	TArray<TVector2<T>> Vertices;

public:

	TPolygon2()
	{
	}

	/**
	 * Construct polygon with given list of vertices
	 */
	TPolygon2(TArray<TVector2<T>> VertexList) : Vertices(MoveTemp(VertexList))
	{
	}

	template<typename OtherVertexType>
	TPolygon2(const TArray<OtherVertexType>& VertexList)
	{
		Vertices.Reserve(VertexList.Num());
		for (const OtherVertexType& OtherVtx : VertexList)
		{
			Vertices.Add( TVector2<T>(OtherVtx.X, OtherVtx.Y) );
		}
	}


	/**
	 * Construct polygon with given indices into a vertex array
	 */
	TPolygon2(TArrayView<const TVector2<T>> VertexArray, TArrayView<const int32> VertexIndices)
	{
		Vertices.SetNum(VertexIndices.Num());
		for (int32 Idx = 0; Idx < VertexIndices.Num(); Idx++)
		{
			Vertices[Idx] = VertexArray[VertexIndices[Idx]];
		}
	}

	/**
	 * Get the vertex at a given index
	 */
	const TVector2<T>& operator[](int Index) const
	{
		return Vertices[Index];
	}

	/**
	 * Get the vertex at a given index
	 */
	TVector2<T>& operator[](int Index)
	{
		return Vertices[Index];
	}


	/**
	 * @return first vertex of Polygon
	 */
	const TVector2<T>& Start() const
	{
		return Vertices[0];
	}

	/**
	 * @return list of Vertices of Polygon
	 */
	const TArray<TVector2<T>>& GetVertices() const
	{
		return Vertices;
	}

	/**
	 * @return number of Vertices in Polygon
	 */
	int VertexCount() const
	{
		return Vertices.Num();
	}

	/**
	 * Add a vertex to the Polygon
	 */
	void AppendVertex(const TVector2<T>& Position)
	{
		Vertices.Add(Position);
	}

	/**
	 * Add a list of Vertices to the Polygon
	 */
	void AppendVertices(const TArray<TVector2<T>>& NewVertices)
	{
		Vertices.Append(NewVertices);
	}

	/**
	 * Set vertex at given index to a new Position
	 */
	void Set(int VertexIndex, const TVector2<T>& Position)
	{
		Vertices[VertexIndex] = Position;
	}

	/**
	 * Remove a vertex of the Polygon (existing Vertices are shifted)
	 */
	void RemoveVertex(int VertexIndex)
	{
		Vertices.RemoveAt(VertexIndex);
	}

	/**
	 * Replace the list of Vertices with a new list
	 */
	void SetVertices(const TArray<TVector2<T>>& NewVertices)
	{
		Vertices = NewVertices;
	}


	/**
	 * Reverse the order of the Vertices in the Polygon (ie switch between Clockwise and CounterClockwise)
	 */
	void Reverse()
	{
		Algo::Reverse(Vertices);
	}


	/**
	 * Get the tangent vector at a vertex of the polygon, which is the normalized
	 * vector from the previous vertex to the next vertex
	 */
	TVector2<T> GetTangent(int VertexIndex) const
	{
		return CurveUtil::Tangent<T, TVector2<T>, true>(Vertices, VertexIndex);
	}


	/**
	 * Get the normal vector at a vertex of the polygon, which is perpendicular to GetTangent()
	 * Points "inward" for a Clockwise Polygon, and outward for CounterClockwise
	 */
	TVector2<T> GetNormal(int VertexIndex) const
	{
		return PerpCW(GetTangent(VertexIndex));
	}


	/**
	 * Construct a normal at a vertex of the Polygon by averaging the adjacent face normals.
	 * This vector is independent of the lengths of the adjacent segments.
	 * Points "inward" for a Clockwise Polygon, and outward for CounterClockwise
	 */
	TVector2<T> GetNormal_FaceAvg(int VertexIndex) const
	{
		return CurveUtil::GetNormal_FaceAvg2<T, TVector2<T>, true>(Vertices, VertexIndex);
	}


	/**
	 * @return the bounding box of the Polygon Vertices
	 */
	TAxisAlignedBox2<T> Bounds() const
	{
		return TAxisAlignedBox2<T>(Vertices);
	}

	
	/**
	 * SegmentIterator is used to iterate over the TSegment2<T> segments of the polygon
	 */
	class SegmentIterator 
	{
	public:
		inline bool operator!()
		{
			return i < polygon->VertexCount();
		}
		inline TSegment2<T> operator*() const
		{
			check(polygon != nullptr && i < polygon->VertexCount());
			return TSegment2<T>(polygon->Vertices[i], polygon->Vertices[(i+1) % polygon->VertexCount()]);
		}
		//inline TSegment2<T> & operator*();
		inline SegmentIterator & operator++() 		// prefix
		{
			i++;
			return *this;
		}
		inline SegmentIterator operator++(int) 		// postfix
		{
			SegmentIterator copy(*this);
			i++;
			return copy;
		}
		inline bool operator==(const SegmentIterator & i2) const { return i2.polygon == polygon && i2.i == i; }
		inline bool operator!=(const SegmentIterator & i2) const { return i2.polygon != polygon || i2.i != i; }
	protected:
		const TPolygon2 * polygon;
		int i;
		inline SegmentIterator(const TPolygon2 * p, int iCur) : polygon(p), i(iCur) {}
		friend class TPolygon2;
	};
	friend class SegmentIterator;

	SegmentIterator SegmentItr() const 
	{
		return SegmentIterator(this, 0);
	}

	/**
	 * Wrapper around SegmentIterator that has begin() and end() suitable for range-based for loop
	 */
	class SegmentEnumerable
	{
	public:
		const TPolygon2<T>* polygon;
		SegmentEnumerable() : polygon(nullptr) {}
		SegmentEnumerable(const TPolygon2<T> * p) : polygon(p) {}
		SegmentIterator begin() { return polygon->SegmentItr(); }
		SegmentIterator end() { return SegmentIterator(polygon, polygon->VertexCount()); }
	};

	/**
	 * @return an object that can be used in a range-based for loop to iterate over the Segments of the Polygon
	 */
	SegmentEnumerable Segments() const
	{
		return SegmentEnumerable(this);
	}


	/**
	 * @return true if the Polygon Vertices have Clockwise winding order / orientation  (signed area is negative)
	 */
	bool IsClockwise() const
	{
		return SignedArea() < 0;
	}

	/**
	 * @return the signed area of the Polygon
	 */
	T SignedArea() const
	{
		return CurveUtil::SignedArea2<T, TVector2<T>>(Vertices);
	}

	/**
	 * @return the unsigned area of the Polygon
	 */
	T Area() const
	{
		return TMathUtil<T>::Abs(SignedArea());
	}

	/**
	 * @return the total perimeter length of the Polygon
	 */
	T Perimeter() const
	{
		return CurveUtil::ArcLength<T, TVector2<T>>(Vertices, true);
	}


	/**
	 * Get the previous and next vertex positions for a given vertex of the Polygon
	 */
	void NeighbourPoints(int VertexIdx, TVector2<T>& OutPrevNbr, TVector2<T>& OutNextNbr) const
	{
		CurveUtil::GetPrevNext<T, TVector2<T>, true>(Vertices, VertexIdx, OutPrevNbr, OutNextNbr);
	}


	/**
	 * Get the vectors from a given vertex to the previous and next Vertices, optionally normalized
	 */
	void NeighbourVectors(int VertexIdx, TVector2<T>& OutToPrev, TVector2<T>& OutToNext, bool bNormalize = false) const
	{
		CurveUtil::GetVectorsToPrevNext<T, TVector2<T>, true>(Vertices, VertexIdx, OutToPrev, OutToNext, bNormalize);
	}


	/**
	 * @return the opening angle in degrees at a vertex of the Polygon
	 */
	T OpeningAngleDeg(int iVertex) const
	{
		TVector2<T> e0, e1;
		NeighbourVectors(iVertex, e0, e1, true);
		return AngleD(e0, e1);
	}


	/**
	 * @return analytic winding integral for this Polygon at an arbitrary point
	 */
	T WindingIntegral(const TVector2<T>& QueryPoint) const
	{
		return CurveUtil::WindingIntegral2<T, TVector2<T>>(Vertices, QueryPoint);
	}

	/**
	 * @param RadiansTolerance		Maximum turn in the 'wrong' direction that will still be considered convex (in radians)
	 * @param bDegenerateIsConvex	What to return for degenerate input (less than 3 points or equivalent due to repeated points)
	 * @return						true if polygon is convex
	 */
	bool IsConvex(T RadiansTolerance = TMathUtil<T>::ZeroTolerance, bool bDegenerateIsConvex = true) const
	{
		return CurveUtil::IsConvex2<T, TVector2<T>>(Vertices, RadiansTolerance, bDegenerateIsConvex);
	}


	/**
	 * @return true if the given query point is inside the Polygon, based on the winding integral
	 */
	bool Contains(const TVector2<T>& QueryPoint) const
	{
		return CurveUtil::Contains2<T, TVector2<T>>(Vertices, QueryPoint);
	}

	/**
	 * Check for polygon overlap, aka solid intersection.  (In contrast, note that the "Intersects" method checks for edge intersection)
	 *
	 * @return true if the Polygon overlaps the OtherPolygon.
	 */
	bool Overlaps(const TPolygon2<T>& OtherPoly) const
	{
		if (!Bounds().Intersects(OtherPoly.Bounds()))
		{
			return false;
		}

		for (int i = 0, N = OtherPoly.VertexCount(); i < N; ++i) 
		{
			if (Contains(OtherPoly[i]))
			{
				return true;
			}
		}

		for (int i = 0, N = VertexCount(); i < N; ++i) 
		{
			if (OtherPoly.Contains(Vertices[i]))
			{
				return true;
			}
		}

		for (TSegment2<T> seg : Segments()) 
		{
			for (TSegment2<T> oseg : OtherPoly.Segments()) 
			{
				if (seg.Intersects(oseg))
				{
					return true;
				}
			}
		}
		return false;
	}

	/**
	 * @return true if the Polygon fully contains the OtherPolygon
	 */
	bool Contains(const TPolygon2<T>& OtherPoly) const
	{
		// @todo fast bbox check?

		int N = OtherPoly.VertexCount();
		for (int i = 0; i < N; ++i) 
		{
			if (Contains(OtherPoly[i]) == false)
			{
				return false;
			}
		}

		if (Intersects(OtherPoly))
		{
			return false;
		}

		return true;
	}


	/**
	 * @return true if the Segment is fully contained inside the Polygon
	 */
	bool Contains(const TSegment2<T>& Segment) const
	{
		// [TODO] Add bbox check
		if (Contains(Segment.StartPoint()) == false || Contains(Segment.EndPoint()) == false)
		{
			return false;
		}

		for (TSegment2<T> seg : Segments())
		{
			if (seg.Intersects(Segment))
			{
				return false;
			}
		}
		return true;
	}
	
	/**
	 * Clip polygon to the given bounds. Note that if the polygon is not convex, it may have overlapping edges at the bounds.
	 */
	void ClipConvex(const TAxisAlignedBox2<T>& Bounds)
	{
		CurveUtil::ClipConvexToBounds<T, TVector2<T>, true, 2>(Vertices, Bounds.Min, Bounds.Max);
	}


	/**
	 * @return true if at least one edge of the OtherPolygon intersects the Polygon
	 */
	bool Intersects(const TPolygon2<T>& OtherPoly) const
	{
		if (!Bounds().Intersects(OtherPoly.Bounds()))
		{
			return false;
		}

		for (TSegment2<T> seg : Segments()) 
		{
			for (TSegment2<T> oseg : OtherPoly.Segments()) 
			{
				if (seg.Intersects(oseg))
				{
					return true;
				}
			}
		}
		return false;
	}


	/**
	 * @return true if the Segment intersects an edge of the Polygon
	 */
	bool Intersects(const TSegment2<T>& Segment) const
	{
		// [TODO] Add bbox check
		if (Contains(Segment.StartPoint()) == true || Contains(Segment.EndPoint()) == true)
		{
			return true;
		}

		// [TODO] Add bbox check
		for (TSegment2<T> seg : Segments())
		{
			if (seg.Intersects(Segment))
			{
				return true;
			}
		}
		return false;
	}


	/**
	 * Find all the points where an edge of the Polygon intersects an edge of the OtherPolygon
	 * @param OtherPoly polygon to test against
	 * @param OutArray intersection points are stored here
	 * @return true if any intersections were found
	 */
	bool FindIntersections(const TPolygon2<T>& OtherPoly, TArray<TVector2<T>>& OutArray) const
	{
		if (!Bounds().Intersects(OtherPoly.Bounds()))
		{
			return false;
		}

		bool bFoundIntersections = false;
		for (TSegment2<T> seg : Segments()) 
		{
			for (TSegment2<T> oseg : OtherPoly.Segments())
			{
				// this computes test twice for intersections, but seg.intersects doesn't
				// create any new objects so it should be much faster for majority of segments (should profile!)
				if (seg.Intersects(oseg)) 
				{
					//@todo can we replace with something like seg.intersects?
					TIntrSegment2Segment2<T> intr(seg, oseg);
					if (intr.Find()) 
					{
						bFoundIntersections = true;
						OutArray.Add(intr.Point0);
						if (intr.Quantity == 2) 
						{
							OutArray.Add(intr.Point1);
						}
					}
				}
			}
		}

		return bFoundIntersections;
	}


	/**
	 * @return edge of the polygon starting at vertex SegmentIndex
	 */
	TSegment2<T> Segment(int SegmentIndex) const
	{
		return TSegment2<T>(Vertices[SegmentIndex], Vertices[(SegmentIndex + 1) % Vertices.Num()]);
	}

	/**
	 * @param SegmentIndex index of first vertex of the edge
	 * @param SegmentParam parameter in range [-Extent,Extent] along segment
	 * @return point on the segment at the given parameter value
	 */
	TVector2<T> GetSegmentPoint(int SegmentIndex, T SegmentParam) const
	{
		TSegment2<T> seg(Vertices[SegmentIndex], Vertices[(SegmentIndex + 1) % Vertices.Num()]);
		return seg.PointAt(SegmentParam);
	}

	/**
	 * @param SegmentIndex index of first vertex of the edge
	 * @param SegmentParam parameter in range [0,1] along segment
	 * @return point on the segment at the given parameter value
	 */
	TVector2<T> GetSegmentPointUnitParam(int SegmentIndex, T SegmentParam) const
	{
		TSegment2<T> seg(Vertices[SegmentIndex], Vertices[(SegmentIndex + 1) % Vertices.Num()]);
		return seg.PointBetween(SegmentParam);
	}


	/**
	 * @param SegmentIndex index of first vertex of the edge
	 * @param SegmentParam parameter in range [0,1] along segment
	 * @return interpolated normal to the segment at the given parameter value
	 */
	TVector2<T> GetNormal(int iSeg, T SegmentParam) const
	{
		TSegment2<T> seg(Vertices[iSeg], Vertices[(iSeg + 1) % Vertices.Num()]);
		T t = ((SegmentParam / seg.Extent) + 1.0) / 2.0;

		TVector2<T> n0 = GetNormal(iSeg);
		TVector2<T> n1 = GetNormal((iSeg + 1) % Vertices.Num());
		return Normalized((T(1) - t) * n0 + t * n1);
	}



	/**
	 * Calculate the squared distance from a point to the polygon
	 * @param QueryPoint the query point
	 * @param NearestSegIndexOut The index of the nearest segment
	 * @param NearestSegParamOut the parameter value of the nearest point on the segment
	 * @return squared distance to the polygon
	 */
	T DistanceSquared(const TVector2<T>& QueryPoint, int& NearestSegIndexOut, T& NearestSegParamOut) const
	{
		TVector2<T> NearestPoint_Unused;
		T DistSq, UnitParam_Unused;
		CurveUtil::ProjectToCurve<T, TVector2<T>>(Vertices, true, QueryPoint, DistSq, NearestSegIndexOut, NearestPoint_Unused, UnitParam_Unused, &NearestSegParamOut);
		return DistSq;
	}


	/**
	 * Calculate the squared distance from a point to the polygon
	 * @param QueryPoint the query point
	 * @return squared distance to the polygon
	 */
	T DistanceSquared(const TVector2<T>& QueryPoint) const
	{
		int Seg_Unused;
		TVector2<T> NearestPoint_Unused;
		T DistSq, UnitParam_Unused;
		CurveUtil::ProjectToCurve<T, TVector2<T>>(Vertices, true, QueryPoint, DistSq, Seg_Unused, NearestPoint_Unused, UnitParam_Unused, nullptr);
		return DistSq;
	}


	/**
	 * @return average edge length of all the edges of the Polygon
	 */
	T AverageEdgeLength() const
	{
		T avg = 0; int N = Vertices.Num();
		for (int i = 1; i < N; ++i) {
			avg += Distance(Vertices[i], Vertices[i - 1]);
		}
		avg += Distance(Vertices[N - 1], Vertices[0]);
		return avg / N;
	}


	/**
	 * Translate the polygon
	 * @returns the Polygon, so that you can chain calls like Translate().Scale()
	 */
	TPolygon2<T>& Translate(const TVector2<T>& Translate) 
	{
		int N = Vertices.Num();
		for (int i = 0; i < N; ++i)
		{
			Vertices[i] += Translate;
		}
		return *this;
	}

	/**
	 * Scale the Polygon relative to a given point
	 * @returns the Polygon, so that you can chain calls like Translate().Scale()
	 */
	TPolygon2<T>& Scale(const TVector2<T>& Scale, const TVector2<T>& Origin)
	{
		int N = Vertices.Num();
		for (int i = 0; i < N; ++i)
		{
			Vertices[i] = Scale * (Vertices[i] - Origin) + Origin;
		}
		return *this;
	}


	/**
	 * Apply an arbitrary transformation to the Polygon
	 * @returns the Polygon, so that you can chain calls like Translate().Scale()
	 */
	TPolygon2<T>& Transform(const TFunction<TVector2<T> (const TVector2<T>&)>& TransformFunc)
	{
		int N = Vertices.Num();
		for (int i = 0; i < N; ++i) 
		{
			Vertices[i] = TransformFunc(Vertices[i]);
		}
		return *this;
	}




	/**
	 * Offset each point by the given Distance along vertex "normal" direction
	 * @param OffsetDistance the distance to offset
	 * @param bUseFaceAvg if true, we offset by the average-face normal instead of the perpendicular-tangent normal
	 */
	void VtxNormalOffset(T OffsetDistance, bool bUseFaceAvg = false)
	{
		TArray<TVector2<T>> NewVertices;
		NewVertices.SetNumUninitialized(Vertices.Num());
		if (bUseFaceAvg) 
		{
			for (int k = 0; k < Vertices.Num(); ++k)
			{
				NewVertices[k] = Vertices[k] + OffsetDistance * GetNormal_FaceAvg(k);
			}
		}
		else 
		{
			for (int k = 0; k < Vertices.Num(); ++k)
			{
				NewVertices[k] = Vertices[k] + OffsetDistance * GetNormal(k);
			}
		}
		for (int k = 0; k < Vertices.Num(); ++k)
		{
			Vertices[k] = NewVertices[k];
		}
	}


	/**
	 * Offset polygon by fixed distance, by offsetting and intersecting edges.
	 * CounterClockWise Polygon offsets "outwards", ClockWise "inwards".
	 */
	void PolyOffset(T OffsetDistance)
	{
		// [TODO] possibly can do with half as many normalizes if we do w/ sequential edges,
		//  rather than centering on each v?
		TArray<TVector2<T>> NewVertices;
		NewVertices.SetNumUninitialized(Vertices.Num());
		for (int k = 0; k < Vertices.Num(); ++k) 
		{
			TVector2<T> v = Vertices[k];
			TVector2<T> next = Vertices[(k + 1) % Vertices.Num()];
			TVector2<T> prev = Vertices[k == 0 ? Vertices.Num() - 1 : k - 1];
			TVector2<T> dn = Normalized(next - v);
			TVector2<T> dp = Normalized(prev - v);
			TLine2<T> ln(v + OffsetDistance * PerpCW(dn), dn);
			TLine2<T> lp(v - OffsetDistance * PerpCW(dp), dp);

			bool bIntersectsAtPoint = ln.IntersectionPoint(lp, NewVertices[k]);
			if (!bIntersectsAtPoint)  // lines were parallel
			{
				NewVertices[k] = Vertices[k] + OffsetDistance * GetNormal_FaceAvg(k);
			}
		}
		for (int k = 0; k < Vertices.Num(); ++k)
		{
			Vertices[k] = NewVertices[k];
		}
	}


private:
	// Polygon simplification
	// code adapted from: http://softsurfer.com/Archive/algorithm_0205/algorithm_0205.htm
	// simplifyDP():
	//  This is the Douglas-Peucker recursive simplification routine
	//  It just marks Vertices that are part of the simplified polyline
	//  for approximating the polyline subchain v[j] to v[k].
	//    Input:  tol = approximation tolerance
	//            v[] = polyline array of vertex points
	//            j,k = indices for the subchain v[j] to v[k]
	//    Output: mk[] = array of markers matching vertex array v[]
	static void SimplifyDouglasPeucker(T Tolerance, const TArray<TVector2<T>>& Vertices, int j, int k, TArray<bool>& Marked)
	{
		Marked.SetNum(Vertices.Num());
		if (k <= j + 1) // there is nothing to simplify
			return;

		// check for adequate approximation by segment S from v[j] to v[k]
		int maxi = j;          // index of vertex farthest from S
		T maxd2 = 0;         // distance squared of farthest vertex
		T tol2 = Tolerance * Tolerance;  // tolerance squared
		TSegment2<T> S = TSegment2<T>(Vertices[j], Vertices[k]);    // segment from v[j] to v[k]

		// test each vertex v[i] for max distance from S
		// Note: this works in any dimension (2D, 3D, ...)
		for (int i = j + 1; i < k; i++)
		{
			T dv2 = S.DistanceSquared(Vertices[i]);
			if (dv2 <= maxd2)
				continue;
			// v[i] is a max vertex
			maxi = i;
			maxd2 = dv2;
		}
		if (maxd2 > tol2)       // error is worse than the tolerance
		{
			// split the polyline at the farthest vertex from S
			Marked[maxi] = true;      // mark v[maxi] for the simplified polyline
			// recursively simplify the two subpolylines at v[maxi]
			SimplifyDouglasPeucker(Tolerance, Vertices, j, maxi, Marked);  // polyline v[j] to v[maxi]
			SimplifyDouglasPeucker(Tolerance, Vertices, maxi, k, Marked);  // polyline v[maxi] to v[k]
		}
		// else the approximation is OK, so ignore intermediate Vertices
		return;
	}


public:

	/**
	 * Simplify the Polygon to reduce the vertex count
	 * @param ClusterTolerance Vertices closer than this distance will be merged into a single vertex
	 * @param LineDeviationTolerance Vertices are allowed to deviate this much from the input polygon lines
	 */
	void Simplify(T ClusterTolerance = 0.0001, T LineDeviationTolerance = 0.01)
	{
		int n = Vertices.Num();
		if (n < 3)
		{
			return;
		}

		int i, k, pv;            // misc counters
		TArray<TVector2<T>> NewVertices;
		NewVertices.SetNumUninitialized(n + 1);  // vertex buffer
		TArray<bool> Marked;
		Marked.SetNumUninitialized(n + 1);
		for (i = 0; i < n + 1; ++i)		// marker buffer
		{
			Marked[i] = false;
		}

		// STAGE 1.  Vertex Reduction within tolerance of prior vertex cluster
		T clusterTol2 = ClusterTolerance * ClusterTolerance;
		NewVertices[0] = Vertices[0];              // start at the beginning
		for (i = 1, k = 1, pv = 0; i < n; i++) 
		{
			if ( UE::Geometry::DistanceSquared(Vertices[i], Vertices[pv]) < clusterTol2)
			{
				continue;
			}
			NewVertices[k++] = Vertices[i];
			pv = i;
		}
		bool skip_dp = false;
		if (k == 1) 
		{
			NewVertices[k++] = Vertices[1];
			NewVertices[k++] = Vertices[2];
			skip_dp = true;
		}
		else if (k == 2) 
		{
			if (pv + 1 < n) // pv < n-1 case; use vertex pv+1
			{
				NewVertices[2] = Vertices[pv + 1];
			}
			else // pv == n-1 case; use vertices n-2 and n-1
			{
				NewVertices[1] = Vertices[pv - 1];
				NewVertices[2] = Vertices[pv];
			}
			k = 3;
			skip_dp = true;
		}

		// push on start vertex again, because simplifyDP is for polylines, not polygons
		NewVertices[k++] = Vertices[0];

		// STAGE 2.  Douglas-Peucker polyline simplification
		int nv = 0;
		if (skip_dp == false && LineDeviationTolerance > 0) 
		{
			Marked[0] = Marked[k - 1] = true;       // mark the first and last Vertices
			SimplifyDouglasPeucker(LineDeviationTolerance, NewVertices, 0, k - 1, Marked);
			for (i = 0; i < k - 1; ++i) 
			{
				if (Marked[i])
				{
					nv++;
				}
			}
		}
		else 
		{
			for (i = 0; i < k; ++i)
			{
				Marked[i] = true;
			}
			nv = k - 1;
		}

		// polygon requires at least 3 Vertices
		if (nv == 2) 
		{
			for (i = 1; i < k - 1; ++i) 
			{
				if (Marked[1] == false)
				{
					Marked[1] = true;
				}
				else if (Marked[k - 2] == false)
				{
					Marked[k - 2] = true;
				}
			}
			nv++;
		}
		else if (nv == 1) 
		{
			Marked[1] = true;
			Marked[2] = true;
			nv += 2;
		}

		// copy marked Vertices back to this polygon
		Vertices.Reset();
		for (i = 0; i < k - 1; ++i)    // last vtx is copy of first, and definitely marked
		{
			if (Marked[i]) 
			{
				Vertices.Add(NewVertices[i]);
			}
		}

		return;
	}



	/**
	 * Chamfer each vertex corner of the Polygon
	 * @param ChamferDist offset distance from corner that we cut at
	 */
	void Chamfer(T ChamferDist, T MinConvexAngleDeg = 30, T MinConcaveAngleDeg = 30)
	{
		check(IsClockwise());

		TArray<TVector2<T>> OldV = Vertices;
		int N = OldV.Num();
		Vertices.Reset();

		int iCur = 0;
		do {
			TVector2<T> center = OldV[iCur];

			int iPrev = (iCur == 0) ? N - 1 : iCur - 1;
			TVector2<T> prev = OldV[iPrev];
			int iNext = (iCur + 1) % N;
			TVector2<T> next = OldV[iNext];

			TVector2<T> cp = prev - center;
			T cpdist = Normalize(cp);
			TVector2<T> cn = next - center;
			T cndist = Normalize(cn);

			// if degenerate, skip this vert
			if (cpdist < TMathUtil<T>::ZeroTolerance || cndist < TMathUtil<T>::ZeroTolerance) 
			{
				iCur = iNext;
				continue;
			}

			T angle = AngleD(cp, cn);
			// TODO document what this means sign-wise
			// TODO re-test post Unreal port that this DotPerp is doing the right thing
			T sign = DotPerp(cn, cp);
			bool bConcave = (sign > 0);

			T thresh = (bConcave) ? MinConcaveAngleDeg : MinConvexAngleDeg;

			// ok not too sharp
			if (angle > thresh) 
			{
				Vertices.Add(center);
				iCur = iNext;
				continue;
			}


			T prev_cut_dist = TMathUtil<T>::Min(ChamferDist, cpdist*0.5);
			TVector2<T> prev_cut = center + cp * prev_cut_dist;
			T next_cut_dist = TMathUtil<T>::Min(ChamferDist, cndist * 0.5);
			TVector2<T> next_cut = center + cn * next_cut_dist;

			Vertices.Add(prev_cut);
			Vertices.Add(next_cut);
			iCur = iNext;
		} while (iCur != 0);
	}




	/**
	 * Construct a four-vertex rectangle polygon
	 */
	static TPolygon2<T> MakeRectangle(const TVector2<T>& Center, T Width, T Height)
	{
		TPolygon2<T> Rectangle;
		Rectangle.Vertices.SetNumUninitialized(4);
		Rectangle.Set(0, TVector2<T>(Center.X - Width / 2, Center.Y - Height / 2));
		Rectangle.Set(1, TVector2<T>(Center.X + Width / 2, Center.Y - Height / 2));
		Rectangle.Set(2, TVector2<T>(Center.X + Width / 2, Center.Y + Height / 2));
		Rectangle.Set(3, TVector2<T>(Center.X - Width / 2, Center.Y + Height / 2));
		return Rectangle;
	}

	/**
	* Construct a rounded rectangle polygon
	*/
	static TPolygon2<T> MakeRoundedRectangle(const TVector2<T>& Center, T Width, T Height, T Corner, int CornerSteps)
	{
		TPolygon2<T> RRectangle;
		CornerSteps = FMath::Max(2, CornerSteps);
		RRectangle.Vertices.SetNumUninitialized(CornerSteps * 4);
		
		TVector2<T> InnerExtents(Width*.5 - Corner, Height*.5 - Corner);
		TVector2<T> InnerCorners[4]
		{
			Center - InnerExtents,
			TVector2<T>(Center.X + InnerExtents.X, Center.Y - InnerExtents.Y),
			Center + InnerExtents,
			TVector2<T>(Center.X - InnerExtents.X, Center.Y + InnerExtents.Y),
		};

		T IdxToAng = TMathUtil<T>::HalfPi / T(CornerSteps - 1);
		for (int StepIdx = 0; StepIdx < CornerSteps; StepIdx++)
		{
			T Angle = StepIdx * IdxToAng;
			T OffC = TMathUtil<T>::Cos(Angle) * Corner;
			T OffS = TMathUtil<T>::Sin(Angle) * Corner;
			RRectangle.Set(StepIdx + CornerSteps * 0, InnerCorners[0] + TVector2<T>(-OffC, -OffS));
			RRectangle.Set(StepIdx + CornerSteps * 1, InnerCorners[1] + TVector2<T>(OffS, -OffC));
			RRectangle.Set(StepIdx + CornerSteps * 2, InnerCorners[2] + TVector2<T>(OffC, OffS));
			RRectangle.Set(StepIdx + CornerSteps * 3, InnerCorners[3] + TVector2<T>(-OffS, OffC));
		}
		return RRectangle;
	}


	/**
	 * Construct a circular polygon
	 */
	static TPolygon2<T> MakeCircle(T Radius, int Steps, T AngleShiftRadians = 0)
	{
		TPolygon2<T> Circle;
		Circle.Vertices.SetNumUninitialized(Steps);

		for (int i = 0; i < Steps; ++i)
		{
			T t = (T)i / (T)Steps;
			T a = TMathUtil<T>::TwoPi * t + AngleShiftRadians;
			Circle.Set(i, TVector2<T>(Radius * TMathUtil<T>::Cos(a), Radius * TMathUtil<T>::Sin(a)));
		}

		return Circle;
	}

};

typedef TPolygon2<double> FPolygon2d;
typedef TPolygon2<float> FPolygon2f;

} // end namespace UE::Geometry
} // end namespace UE