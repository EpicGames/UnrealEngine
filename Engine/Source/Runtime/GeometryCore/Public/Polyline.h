// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "SegmentTypes.h"
#include "Curve/CurveUtil.h"
#include "LineTypes.h"
#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;


/*
 *  TPolylinePolicy represents a collection of needed support classes for TPolyline depending on the required dimension
 */

template<typename T, int DIM>
class TPolylinePolicy
{};

template<typename T>
class TPolylinePolicy<T, 3>
{
public:
	typedef TVector<T> VectorType;
	typedef TSegment3<T> SegmentType;
	typedef TAxisAlignedBox3<T> BoxType;
};

template<typename T>
class TPolylinePolicy<T, 2>
{
public:
	typedef TVector2<T> VectorType;
	typedef TSegment2<T> SegmentType;
	typedef TAxisAlignedBox2<T> BoxType;
};

/**
 * TPolyline represents a dimensional independent polyline stored as a list of Vertices.
 */
template<typename T, int D>
class TPolyline
{
protected:
	typedef typename TPolylinePolicy<T, D>::VectorType VectorType;
	typedef typename TPolylinePolicy<T, D>::SegmentType SegmentType;
	typedef typename TPolylinePolicy<T, D>::BoxType BoxType;

	/** The list of vertices of the polyline */
	TArray<VectorType> Vertices;
	
public:


	TPolyline()
	{
	}

	/**
	 * Construct polyline with given list of vertices
	 */
	TPolyline(const TArray<VectorType>& VertexList) : Vertices(VertexList)
	{
	}

	/**
	 * Construct a single-segment polyline
	 */
	TPolyline(const VectorType& Point0, const VectorType& Point1)
	{
		Vertices.Add(Point0);
		Vertices.Add(Point1);
	}

	/**
	 * Get the vertex at a given index
	 */
	const VectorType& operator[](int Index) const
	{
		return Vertices[Index];
	}

	/**
	 * Get the vertex at a given index
	 */
	VectorType& operator[](int Index)
	{
		return Vertices[Index];
	}


	/**
	 * @return first vertex of polyline
	 */
	const VectorType& Start() const
	{
		return Vertices[0];
	}

	/**
	 * @return last vertex of polyline
	 */
	const VectorType& End() const
	{
		return Vertices[Vertices.Num()-1];
	}


	/**
	 * @return list of Vertices of polyline
	 */
	const TArray<VectorType>& GetVertices() const
	{
		return Vertices;
	}

	/**
	 * @return number of Vertices in polyline
	 */
	int VertexCount() const
	{
		return Vertices.Num();
	}

	/**
	 * @return number of segments in polyline
	 */
	int SegmentCount() const
	{
		return Vertices.Num()-1;
	}


	/** 
	* Discard all vertices of polyline 
	* 
	* @param NewSize The expected number of vertices to be appended after calling this function.
	*/
	void Clear(int32 NewSize = 0)
	{
		Vertices.Reset(NewSize);
	}

	/** 
	* Reserves memory for the given number of vertices 
	* 
	* @param The number of vertices the polyline should be able to contain after allocation.
	*/
	void Reserve(int32 NewSize)
	{
		Vertices.Reserve(NewSize);
	}

	/**
	 * Add a vertex to the polyline
	 */
	void AppendVertex(const VectorType& Position)
	{
		Vertices.Add(Position);
	}

	/**
	 * Add a list of Vertices to the polyline
	 */
	void AppendVertices(const TArray<VectorType>& NewVertices)
	{
		Vertices.Append(NewVertices);
	}

	/**
	 * Add a list of Vertices to the polyline
	 */
	void AppendVertices(const VectorType* VerticesPtr, int Count)
	{
		Vertices.Append(VerticesPtr, Count);
	}

	/**
	 * Add a list of Vertices to the polyline
	 */
	template<typename OtherVectorType>
	void AppendVertices(const TArray<OtherVectorType>& NewVertices)
	{
		int32 NumV = NewVertices.Num();
		Vertices.Reserve(Vertices.Num() + NumV);
		for (int32 k = 0; k < NumV; ++k)
		{
			Vertices.Append( (VectorType)NewVertices[k] );
		}
	}

	/**
	 * Set vertex at given index to a new Position
	 */
	void Set(int VertexIndex, const VectorType& Position)
	{
		Vertices[VertexIndex] = Position;
	}

	/**
	 * Remove a vertex of the polyline (existing Vertices are shifted)
	 */
	void RemoveVertex(int VertexIndex)
	{
		Vertices.RemoveAt(VertexIndex);
	}

	/**
	* Remove multiple vertices of the polyline (existing Vertices are shifted)
	*/
	void RemoveVertices(int VertexIndex, int Count)
	{
		Vertices.RemoveAt(VertexIndex, Count);
	}

	/**
	 * Replace the list of Vertices with a new list
	 */
	void SetVertices(const TArray<VectorType>& NewVertices)
	{
		int NumVerts = NewVertices.Num();
		Vertices.SetNum(NumVerts, EAllowShrinking::No);
		for (int k = 0; k < NumVerts; ++k)
		{
			Vertices[k] = NewVertices[k];
		}
	}


	/**
	 * Reverse the order of the Vertices in the polyline (ie switch between Clockwise and CounterClockwise)
	 */
	void Reverse()
	{
		int32 j = Vertices.Num() - 1;
		for (int32 VertexIndex = 0; VertexIndex < j; VertexIndex++, j--)
		{
			Swap(Vertices[VertexIndex], Vertices[j]);
		}
	}


	/**
	 * Get the tangent vector at a vertex of the polyline, which is the normalized
	 * vector from the previous vertex to the next vertex
	 */
	VectorType GetTangent(int VertexIndex) const
	{
		if (VertexIndex == 0)
		{
			return (Vertices[1] - Vertices[0]).Normalized();
		} 
		int NumVerts = Vertices.Num();
		if (VertexIndex == NumVerts - 1)
		{
			return (Vertices[NumVerts-1] - Vertices[NumVerts-2]).Normalized();
		}
		return (Vertices[VertexIndex+1] - Vertices[VertexIndex-1]).Normalized();
	}



	/**
	 * @return edge of the polyline starting at vertex SegmentIndex
	 */
	SegmentType GetSegment(int SegmentIndex) const
	{
		return SegmentType(Vertices[SegmentIndex], Vertices[SegmentIndex+1]);
	}


	/**
	 * @param SegmentIndex index of first vertex of the edge
	 * @param SegmentParam parameter in range [-Extent,Extent] along segment
	 * @return point on the segment at the given parameter value
	 */
	VectorType GetSegmentPoint(int SegmentIndex, T SegmentParam) const
	{
		SegmentType seg(Vertices[SegmentIndex], Vertices[SegmentIndex + 1]);
		return seg.PointAt(SegmentParam);
	}


	/**
	 * @param SegmentIndex index of first vertex of the edge
	 * @param SegmentParam parameter in range [0,1] along segment
	 * @return point on the segment at the given parameter value
	 */
	VectorType GetSegmentPointUnitParam(int SegmentIndex, T SegmentParam) const
	{
		SegmentType seg(Vertices[SegmentIndex], Vertices[SegmentIndex + 1]);
		return seg.PointBetween(SegmentParam);
	}



	/**
	 * @return the bounding box of the polyline Vertices
	 */
	BoxType GetBounds() const
	{
		BoxType box = BoxType::Empty();
		int NumVertices = Vertices.Num();
		for (int k = 0; k < NumVertices; ++k)
		{
			box.Contain(Vertices[k]);
		}
		return box;
	}



	/**
	 * @return the total perimeter length of the Polygon
	 */
	T Length() const
	{
		T length = 0;
		int N = SegmentCount();
		for (int i = 0; i < N; ++i)
		{
			length += Distance(Vertices[i], Vertices[i+1]);
		}
		return length;
	}



	/**
	 * SegmentIterator is used to iterate over the segments of the polyline
	 */
	class SegmentIterator
	{
	public:
		inline bool operator!()
		{
			return i < Polyline->SegmentCount();
		}
		inline SegmentType operator*() const
		{
			check(Polyline != nullptr && i < Polyline->SegmentCount());
			return SegmentType(Polyline->Vertices[i], Polyline->Vertices[i+1]);
		}
		inline SegmentIterator & operator++() 		// prefix
		{
			i++;
			return *this;
		}
		inline SegmentIterator operator++(int) 		// postfix
		{
			SegmentIterator copy(*this);
			i++;
			return copy;
		}
		inline bool operator==(const SegmentIterator & i3) { return i3.Polyline == Polyline && i3.i == i; }
		inline bool operator!=(const SegmentIterator & i3) { return i3.Polyline != Polyline || i3.i != i; }
	protected:
		const TPolyline<T, D>* Polyline;
		int i;
		inline SegmentIterator(const  TPolyline<T, D>* p, int iCur) : Polyline(p), i(iCur) {}
		friend class TPolyline;
	};
	friend class SegmentIterator;

	SegmentIterator SegmentItr() const
	{
		return SegmentIterator(this, 0);
	}

	/**
	 * Wrapper around SegmentIterator that has begin() and end() suitable for range-based for loop
	 */
	class SegmentEnumerable
	{
	public:
		const TPolyline<T,D>* Polyline;
		SegmentEnumerable() : Polyline(nullptr) {}
		SegmentEnumerable(const TPolyline<T,D> * p) : Polyline(p) {}
		SegmentIterator begin() { return Polyline->SegmentItr(); }
		SegmentIterator end() { return SegmentIterator(Polyline, Polyline->SegmentCount()); }
	};

	/**
	 * @return an object that can be used in a range-based for loop to iterate over the Segments of the Polyline
	 */
	SegmentEnumerable Segments() const
	{
		return SegmentEnumerable(this);
	}






	/**
	 * Calculate the squared distance from a point to the polyline
	 * @param QueryPoint the query point
	 * @param NearestSegIndexOut The index of the nearest segment
	 * @param NearestSegParamOut the parameter value of the nearest point on the segment
	 * @return squared distance to the polyline
	 */
	T DistanceSquared(const VectorType& QueryPoint, int& NearestSegIndexOut, T& NearestSegParamOut) const
	{
		VectorType NearestPoint_Unused;
		T DistSq, UnitParam_Unused;
		CurveUtil::ProjectToCurve<T, VectorType>(Vertices, false, QueryPoint, DistSq, NearestSegIndexOut, NearestPoint_Unused, UnitParam_Unused, &NearestSegParamOut);
		return DistSq;
	}



	/**
	 * Calculate the squared distance from a point to the polyline
	 * @param QueryPoint the query point
	 * @return squared distance to the polyline
	 */
	T DistanceSquared(const VectorType& QueryPoint) const
	{
		int NearestSegIndex_Unused;
		VectorType NearestPoint_Unused;
		T DistSq, UnitParam_Unused;
		CurveUtil::ProjectToCurve<T, VectorType>(Vertices, false, QueryPoint, DistSq, NearestSegIndex_Unused, NearestPoint_Unused, UnitParam_Unused, nullptr);
		return DistSq;
	}




	/**
	 * @return average edge length of all the edges of the Polygon
	 */
	T AverageEdgeLength() const
	{
		T avg = 0; int N = Vertices.Num();
		for (int i = 1; i < N; ++i) {
			avg += Distance(Vertices[i], Vertices[i - 1]);
		}
		return avg / (T)(N-1);
	}

	/**
	 * Compute a point on the polyline based on its distance from the first vertex 
	 */
	VectorType GetPointFromFirstVertex(const T InDistance) const
	{
		if (InDistance <= 0)
		{
			return Vertices[0];
		}

		T RemainingDistance = InDistance;
		const int N = Vertices.Num();
		for (int i = 1; i < N; ++i)
		{
			const T SegmentLength = Distance(Vertices[i], Vertices[i - 1]);
			if (SegmentLength > 0 && SegmentLength > RemainingDistance)
			{
				return Lerp(Vertices[i - 1], Vertices[i], RemainingDistance / SegmentLength);
			}
			RemainingDistance -= SegmentLength;
		}

		return Vertices.Last();
	}

	/**
	* Compute a point on the polyline based on its distance from the last vertex 
	*/
	VectorType GetPointFromLastVertex(const T InDistance) const
	{
		if (InDistance <= 0)
		{
			return Vertices.Last();
		}

		T RemainingDistance = InDistance;
		const int N = Vertices.Num();
		for (int i = N-1; i > 0; --i)
		{
			const T SegmentLength = Distance(Vertices[i], Vertices[i - 1]);
			if (SegmentLength > 0 && SegmentLength > RemainingDistance)
			{
				return Lerp(Vertices[i], Vertices[i-1], RemainingDistance / SegmentLength);
			}
			RemainingDistance -= SegmentLength;
		}

		return Vertices[0];
	}

	/**
	 * Produce a new polyline that is smoother than this one
	 */
	void SmoothSubdivide(TPolyline<T,D>& NewPolyline) const
	{
		const T Alpha = (T)1 / (T)3;
		const T OneMinusAlpha = (T)2 / (T)3;

		int N = Vertices.Num() - 1;
		NewPolyline.Vertices.SetNum(2*N);
		NewPolyline.Vertices[0] = Vertices[0];
		int k = 1;
		for (int i = 1; i < N; ++i)
		{
			const VectorType& Prev = Vertices[i-1];
			const VectorType& Cur = Vertices[i];
			const VectorType& Next = Vertices[i+1];
			NewPolyline.Vertices[k++] = Alpha * Prev + OneMinusAlpha * Cur;
			NewPolyline.Vertices[k++] = OneMinusAlpha * Cur + Alpha * Next;
		}
		NewPolyline.Vertices[k] = Vertices[N];
	}

	/**
	 * Simplify the Polyline to reduce the vertex count
	 * @param ClusterTolerance Vertices closer than this distance will be merged into a single vertex
	 * @param LineDeviationTolerance Vertices are allowed to deviate this much from the polylines
	 */
	void Simplify(T ClusterTolerance, T LineDeviationTolerance)
	{
		TArray<int32> NewVertexIndices;
		Simplify(ClusterTolerance, LineDeviationTolerance, NewVertexIndices);
	}

	/**
	 * Simplify the Polyline to reduce the vertex count
	 * @param ClusterTolerance Vertices closer than this distance will be merged into a single vertex
	 * @param LineDeviationTolerance Vertices are allowed to deviate this much from the polylines
	 * @param OutNewVertexIndices Output array containing the indices of Vertices that remain after simplification.
	 *        The indices reference the Vertices array before simplification and are ordered along the polyline.
	 */
	void Simplify(T ClusterTolerance, T LineDeviationTolerance, TArray<int32>& OutNewVertexIndices)
	{
		const int32 VertexCount = Vertices.Num();
		if (VertexCount < 3)
		{
			OutNewVertexIndices.SetNumUninitialized(VertexCount);
			for (int32 i = 0; i < VertexCount; ++i)
			{
				OutNewVertexIndices[i] = i;
			}
			// we need at least 3 vertices to be able to simplify a line
			return;
		}

		OutNewVertexIndices.Reset(VertexCount);

		// STAGE 1.  Vertex Reduction within tolerance of prior vertex cluster
		if (ClusterTolerance > 0)
		{
			T ClusterToleranceSquared = ClusterTolerance * ClusterTolerance;
			OutNewVertexIndices.Add(0);				// keep the first vertex
			for (int32 Index = 1; Index < VertexCount-1; Index++) 
			{
				if (Geometry::DistanceSquared(Vertices[Index], Vertices[OutNewVertexIndices.Last()]) < ClusterToleranceSquared)
				{
					continue;
				}
				OutNewVertexIndices.Add(Index);
			}
			OutNewVertexIndices.Add(VertexCount - 1);		// keep the last vertex
		}
		else
		{
			OutNewVertexIndices.SetNumUninitialized(VertexCount);
			for (int32 i = 0; i < VertexCount; ++i)
			{
				OutNewVertexIndices[i] = i;
			}
		}

		// STAGE 2.  Douglas-Peucker polyline simplification
		TArray<bool> Marked;
		if (LineDeviationTolerance > 0 && OutNewVertexIndices.Num() >= 3)
		{
			Marked.SetNumZeroed(OutNewVertexIndices.Num());

			// mark the first and last vertices to make sure we keep them
			Marked[0] = true;
			Marked.Last() = true;
			SimplifyDouglasPeucker(LineDeviationTolerance, Vertices, OutNewVertexIndices, 0, OutNewVertexIndices.Num()-1, Marked);
		}

		// STAGE 3. Copy back values in OutNewVertexIndices
		if (!Marked.IsEmpty()) // If we have not run Douglas-Pecker, OutNewVertexIndices contains our final result
		{
			// only keep the marked ones
			int32 OutNumNewVertexIndices = 0;
			const int32 MarkedCount = Marked.Num();
			for (int32 Index = 0; Index < MarkedCount; ++Index)
			{
				if (Marked[Index])
				{
					OutNewVertexIndices[OutNumNewVertexIndices++] = OutNewVertexIndices[Index];
				}
			}
			OutNewVertexIndices.SetNum(OutNumNewVertexIndices);
		}

		if (OutNewVertexIndices.Num() != Vertices.Num())
		{
			check(OutNewVertexIndices.Num() < Vertices.Num());
			for (int32 i = 0; i < OutNewVertexIndices.Num(); ++i)
			{
				Vertices[i] = Vertices[OutNewVertexIndices[i]]; // This only works since OutNewVertexIndices is sorted from lowest to higest vertex index
			}

			Vertices.SetNum(OutNewVertexIndices.Num());
		}
	}

private:
	// Polygon simplification
	// code adapted from: http://softsurfer.com/Archive/algorithm_0205/algorithm_0205.htm
	// simplifyDP():
	//  This is the Douglas-Peucker recursive simplification routine
	//  It just marks Vertices that are part of the simplified polyline
	//  for approximating the polyline subchain v[j] to v[k].
	//    Input:  tol = approximation tolerance
	//            v[] = polyline array of vertex points
	//            j,k = indices for the subchain v[j] to v[k]
	//    Output: mk[] = array of markers matching vertex array v[]
	static void SimplifyDouglasPeucker(T Tolerance, const TArray<VectorType>& Vertices, const TArray<int32>& VertexIndices, int32 j, int32 k, TArray<bool>& Marked)
	{
		check(Marked.Num() == VertexIndices.Num());
		if (k <= j + 1) // there is nothing to simplify
			return;

		// check for adequate approximation by segment S from v[j] to v[k]
		int maxi = j;          // index of vertex farthest from S
		T maxd2 = 0;         // distance squared of farthest vertex
		T tol2 = Tolerance * Tolerance;  // tolerance squared
		SegmentType S = SegmentType(Vertices[VertexIndices[j]], Vertices[VertexIndices[k]]);    // segment from v[j] to v[k]

		// test each vertex v[i] for max distance from S
		// Note: this works in any dimension (2D, 3D, ...)
		for (int i = j + 1; i < k; i++)
		{
			T dv2 = S.DistanceSquared(Vertices[VertexIndices[i]]);
			if (dv2 <= maxd2)
				continue;
			// v[i] is a max vertex
			maxi = i;
			maxd2 = dv2;
		}
		if (maxd2 > tol2)       // error is worse than the tolerance
		{
			// split the polyline at the farthest vertex from S
			Marked[maxi] = true;      // mark v[maxi] for the simplified polyline
			// recursively simplify the two subpolylines at v[maxi]
			SimplifyDouglasPeucker(Tolerance, Vertices, VertexIndices, j, maxi, Marked);  // polyline v[j] to v[maxi]
			SimplifyDouglasPeucker(Tolerance, Vertices, VertexIndices, maxi, k, Marked);  // polyline v[maxi] to v[k]
		}
		// else the approximation is OK, so ignore intermediate Vertices
		return;
	}
};

} // end namespace UE::Geometry
} // end namespace UE
