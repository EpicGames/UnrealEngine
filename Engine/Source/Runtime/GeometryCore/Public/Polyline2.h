// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Polyline.h"

namespace UE
{
namespace Geometry
{

template<typename T>
using TPolyline2 = TPolyline<T, 2>;

typedef TPolyline2<double> FPolyline2d;
typedef TPolyline2<float> FPolyline2f;

} // end namespace UE::Geometry
} // end namespace UE
