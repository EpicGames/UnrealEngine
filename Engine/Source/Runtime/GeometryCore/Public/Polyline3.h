// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Polyline.h"

namespace UE
{
namespace Geometry
{

template<typename T>
using TPolyline3 = TPolyline<T, 3>;

typedef TPolyline3<double> FPolyline3d;
typedef TPolyline3<float> FPolyline3f;


} // end namespace UE::Geometry
} // end namespace UE
