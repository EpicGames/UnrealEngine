// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "MathUtil.h"
#include "GenericPlatform/GenericPlatformMath.h"

namespace UE {
namespace Geometry {
	
enum class EFBMMode : uint8
{
	/**
	 * Classic Perlin Noise
	 */
	Standard,

	/**
	 * Turbulent Modifier, creating upward bumps.
	 */
	Turbulent,

	/**
	 * Ridge Modifier, creating sharper creases and ridges.
	 */
	Ridge
};

template <typename T>
T FractalBrownianMotionNoise(
	const EFBMMode FBMMode, 
	const uint32 OctaveCount,
	const Math::TVector2<T> Coords, //< sampling coordinates
	const T Lacunarity, //< multiplier to apply to frequency from one octave to the next finer one
	const T Gain, //< weight to apply to amplitude from one octave to the next finer one
	const T Smoothness, //< smoothness amount to apply to turbulent and ridge modes
	const T Gamma) //< gamma to apply to turbulent and ridge
{
	T Amplitude{ 1. };

	checkSlow(Smoothness >= T(0.));
	checkSlow(Gain > T(0.));
	checkSlow(Lacunarity > T(0.));
	checkSlow(Gamma > T(0.));

	const T SmoothEps = Smoothness * T(0.01);
	FVector2d ST(Coords.X, Coords.Y);

	T TotalOffset{ 0. };

	for (uint32 Octave = 0; Octave < OctaveCount; ++Octave)
	{
		T Offset = FMath::PerlinNoise2D(ST);
		switch (FBMMode) {
		case EFBMMode::Standard:
			// nothing to be done
			break;
		case EFBMMode::Turbulent:
			Offset = FMath::Pow(2. * FMath::Sqrt(Offset * Offset + SmoothEps), Gamma);
			break;
		case EFBMMode::Ridge:
			Offset = FMath::Pow(1. - FMath::Sqrt(Offset * Offset + SmoothEps), Gamma);
			break;
		};

		TotalOffset += Amplitude * Offset;
		ST *= Lacunarity;
		Amplitude *= Gain;
	}

	return TotalOffset;
}

} // namespace Geometry
} // namespace UE