// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "Math/UnrealMathUtility.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * QuadricError represents a quadratic function that evaluates distance to plane.
 * Stores minimal 10-coefficient form, following http://mgarland.org/files/papers/qtheory.pdf
 * (symmetric matrix A, vector b, constant c)
 */
template<typename RealType>
struct TQuadricError 
{
	typedef RealType   ScalarType;

	RealType Axx, Axy, Axz, Ayy, Ayz, Azz;
	RealType bx, by, bz; //d = -normal.dot(c); b = d*normal
	RealType c; // d*d

	inline static TQuadricError Zero() { return TQuadricError(); };

	TQuadricError()
	{
		Axx = Axy = Axz = Ayy = Ayz = Azz = bx = by = bz = c = 0;
	}

	/**
	 * Construct TQuadricError a plane with the given normal and a point on plane
	 */
	TQuadricError(const TVector<RealType>& Normal, const TVector<RealType>& Point) 
	{
		Axx = Normal.X * Normal.X;
		Axy = Normal.X * Normal.Y;
		Axz = Normal.X * Normal.Z;
		Ayy = Normal.Y * Normal.Y;
		Ayz = Normal.Y * Normal.Z;
		Azz = Normal.Z * Normal.Z;
		bx = by = bz = c = 0;
		TVector<RealType> v = MultiplyA(Point);
		bx = -v.X; by = -v.Y; bz = -v.Z; // b = -Normal.Dot(Point) Normal
		c = Point.Dot(v); // note this is the same as Normal.Dot(Point) ^2
	}

	/**
	 * Construct TQuadricError that is the sum of two other TQuadricErrors
	 */
	TQuadricError(const TQuadricError& a, const TQuadricError& b) 
	{
		Axx = a.Axx + b.Axx;
		Axy = a.Axy + b.Axy;
		Axz = a.Axz + b.Axz;
		Ayy = a.Ayy + b.Ayy;
		Ayz = a.Ayz + b.Ayz;
		Azz = a.Azz + b.Azz;
		bx = a.bx + b.bx;
		by = a.by + b.by;
		bz = a.bz + b.bz;
		c = a.c + b.c;
	}


	/**
	 * Add scalar multiple of a TQuadricError to this TQuadricError
	 */
	void Add(RealType Weight, const TQuadricError& b) 
	{
		Axx += Weight * b.Axx;
		Axy += Weight * b.Axy;
		Axz += Weight * b.Axz;
		Ayy += Weight * b.Ayy;
		Ayz += Weight * b.Ayz;
		Azz += Weight * b.Azz;
		bx += Weight * b.bx;
		by += Weight * b.by;
		bz += Weight * b.bz;
		c += Weight * b.c;
	}

	void RemoveFaceQuadric(const int CornerIdx, RealType TriArea, RealType RemoveScale, const TQuadricError& b)
	{
		Add(-TriArea * RemoveScale, b);
	}

	void AccumulateFaceQuadric(const int CornerIdx, RealType TriArea, RealType Weight, const TQuadricError& b)
	{
		Add(TriArea * Weight, b);
	}

	void Add(const TQuadricError& b)
	{
		Axx += b.Axx;
		Axy += b.Axy;
		Axz += b.Axz;
		Ayy += b.Ayy;
		Ayz += b.Ayz;
		Azz += b.Azz;
		bx += b.bx;
		by += b.by;
		bz += b.bz;
		c +=  b.c;
	}

	void Subtract(const TQuadricError& b)
	{
		Axx -= b.Axx;
		Axy -= b.Axy;
		Axz -= b.Axz;
		Ayy -= b.Ayy;
		Ayz -= b.Ayz;
		Azz -= b.Azz;
		bx -= b.bx;
		by -= b.by;
		bz -= b.bz;
		c -= b.c;
	}

	void AddSeamQuadric(const TQuadricError& b)
	{
		Add(b);
	}

	void SubtractSeamQuadric(const TQuadricError& b)
	{
		Subtract(b);
	}

	/** 
	* Scale this quadric by the weight 'w'
	*/
	void Scale(RealType w)
	{
		Axx *= w;
		Axy *= w;
		Axz *= w;
		Ayy *= w;
		Ayz *= w;
		Azz *= w;
		bx *= w;
		by *= w;
		bz *= w;
		c *= w;
	}

	/**
	 * Evaluates p*A*p + 2*dot(p,b) + c
	 * @return 
	 */
	RealType Evaluate(const UE::Math::TVector<RealType>& pt) const
	{
		RealType x = Axx * pt.X + Axy * pt.Y + Axz * pt.Z;
		RealType y = Axy * pt.X + Ayy * pt.Y + Ayz * pt.Z;
		RealType z = Axz * pt.X + Ayz * pt.Y + Azz * pt.Z;
		return (pt.X * x + pt.Y * y + pt.Z * z) +
			2.0 * (pt.X * bx + pt.Y * by + pt.Z * bz) + c;
	}

	/**
	 * 
	 */
	TVector<RealType> MultiplyA(const UE::Math::TVector<RealType>& pt) const
	{
		RealType x = Axx * pt.X + Axy * pt.Y + Axz * pt.Z;
		RealType y = Axy * pt.X + Ayy * pt.Y + Ayz * pt.Z;
		RealType z = Axz * pt.X + Ayz * pt.Y + Azz * pt.Z;
		return TVector<RealType>(x, y, z);
	}


	bool SolveAxEqualsb(UE::Math::TVector<RealType>& OutResult, const RealType bvecx, const RealType bvecy, const RealType bvecz, const RealType minThresh = 1000.0*TMathUtil<RealType>::Epsilon) const
	{
		RealType a11 = Azz * Ayy - Ayz * Ayz;
		RealType a12 = Axz * Ayz - Azz * Axy;
		RealType a13 = Axy * Ayz - Axz * Ayy;
		RealType a22 = Azz * Axx - Axz * Axz;
		RealType a23 = Axy * Axz - Axx * Ayz;
		RealType a33 = Axx * Ayy - Axy * Axy;
		RealType det = (Axx * a11) + (Axy * a12) + (Axz * a13);

		// TODO: not sure what we should be using for this threshold...have seen
		//  det less than 10^-9 on "normal" meshes.
		if (FMath::Abs(det) > minThresh)
		{
			det = 1.0 / det;
			a11 *= det; a12 *= det; a13 *= det;
			a22 *= det; a23 *= det; a33 *= det;
			RealType x = a11 * bvecx + a12 * bvecy + a13 * bvecz;
			RealType y = a12 * bvecx + a22 * bvecy + a23 * bvecz;
			RealType z = a13 * bvecx + a23 * bvecy + a33 * bvecz;
			OutResult = TVector<RealType>(x, y, z);
			return true;
		}
		else 
		{
			return false;
		}
	}

	static bool InvertSymmetricMatrix(const RealType SM[6], RealType InvSM[6], RealType minThresh = 1000.0*TMathUtil<RealType>::Epsilon)
	{
		bool Result = false;
		// Axx = SM[0];  Axy = SM[1]; Axz = SM[2]
		//               Ayy = SM[3]; Ayz = SM[4]
		//                            Azz = SM[5]

		InvSM[0] = SM[5] * SM[3] - SM[4] * SM[4]; //s11
		InvSM[1] = SM[2] * SM[4] - SM[5] * SM[1]; //s12
		InvSM[2] = SM[1] * SM[4] - SM[2] * SM[3]; //s13
		InvSM[3] = SM[5] * SM[0] - SM[2] * SM[2]; //s22
		InvSM[4] = SM[1] * SM[2] - SM[0] * SM[4]; //s23
		InvSM[5] = SM[0] * SM[3] - SM[1] * SM[1]; //s33
		RealType Det = (SM[0] * InvSM[0]) + (SM[1] * InvSM[1]) + (SM[2] * InvSM[2]);
		if (FMath::Abs(Det) > minThresh)
		{
			RealType InvDet = RealType(1) / Det;
			InvSM[0] *= InvDet; InvSM[1] *= InvDet; InvSM[2] *= InvDet; 
			                    InvSM[3] *= InvDet; InvSM[4] *= InvDet;  
							                        InvSM[5] *= InvDet;
			Result = true;
	}
		return Result;
	}

	static TVector<RealType> MultiplySymmetricMatrix(const RealType SM[6], const RealType vec[3])
	{
		
		RealType a =  SM[0] * vec[0] + SM[1] * vec[1] + SM[2] * vec[2];
		RealType b =  SM[1] * vec[0] + SM[3] * vec[1] + SM[4] * vec[2];
		RealType c =  SM[2] * vec[0] + SM[4] * vec[1] + SM[5] * vec[2];

		return TVector<RealType>(a, b, c);
	}
	static TVector<RealType> MultiplySymmetricMatrix(const RealType SM[6], const UE::Math::TVector<RealType>& vec)
	{
		RealType vectmp[3] = { vec.X, vec.Y, vec.Z };
		return MultiplySymmetricMatrix(SM, vectmp);
	}


	bool OptimalPoint(UE::Math::TVector<RealType>& OutResult, RealType minThresh = 1000.0*TMathUtil<RealType>::Epsilon ) const
	{
		return SolveAxEqualsb(OutResult, -bx, -by, -bz, minThresh);
	}

	// Add regularizing term that penalizes movement away from Pos
	void AddPositionRegularizer(const TVector<RealType>& Pos, RealType Weight)
	{
		Axx += Weight;
		Ayy += Weight;
		Azz += Weight;
		bx -= Pos.X * Weight;
		by -= Pos.Y * Weight;
		bz -= Pos.Z * Weight;
		c += Pos.Dot(Pos) * Weight;
	}

	// Add regularizing term that penalizes movement away from the line through Pos along LineDir
	void AddLineRegularizer(const TVector<RealType>& Pos, const TVector<RealType>& LineDir, RealType Weight)
	{
		TVector<RealType> NormalizedLineDir = LineDir;
		if (!NormalizedLineDir.Normalize())
		{
			// if we fail to normalize, fall back to the position regularizer
			AddPositionRegularizer(Pos, Weight);
			return;
		}
		TVector<RealType> N1, N2;
		VectorUtil::MakePerpVectors(NormalizedLineDir, N1, N2);
		Add(Weight * RealType(0.5), TQuadricError<RealType>(N1, Pos));
		Add(Weight * RealType(0.5), TQuadricError<RealType>(N2, Pos));
	}

	// TODO remove the need to track the area in the simplifier and keep it quadric-internal
	RealType GetArea() const
	{
		return -1.f;
	}

	bool CanCollapse() const { return true; }

};


typedef TQuadricError<float> FQuadricErrorf;
typedef TQuadricError<double> FQuadricErrord;

/**
* Quadric Error type for use in memory-less simplification with volume preservation constraints.
* 
* See: http://hhoppe.com/newqem.pdf or https://www.cc.gatech.edu/~turk/my_papers/memless_vis98.pdf
* for information about the volume preservation. 
*/
template<typename RealType>
class TVolPresQuadricError : public TQuadricError<RealType>
{
public:

	typedef TQuadricError<RealType>  BaseStruct;

	struct FPlaneData
	{
		FPlaneData(const TVector<RealType>& Normal, const TVector<RealType>& Point)
		{
			N    = Normal;
			Dist = -Normal.Dot(Point);
		}

		FPlaneData(RealType w, const FPlaneData& other)
		{
			N    = w * other.N;
			Dist = w * other.Dist;
		}

		FPlaneData(const FPlaneData& other)
		{
			N    =  other.N;
			Dist =  other.Dist;
		}

		FPlaneData() 
		{
			N    = TVector<RealType>::Zero(); 
			Dist = RealType(0);
		}

		void Add(const FPlaneData& other)
		{
			N    += other.N;
			Dist += other.Dist;
		}

		/**
		* Add scalar multiple of a FPlaneData to this FPlaneData
		*/
		void Add(RealType w, const FPlaneData& other)
		{
			N    += w * other.N;
			Dist += w * other.Dist;
		}

		FPlaneData& operator=(const FPlaneData& other)
		{
			N    = other.N;
			Dist = other.Dist;

			return *this;
		}

		TVector<RealType> N;
		RealType Dist;
	};

	FPlaneData PlaneData;

public:

	inline static TVolPresQuadricError Zero() { return TVolPresQuadricError(); };
	
	TVolPresQuadricError() : BaseStruct()
	{
	}

	TVolPresQuadricError(const TVector<RealType>& Normal, const TVector<RealType>& Point)
		: BaseStruct(Normal, Point)
		, PlaneData(Normal, Point)
	{ }


	TVolPresQuadricError(const TVolPresQuadricError& a, const TVolPresQuadricError& b)
		: BaseStruct(a, b)
		, PlaneData(a.PlaneData)
	{
		PlaneData.Add(b.PlaneData);
	}

	TVolPresQuadricError(const TVolPresQuadricError& a, const TVolPresQuadricError& b, const FPlaneData& DuplicatePlaneData)
		: BaseStruct(a, b)
		, PlaneData(a.PlaneData)
	{
		PlaneData.Add(b.PlaneData);

		// Subtract a single copy of the duplicate plane data.
		PlaneData.Add(-1., DuplicatePlaneData);
	}

	/**
	* Area Weighted Add
	*/
	void Add(RealType Weight, const TVolPresQuadricError& b)
	{
		BaseStruct::Add(Weight, b);

		// this should probably only be weighted by TriArea
		PlaneData.Add(Weight, b.PlaneData);
	}

	void RemoveFaceQuadric(const int CornerIdx, RealType TriArea, RealType RemoveScale, const TVolPresQuadricError& b)
	{
		Add(-TriArea * RemoveScale, b);
	}

	void AccumulateFaceQuadric(const int CornerIdx, RealType TriArea, RealType Weight, const TVolPresQuadricError& b)
	{
		Add(TriArea * Weight, b);
	}

	void Scale(RealType Wt)
	{
		BaseStruct::Scale(Wt);
		PlaneData.N *= Wt;
		PlaneData.Dist *= Wt;
	}

	/**
	* The optimal point minimizing the quadric error with respect to a volume conserving constraint
	*/
	bool OptimalPoint(UE::Math::TVector<RealType>& OutResult, RealType minThresh = 1000.0*TMathUtil<RealType>::Epsilon) const
	{
		// Compute the unconstrained optimal point
		bool bValid = BaseStruct::OptimalPoint(OutResult, minThresh);

		// if it failed ( the A matrix wasn't invertible) we early out
		if (!bValid)
		{
			return bValid;
		}

		// Adjust with the volume constraint.  
		// See: http://hhoppe.com/newqem.pdf or https://www.cc.gatech.edu/~turk/my_papers/memless_vis98.pdf

		FPlaneData gvol(1. / 3., PlaneData);

		// Compute the effect of the volumetric constraint.
		RealType Tol = minThresh; //(1.e-7);  NB: using minThresh here to better compare with the attribute version..
		//constexpr RealType Tol = (1.e-7);  //NB: using minThresh here to better compare with the attribute version..
		
		TVector<RealType> Ainv_g;
		if (BaseStruct::SolveAxEqualsb(Ainv_g, gvol.N.X, gvol.N.Y, gvol.N.Z, Tol))
		{

			RealType gt_Ainv_g = gvol.N.Dot(Ainv_g);

			// move the point to the constrained optimal
			RealType gt_unopt = gvol.N.Dot(OutResult);
			RealType lambda = (gvol.Dist + gt_unopt) / gt_Ainv_g;

			// Check if the constraint failed.

			if (FMath::Abs(lambda) > 1.e5)
			{
				return bValid;
			}

			OutResult -= lambda * Ainv_g;
		}

		return bValid;
	}

	// Add regularizing term that penalizes movement away from Pos
	void AddPositionRegularizer(const UE::Math::TVector<RealType>& Pos, RealType Weight)
	{
		BaseStruct::AddPositionRegularizer(Pos, Weight);
	}

	// Add regularizing term that penalizes movement away from the line through Pos along LineDir
	void AddLineRegularizer(const UE::Math::TVector<RealType>& Pos, const UE::Math::TVector<RealType>& LineDir, RealType Weight)
	{
		BaseStruct::AddLineRegularizer(Pos, LineDir, Weight);
	}
};


typedef TVolPresQuadricError<float>  FVolPresQuadricErrorf;
typedef TVolPresQuadricError<double> FVolPresQuadricErrord;

/**
* Quadric Error type for use in volume memory-less simplification with volume preservation constraints.
* using the normal as three additional attributes to contribute to the quadric error.
* See: http://hhoppe.com/newqem.pdf 
*/
template<typename RealType>
class TAttrBasedQuadricError : public TVolPresQuadricError<RealType>
{
public:
	typedef RealType  ScalarType;
	typedef TVolPresQuadricError<RealType>  BaseClass;
	typedef TQuadricError<RealType>  BaseStruct;

	// Triangle Quadric constructor.  Take vertex locations, vertex normals, face normal, and center of face.
	TAttrBasedQuadricError(const TVector<RealType>& P0, const TVector<RealType>& P1, const TVector<RealType>& P2,
		const TVector<RealType>& N0, const TVector<RealType>& N1, const TVector<RealType>& N2,
		const TVector<RealType>& NFace, const TVector<RealType>& CenterPoint, RealType AttrWeight)
		: BaseClass(NFace, CenterPoint),
		a(0),
		attrweight(AttrWeight)
	{
		/**
		* Given a scalar attribute 'a' defined at the vertices e.g. a0, a1, a2 (in this case the attribute is a single component of the vertex normal)
		*
		* solve 4x4 system:
		*       (p0^t   1) (g[0])  = (a0)
		*       (p1^t   1) (g[1])    (a1)
		*       (p2^t   1) (g[2])    (a2)
		*       (n^t    0) (  d )    (0 )
		*
		* for the interpolating gradient 'g'   -- note this is really p.dot(g) + d = a0  and n.dot(g) = 0
		*
		* this system can be re-written as 
		*       < e_20 | g > = (a2-a0)
		*       < e_10 | g > = (a1-a0)
		*       <   n  | g > = 0
		* and   
		*       < p0 | g > + d = a0.
		* where e_20 is the edge vector p2 - p0
		*       e_10 is the edge vector p1 - p0.
		* 
		* the first 3 equations can be solved for the vector 'g' either as a 3x3 matrix system 
		* or one could solve a 2x2 system  since 'g' must be in the plane spanned by {e20, e10}  
		* ( give 'n' is orthogonal to the triangle face).
		*
		* For this quadric, each component of the normal is treated as separate attribute.
		*/
		 
		// the basis matrix is composed of two edges of the triangle and the face normal.
		// unless the triangle is degenerate (or too small), this should be invertible. 
		//@todo replace the 3x3 system with a 2x2 system that results from g = a1 e_10 + a2 e_20
		// and solving for a1, a2
		const TMatrix3<RealType> BasisMatrix(P2-P0, P1-P0, NFace, true); // row-based matrix constructor.

		const RealType det = BasisMatrix.Determinant(); // geometrically the det is related to the triangle area squared
		if (FMath::Abs(det) > 1.e-8)
		{
			// invert the matrix
			const TMatrix3<RealType> CoBasisMatrix = BasisMatrix.Inverse();

			// for each component of the attribute vector, 
			for (int i = 0; i < 3; ++i)
			{
				const FVector3d ScaledAttr = AttrWeight * FVector3d(N0[i], N1[i], N2[i]);
				const FVector3d DataVec(ScaledAttr[2] - ScaledAttr[0], ScaledAttr[1] - ScaledAttr[0], 0.);
				
				// compute vector 'g' and scalar 'd'
				grad[i] = CoBasisMatrix * DataVec;
				d[i] = ScaledAttr[0] - P0.Dot(grad[i]);

				// add the values to the quadric 
				BaseStruct::Axx += grad[i].X * grad[i].X;

				BaseStruct::Axy += grad[i].X * grad[i].Y;
				BaseStruct::Axz += grad[i].X * grad[i].Z;

				BaseStruct::Ayy += grad[i].Y * grad[i].Y;
				BaseStruct::Ayz += grad[i].Y * grad[i].Z;
				BaseStruct::Azz += grad[i].Z * grad[i].Z;

				BaseStruct::bx += d[i] * grad[i].X;
				BaseStruct::by += d[i] * grad[i].Y;
				BaseStruct::bz += d[i] * grad[i].Z;

				BaseStruct::c += d[i] * d[i];
			}
		}
		else // the triangle was too small or degenerate in some other way
		{
			for (int i = 0; i < 3; ++i)
			{
				grad[i] = TVector<RealType>::Zero();
				d[i] = AttrWeight * (N0[i] + N1[i] + N2[i]) / 3.; // just average the attribute.
			}
		}
	}

	TAttrBasedQuadricError()
		:BaseClass(),
		a(0.),
		attrweight(1.)
	{
		grad[0] = TVector<RealType>::Zero();
		grad[1] = TVector<RealType>::Zero();
		grad[2] = TVector<RealType>::Zero();

		d[0] = RealType(0);
		d[1] = RealType(0);
		d[2] = RealType(0);
	}

	TAttrBasedQuadricError(const TAttrBasedQuadricError& Aother, const TAttrBasedQuadricError& Bother)
		:BaseClass(Aother, Bother)
	{
		a = Aother.a + Bother.a;
		attrweight = 0.5 * (Aother.attrweight + Bother.attrweight);

		grad[0] = Aother.grad[0] + Bother.grad[0];
		grad[1] = Aother.grad[1] + Bother.grad[1];
		grad[2] = Aother.grad[2] + Bother.grad[2];

		d[0] = Aother.d[0] + Bother.d[0];
		d[1] = Aother.d[1] + Bother.d[1];
		d[2] = Aother.d[2] + Bother.d[2];
	}

	static TAttrBasedQuadricError Zero() { return TAttrBasedQuadricError(); }

	/**
	* Area Weighted Add
	*/
	void Add(RealType Weight, const TAttrBasedQuadricError& other)
	{
		BaseClass::Add(Weight, other);

		a += Weight; // accumulate weight
		attrweight = other.attrweight;

		grad[0] += Weight * other.grad[0];
		grad[1] += Weight * other.grad[1];
		grad[2] += Weight * other.grad[2];

		d[0] += Weight * other.d[0];
		d[1] += Weight * other.d[1];
		d[2] += Weight * other.d[2];
	}

	void RemoveFaceQuadric(const int CornerIdx, RealType TriArea, RealType RemoveScale, const TAttrBasedQuadricError& b)
	{
		Add(-TriArea * RemoveScale, b);
	}

	void AccumulateFaceQuadric(const int CornerIdx, RealType TriArea, RealType Weight, const TAttrBasedQuadricError& b)
	{
		Add(TriArea * Weight, b);
	}

	void Scale(RealType Wt)
	{
		BaseClass::Scale(Wt);
		a *= Wt;
		grad[0] *= Wt;
		grad[1] *= Wt;
		grad[2] *= Wt;
		d[0] *= Wt;
		d[1] *= Wt;
		d[2] *= Wt;
		// note: we intentionally do not scale attrweight here; it characterizes the relative importance of attribute error and doesn't scale w/ data
	}

	/**
	* The optimal point minimizing the quadric error with respect to a volume conserving constraint
	*/
	bool OptimalPoint(UE::Math::TVector<RealType>& OptPoint, RealType minThresh = 1000.0*TMathUtil<RealType>::Epsilon) const
	{
		// Generate symmetric matrix
		RealType SM[6] = { 0., 0., 0., 0., 0., 0. };


		for (int i = 0; i < 3; ++i)
		{
			SM[0] -= grad[i].X * grad[i].X;  SM[1] -= grad[i].X * grad[i].Y;  SM[2] -= grad[i].X * grad[i].Z;
			                                 SM[3] -= grad[i].Y * grad[i].Y;  SM[4] -= grad[i].Y * grad[i].Z;
			                                                                  SM[5] -= grad[i].Z * grad[i].Z;
		}

		RealType InvA = (FMath::Abs(a) > 1.e-7) ?  RealType(1) / a : 0.;

		SM[0] *= InvA;  SM[1] *= InvA; SM[2] *= InvA;
		                SM[3] *= InvA; SM[4] *= InvA;
						               SM[5] *= InvA;

		// A - (grad_attr0, grad_attr1.. grad_attrn) . (grad_attr0, grad_attr1.. grad_attrn)^T

		SM[0] += BaseStruct::Axx;	SM[1] += BaseStruct::Axy;	SM[2] += BaseStruct::Axz;
									SM[3] += BaseStruct::Ayy;	SM[4] += BaseStruct::Ayz;
																SM[5] += BaseStruct::Azz;



		RealType InvSM[6];
		bool bValid = BaseStruct::InvertSymmetricMatrix(SM, InvSM, minThresh);

		RealType b[3] = { -BaseStruct::bx, -BaseStruct::by, -BaseStruct::bz };
		b[0] += InvA * (grad[0].X * d[0] + grad[1].X * d[1] + grad[2].X * d[2]);
		b[1] += InvA * (grad[0].Y * d[0] + grad[1].Y * d[1] + grad[2].Y * d[2]);
		b[2] += InvA * (grad[0].Z * d[0] + grad[1].Z * d[1] + grad[2].Z * d[2]);

		// add volume constraint 
		typename BaseClass::FPlaneData gvol(1. / 3., BaseClass::PlaneData);

		if (bValid)
		{
			// optimal point prior to volume correction

			OptPoint = BaseStruct::MultiplySymmetricMatrix(InvSM, b);
			//SolveAxEqualsb(OptPoint, b[0], b[1], b[2]);

			TVector<RealType> InvSMgvol = BaseStruct::MultiplySymmetricMatrix(InvSM, gvol.N);
			//SolveAxEqualsb(InvSMgvol, gvol.N.X, gvol.N.Y, gvol.N.Z);

			RealType gvolDotInvSMgvol = gvol.N.Dot(InvSMgvol);

			// move the point to the constrained optimal
			RealType gvolDotuncopt = gvol.N.Dot(OptPoint);
			RealType lambda = (gvol.Dist + gvolDotuncopt) / gvolDotInvSMgvol;

			// Check if the constraint failed.

			if (FMath::Abs(lambda) > 1.e5)
			{
				return bValid;
			}

			OptPoint -= lambda * InvSMgvol;
		}

		return bValid;
		
	}

	RealType Evaluate(const TVector<RealType>& point, const TVector<RealType>& InAttr) const
	{
		// 6x6 symmetric matrix   (    A      -grad[0], -grad[1], -grad[2] )
		//                        ( -grad[0]^T                             ) 
		//                        ( -grad[1]^T             aI              )
		//                        ( -grad[2]^T                             )          
		//
		//  aI is Identity matrix scaled by the accumulated area 'a'
		//
		// extended state vector  v = ( point )
		//                            ( attr  )
		//

		const TVector<RealType> attr = attrweight * InAttr;

		RealType ptAp = point.Dot(BaseStruct::MultiplyA(point));
		RealType attrDotattr = attr.Dot(attr);
		TVector<RealType> Gradattr(grad[0].X * attr[0] + grad[1].X * attr[1] + grad[2].X * attr[2],
			                        grad[0].Y * attr[0] + grad[1].Y * attr[1] + grad[2].Y * attr[2],
			                        grad[0].Z * attr[0] + grad[1].Z * attr[1] + grad[2].Z * attr[2]);
		RealType ptGradattr = point.Dot(Gradattr);

		RealType vtSv = ptAp + a * attrDotattr - RealType(2) * ptGradattr;

		// extended 'b' vector
		//                     ( b )
		//                     (-d )
		// compute 2<v|b>
		RealType vtb = point.X  * BaseStruct::bx + point.Y * BaseStruct::by + point.Z * BaseStruct::bz 
			- (d[0] * attr[0] + d[1] * attr[1] + d[2] * attr[2]);

		RealType QuadricError =  vtSv + RealType(2) * vtb + BaseStruct::c;

		return QuadricError;
	}

	void ComputeAttributes(const TVector<RealType>& point, TVector<RealType>& attr) const
	{
		if (FMath::Abs(a) > 1.e-5)
		{
			RealType aInv = RealType(1) / a;

			attr.X = grad[0].Dot(point) + d[0];
			attr.Y = grad[1].Dot(point) + d[1];
			attr.Z = grad[2].Dot(point) + d[2];

			attr *= aInv;
		} 
		else
		{
			attr.X = RealType(0);
			attr.Y = RealType(0);
			attr.Z = RealType(0);
		}
		
		// the quadric was constructed around a scaled version of the attribute.  need to un-scale the result
		attr *= 1. / attrweight;
	}

	RealType Evaluate(const TVector<RealType>& point) const
	{
		TVector<RealType> attr;
		ComputeAttributes(point, attr);

		return Evaluate(point, attr);
	}

	// Add regularizing term that penalizes movement away from Pos
	void AddPositionRegularizer(const UE::Math::TVector<RealType>& Pos, RealType Weight)
	{
		BaseStruct::AddPositionRegularizer(Pos, Weight);
	}

	// Add regularizing term that penalizes movement away from the line through Pos along LineDir
	void AddLineRegularizer(const UE::Math::TVector<RealType>& Pos, const UE::Math::TVector<RealType>& LineDir, RealType Weight)
	{
		BaseStruct::AddLineRegularizer(Pos, LineDir, Weight);
	}

public:

	// accumulated area
	RealType           a;

	// weight used to internally scale the attribute/
	// used to increase or decrease the importance of the normal in computing the optimal position 
	RealType           attrweight = 1.;

	// Additional planes for the attributes.
	TVector<RealType> grad[3];
	RealType           d[3];


};

typedef TAttrBasedQuadricError<float>  FAttrBasedQuadricErrorf;
typedef TAttrBasedQuadricError<double> FAttrBasedQuadricErrord;

/**
* A "Seam Quadric" is a quadric defined with respect to the plane passing through the edge p1-p0, but perpendicular to the adjacent face.
* On return the quadric has been weighted by the length of the (p1-p0).
*/
template<typename RealType>
TQuadricError<RealType>  CreateSeamQuadric(const TVector<RealType>& p0, const TVector<RealType>& p1, const TVector<RealType>& AdjFaceNormal)
{

	RealType LenghtSqrd = AdjFaceNormal.SquaredLength();
	RealType error = 100000.0 * TMathUtil<RealType>::Epsilon;
	if (!FMath::IsNearlyEqual(LenghtSqrd, (RealType)1., error) )
	{
		// zero quadric
		return TQuadricError<RealType>::Zero();
	}


	TVector<RealType> Edge = p1 - p0;

	// Weight scaled on edge length 

	const RealType EdgeLengthSqrd = Edge.SquaredLength();
	if (FMath::IsNearlyZero(EdgeLengthSqrd, (RealType)1000.0 * TMathUtil<RealType>::Epsilon))
	{
		// zero quadric
		return TQuadricError<RealType>::Zero();
	}

	const RealType EdgeLength = FMath::Sqrt(EdgeLengthSqrd);
	// normalize
	Edge *= 1. / EdgeLength;


	const RealType Weight = EdgeLength;

	// Normal that is perpendicular to the edge, and face Normal. - i.e. in the face plane and perpendicular to the edge.
	// The constraint should try to keep points on the plane associated with this constraint plane.
	TVector<RealType> ConstraintNormal = Edge.Cross(AdjFaceNormal);


	TQuadricError<RealType> SeamQuadric(ConstraintNormal, p0);

	SeamQuadric.Scale(EdgeLength);

	return SeamQuadric;
}


} // end namespace UE::Geometry
} // end namespace UE