// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"		// Engine types
#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "IndexTypes.h"

// ported from geometry3Sharp Quaternion

namespace UE
{
namespace Geometry
{

using namespace UE::Math;


template<typename RealType>
struct TQuaternion
{
	union
	{
		struct
		{
			// note: in Wm5 version, this is a 4-element arraY stored in order (w,x,y,z).
			RealType X;
			RealType Y;
			RealType Z;
			RealType W;
		};

		UE_DEPRECATED(all, "For internal use only")
		RealType XYZW[4];
	};

	TQuaternion();
	TQuaternion(RealType X, RealType Y, RealType Z, RealType W);
	explicit TQuaternion(const RealType* Values);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TQuaternion(const TQuaternion& Copy) = default;
	TQuaternion& operator=(const TQuaternion& Copy) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	template<typename RealType2>
	explicit TQuaternion(const TQuaternion<RealType2>& Copy);
	TQuaternion(const TVector<RealType>& Axis, RealType Angle, bool bAngleIsDegrees);
	TQuaternion(const TVector<RealType>& From, const TVector<RealType>& To);
	TQuaternion(const TQuaternion<RealType>& From, const TQuaternion<RealType>& To, RealType InterpT);
	TQuaternion(const TMatrix3<RealType>& RotationMatrix);

	void SetAxisAngleD(const TVector<RealType>& Axis, RealType AngleDeg);
	void SetAxisAngleR(const TVector<RealType>& Axis, RealType AngleRad);
	void SetFromTo(const TVector<RealType>& From, const TVector<RealType>& To);
	void SetToSlerp(TQuaternion<RealType> From, TQuaternion<RealType> To, RealType InterpT);
	void SetFromRotationMatrix(const TMatrix3<RealType>& RotationMatrix);

	static TQuaternion<RealType> Zero() { return TQuaternion<RealType>((RealType)0, (RealType)0, (RealType)0, (RealType)0); }
	static TQuaternion<RealType> Identity() { return TQuaternion<RealType>((RealType)0, (RealType)0, (RealType)0, (RealType)1); }

	RealType& operator[](int i)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[i];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const RealType& operator[](int i) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return XYZW[i];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Test whether quaternions represent the same rotation w/in a tolerance. (Note: Considers the negative representation of the same rotation as equal.)
	bool EpsilonEqual(const TQuaternion<RealType>& Other, RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const;
	// Test whether the quaternion is EpsilonEqual to the identity quaternion (i.e., W=1 or W=-1, within tolerance)
	bool IsIdentity(RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		return EpsilonEqual(Identity(), Tolerance);
	}

	RealType Length() const { return (RealType)sqrt(X*X + Y*Y + Z*Z + W*W); }
	RealType SquaredLength() const { return X*X + Y*Y + Z*Z + W*W; }

	TVector<RealType> AxisX() const;
	TVector<RealType> AxisY() const;
	TVector<RealType> AxisZ() const;
	void GetAxes(TVector<RealType>& X, TVector<RealType>& Y, TVector<RealType>& Z) const;

	RealType Normalize(const RealType epsilon = 0);
	TQuaternion<RealType> Normalized(const RealType epsilon = 0) const;

	RealType Dot(const TQuaternion<RealType>& Other) const;

	TQuaternion<RealType> Inverse() const;
	TVector<RealType> InverseMultiply(const TVector<RealType>& Other) const;

	TMatrix3<RealType> ToRotationMatrix() const;

	constexpr TQuaternion<RealType> operator-() const
	{
		return TQuaternion<RealType>(-X, -Y, -Z, -W);
	}


	// available for porting:
	//   SetFromRotationMatrix(FMatrix3f)


	explicit inline operator FQuat4f() const
	{
		FQuat4f Quat; 
		Quat.X = (float)X;
		Quat.Y = (float)Y;
		Quat.Z = (float)Z;
		Quat.W = (float)W;
		return Quat;
	}
	explicit inline operator FQuat4d() const
	{
		FQuat4d Quat; 
		Quat.X = (double)X;
		Quat.Y = (double)Y;
		Quat.Z = (double)Z;
		Quat.W = (double)W;
		return Quat;
	}
	explicit inline operator FRotator() const
	{
		return ((FQuat)*this).Rotator();
	}
	explicit inline TQuaternion(const FQuat4f& Quat)
	{
		X = (RealType)Quat.X;
		Y = (RealType)Quat.Y;
		Z = (RealType)Quat.Z;
		W = (RealType)Quat.W;
	}
	explicit inline TQuaternion(const FQuat4d& Quat)
	{
		X = (RealType)Quat.X;
		Y = (RealType)Quat.Y;
		Z = (RealType)Quat.Z;
		W = (RealType)Quat.W;
	}
	explicit inline TQuaternion(const FRotator& Rotator)
	{
		FQuat4d Quat(Rotator);
		X = (RealType)Quat.X;
		Y = (RealType)Quat.Y;
		Z = (RealType)Quat.Z;
		W = (RealType)Quat.W;
	}
	explicit inline operator UE::Math::TVector4<RealType>() const
	{
		return UE::Math::TVector4<RealType>(X,Y,Z,W);
	}
};



typedef TQuaternion<float> FQuaternionf;
typedef TQuaternion<double> FQuaterniond;




template<typename RealType>
TQuaternion<RealType>::TQuaternion()
{
	X = Y = Z = 0; W = 1;
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(RealType x, RealType y, RealType z, RealType w)
{
	X = x;
	Y = y;
	Z = z;
	W = w;
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(const RealType* Values)
{
	X = Values[0];
	Y = Values[1];
	Z = Values[2];
	W = Values[3];
}

template<typename RealType>
template<typename RealType2>
TQuaternion<RealType>::TQuaternion(const TQuaternion<RealType2>& Copy)
{
	X = (RealType)Copy.X;
	Y = (RealType)Copy.Y;
	Z = (RealType)Copy.Z;
	W = (RealType)Copy.W;
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(const TVector<RealType>& Axis, RealType Angle, bool bAngleIsDegrees)
{
	X = Y = Z = 0; W = 1;
	SetAxisAngleR(Axis, Angle * (bAngleIsDegrees ? TMathUtil<RealType>::DegToRad : (RealType)1));
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(const TVector<RealType>& From, const TVector<RealType>& To)
{
	X = Y = Z = 0; W = 1;
	SetFromTo(From, To);
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(const TQuaternion<RealType>& From, const TQuaternion<RealType>& To, RealType InterpT)
{
	X = Y = Z = 0; W = 1;
	SetToSlerp(From, To, InterpT);
}

template<typename RealType>
TQuaternion<RealType>::TQuaternion(const TMatrix3<RealType>& RotationMatrix)
{
	X = Y = Z = 0; W = 1;
	SetFromRotationMatrix(RotationMatrix);
}





template<typename RealType>
RealType TQuaternion<RealType>::Normalize(const RealType Epsilon)
{
	RealType length = Length();
	if (length > Epsilon)
	{
		RealType invLength = ((RealType)1) / length;
		X *= invLength;
		Y *= invLength;
		Z *= invLength;
		W *= invLength;
		return invLength;
	}
	X = Y = Z = W = (RealType)0;
	return 0;
}

template<typename RealType>
TQuaternion<RealType> TQuaternion<RealType>::Normalized(const RealType Epsilon) const
{
	RealType length = Length();
	if (length > Epsilon)
	{
		RealType invLength = ((RealType)1) / length;
		return TQuaternion<RealType>(X*invLength, Y*invLength, Z*invLength, W*invLength);
	}
	return Zero();
}

template<typename RealType>
RealType TQuaternion<RealType>::Dot(const TQuaternion<RealType>& Other) const
{
	return X * Other.X + Y * Other.Y + Z * Other.Z + W * Other.W;
}


template<typename RealType>
TQuaternion<RealType> operator*(const TQuaternion<RealType>& A, const TQuaternion<RealType>& B) 
{
	RealType W = A.W * B.W - A.X * B.X - A.Y * B.Y - A.Z * B.Z;
	RealType X = A.W * B.X + A.X * B.W + A.Y * B.Z - A.Z * B.Y;
	RealType Y = A.W * B.Y + A.Y * B.W + A.Z * B.X - A.X * B.Z;
	RealType Z = A.W * B.Z + A.Z * B.W + A.X * B.Y - A.Y * B.X;
	return TQuaternion<RealType>(X, Y, Z, W);
}


template<typename RealType>
TQuaternion<RealType> operator*(RealType Scalar, const TQuaternion<RealType>& Q) 
{
	return TQuaternion<RealType>(Scalar * Q.X, Scalar * Q.Y, Scalar * Q.Z, Scalar * Q.W);
}

template<typename RealType>
TQuaternion<RealType> operator*(const TQuaternion<RealType>& Q, RealType Scalar) 
{
	return TQuaternion<RealType>(Scalar * Q.X, Scalar * Q.Y, Scalar * Q.Z, Scalar * Q.W);
}


template<typename RealType>
TQuaternion<RealType> operator+(const TQuaternion<RealType>& A, const TQuaternion<RealType>& B) 
{
	return TQuaternion<RealType>(A.X + B.X, A.Y + B.Y, A.Z + B.Z, A.W + B.W);
}


template<typename RealType>
TQuaternion<RealType> operator -(const TQuaternion<RealType>& A, const TQuaternion<RealType>& B) 
{
	return TQuaternion<RealType>(A.X - B.X, A.Y - B.Y, A.Z - B.Z, A.W - B.W);
}

template<typename RealType>
TVector<RealType> operator*(const TQuaternion<RealType>& Q, const UE::Math::TVector<RealType>& V)
{
	//return q.ToRotationMatrix() * v;
	// inline-expansion of above:
	RealType twoX = (RealType)2*Q.X; RealType twoY = (RealType)2*Q.Y; RealType twoZ = (RealType)2*Q.Z;
	RealType twoWX = twoX * Q.W; RealType twoWY = twoY * Q.W; RealType twoWZ = twoZ * Q.W;
	RealType twoXX = twoX * Q.X; RealType twoXY = twoY * Q.X; RealType twoXZ = twoZ * Q.X;
	RealType twoYY = twoY * Q.Y; RealType twoYZ = twoZ * Q.Y; RealType twoZZ = twoZ * Q.Z;
	return TVector<RealType>(
		V.X * ((RealType)1 - (twoYY + twoZZ)) + V.Y * (twoXY - twoWZ) + V.Z * (twoXZ + twoWY),
		V.X * (twoXY + twoWZ) + V.Y * ((RealType)1 - (twoXX + twoZZ)) + V.Z * (twoYZ - twoWX),
		V.X * (twoXZ - twoWY) + V.Y * (twoYZ + twoWX) + V.Z * ((RealType)1 - (twoXX + twoYY))); ;
}


template<typename RealType>
TVector<RealType> TQuaternion<RealType>::InverseMultiply(const TVector<RealType>& V) const
{
	RealType norm = SquaredLength();
	if (norm > 0) 
	{
		RealType invNorm = (RealType)1 / norm;
		RealType qX = -X * invNorm, qY = -Y * invNorm, qZ = -Z * invNorm, qW = W * invNorm;
		RealType twoX = (RealType)2 * qX; RealType twoY = (RealType)2 * qY; RealType twoZ = (RealType)2 * qZ;
		RealType twoWX = twoX * qW; RealType twoWY = twoY * qW; RealType twoWZ = twoZ * qW;
		RealType twoXX = twoX * qX; RealType twoXY = twoY * qX; RealType twoXZ = twoZ * qX;
		RealType twoYY = twoY * qY; RealType twoYZ = twoZ * qY; RealType twoZZ = twoZ * qZ;
		return TVector<RealType>(
			V.X * ((RealType)1 - (twoYY + twoZZ)) + V.Y * (twoXY - twoWZ) + V.Z * (twoXZ + twoWY),
			V.X * (twoXY + twoWZ) + V.Y * ((RealType)1 - (twoXX + twoZZ)) + V.Z * (twoYZ - twoWX),
			V.X * (twoXZ - twoWY) + V.Y * (twoYZ + twoWX) + V.Z * ((RealType)1 - (twoXX + twoYY)));
	}
	return TVector<RealType>::Zero();
}


template<typename RealType>
TVector<RealType> TQuaternion<RealType>::AxisX() const
{
	RealType twoY = (RealType)2 * Y; RealType twoZ = (RealType)2 * Z;
	RealType twoWY = twoY * W; RealType twoWZ = twoZ * W;
	RealType twoXY = twoY * X; RealType twoXZ = twoZ * X;
	RealType twoYY = twoY * Y; RealType twoZZ = twoZ * Z;
	return TVector<RealType>((RealType)1 - (twoYY + twoZZ), twoXY + twoWZ, twoXZ - twoWY);
}

template<typename RealType>
TVector<RealType> TQuaternion<RealType>::AxisY() const
{
	RealType twoX = (RealType)2 * X; RealType twoY = (RealType)2 * Y; RealType twoZ = (RealType)2 * Z;
	RealType twoWX = twoX * W; RealType twoWZ = twoZ * W; RealType twoXX = twoX * X;
	RealType twoXY = twoY * X; RealType twoYZ = twoZ * Y; RealType twoZZ = twoZ * Z;
	return TVector<RealType>(twoXY - twoWZ, (RealType)1 - (twoXX + twoZZ), twoYZ + twoWX);
}

template<typename RealType>
TVector<RealType> TQuaternion<RealType>::AxisZ() const
{
	RealType twoX = (RealType)2 * X; RealType twoY = (RealType)2 * Y; RealType twoZ = (RealType)2 * Z;
	RealType twoWX = twoX * W; RealType twoWY = twoY * W; RealType twoXX = twoX * X;
	RealType twoXZ = twoZ * X; RealType twoYY = twoY * Y; RealType twoYZ = twoZ * Y;
	return TVector<RealType>(twoXZ + twoWY, twoYZ - twoWX, (RealType)1 - (twoXX + twoYY));
}

template<typename RealType>
void TQuaternion<RealType>::GetAxes(TVector<RealType>& XOut, TVector<RealType>& YOut, TVector<RealType>& ZOut) const
{
	RealType twoX = (RealType)2 * X; RealType twoY = (RealType)2 * Y; RealType twoZ = (RealType)2 * Z;
	RealType twoWX = twoX * W; RealType twoWY = twoY * W; RealType twoWZ = twoZ * W;
	RealType twoXX = twoX * X; RealType twoXY = twoY * X; RealType twoXZ = twoZ * X;
	RealType twoYY = twoY * Y; RealType twoYZ = twoZ * Y; RealType twoZZ = twoZ * Z;
	XOut = TVector<RealType>((RealType)1 - (twoYY + twoZZ), twoXY + twoWZ, twoXZ - twoWY);
	YOut = TVector<RealType>(twoXY - twoWZ, (RealType)1 - (twoXX + twoZZ), twoYZ + twoWX);
	ZOut = TVector<RealType>(twoXZ + twoWY, twoYZ - twoWX, (RealType)1 - (twoXX + twoYY));
}


template<typename RealType>
TQuaternion<RealType> TQuaternion<RealType>::Inverse() const
{
	RealType norm = SquaredLength();
	if (norm > (RealType)0) 
	{
		RealType invNorm = (RealType)1 / norm;
		return TQuaternion<RealType>(
			-X * invNorm, -Y * invNorm, -Z * invNorm, W * invNorm);
	}
	return TQuaternion<RealType>::Zero();
}


template<typename RealType>
void TQuaternion<RealType>::SetAxisAngleD(const TVector<RealType>& Axis, RealType AngleDeg) 
{
	SetAxisAngleR(Axis, TMathUtil<RealType>::DegToRad * AngleDeg);
}

template<typename RealType>
void TQuaternion<RealType>::SetAxisAngleR(const TVector<RealType>& Axis, RealType AngleRad)
{
	RealType halfAngle = (RealType)0.5 * AngleRad;
	RealType sn = (RealType)sin(halfAngle);
	W = (RealType)cos(halfAngle);
	X = (sn * Axis.X);
	Y = (sn * Axis.Y);
	Z = (sn * Axis.Z);
}



// this function can take non-normalized vectors vFrom and vTo (normalizes internally)
template<typename RealType>
void TQuaternion<RealType>::SetFromTo(const TVector<RealType>& From, const TVector<RealType>& To)
{
	// [TODO] this page seems to have optimized version:
	//    http://lolengine.net/blog/2013/09/18/beautiful-maths-quaternion-from-vectors

	// [RMS] not ideal to explicitly normalize here, but if we don't,
	//   output TQuaternion is not normalized and this causes problems,
	//   eg like drift if we do repeated SetFromTo()
	TVector<RealType> from = UE::Geometry::Normalized(From), to = UE::Geometry::Normalized(To);
	TVector<RealType> bisector = UE::Geometry::Normalized(from + to, TMathUtil<RealType>::ZeroTolerance);
	W = from.Dot(bisector);
	if (W != 0) 
	{
		TVector<RealType> cross = from.Cross(bisector);
		X = cross.X;
		Y = cross.Y;
		Z = cross.Z;
	}
	else 
	{
		RealType invLength;
		if ( (RealType)fabs(from.X) >= (RealType)fabs(from.Y)) 
		{
			// V1.X or V1.Z is the largest magnitude component.
			invLength = ((RealType)1 / (RealType)sqrt(from.X * from.X + from.Z * from.Z));
			X = -from.Z * invLength;
			Y = 0;
			Z = +from.X * invLength;
		}
		else 
		{
			// V1.Y or V1.Z is the largest magnitude component.
			invLength = ((RealType)1 / (RealType)sqrt(from.Y * from.Y + from.Z * from.Z));
			X = 0;
			Y = +from.Z * invLength;
			Z = -from.Y * invLength;
		}
	}
	Normalize();   // just to be safe...
}



template<typename RealType>
void TQuaternion<RealType>::SetToSlerp( TQuaternion<RealType> From, TQuaternion<RealType> To, RealType InterpT)
{
	From.Normalize();
	To.Normalize();
	RealType cs = From.Dot(To);

	// Q and -Q are equivalent, but if we try to Slerp between them we will get nonsense instead of 
	// just returning Q. Depending on how the Quaternion was constructed it is possible
	// that the sign flips. So flip it back.
	if (cs < (RealType)-0.99)
	{
		From = -From;
	}

	RealType angle = TMathUtil<RealType>::ACos(cs);
	if (TMathUtil<RealType>::Abs(angle) >= TMathUtil<RealType>::ZeroTolerance)
	{
		RealType sn = TMathUtil<RealType>::Sin(angle);
		RealType invSn = (RealType)1 / sn;
		RealType tAngle = InterpT * angle;
		RealType coeff0 = TMathUtil<RealType>::Sin(angle - tAngle) * invSn;
		RealType coeff1 = TMathUtil<RealType>::Sin(tAngle) * invSn;
		X = coeff0 * From.X + coeff1 * To.X;
		Y = coeff0 * From.Y + coeff1 * To.Y;
		Z = coeff0 * From.Z + coeff1 * To.Z;
		W = coeff0 * From.W + coeff1 * To.W;
	}
	else 
	{
		X = From.X;
		Y = From.Y;
		Z = From.Z;
		W = From.W;
	}

	Normalize();	// be safe
}



template<typename RealType>
void TQuaternion<RealType>::SetFromRotationMatrix(const TMatrix3<RealType>& RotationMatrix)
{
	// Algorithm in Ken Shoemake's article in 1987 SIGGRAPH course notes
	// article "TQuaternion Calculus and Fast Animation".
	FIndex3i next(1, 2, 0);

	RealType trace = RotationMatrix(0,0) + RotationMatrix(1,1) + RotationMatrix(2, 2);
	RealType root;

	if (trace > 0) 
	{
		// |w| > 1/2, may as well choose w > 1/2
		root = (RealType)sqrt(trace + (RealType)1);  // 2w
		W = ((RealType)0.5) * root;
		root = ((RealType)0.5) / root;  // 1/(4w)
		X = (RotationMatrix(2, 1) - RotationMatrix(1, 2)) * root;
		Y = (RotationMatrix(0, 2) - RotationMatrix(2, 0)) * root;
		Z = (RotationMatrix(1, 0) - RotationMatrix(0, 1)) * root;
	}
	else 
	{
		// |w| <= 1/2
		int i = 0;
		if (RotationMatrix(1, 1) > RotationMatrix(0, 0))
		{
			i = 1;
		}
		if (RotationMatrix(2, 2) > RotationMatrix(i, i))
		{
			i = 2;
		}
		int j = next[i];
		int k = next[j];

		root = (RealType)sqrt(RotationMatrix(i, i) - RotationMatrix(j, j) - RotationMatrix(k, k) + (RealType)1);

		TVector<RealType> quat(X, Y, Z);
		quat[i] = ((RealType)0.5) * root;
		root = ((RealType)0.5) / root;
		W = (RotationMatrix(k, j) - RotationMatrix(j, k)) * root;
		quat[j] = (RotationMatrix(j, i) + RotationMatrix(i, j)) * root;
		quat[k] = (RotationMatrix(k, i) + RotationMatrix(i, k)) * root;
		X = quat.X; Y = quat.Y; Z = quat.Z;
	}

	Normalize();   // we prefer normalized TQuaternions...
}




template<typename RealType>
TMatrix3<RealType> TQuaternion<RealType>::ToRotationMatrix() const
{
	RealType twoX = 2 * X; RealType twoY = 2 * Y; RealType twoZ = 2 * Z;
	RealType twoWX = twoX * W; RealType twoWY = twoY * W; RealType twoWZ = twoZ * W;
	RealType twoXX = twoX * X; RealType twoXY = twoY * X; RealType twoXZ = twoZ * X;
	RealType twoYY = twoY * Y; RealType twoYZ = twoZ * Y; RealType twoZZ = twoZ * Z;
	TMatrix3<RealType> m = TMatrix3<RealType>::Zero();
	m.Row0 = TVector<RealType>(1 - (twoYY + twoZZ), twoXY - twoWZ, twoXZ + twoWY);
	m.Row1 = TVector<RealType>(twoXY + twoWZ, 1 - (twoXX + twoZZ), twoYZ - twoWX);
	m.Row2 = TVector<RealType>(twoXZ - twoWY, twoYZ + twoWX, 1 - (twoXX + twoYY));
	return m;
}



template<typename RealType>
bool TQuaternion<RealType>::EpsilonEqual(const TQuaternion<RealType>& Other, RealType Tolerance) const
{
	return 
		((RealType)fabs(X - Other.X) <= Tolerance &&
		(RealType)fabs(Y - Other.Y) <= Tolerance &&
		(RealType)fabs(Z - Other.Z) <= Tolerance &&
		(RealType)fabs(W - Other.W) <= Tolerance) 
		||
		((RealType)fabs(X + Other.X) <= Tolerance &&
		(RealType)fabs(Y + Other.Y) <= Tolerance &&
		(RealType)fabs(Z + Other.Z) <= Tolerance &&
		(RealType)fabs(W + Other.W) <= Tolerance);
}


} // end namespace UE::Geometry
} // end namespace UE