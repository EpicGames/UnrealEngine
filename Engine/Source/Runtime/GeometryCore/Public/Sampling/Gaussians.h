// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntVectorTypes.h"
#include "MathUtil.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * 1D Gaussian
 */
template<typename RealType>
class TGaussian1
{
public:
	RealType Sigma;

	TGaussian1(RealType SigmaIn = 1.0) : Sigma(SigmaIn) {}

	RealType Evaluate(RealType X) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return  TMathUtil<RealType>::InvPi * InvTwoSigmaSqr * TMathUtil<RealType>::Exp( -(X*X) * InvTwoSigmaSqr );
	}

	RealType EvaluateSqr(RealType XSquared) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return  TMathUtil<RealType>::InvPi * InvTwoSigmaSqr * TMathUtil<RealType>::Exp(-XSquared * InvTwoSigmaSqr);
	}

	RealType EvaluateUnscaled(RealType X) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return  TMathUtil<RealType>::Exp( -(X*X) * InvTwoSigmaSqr);
	}

	RealType EvaluateSqrUnscaled(RealType XSquared) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return  TMathUtil<RealType>::Exp(-XSquared * InvTwoSigmaSqr);
	}
};
typedef TGaussian1<float> TGaussian1f;
typedef TGaussian1<double> TGaussian1d;




/**
 * 2D Discretized Kernel (ie 2D grid/matrix of values)
 */
template<typename RealType>
class TDiscreteKernel2
{
public:
	int32 IntRadius;
	TArray<RealType> Kernel;

	RealType EvaluateFromOffset(const FVector2i& Offset) const
	{
		int32 X = Offset.X + IntRadius;
		int32 Y = Offset.Y + IntRadius;
		return Kernel[Y * (2*IntRadius+1) + X];
	}

};
typedef TDiscreteKernel2<float> TDiscreteKernel2f;
typedef TDiscreteKernel2<double> TDiscreteKernel2d;




/**
 * 2D Gaussian
 */
template<typename RealType>
class TGaussian2
{
public:
	RealType Sigma;

	TGaussian2(RealType SigmaIn = 1.0) : Sigma(SigmaIn) {}

	RealType Evaluate(const UE::Math::TVector2<RealType>& XY) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return TMathUtil<RealType>::InvPi * InvTwoSigmaSqr * TMathUtil<RealType>::Exp( -XY.SquaredLength() * InvTwoSigmaSqr );
	}

	RealType EvaluateSqrUnscaled(const UE::Math::TVector2<RealType>& XY) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return TMathUtil<RealType>::Exp(-XY.SquaredLength() * InvTwoSigmaSqr);
	}


	static void MakeKernelFromWidth(int32 IntRadius, RealType Sigma, TArray<RealType>& KernelOut, bool bNormalized = true)
	{
		TGaussian2<RealType> Gaussian(Sigma);

		int32 KernelSize = (2 * IntRadius) + 1;
		int32 N = KernelSize * KernelSize;
		KernelOut.SetNum(N);

		RealType InvTwoSigmaSqr = (RealType)1 / ((RealType)2 * Sigma * Sigma);
		RealType NormalizeFactor = (bNormalized) ? (RealType)1 : (TMathUtil<RealType>::InvPi * InvTwoSigmaSqr);

		FVector2i CenterIdx(IntRadius, IntRadius);
		RealType KernelSum = 0;
		for (int32 Y = 0; Y < KernelSize; ++Y)
		{
			for (int32 X = 0; X < KernelSize; ++X)
			{
				FVector2i RelPos = FVector2i(X,Y) - CenterIdx;
				RealType DistSqr = (RealType)RelPos.SquaredLength();
				KernelOut[Y*KernelSize + X] = NormalizeFactor * TMathUtil<RealType>::Exp(-DistSqr * InvTwoSigmaSqr);
				KernelSum += KernelOut[Y*KernelSize + X];
			}
		}
		if (bNormalized)
		{
			for (int32 k = 0; k < N; ++k)
			{
				KernelOut[k] /= KernelSum;
			}
		}
	}


	static void MakeKernelFromRadius(RealType Radius, TArray<RealType>& KernelOut, int32& IntRadiusOut, bool bNormalized = true)
	{
		IntRadiusOut = (int32)TMathUtil<RealType>::Ceil(Radius);
		RealType Sigma = Radius / (RealType)2;			// ??
		MakeKernelFromWidth(IntRadiusOut, Sigma, KernelOut, bNormalized);
	}


	static void MakeKernelFromRadius(RealType Radius, TDiscreteKernel2<RealType>& KernelOut, bool bNormalized = true)
	{
		KernelOut.IntRadius = (int32)TMathUtil<RealType>::Ceil(Radius);
		RealType Sigma = Radius / (RealType)2;			// ??
		MakeKernelFromWidth(KernelOut.IntRadius, Sigma, KernelOut.Kernel, bNormalized);
	}



};
typedef TGaussian2<float> TGaussian2f;
typedef TGaussian2<double> TGaussian2d;




/**
 * 3D Gaussian
 */
template<typename RealType>
class TGaussian3
{
public:
	RealType Sigma;

	TGaussian3(RealType SigmaIn = 1.0) : Sigma(SigmaIn) {}

	RealType Evaluate(const TVector<RealType>& XYZ) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return TMathUtil<RealType>::InvPi * InvTwoSigmaSqr * TMathUtil<RealType>::Exp( -XYZ.SquaredLength() * InvTwoSigmaSqr );
	}

	RealType EvaluateSqrUnscaled(const TVector<RealType>& XYZ) const
	{
		RealType InvTwoSigmaSqr = 1.0 / (2.0 * Sigma * Sigma);
		return TMathUtil<RealType>::Exp(-XYZ.SquaredLength() * InvTwoSigmaSqr);
	}
};
typedef TGaussian3<float> TGaussian3f;
typedef TGaussian3<double> TGaussian3d;


} // end namespace UE::Geometry
} // end namespace UE