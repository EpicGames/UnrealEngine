// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * Given a linear index into a square grid of given size, gives back the [0,1)
 * XY coordinate of the grid cell center (assuming linear index increases first
 * in X and then in Y).
 */
template<typename RealType>
class TGridSampler
{
private:
	int32 Size = 1;

public:
	TGridSampler(int32 SizeIn)
		: Size(SizeIn)
	{
		checkSlow(Size >= 1);
	}

	/** @return total number of grid points. */
	int32 Num() const
	{
		return Size * Size;
	}

	/**
	 * @param Index linear point index in range [0,Num()-1]
	 * @return grid sample for given linear index
	 */
	UE::Math::TVector2<RealType> Sample(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < Num());
		RealType X = (static_cast<RealType>(Index % Size) + RealType{ 1 } / RealType{ 2 }) / static_cast<RealType>(Size); //-V1064
		RealType Y = (static_cast<RealType>(Index / Size) + RealType{ 1 } / RealType{ 2 }) / static_cast<RealType>(Size); //-V1064
		return UE::Math::TVector2<RealType>(X, Y);
	}
};

} // end namespace UE::Geometry
} // end namespace UE

