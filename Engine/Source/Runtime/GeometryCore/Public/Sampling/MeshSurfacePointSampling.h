// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "Util/ProgressCancel.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

// class for efficiently drawing weighted samples (when you need to draw more than a very small amount of them)
// note this is a more flexible version of the algorithm in Engine/WeightedRandomSampler.h, 
// with support for cases where some IDs cannot be sampled (e.g. when sampling triangles of non-compact meshes)
template<typename RealType>
class TWeightedSamplingAliasTable
{
public:
	/**
	 * Initialize sampling table
	 * 
	 * @param Weights The relative frequency that the element should be sampled -- e.g., for uniform mesh sampling, these may be triangle areas.
	 * @param SumOfValidWeights Sum of all valid weights. Must not include negative (invalid) weights.
	 * @param bAllowInvalidWeights If true, then negative weights can be used to indicate invalid IDs, which must never be sampled. Otherwise, all weights should be non-negative.
	 * @return true if successfully created a valid sampling table, false otherwise (e.g., if there were no valid weights)
	 */
	GEOMETRYCORE_API bool Init(TConstArrayView<RealType> Weights, RealType SumOfValidWeights, bool bAllowInvalidWeights);

	TWeightedSamplingAliasTable() = default;
	TWeightedSamplingAliasTable(TConstArrayView<RealType> Weights, RealType SumOfValidWeights, bool bAllowInvalidWeights)
	{
		Init(Weights, SumOfValidWeights, bAllowInvalidWeights);
	}
	TWeightedSamplingAliasTable(TConstArrayView<RealType> Weights)
	{
		bool bHasInvalidWeight = false;
		RealType WtSum = (RealType)0;
		for (RealType Wt : Weights)
		{
			if (Wt < 0)
			{
				bHasInvalidWeight = true;
				continue;
			}
			WtSum += Wt;
		}
		Init(Weights, WtSum, bHasInvalidWeight);
	}
	
	inline int32 Num() const
	{
		return Probability.Num();
	}

	inline bool IsValid() const
	{
		return !Probability.IsEmpty();
	}
	
	// UniformRandom1, UniformRandom2 must be in the [0,1) range
	inline int32 Sample(RealType UniformRandom1, RealType UniformRandom2) const
	{
		checkSlow(IsValid() && UniformRandom1 < (RealType)1);
		int32 Idx = FMath::TruncToInt32(UniformRandom1 * (RealType)Probability.Num());
		return UniformRandom2 < Probability[Idx] ? Idx : Alias[Idx];
	}

private:
	// Table of probability that you should keep a given index chosen by unweighted random sampling (can be MaxReal if should always pick the index)
	TArray<RealType> Probability;
	// Table indicating the alternative index you should choose, if you don't keep the initial index
	TArray<int32> Alias;
};

// Class to help draw uniform-random surface samples from a (possibly non-compact) triangle mesh. 
// TMeshType must implement: IsTriangle, MaxTriangleID, and GetTriVertices. RealType must be float or double.
template<typename TMeshType, typename RealType>
class TMeshUniformSurfaceSampling
{
public:
	
	TMeshUniformSurfaceSampling() = default;
	TMeshUniformSurfaceSampling(const TMeshType& Mesh)
	{
		Init(Mesh);
	}
	
	// @return true on successful initialization
	bool Init(const TMeshType& Mesh)
	{
		const int32 MaxTID = Mesh.MaxTriangleID();
		TArray<RealType> Wts;
		RealType WtSum = (RealType)0;
		bool bHasInvalidWeights = false;
		for (int32 TID = 0; TID < MaxTID; ++TID)
		{
			if (Mesh.IsTriangle(TID))
			{
				TVector<RealType> A, B, C;
				Mesh.GetTriVertices(TID, A, B, C);
				RealType Area = (RealType)VectorUtil::Area(A, B, C);
				WtSum += Area;
				Wts.Add(Area);
			}
			else
			{
				bHasInvalidWeights = true;
				Wts.Add(-1);
			}
		}
		return AliasTable.Init(Wts, WtSum, bHasInvalidWeights);
	}

	// @return true if we can generate samples (e.g., if the class has been initialized with a non-empty mesh)
	bool IsValid() const
	{
		return AliasTable.IsValid();
	}

	/**
	 * Sample a random triangle, with probability proportional to relative triangle area.
	 * Note: Use VectorUtil::UniformSampleTrianglePoint to generate a uniform random point on a given triangle,
	 * or VectorUtil::UniformSampleTriangleBarycentricCoords for a uniform random barycentric coordinate
	 *
	 * Parameters UniformRandom1, UniformRandom2 must be in the [0,1) range.
	 */
	int32 DrawTriangleIDSample(RealType UniformRandom1, RealType UniformRandom2) const
	{
		return AliasTable.Sample(UniformRandom1, UniformRandom2);
	}
	
private:
	TWeightedSamplingAliasTable<RealType> AliasTable;
};

using FMeshUniformSurfaceSampling = TMeshUniformSurfaceSampling<FDynamicMesh3, double>;
using FWeightedSamplingAliasTabled = TWeightedSamplingAliasTable<double>;
using FWeightedSamplingAliasTablef = TWeightedSamplingAliasTable<float>;


/**
 * FMeshSurfacePointSampling computes oriented point samples on the surface of
 * a Mesh using various sampling strategies, with controls over sample spacing.
 */
class FMeshSurfacePointSampling
{
public:
	// 
	// Basic sampling parameters
	//

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	double SampleRadius = 10.0;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	uint32 MaxSamples = TNumericLimits<uint32>::Max();
	
	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	double SubSampleDensity = 10.0;

	/** Random Seed used to initialize sampling strategies */
	int32 RandomSeed = 0;

	/** Safety threshold for maximum number of points in subsampling. */
	int32 MaxSubSamplePoints = 50000000;

	/**
	 * If < 0, the latest, recommended sampling methods will be used. Otherwise, requests a specific sampling method version.
	 * Set this for more consistent results across UE versions, at the risk of worse performance or quality.
	 * Valid versions are:
	 * Method 0: UE 5.5 and earlier sampling method. Slower initial (dense) point sampling, less robust to degenerate triangles.
	 * Method 1: Point downsampling is performed in a fixed order regardless of seed, which gives overly-similar sample patterns across multiple runs
	 * Method 2: Currently the default method.
	 */
	int32 SamplingMethodVersion = INDEX_NONE;

	//
	// Parameters for non-uniform / variable-radius sampling. 
	// Disabled if MaxSampleRadius <= SampleRadius
	//

	/** If MaxSampleRadius > SampleRadius, then output point radii will be in range [SampleRadius, MaxSampleRadius] */
	double MaxSampleRadius = 0.0;

	/** Controls the distribution of sample radii */
	enum class ESizeDistribution
	{
		/** Uniform distribution of sizes, ie all equally likely */
		Uniform = 0,
		/** Distribution is weighted towards smaller points (T^Power) */
		Smaller = 1,
		/** Distribution is weighted towards larger points (T^1/Power) */
		Larger = 2
	};
	/** Active Size Distribution mode */
	ESizeDistribution SizeDistribution = ESizeDistribution::Uniform;

	/** Used to define how extreme the Size Distribution shift is. Valid range is [1,10] */
	double SizeDistributionPower = 2.0;


	/** Control whether VertexWeights (if valid) will be interpolated to modulate sampling */
	bool bUseVertexWeights = false;

	/** Per-vertex weights, size must be == Max Vertex Index of input Mesh */
	TArray<double> VertexWeights;

	/** Control how active Weights are used to affect point radius */
	enum class EInterpretWeightMode
	{
		/** Weights are clamped to [0,1] and used to interpolate Min/Max Radius */
		RadiusInterp = 0,
		/** Weights are clamped to [0,1] and used to interpolate Min/Max Radius, with decay, so that smaller-radius samples will infill between large ones */
		RadiusInterpWithFill = 1,
		/** Weight is used to create nonuniform random sampling, ie it nudges the random point radius distribution but does not directly control it */
		WeightedRandom = 2
	};
	/** Active weight interpretation mode */
	EInterpretWeightMode InterpretWeightMode = EInterpretWeightMode::RadiusInterpWithFill;

	/** If true, weights are inverted */
	bool bInvertWeights = false;

	/** If true, barycentric coordinates output array will be populated */
	bool bComputeBarycentrics = false;

	//
	// TODO: when MaxSamples is set, it would be useful to be able to use Weight to modulate
	// positional distribution instead of radius, (or both!)
	//

public:
	//
	// Outputs
	//

	/** Result of last computation */
	FGeometryResult Result;

	/** Oriented Sample Points on the mesh surface. Z axis of frame points along mesh normal, X and Y are arbitrary */
	TArray<FFrame3d> Samples;

	/** Radius of each Sample Point, length is the same as Samples array */
	TArray<double> Radii;

	/** Triangle that contains each Sample Point, length is the same as Samples array */
	TArray<int32> TriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. Only computed if bComputeBarycentrics = true */
	TArray<FVector3d> BarycentricCoords;

public:

	/**
	 * Compute an approximate Poisson sampling of the mesh, either uniform or non-uniform depending on the settings above.
	 * The sampling will attempt to fully cover the mesh unless .MaxSamples is provided, in which case exactly
	 * that many samples will be generated and they will be randomly distributed (so not at all Poisson...!)
	 * 
	 * By default the sampling will be uniform - all points will be spaced at least .SampleRadius*2, ie no "collisions"
	 * between their bounding spheres.
	 * 
	 * If .MaxSampleRadius is larger than .SampleRadius, the sampling will be non-uniform, ie samples will be emitted
	 * with radii within this range. By default a random uniform distribution of radii will be attempted, .SizeDistribution 
	 * and related parameters can be used to make this distribution non-uniform. 
	 * The spacing between points will always be greater than the sum of their two sample radii, so again no collisions. 
	 * However the "density" of the sampling will vary depending on how well the algorithm can find gaps to fill.
	 * 
	 * If .bUseVertexWeights is defined and valid VertexWeights are provided, they will be used to modulate the sampling radii.
	 * The .InterpretWeightMode setting controls how the weights are used to influence the sample radii.
	 * 
	 * The strategy used is to compute a much higher density sampling than needed (based on .SubSampleDensity),
	 * then iteratively select from that point set and decimate it within the radius of selected samples. The implementation
	 * also introduces various biases to increase performance. Generally increasing SubSampleDensity will result in 
	 * more tightly-packed results, but at increasingly expensive computation time.
	 */
	GEOMETRYCORE_API void ComputePoissonSampling(const FDynamicMesh3& Mesh, FProgressCancel* Progress = nullptr);


	// ability to incrementally increase existing sample sizes to more tightly pack them?
	// ability to iteratively redistribute samples to improve uniformity (ie potential field / mass-spring type method, point smoothing, ?)
};






} // end namespace UE::Geometry
} // end namespace UE

