// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "Sampling/SphericalFibonacci.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TNormalHistogram calculates/represents a histogram on a 3D sphere. The histogram bins
 * are defined by a Spherical Fibonacci distribution, ie each bin is the voronoi cell of
 * the Fibonacci point. This point set can be calculated for any N.
 * 
 * One current limitation of this implementation is that the unit +/- X/Y/Z axes are not bin center points.
 */
template<typename RealType>
class TNormalHistogram
{
public:
	int32 Bins;
	TSphericalFibonacci<RealType> BinPoints;
	TArray<RealType> WeightedCounts;

	/**
	 * Initialize a normal histogram with the given number of bins
	 */
	TNormalHistogram(int32 NumBins)
	{
		Bins = NumBins;
		BinPoints = TSphericalFibonacci<RealType>(NumBins);
		WeightedCounts.Init(0, BinPoints.Num());
	}

	/**
	 * Count a Normal in the histogram, ie find the Bin it should be included in and add the Weight to that Bin (The Normal does not strictly need to be normalized)
	 * @param Weight optional weight for this normal
	 */
	void Count(const TVector<RealType>& Normal, RealType Weight = 1)
	{
		int32 BinIndex = BinPoints.FindIndex(Normal);
		WeightedCounts[BinIndex] += Weight;
	}

	/**
	 * @return the normal direction for the Histogram bin that contains the most points / largest mass
	 */
	TVector<RealType> FindMaxNormal() const
	{
		int MaxIndex = 0;
		for (int k = 1; k < Bins; ++k)
		{
			if (WeightedCounts[k] > WeightedCounts[MaxIndex])
			{
				MaxIndex = k;
			}
		}
		return BinPoints[MaxIndex];
	}


	/**
	 * Attempt to compute a measure of the range of variation in normals across the occupied bins
	 */
	RealType WeightedSpreadMetric() const
	{
		int32 NumOccupiedBins = 0;
		RealType TotalWeightedCount = 0;
		for (int k = 0; k < Bins; ++k)
		{
			if (WeightedCounts[k] > 0)
			{
				TotalWeightedCount += WeightedCounts[k];
				NumOccupiedBins++;
			}
		}

		RealType Metric = 0;
		for (int32 k = 0; k < Bins; ++k)
		{
			if (WeightedCounts[k] > 0)
			{
				RealType MassFraction = WeightedCounts[k] / TotalWeightedCount;
				MassFraction /= (RealType)NumOccupiedBins;
				Metric += MassFraction;
			}
		}

		return 1.0 / Metric;
	}
};



} // end namespace UE::Geometry
} // end namespace UE
