// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{


/**
 * TSampleSetStatistics stores and calculates statistics for a scalar sample set,
 * such as the range of values, mean, variance, etc.
 */
template<typename RealType>
struct TSampleSetStatistics
{
	/** Number of values used to compute statistics */
	int64 Count = 0;

	/** Range of values */
	TInterval1<RealType> Range = TInterval1<RealType>::Empty();

	/** Mean / Average of values */
	RealType Mean = 0;

	/** Standard Deviation of values */
	RealType StandardDeviation = 0;



	//
	// two-pass construction for a sample set with known size.
	//
	// Usage:
	//    FFixedCountData Data = Stat.Begin_FixedCount(N);
	//    for ( Value in Values ) 
	//        Stat.AccumulateValue_FixedCount(Value);
	//    Stat.StartSecondPass_FixedCount(Data);
	//    for ( Value in Values ) 
	//        Stat.AccumulateValue_FixedCount(Value);
	//    Stat.CompleteSecondPass_FixedCount(Data);


	struct FFixedCountData
	{
		int32 PassNum;
		RealType CountDivide;
	};

	FFixedCountData Begin_FixedCount(int64 CountIn)
	{
		Count = CountIn;
		Range = TInterval1<RealType>::Empty();
		Mean = 0;
		StandardDeviation = 0;

		FFixedCountData Data;
		Data.CountDivide = 1.0 / (RealType)Count;
		Data.PassNum = 0;
		return Data;
	}

	void StartSecondPass_FixedCount(FFixedCountData& DataInOut)
	{
		StandardDeviation = 0;
		DataInOut.CountDivide = 1.0 / (RealType)(Count - 1);
		DataInOut.PassNum = 1;
	}

	void CompleteSecondPass_FixedCount(FFixedCountData& DataInOut)
	{
		StandardDeviation = TMathUtil<RealType>::Sqrt(StandardDeviation);
	}

	void AccumulateValue_FixedCount(const RealType Value, FFixedCountData& DataInOut)
	{
		if (DataInOut.PassNum == 0)
		{
			Range.Contain(Value);
			Mean += Value * DataInOut.CountDivide;
		}
		else
		{
			StandardDeviation += (Value - Mean) * (Value - Mean) * DataInOut.CountDivide;
		}
	}

};

typedef TSampleSetStatistics<float> FSampleSetStatisticsf;
typedef TSampleSetStatistics<double> FSampleSetStatisticsd;


/**
 * Helper class for reducing the amount of boilerplace code when 
 * building a set of TSampleSetStatistics values.
 */
template<typename RealType>
struct TSampleSetStatisticBuilder
{
	int32 NumStatistics;
	TArray<TSampleSetStatistics<RealType>> Statistics;
	TArray<typename TSampleSetStatistics<RealType>::FFixedCountData> FixedCountBuildData;


	TSampleSetStatisticBuilder(int32 NumStatisticsIn = 1)
	{
		NumStatistics = NumStatisticsIn;
		Statistics.Init(TSampleSetStatistics<RealType>(), NumStatistics);
		FixedCountBuildData.SetNum(NumStatistics);
	}

	const TSampleSetStatistics<RealType>& operator[](int32 Index) const { return Statistics[Index]; }

	void Begin_FixedCount(int64 Count)
	{
		for (int32 j = 0; j < NumStatistics; ++j)
		{
			FixedCountBuildData[j] = Statistics[j].Begin_FixedCount(Count);
		}
	}

	void StartSecondPass_FixedCount()
	{
		for (int32 j = 0; j < NumStatistics; ++j)
		{
			Statistics[j].StartSecondPass_FixedCount(FixedCountBuildData[j]);
		}
	}

	void CompleteSecondPass_FixedCount()
	{
		for (int32 j = 0; j < NumStatistics; ++j)
		{
			Statistics[j].CompleteSecondPass_FixedCount(FixedCountBuildData[j]);
		}
	}

	void AccumulateValue_FixedCount(int32 StatisticIndex, const RealType Value)
	{
		Statistics[StatisticIndex].AccumulateValue_FixedCount(Value, FixedCountBuildData[StatisticIndex]);
	}


	template<typename EnumerableType>
	void ComputeMultiPass(int64 Count, const EnumerableType& MultiEnumerableType)
	{
		check(NumStatistics == 1);
		Begin_FixedCount(Count);
		for (RealType Value : MultiEnumerableType)
		{
			Statistics[0].AccumulateValue_FixedCount(Value, FixedCountBuildData[0]);
		}
		StartSecondPass_FixedCount();
		for (RealType Value : MultiEnumerableType)
		{
			Statistics[0].AccumulateValue_FixedCount(Value, FixedCountBuildData[0]);
		}
		CompleteSecondPass_FixedCount();
	}

};


} // end namespace UE::Geometry
} // end namespace UE
