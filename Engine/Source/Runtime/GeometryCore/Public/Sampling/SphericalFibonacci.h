// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "MatrixTypes.h"
#include "Math/UnrealMathUtility.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * A Spherical Fibonacci (SF) Point Set is a set of points that are roughly evenly distributed on a sphere.
 * The points lie on a spiral, see https://dl.acm.org/doi/10.1145/2816795.2818131 for more information.
 * The i-th SF point an N-point set can be calculated directly.
 * For a given (normalized) point P, finding the nearest SF point (ie mapping back to i) can be done in constant time.
 *
 * Z is "up" in the sphere.
 */
template<typename RealType>
class TSphericalFibonacci
{
public:
	int32 N = 64;

	TSphericalFibonacci(int32 NumPoints = 64)
		: N(NumPoints)
	{
	}

	int32 Num() const
	{
		return N;
	}

	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector<RealType> Point(int32 Index) const
	{
		constexpr RealType PHI = static_cast<RealType>(UE_DOUBLE_GOLDEN_RATIO);

		checkSlow(Index >= 0 && Index < N);
		RealType div = (RealType)Index / PHI;
		RealType phi = TMathUtil<RealType>::TwoPi * (div - TMathUtil<RealType>::Floor(div));
		RealType cos_phi = TMathUtil<RealType>::Cos(phi), sin_phi = TMathUtil<RealType>::Sin(phi);

		RealType z = 1.0 - (2.0 * (RealType)Index + 1.0) / (RealType)N;
		RealType theta = TMathUtil<RealType>::ACos(z);
		RealType sin_theta = TMathUtil<RealType>::Sin(theta);

		return TVector<RealType>(cos_phi * sin_theta, sin_phi * sin_theta, z);
	}


	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector<RealType> operator[](int32 Index) const
	{
		return Point(Index);
	}


	/**
	 * @return Index of the sphere point closest to the given point P
	 */
	int32 FindIndex(const TVector<RealType>& P)
	{
		constexpr RealType PHI = static_cast<RealType>(UE_DOUBLE_GOLDEN_RATIO);

		RealType phi = TMathUtil<RealType>::Min(TMathUtil<RealType>::Atan2(P.Y, P.X), TMathUtil<RealType>::Pi);
		RealType cosTheta = P.Z;
		RealType k = TMathUtil<RealType>::Max(2.0, TMathUtil<RealType>::Floor(
			TMathUtil<RealType>::Log(N * TMathUtil<RealType>::Pi * TMathUtil<RealType>::Sqrt(5.0) * (1.0 - cosTheta * cosTheta)) / TMathUtil<RealType>::Log(PHI * PHI)));
		RealType Fk = TMathUtil<RealType>::Pow(PHI, k) / TMathUtil<RealType>::Sqrt(5.0);

		RealType F0 = TMathUtil<RealType>::Round(Fk);
		RealType F1 = TMathUtil<RealType>::Round(Fk * PHI);

		TMatrix2<RealType> B(
			2.0 * TMathUtil<RealType>::Pi * MultiplyAddFrac(F0 + 1.0, PHI - 1.0) - 2.0 * TMathUtil<RealType>::Pi * (PHI - 1),
			2.0 * TMathUtil<RealType>::Pi * MultiplyAddFrac(F1 + 1.0, PHI - 1.0) - 2.0 * TMathUtil<RealType>::Pi * (PHI - 1),
			-2.0 * F0 / N, -2.0 * F1 / N);
		TMatrix2<RealType> invB = B.Inverse();

		//Vector2d c = floor(mul(invB, RealType2(phi, cosTheta - (1 - 1.0/N))));
		TVector2<RealType> c(phi, cosTheta - (1.0 - 1.0/N));
		c = invB * c;
		c.X = TMathUtil<RealType>::Floor(c.X); c.Y = TMathUtil<RealType>::Floor(c.Y);

		RealType d = TMathUtil<RealType>::MaxReal, j = 0;
		for (int32 s = 0; s < 4; ++s) 
		{
			TVector2<RealType> cosTheta_second((RealType)(s % 2) + c.X, (RealType)(s / 2) + c.Y);
			cosTheta = B.Row1.Dot(cosTheta_second) + (1.0 - 1.0 / N);
			cosTheta = TMathUtil<RealType>::Clamp(cosTheta, -1.0, +1.0) * 2.0 - cosTheta;
			RealType i = TMathUtil<RealType>::Floor(N * 0.5 - cosTheta * N * 0.5);
			phi = 2.0 * TMathUtil<RealType>::Pi * MultiplyAddFrac(i, PHI - 1);
			cosTheta = 1.0 - (2.0 * i + 1.0) * (1.0 / N); // rcp(n);
			RealType sinTheta = TMathUtil<RealType>::Sqrt(1.0 - cosTheta * cosTheta);
			TVector<RealType> q( 
				TMathUtil<RealType>::Cos(phi) * sinTheta, 
				TMathUtil<RealType>::Sin(phi) * sinTheta, 
				cosTheta );
			RealType SquaredDistance = DistanceSquared(q, P);
			if (SquaredDistance < d)
			{
				d = SquaredDistance;
				j = i;
			}
		}

		return (int32)j;
	}





protected:
	// MultiplyAddFrac(A,B) = multiply_add( A ,B, -floor(A*B) )
	static RealType MultiplyAddFrac(RealType a, RealType b)
	{
		return a * b + -TMathUtil<RealType>::Floor(a * b);
	}
};


/** 
 * A 2D point set based on the Fibonacci sequence where the i'th point
 * of an N point set can be directly computed.
 * 
 * Reference: Spherical Fibonacci Point Sets for Illumination Integrals
 * https://repositori.upf.edu/bitstream/handle/10230/35552/marques_CGForum32_sphe.pdf?sequence=1&isAllowed=y
 * 
 */
template<typename RealType>
class TFibonacciLattice
{
public:
	int32 N = 64;

	enum class EType
	{
		Square,	// Fibonacci lattice on unit square
		Disc	// Fibonacci lattice on unit disc (spiral)
	};
	EType Type = EType::Square;

public:
	TFibonacciLattice(int32 NumPoints = 64, EType InType = EType::Square)
		: N(NumPoints), Type(InType)
	{
	}

	int32 Num() const
	{
		return N;
	}

	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector2<RealType> Point(int32 Index) const
	{
		constexpr RealType PHI = static_cast<RealType>(UE_DOUBLE_GOLDEN_RATIO);
		checkSlow(Index >= 0 && Index < N);

		// Use 1-based indices to exclude the zero/origin term.
		// Internally remap the range: [1, N+1].
		const int32 Idx = Index + 1;
		const int32 NumPts = N + 1;

		// Unit Square Lattice:
		// (xs,ys) --> (modf(i/phi, unused), i/n)
		RealType Div = (RealType)Idx / PHI;
		RealType X = (Div - TMathUtil<RealType>::Floor(Div));
		RealType Y = (RealType)Idx / (RealType)NumPts;

		if (Type == EType::Disc)
		{
			// Unit Disc Lattice:
			// (theta, r) --> (2*pi*xs, sqrt(ys))
			// (xd, yd) --> (r*cos(theta), r*sin(theta))
			RealType Theta = TMathUtil<RealType>::TwoPi * X;
			RealType R = TMathUtil<RealType>::Sqrt(Y);
			X = R * TMathUtil<RealType>::Cos(Theta);
			Y = R * TMathUtil<RealType>::Sin(Theta);
		}
		return TVector2<RealType>(X,Y);
	}

	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector2<RealType> operator[](int32 Index) const
	{
		return Point(Index);
	}
};


/**
 * A hemisphere point set generated using a Fibonacci lattice
 * where the i'th point of an N point set can be computed directly.
 * 
 * Z is "up" in the hemisphere.
 */
template<typename RealType>
class THemisphericalFibonacci
{
public:
	int32 N = 64;

	enum class EDistribution
	{
		Uniform,
		Cosine
	};
	EDistribution Distribution = EDistribution::Uniform;

public:
	THemisphericalFibonacci(int32 NumPoints = 64, EDistribution Dist = EDistribution::Uniform)
		: N(NumPoints), Distribution(Dist)
	{
	}

	int32 Num() const
	{
		return N;
	}

	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector<RealType> Point(int32 Index) const
	{
		checkSlow(Index >= 0 && Index < N);

		TVector<RealType> Point = {};
		switch (Distribution)
		{
		case EDistribution::Uniform:
		{
			TSphericalFibonacci<RealType> Points(2 * N);
			Point = Points[Index];
			break;
		}
		case EDistribution::Cosine:
		{
			TFibonacciLattice<RealType> Points(N, TFibonacciLattice<RealType>::EType::Disc);
			const TVector2<RealType> Pt = Points[Index];
            
			// Planar projection of Fibonacci spiral (unit disc [xd, yd]) to
			// hemisphere to achieve cosine weighted ray distribution.
			// (Malley's method).
			//
			// Hemisphere: (x,y,z) --> (xd, yd, 1-xd^2-yd^2)
			//
			// Reference: Physically Based Rendering: From Theory to Implementation
			// http://www.pbr-book.org/3ed-2018/Monte_Carlo_Integration/2D_Sampling_with_Multidimensional_Transformations.html#Cosine-WeightedHemisphereSampling
			RealType Z = TMathUtil<RealType>::Sqrt(TMathUtil<RealType>::Max(0.0, 1.0 - Pt.X * Pt.X - Pt.Y * Pt.Y));
			Point = TVector<RealType>(Pt.X, Pt.Y, Z);
			break;
		}
		}
		return Point;
	}

	/**
	 * @param Index point index in range [0,Num()-1]
	 * @return sphere point for given Index
	 */
	TVector<RealType> operator[](int32 Index) const
	{
		return Point(Index);
	}
};

} // end namespace UE::Geometry
} // end namespace UE