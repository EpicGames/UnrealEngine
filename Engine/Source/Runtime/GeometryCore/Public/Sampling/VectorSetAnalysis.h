// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TVectorSetAnalysis3 computes various analyses of a set of input Vectors (currently mainly clustering.
 */
template<typename RealType>
class TVectorSetAnalysis3
{
public:
	//
	// Input data
	//

	TArray<TVector<RealType>> Vectors;
	TArray<int32> VectorIDs;
	bool bNormalized = false;

	//
	// Calculated values
	//

	/** Set of vectors that represent centers of clusters */
	TArray<TVector<RealType>> ClusterVectors;
	/** Mapping from Vector index to ClusterVectors index */
	TArray<int32> VectorToClusterMap;


	/**
	 * Initialize the internal set of Vectors and IDs using an external integer-Enumerable and associated GetVector(ID) function
	 * @param NumVectorsHint provide hint as to number of elements in Enumerable, to allow memory to be pre-allocated
	 * @param bIsNormalizedHint indicate whether vectors are normalized
	 */
	template<typename EnumerableIDType>
	void Initialize(EnumerableIDType EnumerableIDs, TFunctionRef<TVector<RealType>(int32)> GetVectorFunc, int32 NumVectorsHint = 0, bool bIsNormalizedHint = false)
	{
		Vectors.Reserve(NumVectorsHint);
		VectorIDs.Reserve(NumVectorsHint);
		for (int32 ID : EnumerableIDs)
		{
			VectorIDs.Add(ID);
			Vectors.Add(GetVectorFunc(ID));
		}

		bNormalized = bIsNormalizedHint;
	}

	/** @return number of input vectors */
	int32 NumVectors() const { return Vectors.Num(); }

	/** @return number of clusters found by last clustering algorithm (may be zero if not initialized) */
	int32 NumClusters() const { return ClusterVectors.Num(); }


	/**
	 * Run simple greedy clustering algorithm on input ClusterVectors.
	 * Done in a single pass over vectors, each successive Vector is either grouped with one of the
	 * existing clusters if it's direction is within AngleToleranceDeg, or creates a new cluster.
	 */
	inline void GreedyClusterVectors(RealType AngleToleranceDeg)
	{
		check(bNormalized);		// otherwise code below is incorrect

		RealType DotTolerance = TMathUtil<RealType>::Cos(AngleToleranceDeg * TMathUtil<RealType>::DegToRad);

		int32 N = NumVectors(), M = 0;
		VectorToClusterMap.SetNum(N);
		for (int32 k = 0; k < N; ++k)
		{
			bool bFound = false;
			if (M > 0)
			{
				// try to find an existing cluster
				for (int32 j = 0; j < M && !bFound; ++j)
				{
					if (Vectors[k].Dot(ClusterVectors[j]) > DotTolerance)
					{
						VectorToClusterMap[k] = j;
						bFound = true;
					}
				}
			}

			// cluster not found, spawn a new one
			if (!bFound)
			{
				ClusterVectors.Add(Vectors[k]);
				VectorToClusterMap[k] = k;
				M++;
			}
		}
	}


};

typedef TVectorSetAnalysis3<float> FVectorSetAnalysis3f;
typedef TVectorSetAnalysis3<double> FVectorSetAnalysis3d;

} // end namespace UE::Geometry
} // end namespace UE
