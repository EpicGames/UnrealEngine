// Copyright Epic Games, Inc. All Rights Reserved.

// ported from geometry3Sharp Segment2

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"
#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/*
 * 2D Line Segment stored as Center point, normalized Direction vector, and scalar Extent
 */
template<typename T>
struct TSegment2
{
public:
	/** Center point of segment */
	TVector2<T> Center = TVector2<T>::Zero();
	/** normalized Direction vector of segment */
	TVector2<T> Direction = TVector2<T>::UnitX();
	/** Extent of segment, which is half the total length */
	T Extent = (T)0;

	TSegment2() = default;

	/**
	 * Construct a Segment from two Points
	 */
	TSegment2(const TVector2<T>& Point0, const TVector2<T>& Point1)
	{
		// set from endpoints 
		Center = T(.5) * (Point0 + Point1);
		Direction = Point1 - Point0;
		Extent = T(.5) * Normalize(Direction);
	}

	/**
	 * Construct a segment from a Center Point, normalized Direction, and scalar Extent
	 */
	TSegment2(const TVector2<T>& CenterIn, const TVector2<T>& DirectionIn, T ExtentIn)
	{
		Center = CenterIn;
		Direction = DirectionIn; 
		Extent = ExtentIn;
	}



	/** Update the Segment with a new start point */
	inline void SetStartPoint(const TVector2<T>& Point)
	{
		update_from_endpoints(Point, EndPoint());
	}

	/** Update the Segment with a new end point */
	inline void SetEndPoint(const TVector2<T>& Point)
	{
		update_from_endpoints(StartPoint(), Point);
	}

	/** Reverse the segment */
	void Reverse()
	{
		update_from_endpoints(EndPoint(), StartPoint());
	}




	/** @return start point of segment */
	inline TVector2<T> StartPoint() const
	{
		return Center - Extent * Direction;
	}

	/** @return end point of segment */
	inline TVector2<T> EndPoint() const
	{
		return Center + Extent * Direction;
	}


	/** @return the Length of the segment */
	inline T Length() const
	{
		return (T)2 * Extent;
	}

	/** @return first (i == 0) or second (i == 1) endpoint of the Segment  */
	inline TVector2<T> GetPointFromIndex(int i) const
	{
		return (i == 0) ? (Center - Extent * Direction) : (Center + Extent * Direction);
	}

	/**
	 * @return point on segment at given (signed) Distance from the segment Origin
	 */
	inline TVector2<T> PointAt(T DistanceParameter) const
	{
		return Center + DistanceParameter * Direction;
	}

	/**
	 * @param UnitParameter value in range [0,1]
	 * @return point on segment at that linearly interpolates between start and end based on Unit Parameter
	 */
	inline TVector2<T> PointBetween(T UnitParameter) const
	{
		return Center + ((T)2 * UnitParameter - (T)1) * Extent * Direction;
	}

	/**
	 * @param DistanceParameter distance on the segment from the segment Origin in range [-Extent, Extent]
	 * @return unit value in range [0,1]
	 */
	inline T ConvertToUnitRange(T DistanceParameter) const
	{
		return T(0.5) * (T(1) + DistanceParameter / Extent); 
	}

	/**
	 * @return minimum squared distance from Point to segment
	 */
	inline T DistanceSquared(const TVector2<T>& Point) const
	{
		T DistParameter;
		return DistanceSquared(Point, DistParameter);
	}



	/**
	 * @param Point query point
	 * @param DistParameterOut calculated distance parameter in range [-Extent,Extent]
	 * @return minimum squared distance from Point to Segment
	 */	
	T DistanceSquared(const TVector2<T>& Point, T& DistParameterOut) const
	{
		DistParameterOut = (Point - Center).Dot(Direction);
		if (DistParameterOut >= Extent)
		{
			DistParameterOut = Extent;
			return UE::Geometry::DistanceSquared(Point, EndPoint());
		}
		else if (DistParameterOut <= -Extent)
		{
			DistParameterOut = -Extent;
			return UE::Geometry::DistanceSquared(Point, StartPoint());
		}
		TVector2<T> ProjectedPt = Center + DistParameterOut * Direction;
		return UE::Geometry::DistanceSquared(ProjectedPt, Point);
	}


	/**
	 * @return nearest point on segment to QueryPoint
	 */
	inline TVector2<T> NearestPoint(const TVector2<T>& QueryPoint) const
	{
		T t = (QueryPoint - Center).Dot(Direction);
		if (t >= Extent)
		{
			return EndPoint();
		}
		if (t <= -Extent)
		{
			return StartPoint();
		}
		return Center + t * Direction;
	}


	/**
	 * @return scalar projection of QueryPoint onto line of Segment (not clamped to Extents)
	 */
	inline T Project(const TVector2<T>& QueryPoint) const
	{
		return (QueryPoint - Center).Dot(Direction);
	}


	/**
	 * @return scalar projection of QueryPoint onto line of Segment, mapped to [0,1] range along segment
	 */
	inline T ProjectUnitRange(const TVector2<T>& QueryPoint) const
	{
		T ProjT = (QueryPoint - Center).Dot(Direction);
		T Alpha = ((ProjT / Extent) + (T)1) * (T)0.5;
		return TMathUtil<T>::Clamp(Alpha, (T)0, (T)1);
	}

	/**
	 * Determine which side of the segment the query point lies on.
	 * @param QueryPoint test point
	 * @param Tolerance tolerance band in which we return 0
	 * @return -1 if point is to right of line, +1 if left, and 0 if on line or within tolerance band
	 */
	int GetSide(const TVector2<T>& QueryPoint, T Tolerance = 0)
	{
		TVector2<T> StartPt = Center - Extent * Direction;
		TVector2<T> AQ = QueryPoint - StartPt;

		// Note that we don't have to adjust tolerance because direction is already normalized
		double Dist = DotPerp(AQ, Direction);
		return (Dist > Tolerance ? -1 : (Dist < -Tolerance ? 1 : 0));
	}

	/**
	 * Determine which side of the segment the query point lies on. This version is deprecated because it
	 *  and the static function give opposite results. It also does not correctly handle tolerance.
	 * @param QueryPoint test point
	 * @param Tolerance tolerance band in which we return 0
	 * @return -1 if point is to right of line, +1 if left, and 0 if on line or within tolerance band
	 */
	UE_DEPRECATED(5.5, "Use GetSide instead.")
	int WhichSide(const TVector2<T>& QueryPoint, T Tolerance = 0)
	{
		// [TODO] subtract Center from test?
		TVector2<T> EndPt = Center + Extent * Direction;
		TVector2<T> StartPt = Center - Extent * Direction;
		T det = -Orient(EndPt, StartPt, QueryPoint);
		return (det > Tolerance ? +1 : (det < -Tolerance ? -1 : 0));
	}


	/**
	 * Test if this segment intersects with OtherSegment. Returns true for parallel-line overlaps. Returns same result as IntrSegment2Segment2
	 * @param OtherSegment segment to test against
	 * @param DotThresh dot-product tolerance used to determine if segments are parallel
	 * @param IntervalThresh distance tolerance used to allow slighly-not-touching segments to be considered overlapping
	 * @return true if segments intersect
	 */
	bool Intersects(const TSegment2<T>& OtherSegment, T DotThresh = TMathUtil<T>::Epsilon, T IntervalThresh = 0) const
	{
		// see IntrLine2Line2 and IntrSegment2Segment2 for details on this code

		// Special handling of degenerate segments; by convention if Direction was too small to normalize, Extent and Direction will be zero
		bool bIsPoint = Extent == 0;
		bool bIsPointOther = OtherSegment.Extent == 0;
		int IsPointCount = (int)bIsPoint + (int)bIsPointOther;
		if (IsPointCount == 2)
		{
			return UE::Geometry::DistanceSquared(Center, OtherSegment.Center) <= IntervalThresh * IntervalThresh;
		}
		else if (IsPointCount == 1)
		{
			T DistSq;
			if (bIsPoint)
			{
				DistSq = OtherSegment.DistanceSquared(Center);
			}
			else
			{
				DistSq = DistanceSquared(OtherSegment.Center);
			}
			return DistSq <= IntervalThresh * IntervalThresh;
		}

		TVector2<T> diff = OtherSegment.Center - Center;
		T D0DotPerpD1 = DotPerp(Direction, OtherSegment.Direction);
		if (TMathUtil<T>::Abs(D0DotPerpD1) > DotThresh)      // Lines intersect in a single point.
		{
			T invD0DotPerpD1 = ((T)1) / D0DotPerpD1;
			T diffDotPerpD0 = DotPerp(diff, Direction);
			T diffDotPerpD1 = DotPerp(diff, OtherSegment.Direction);
			T s = diffDotPerpD1 * invD0DotPerpD1;
			T s2 = diffDotPerpD0 * invD0DotPerpD1;
			return TMathUtil<T>::Abs(s) <= (Extent + IntervalThresh)
				&& TMathUtil<T>::Abs(s2) <= (OtherSegment.Extent + IntervalThresh);
		}

		// Lines are parallel.
		Normalize(diff);
		T diffNDotPerpD1 = DotPerp(diff, OtherSegment.Direction);
		if (TMathUtil<T>::Abs(diffNDotPerpD1) <= DotThresh)
		{
			// Compute the location of OtherSegment endpoints relative to our Segment
			diff = OtherSegment.Center - Center;
			T t1 = Direction.Dot(diff);
			// Note: IntervalThresh not used here; this is a bit inconsistent but simplifies the code
			// If you add it, make sure that segments that intersect without it don't end up with too-wide intersection intervals
			T tmin = t1 - OtherSegment.Extent;
			T tmax = t1 + OtherSegment.Extent;
			TInterval1<T> extents(-Extent, Extent);
			if (extents.Overlaps(TInterval1<T>(tmin, tmax)))
			{
				return true;
			}
			return false;
		}

		// lines are parallel but not collinear
		return false;
	}





	// 2D segment utility functions


	/**
	 * Calculate distance from QueryPoint to segment (StartPt,EndPt)
	 */
	static T FastDistanceSquared(const TVector2<T>& StartPt, const TVector2<T>& EndPt, const TVector2<T>& QueryPt, T Tolerance = TMathUtil<T>::Epsilon)
	{
		T vx = EndPt.X - StartPt.X, vy = EndPt.Y - StartPt.Y;
		T len2 = vx * vx + vy * vy;
		T dx = QueryPt.X - StartPt.X, dy = QueryPt.Y - StartPt.Y;
		//if (len2 < 1e-13) 
		if (len2 < Tolerance)
		{
			return dx * dx + dy * dy;
		}
		T t = (dx*vx + dy * vy);
		if (t <= 0) 
		{
			return dx * dx + dy * dy;
		}
		else if (t >= len2) 
		{
			dx = QueryPt.X - EndPt.X; 
			dy = QueryPt.Y - EndPt.Y;
			return dx * dx + dy * dy;
		}
		dx = QueryPt.X - (StartPt.X + ((t * vx) / len2));
		dy = QueryPt.Y - (StartPt.Y + ((t * vy) / len2));
		return dx * dx + dy * dy;
	}

	/**
	 * Determine which side of the segment the query point lies on. 
	 * @param StartPt first point of Segment
	 * @param EndPt second point of Segment
	 * @param QueryPoint test point
	 * @param Tolerance tolerance band in which we return 0. Note that using tolerance == 0 is faster because
	 *   we do not need to get distance of point from segment line.
	 * @return -1 if point is to right of line, +1 if left, and 0 if on line or within tolerance band, or segment was degenerate
	 */
	static int GetSide(const TVector2<T>& StartPt, const TVector2<T>& EndPt, const TVector2<T>& QueryPt, T Tolerance = (T)0)
	{
		TVector2<T> AB = EndPt - StartPt;
		T DotPerpResult = DotPerp(QueryPt - StartPt, AB);
		if (Tolerance == 0)
		{
			return (DotPerpResult > 0 ? -1 : (DotPerpResult < 0 ? 1 : 0));
		}

		// If tolerance is nonzero, we have to use segment length to adjust it, since we use unnormalized
		//  direction in DotPerp.
		T LengthSquared = AB.SizeSquared();
		if (LengthSquared == 0)
		{
			// It's possible to get here without DotPerp underflowing, so we early out to avoid incorrectly 
			//  zeroing out the tolerance below while comparing against a nonzero DotPerpResult. 
			// We could scale AB to deal with this case to try to better match behavior with the tolerance
			//  0 case (which blindly uses DotPerp result), but it's fair to expect tolerance to affect the 
			//  treatment of near-degenerate segments, so we don't bother.
			return 0;
		}
		T ToleranceToUse = Tolerance * FMath::Sqrt(LengthSquared);

		return (DotPerpResult > ToleranceToUse ? -1 : (DotPerpResult < -ToleranceToUse ? 1 : 0));
	}

	/**
	 * Determine which side of the segment the query point lies on. This version is deprecated because it
	 *  and the member function give opposite results. It also does not correctly handle tolerance.
	 * @param StartPt first point of Segment
	 * @param EndPt second point of Segment
	 * @param QueryPoint test point
	 * @param Tolerance tolerance band in which we return 0
	 * @return +1 if point is to right of line, -1 if left, and 0 if on line or within tolerance band
	 */
	UE_DEPRECATED(5.5, "Use GetSide instead.")
	static int WhichSide(const TVector2<T>& StartPt, const TVector2<T>& EndPt, const TVector2<T>& QueryPt, T Tolerance = (T)0)
	{
		T det = -Orient(StartPt, EndPt, QueryPt);
		return (det > Tolerance ? +1 : (det < -Tolerance ? -1 : 0));
	}

	/**
	 * Return true if QueryPt is on the segment between A and B.
	 */
	static bool IsOnSegment(const TVector2<T>& A, const TVector2<T>& B, const TVector2<T>& QueryPt, T Tolerance = (T)0)
	{
		const TVector2<T> AB = B - A;

		T SquaredLength = AB.SquaredLength();
		if (SquaredLength == 0)
		{
			// It's possible to underflow here (i.e. A and B are not coincident), but the distance between
			//  A and B is negligible in that case, and we'll still correctly identify B as on the segment
			//  (since we'll underflow the same way). The fact that we're slighly more tolerant near A at
			//  these scales seems ok.
			return TVector2<T>::DistSquared(A, QueryPt) <= Tolerance * Tolerance;
		}
		// We can either normalize AB so that the values we get from dot products are distances,
		//  or just multiply the other side by length.
		T DotTolerance = FMath::Sqrt(SquaredLength) * Tolerance;
		const TVector2<T> AQ = QueryPt - A;
		if (FMath::Abs(DotPerp(AB, AQ)) > DotTolerance)
		{
			// Must not be on the line
			return false;
		}

		// Make sure it's between the endpoints
		return (AB.Dot(AQ) >= -DotTolerance && AB.Dot(QueryPt - B) <= DotTolerance);
	}

protected:

	// update segment based on new endpoints
	inline void update_from_endpoints(const TVector2<T>& p0, const TVector2<T>& p1)
	{
		Center = 0.5 * (p0 + p1);
		Direction = p1 - p0;
		Extent = 0.5 * Normalize(Direction);
	}

};
typedef TSegment2<float> FSegment2f;
typedef TSegment2<double> FSegment2d;








/*
 * 3D Line Segment stored as Center point, normalized Direction vector, and scalar Extent
 */
template<typename T>
struct TSegment3
{
public:
	/** Center point of segment */
	TVector<T> Center = TVector<T>::Zero();
	/** normalized Direction vector of segment */
	TVector<T> Direction = TVector<T>::UnitX();
	/** Extent of segment, which is half the total length */
	T Extent = (T)0;

	TSegment3() = default;

	/**
	 * Construct a Segment from two Points
	 */
	TSegment3(const TVector<T>& Point0, const TVector<T>& Point1)
	{
		// set from endpoints 
		Center = T(.5) * (Point0 + Point1);
		Direction = Point1 - Point0;
		Extent = T(.5) * Normalize(Direction);
	}

	/**
	 * Construct a segment from a Center Point, normalized Direction, and scalar Extent
	 */
	TSegment3(const TVector<T>& CenterIn, const TVector<T>& DirectionIn, T ExtentIn)
	{
		Center = CenterIn;
		Direction = DirectionIn;
		Extent = ExtentIn;
	}



	/** Update the Segment with a new start point */
	inline void SetStartPoint(const TVector<T>& Point)
	{
		update_from_endpoints(Point, EndPoint());
	}

	/** Update the Segment with a new end point */
	inline void SetEndPoint(const TVector<T>& Point)
	{
		update_from_endpoints(StartPoint(), Point);
	}

	/** Reverse the segment */
	void Reverse()
	{
		update_from_endpoints(EndPoint(), StartPoint());
	}




	/** @return start point of segment */
	inline TVector<T> StartPoint() const
	{
		return Center - Extent * Direction;
	}

	/** @return end point of segment */
	inline TVector<T> EndPoint() const
	{
		return Center + Extent * Direction;
	}


	/** @return the Length of the segment */
	inline T Length() const
	{
		return (T)2 * Extent;
	}

	/** @return first (i == 0) or second (i == 1) endpoint of the Segment  */
	inline TVector<T> GetPointFromIndex(int i) const
	{
		return (i == 0) ? (Center - Extent * Direction) : (Center + Extent * Direction);
	}

	/**
	 * @return point on segment at given (signed) Distance from the segment Origin
	 */
	inline TVector<T> PointAt(T DistanceParameter) const
	{
		return Center + DistanceParameter * Direction;
	}

	/**
	 * @param UnitParameter value in range [0,1]
	 * @return point on segment at that linearly interpolates between start and end based on Unit Parameter
	 */
	inline TVector<T> PointBetween(T UnitParameter) const
	{
		return Center + ((T)2 * UnitParameter - (T)1) * Extent * Direction;
	}

	/**
	 * @param DistanceParameter distance on the segment from the segment Origin in range [-Extent, Extent]
	 * @return unit value in range [0,1]
	 */
	inline T ConvertToUnitRange(T DistanceParameter) const
	{
		return T(0.5) * (T(1) + DistanceParameter / Extent); 
	}

	/**
	 * @return minimum squared distance from Point to segment
	 */
	inline T DistanceSquared(const TVector<T>& Point) const
	{
		T DistParameter;
		return DistanceSquared(Point, DistParameter);
	}



	/**
	 * @param Point query point
	 * @param DistParameterOut calculated distance parameter in range [-Extent,Extent]
	 * @return minimum squared distance from Point to Segment
	 */
	T DistanceSquared(const TVector<T>& Point, T& DistParameterOut) const
	{
		DistParameterOut = (Point - Center).Dot(Direction);
		if (DistParameterOut >= Extent)
		{
			DistParameterOut = Extent;
			return UE::Geometry::DistanceSquared(Point, EndPoint());
		}
		else if (DistParameterOut <= -Extent)
		{
			DistParameterOut = -Extent;
			return UE::Geometry::DistanceSquared(Point, StartPoint());
		}
		TVector<T> ProjectedPt = Center + DistParameterOut * Direction;
		return UE::Geometry::DistanceSquared(ProjectedPt, Point);
	}


	/**
	 * @return nearest point on segment to QueryPoint
	 */
	inline TVector<T> NearestPoint(const TVector<T>& QueryPoint) const
	{
		T t = (QueryPoint - Center).Dot(Direction);
		if (t >= Extent)
		{
			return EndPoint();
		}
		if (t <= -Extent)
		{
			return StartPoint();
		}
		return Center + t * Direction;
	}


	/**
	 * @return scalar projection of QueryPoint onto line of Segment (not clamped to Extents)
	 */
	inline T Project(const TVector<T>& QueryPoint) const
	{
		return (QueryPoint - Center).Dot(Direction);
	}


	/**
	 * @return scalar projection of QueryPoint onto line of Segment, mapped to [0,1] range along segment
	 */
	inline T ProjectUnitRange(const TVector<T>& QueryPoint) const
	{
		T ProjT = (QueryPoint - Center).Dot(Direction);
		T Alpha = ((ProjT / Extent) + (T)1) * (T)0.5;
		return TMathUtil<T>::Clamp(Alpha, (T)0, (T)1);
	}

	/**
	 * @return axis-aligned bounding box of line segment
	 */
	TAxisAlignedBox3<T> GetBounds() const
	{
		TAxisAlignedBox3<T> Result;
		
		for (int32 j = 0; j < 3; ++j)
		{
			T LowValue = Center[j] - Extent * Direction[j];
			T HighValue = Center[j] + Extent * Direction[j];
			Result.Min[j] = TMathUtil<T>::Min(LowValue, HighValue);
			Result.Max[j] = TMathUtil<T>::Max(LowValue, HighValue);
		}
		return Result;
	}

	/**
	 * @return axis-aligned bounding box of line segment thickened to given SegmentRadius
	 */
	TAxisAlignedBox3<T> GetBounds(T SegmentRadius) const
	{
		TAxisAlignedBox3<T> Result = GetBounds();
		Result.Expand(SegmentRadius);
		return Result;
	}

protected:

	// update segment based on new endpoints
	inline void update_from_endpoints(const TVector<T>& p0, const TVector<T>& p1)
	{
		Center = 0.5 * (p0 + p1);
		Direction = p1 - p0;
		Extent = 0.5 * Normalize(Direction);
	}

};
typedef TSegment3<float> FSegment3f;
typedef TSegment3<double> FSegment3d;

} // end namespace UE::Geometry
} // end namespace UE

