// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3sharp MeshConnectedComponents

#pragma once

#include "CoreMinimal.h"
#include "IntBoxTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Containers/IndirectArray.h"

namespace UE
{
namespace Geometry
{

/**
 * FMeshConnectedComponents calculates Connected Components of a Mesh, or sub-regions of a Mesh.
 * By default the actual mesh connectivity is used, but an optional connectivity predicate
 * can be provided to specify when two elements should be considered connected.
 */
class FMeshConnectedComponents
{
public:
	const FDynamicMesh3* Mesh;

	/**
	 * Connected Component found by one of the calculation functions
	 */
	struct FComponent
	{
		/** List of indices contained in component */
		TArray<int> Indices;
	};

	/**
	 * List of Connected Components that have been found by one of the calculation functions
	 */
	TIndirectArray<FComponent> Components;

public:

	FMeshConnectedComponents(const FDynamicMesh3* MeshIn)
		: Mesh(MeshIn)
	{
	}

	//
	// Calculation functions. Call these to calculate different types of Components.
	//

	/**
	 * Find all connected triangle components of the Mesh and store in Components array.
	 * Triangle connectivity is based on edge connectivity, ie bowtie-vertices are not connections between triangles.
	 * @param TrisConnectedPredicate optional function that specifies whether two edge-connected triangles should be considered connected by the search
	 */
	GEOMETRYCORE_API void FindConnectedTriangles(TFunction<bool(int32, int32)> TrisConnectedPredicate = nullptr);

	/**
	 * Find all connected triangle components of a subset of triangles of the Mesh and store in Components array.
	 * Triangle connectivity is based on edge connectivity, ie bowtie-vertices are not connections between triangles.
	 * @param TriangleROI list of triangles to search across
	 * @param TrisConnectedPredicate optional function that specifies whether two edge-connected triangles should be considered connected by the search
	 */
	GEOMETRYCORE_API void FindConnectedTriangles(const TArray<int>& TriangleROI, TFunction<bool(int32, int32)> TrisConnectedPredicate = nullptr);

	/**
	 * Find all connected triangle components of a subset of triangles of the Mesh and store in Components array.
	 * Triangle connectivity is based on edge connectivity, ie bowtie-vertices are not connections between triangles.
	 * @param IndexFilterFunc defines set of triangles to search across, return true for triangle IDs that are to be considered
	 * @param TrisConnectedPredicate optional function that specifies whether two edge-connected triangles should be considered connected by the search
	 */
	GEOMETRYCORE_API void FindConnectedTriangles(TFunctionRef<bool(int)> IndexFilterFunc, TFunction<bool(int32, int32)> TrisConnectedPredicate = nullptr);

	/**
	 * Find all connected triangle components that contain one or more Seed Triangles and store in Components array.
	 * Search only starts from Seed Triangles.
	 * Triangle connectivity is based on edge connectivity, ie bowtie-vertices are not connections between triangles.
	 * @param SeedTriangles list of start triangles, each component contains at least one of these triangles
	 * @param TrisConnectedPredicate optional function that specifies whether two edge-connected triangles should be considered connected by the search
	 */
	GEOMETRYCORE_API void FindTrianglesConnectedToSeeds(const TArray<int>& SeedTriangles, TFunction<bool(int32, int32)> TrisConnectedPredicate = nullptr);

	/**
	 * Initialize the internal FComponent list from the input ComponentLists, skipping any empty input lists
	 * @param bValidateIDs if true, test that each value corresponds to a valid triangle ID on the Mesh
	 * @return true if all IDs are valid, or if check was skipped
	 */
	GEOMETRYCORE_API bool InitializeFromTriangleComponents(const TArray<TArray<int32>>& ComponentLists, bool bValidateIDs);

	/**
	* Initialize the internal FComponent list from the input ComponentLists, skipping any empty input lists
	* @param bMoveSubLists if true, steal the arrays inside the ComponentLists (via MoveTemp), to avoid memory copies
	* @param bValidateIDs if true, test that each value corresponds to a valid triangle ID on the Mesh
	* @return true if all IDs are valid, or if check was skipped
	*/
	GEOMETRYCORE_API bool InitializeFromTriangleComponents(TArray<TArray<int32>>& ComponentLists, bool bMoveSubLists, bool bValidateIDs);

	/**
	* Find all connected vertex components of the Mesh and store in Components array.
	* @param VertsConnectedPredicate optional function that specifies whether two edge-connected vertices should be considered connected by the search
	*/
	GEOMETRYCORE_API void FindConnectedVertices(TFunction<bool(int32, int32)> VertsConnectedPredicate = nullptr);

	/**
	* Find all connected vertex components of a subset of vertices of the Mesh and store in Components array.
	* @param VertexROI list of vertices to search across
	* @param VertsConnectedPredicate optional function that specifies whether two edge-connected vertices should be considered connected by the search
	*/
	GEOMETRYCORE_API void FindConnectedVertices(const TArray<int>& VertexROI, TFunction<bool(int32, int32)> VertsConnectedPredicate = nullptr);

	/**
	* Find all connected vertex components of a subset of vertices of the Mesh and store in Components array.
	* @param IndexFilterFunc defines set of vertices to search across, return true for vertex IDs that are to be considered
	* @param VertsConnectedPredicate optional function that specifies whether two edge-connected vertices should be considered connected by the search
	*/
	GEOMETRYCORE_API void FindConnectedVertices(TFunctionRef<bool(int)> IndexFilterFunc, TFunction<bool(int32, int32)>VertsConnectedPredicate = nullptr);

	/**
	* Find all connected vertex components that contain one or more Seed Vertices and store in Components array.
	 Search only starts from Seed Vertices.
	* @param SeedVertices list of start vertices, each component contains at least one of these vertices
	* @param VertsConnectedPredicate optional function that specifies whether two edge-connected vertices should be considered connected by the search
	*/
	GEOMETRYCORE_API void FindVerticesConnectedToSeeds(const TArray<int>& SeedVertices, TFunction<bool(int32, int32)> VertsConnectedPredicate = nullptr);

	/**
	 * Initialize the internal FComponent list from the input ComponentLists, skipping any empty input lists
	 * @param bValidateIDs if true, test that each value corresponds to a valid vertex ID on the Mesh
	 * @return true if all IDs are valid, or if check was skipped
	 */
	GEOMETRYCORE_API bool InitializeFromVertexComponents(const TArray<TArray<int32>>& ComponentLists, bool bValidateIDs);

	/**
	* Initialize the internal FComponent list from the input ComponentLists, skipping any empty input lists
	* @param bMoveSubLists if true, steal the arrays inside the ComponentLists (via MoveTemp), to avoid memory copies
	* @param bValidateIDs if true, test that each value corresponds to a valid vertex ID on the Mesh
	* @return true if all IDs are valid, or if check was skipped
	*/
	GEOMETRYCORE_API bool InitializeFromVertexComponents(TArray<TArray<int32>>& ComponentLists, bool bMoveSubLists, bool bValidateIDs);

	//
	// Query functions. Only valid to call after a Calculation function has been called.
	//

	/** @return Number of Components that were found */
	int32 Num() const
	{
		return Components.Num();
	}

	/** @return element of Components array at given Index */
	const FComponent& GetComponent(int32 Index) const { return Components[Index]; }

	/** @return element of Components array at given Index */
	FComponent& GetComponent(int32 Index) { return Components[Index]; }

	/** @return element of Components array at given Index */
	const FComponent& operator[](int32 Index) const { return Components[Index]; }

	/** @return element of Components array at given Index */
	FComponent& operator[](int32 Index) { return Components[Index]; }

	/** 
	 * @return index of largest component by element count 
	 */
	GEOMETRYCORE_API int32 GetLargestIndexByCount() const;

	/**
	 * Sort the Components array by component element count
	 * @param bLargestFirst if true, sort by decreasing count, otherwise by increasing count
	 */
	GEOMETRYCORE_API void SortByCount(bool bLargestFirst = true);



public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable ranged-based for loop support (forwarding TIndirectArray declarations)
	 */
	auto begin() { return Components.begin(); }
	auto begin() const { return Components.begin(); }
	auto end() { return Components.end();  }
	auto end() const { return Components.end(); }



protected:
	//
	// Internal functions to calculate ROI
	//
	GEOMETRYCORE_API void FindTriComponents(FInterval1i ActiveRange, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> TriConnectedPredicate);
	GEOMETRYCORE_API void FindTriComponents(const TArray<int32>& SeedList, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> TriConnectedPredicate);
	GEOMETRYCORE_API void FindTriComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet);
	GEOMETRYCORE_API void FindTriComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet, TFunctionRef<bool(int32, int32)> TriConnectedPredicate);
	GEOMETRYCORE_API void RemoveFromActiveSet(const FComponent* Component, TArray<uint8>& ActiveSet);

	GEOMETRYCORE_API void FindVertComponents(FInterval1i ActiveRange, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> VertsConnectedPredicate);
	GEOMETRYCORE_API void FindVertComponents(const TArray<int32>& SeedList, TArray<uint8>& ActiveSet, TFunction<bool(int32, int32)> VertsConnectedPredicate);
	GEOMETRYCORE_API void FindVertComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet);
	GEOMETRYCORE_API void FindVertComponent(FComponent* Component, TArray<int32>& ComponentQueue, TArray<uint8>& ActiveSet, TFunctionRef<bool(int32, int32)> VertsConnectedPredicate);

public:
	/**
	 * Utility function to expand a triangle selection to all triangles considered "connected".
	 * More efficient than using full FMeshConnectedComponents instance if ROI is small relative to Mesh size (or if temp buffers can be re-used)
	 * @param Mesh Mesh to calculate on
	 * @param InputROI input set of triangles
	 * @param ResultROI output set of triangles connected to InputROI
	 * @param QueueBuffer optional buffer used as internal Queue. If passed as nullptr, a TArray will be locally allocated
	 * @param DoneBuffer optional set used to track which triangles have already been processed. If passed as nullptr, an TSet will be locally allocated
	 * @param CanGrowPredicate determines whether two connected mesh triangles should be considered connected while growing
	 */
	static GEOMETRYCORE_API void GrowToConnectedTriangles(const FDynamicMesh3* Mesh, 
		const TArray<int>& InputROI, 
		TArray<int>& ResultROI,
		TArray<int32>* QueueBuffer = nullptr, 
		TSet<int32>* DoneBuffer = nullptr,
		TFunctionRef<bool(int32, int32)> CanGrowPredicate = [](int32, int32) { return true; }
	);


	/**
	 * Utility function to expand a triangle selection to all triangles considered "connected".
	 * More efficient than using full FMeshConnectedComponents instance if ROI is small relative to Mesh size (or if temp buffers can be re-used)
	 * This version computes an output TSet instead of output TArray, which is preferable in some cases.
	 * @param Mesh Mesh to calculate on
	 * @param InputROI input set of triangles
	 * @param ResultROI output set of triangles connected to InputROI
	 * @param QueueBuffer optional buffer used as internal Queue. If passed as nullptr, a TArray will be locally allocated
	 * @param CanGrowPredicate determines whether two connected mesh triangles should be considered connected while growing
	 */
	static GEOMETRYCORE_API void GrowToConnectedTriangles(const FDynamicMesh3* Mesh,
		const TArray<int>& InputROI,
		TSet<int>& ResultROI,
		TArray<int32>* QueueBuffer = nullptr,
		TFunctionRef<bool(int32, int32)> CanGrowPredicate = [](int32, int32) { return true; }
	);


	/**
	 * Utility function to expand a vertex selection to all vertices considered "connected".
	 * @param Mesh Mesh to calculate on
	 * @param InputROI input set of vertices
	 * @param ResultROI output set of vertices connected to InputROI
	 * @param QueueBuffer optional buffer used as internal Queue. If passed as nullptr, a TArray will be locally allocated
	 * @param CanGrowPredicate determines whether two connected mesh vertices should be considered connected while growing
	 */
	static GEOMETRYCORE_API void GrowToConnectedVertices(
		const FDynamicMesh3& Mesh,
		const TArray<int>& InputROI,
		TSet<int>& ResultROI,
		TArray<int32>* QueueBuffer = nullptr,
		TFunctionRef<bool(int32, int32)> CanGrowPredicate = [](int32, int32) { return true; }
	);


	/**
	 * Utility function to expand an edge selection to all edges considered "connected".
	 * @param Mesh Mesh to calculate on
	 * @param InputROI input set of edges
	 * @param ResultROI output set of edges connected to InputROI
	 * @param QueueBuffer optional buffer used as internal Queue. If passed as nullptr, a TArray will be locally allocated
	 * @param CanGrowPredicate determines whether two connected mesh edges should be considered connected while growing
	 */
	static GEOMETRYCORE_API void GrowToConnectedEdges(
		const FDynamicMesh3& Mesh,
		const TArray<int>& InputROI,
		TSet<int>& ResultROI,
		TArray<int32>* QueueBuffer = nullptr,
		TFunctionRef<bool(int32, int32)> CanGrowPredicate = [](int32, int32) { return true; }
	);


};


} // end namespace UE::Geometry
} // end namespace UE
