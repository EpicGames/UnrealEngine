// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3sharp MeshEdgeSelection

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "CoreMinimal.h"
#include "Containers/Set.h"
#include "Selections/MeshVertexSelection.h"

namespace UE
{
namespace Geometry
{


/**
 * Currently a thin wrapper of a TSet<int> of Edge IDs paired with a Mesh; the backing storage will likely change as we need to optimize in the future
 */
class FMeshEdgeSelection
{
private:
	const FDynamicMesh3* Mesh;

	TSet<int> Selected;

public:

	FMeshEdgeSelection(const FDynamicMesh3* mesh)
	{
		Mesh = mesh;
	}


	// convert vertex selection to edge selection. Require at least minCount verts of edge to be selected
	GEOMETRYCORE_API FMeshEdgeSelection(const FDynamicMesh3* mesh, const FMeshVertexSelection& convertV, int minCount = 2);

	// convert face selection to edge selection. Require at least minCount tris of edge to be selected
	GEOMETRYCORE_API FMeshEdgeSelection(const FDynamicMesh3* mesh, const FMeshFaceSelection& convertT, int minCount = 1);



	TSet<int> AsSet() const
	{
		return Selected;
	}
	TArray<int> AsArray() const
	{
		return Selected.Array();
	}
	TBitArray<FDefaultBitArrayAllocator> AsBitArray() const
	{
		TBitArray<FDefaultBitArrayAllocator> Bitmap(false, Mesh->MaxEdgeID());
		for (int tid : Selected)
		{
			Bitmap[tid] = true;
		}
		return Bitmap;
	}

public:
	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	TSet<int>::TRangedForIterator      begin() { return Selected.begin(); }
	TSet<int>::TRangedForConstIterator begin() const { return Selected.begin(); }
	TSet<int>::TRangedForIterator      end() { return Selected.end(); }
	TSet<int>::TRangedForConstIterator end() const { return Selected.end(); }

private:
	void add(int eid)
	{
		Selected.Add(eid);
	}
	void remove(int eid)
	{
		Selected.Remove(eid);
	}


public:
	int Num()
	{
		return Selected.Num();
	}



	bool IsSelected(int eid)
	{
		return Selected.Contains(eid);
	}


	void Select(int eid)
	{
		ensure(Mesh->IsEdge(eid));
		if (Mesh->IsEdge(eid))
		{
			add(eid);
		}
	}
	
	void Select(const TArray<int>& edges)
	{
		for (int eid : edges)
		{
			if (Mesh->IsEdge(eid))
			{
				add(eid);
			}
		}
	}
	void Select(TArrayView<const int> edges)
	{
		for (int eid : edges)
		{
			if (Mesh->IsEdge(eid))
			{
				add(eid);
			}
		}
	}
	void Select(TFunctionRef<bool(int)> SelectF)
	{
		int NT = Mesh->MaxEdgeID();
		for (int eid = 0; eid < NT; ++eid)
		{
			if (Mesh->IsEdge(eid) && SelectF(eid))
			{
				add(eid);
			}
		}
	}

	void SelectVertexEdges(TArrayView<const int> vertices)
	{
		for (int vid : vertices) {
			for (int eid : Mesh->VtxEdgesItr(vid))
			{
				add(eid);
			}
		}
	}

	void SelectTriangleEdges(TArrayView<const int> Triangles)
	{
		for (int tid : Triangles)
		{
			FIndex3i et = Mesh->GetTriEdges(tid);
			add(et.A); add(et.B); add(et.C);
		}
	}

private:
	template<bool bHasFilter, typename ExpandContainerType>
	void ExpandToOneRingNeighbors_FindNeighborsHelper(const ExpandContainerType& ToExpand, TArray<int32>& ToAdd, TFunctionRef<bool(int)> FilterF)
	{
		for (int EID : ToExpand)
		{
			FIndex2i EdgeV = Mesh->GetEdgeV(EID);
			for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
			{
				for (int32 NbrEID : Mesh->VtxEdgesItr(EdgeV[SubIdx]))
				{
					if constexpr (bHasFilter)
					{
						if (!FilterF(NbrEID))
						{
							continue;
						}
					}
					if (!IsSelected(NbrEID))
					{
						ToAdd.Add(NbrEID);
					}
				}
			}
		}
	}

	template<bool bHasFilter>
	void ExpandToOneRingNeighbors_Helper(int32 NumRings, TFunctionRef<bool(int)> FilterF)
	{
		if (NumRings <= 0)
		{
			return;
		}

		// ToAdd can have the same edge multiple times, so to avoid accumulating duplicates
		// this will clean up the extra copies as it add to the selection
		auto AddToSelectionAndRemoveRedundant = [this](TArray<int32>& ToAdd)
		{
			for (int32 Idx = 0; Idx < ToAdd.Num(); ++Idx)
			{
				bool bAlreadyInSet;
				Selected.Add(ToAdd[Idx], &bAlreadyInSet);
				if (bAlreadyInSet)
				{
					ToAdd.RemoveAtSwap(Idx, EAllowShrinking::No);
					--Idx;
				}
			}
		};

		TArray<int32> ToAdd;
		ExpandToOneRingNeighbors_FindNeighborsHelper<bHasFilter>(Selected, ToAdd, FilterF);
		if (NumRings == 1)
		{
			// In the one ring case, no need to clean duplicates from ToAdd since we'll never use it again
			// (note: could also do this on the final iteration for the NumRings > 1 case ...)
			for (int32 ID : ToAdd)
			{
				add(ID);
			}
			return;
		}
		else
		{
			AddToSelectionAndRemoveRedundant(ToAdd);
			TArray<int32> ToExpand;
			for (int32 Iter = 1; Iter < NumRings; ++Iter)
			{
				Swap(ToAdd, ToExpand);
				ToAdd.Reset();
				ExpandToOneRingNeighbors_FindNeighborsHelper<bHasFilter>(ToExpand, ToAdd, FilterF);
				AddToSelectionAndRemoveRedundant(ToAdd);
			}
		}
	}

public:

	/**
	 *  Add all one-ring neighbors of current selection to set.
	 *  On a large selection this is quite expensive as we don't know the boundary,
	 *  so we have to iterate over all selected edges.
	 *
	 *  Return false from FilterF to prevent vertices from being included.
	 */
	void ExpandToOneRingNeighbors(TFunctionRef<bool(int32)> FilterF)
	{
		ExpandToOneRingNeighbors_Helper<true>(1, FilterF);
	}

	void ExpandToOneRingNeighbors()
	{
		ExpandToOneRingNeighbors_Helper<false>(1, [](int32) {return true;});
	}

	void ExpandToOneRingNeighbors(int NumRings, TFunctionRef<bool(int32)> FilterF)
	{
		ExpandToOneRingNeighbors_Helper<true>(NumRings, FilterF);
	}
	
	void ExpandToOneRingNeighbors(int NumRings)
	{
		ExpandToOneRingNeighbors_Helper<false>(NumRings, [](int32) {return true;});
	}


	/**
	 * For each contraction, remove edges in current selection set that have 
	 * any unselected edge neighboring either of the edge's vertices
	 */
	void ContractByBorderEdges(int32 nRings = 1)
	{
		// find set of boundary edges
		TArray<int> BorderEdges;
		for (int32 k = 0; k < nRings; ++k)
		{
			BorderEdges.Reset();

			for (int EID : Selected)
			{
				FIndex2i EdgeV = Mesh->GetEdgeV(EID);
				bool bEitherSideBoundary = false;
				for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
				{
					bool bIsBoundary = false;
					for (int32 NbrEID : Mesh->VtxEdgesItr(EdgeV[SubIdx]))
					{
						if (NbrEID != EID && !IsSelected(NbrEID))
						{
							bIsBoundary = true;
						}
					}
					if (bIsBoundary)
					{
						bEitherSideBoundary = true;
						break;
					}
				}
				if (bEitherSideBoundary)
				{
					BorderEdges.Add(EID);
				}
			}
			Deselect(BorderEdges);
		}
	}


	GEOMETRYCORE_API void SelectBoundaryTriEdges(const FMeshFaceSelection& Triangles);

	void Deselect(int tid) {
		remove(tid);
	}
	void Deselect(TArrayView<const int> edges) {
		for (int tid : edges)
		{
			remove(tid);
		}
	}
	void DeselectAll()
	{
		Selected.Empty();
	}

};

} // end namespace UE::Geometry
} // end namespace UE
