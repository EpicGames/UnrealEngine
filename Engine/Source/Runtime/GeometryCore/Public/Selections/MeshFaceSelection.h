// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshFaceSelection

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Util/DynamicVector.h"
#include "EdgeLoop.h"

namespace UE
{
namespace Geometry
{

class FMeshVertexSelection;
class FMeshEdgeSelection;


class FMeshFaceSelection
{
private:
	const FDynamicMesh3* Mesh;

	TSet<int> Selected;

public:

	FMeshFaceSelection(const FDynamicMesh3* mesh)
	{
		Mesh = mesh;
	}

	// convert vertex selection to face selection. Require at least minCount verts of
	// tri to be selected (valid values are 1,2,3)
	GEOMETRYCORE_API FMeshFaceSelection(const FDynamicMesh3* mesh, const FMeshVertexSelection& convertV, int minCount = 3);

	// select a group
	FMeshFaceSelection(const FDynamicMesh3* mesh, int group_id) : Mesh(mesh)
	{
		SelectGroup(group_id);
	}

	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	TSet<int>::TRangedForIterator      begin() { return Selected.begin(); }
	TSet<int>::TRangedForConstIterator begin() const { return Selected.begin(); }
	TSet<int>::TRangedForIterator      end() { return Selected.end(); }
	TSet<int>::TRangedForConstIterator end() const { return Selected.end(); }


private:

	void add(int tid)
	{
		Selected.Add(tid);
	}
	void remove(int tid)
	{
		Selected.Remove(tid);
	}

public:


	int Num() const
	{
		return Selected.Num();
	}



	bool IsSelected(int tid) const
	{
		return Selected.Contains(tid);
	}

	bool Contains(int tid) const
	{
		return Selected.Contains(tid);
	}

	void Select(int tid)
	{
		ensure(Mesh->IsTriangle(tid));
		if (Mesh->IsTriangle(tid))
		{
			add(tid);
		}
	}

	template<typename EnumerableType, typename E = decltype(DeclVal<EnumerableType>().begin())>
	void Select(const EnumerableType& Enumerable)
	{
		for (int32 tid : Enumerable)
		{
			if (Mesh->IsTriangle(tid))
			{
				add(tid);
			}
		}
	}

	void Select(TFunctionRef<bool(int)> SelectF)
	{
		int NT = Mesh->MaxTriangleID();
		for (int tID = 0; tID < NT; ++tID)
		{
			if (Mesh->IsTriangle(tID) && SelectF(tID))
			{
				add(tID);
			}
		}
	}


	void SelectVertexOneRing(int vid) 
	{
		Mesh->EnumerateVertexTriangles(vid, [&](int tid)
		{
			add(tid);
		});
	}
	void SelectVertexOneRings(TArrayView<const int> Vertices)
	{
		for (int vid : Vertices)
		{
			SelectVertexOneRing(vid);
		}
	}

	void SelectEdgeTris(int eid)
	{
		FIndex2i et = Mesh->GetEdgeT(eid);
		add(et.A);
		if (et.B != FDynamicMesh3::InvalidID)
		{
			add(et.B);
		}
	}


	void Deselect(int tid)
	{
		remove(tid);
	}
	void Deselect(TArrayView<const int> Triangles)
	{
		for (int TID : Triangles)
		{
			remove(TID);
		}
	}

	template<typename EnumerableType>
	void Deselect(const EnumerableType& Enumerable)
	{
		for (int32 tid : Enumerable)
		{
			remove(tid);
		}
	}

	void DeselectAll()
	{
		Selected.Empty();
	}


	void SelectGroup(int gid)
	{
		int NT = Mesh->MaxTriangleID();
		for (int tid = 0; tid < NT; ++tid)
		{
			if (Mesh->IsTriangle(tid) && Mesh->GetTriangleGroup(tid) == gid)
			{
				add(tid);
			}
		}
	}
	void SelectGroupInverse(int gid)
	{
		int NT = Mesh->MaxTriangleID();
		for (int tid = 0; tid < NT; ++tid)
		{
			if (Mesh->IsTriangle(tid) && Mesh->GetTriangleGroup(tid) != gid)
			{
				add(tid);
			}
		}
	}
	void DeselectGroup(int gid)
	{
		// cannot just iterate over selected tris because remove() will change them...
		int NT = Mesh->MaxTriangleID();
		for (int tid = 0; tid < NT; ++tid)
		{
			if (Mesh->IsTriangle(tid) && Mesh->GetTriangleGroup(tid) == gid)
			{
				remove(tid);
			}
		}
	}

	/**
	 * Find the elements in current selection that are not in SubtractSet, and store in DifferenceStorage
	 */
	template<typename EnumerableType, typename StorageType>
	void SetDifference(const EnumerableType& SubtractSet, StorageType& DifferenceStorage) const
	{
		for ( int32 tid : Selected )
		{
			if (SubtractSet.Contains(tid) == false)
			{
				DifferenceStorage.Add(tid);
			}
		}
	}

	/**
	 * Find the elements in current selection that are also in IntersectSet, and store in IntersectionStorage
	 */
	template<typename EnumerableType, typename StorageType>
	void SetIntersection(const EnumerableType& IntersectSet, StorageType& IntersectionStorage) const
	{
		for (int32 tid : Selected)
		{
			if (IntersectSet.Contains(tid))
			{
				IntersectionStorage.Add(tid);
			}
		}
	}

	const TSet<int>& AsSet() const
	{
		return Selected;
	}
	TArray<int> AsArray() const
	{
		return Selected.Array();
	}
	TBitArray<FDefaultBitArrayAllocator> AsBitArray() const
	{
		TBitArray<FDefaultBitArrayAllocator> Bitmap(false, Mesh->MaxTriangleID());
		for (int tid : Selected)
		{
			Bitmap[tid] = true;
		}
		return Bitmap;
	}

	/**
	 * @return pair (bIsSelectionBoundaryEdge, bIsMeshBoundaryEdge) for a given edge
	 */
	TPair<bool, bool> IsSelectionBoundaryEdge(int32 eid) const
	{
		if (Mesh->IsEdge(eid))
		{
			FIndex2i EdgeT = Mesh->GetEdgeT(eid);
			bool bA = Selected.Contains(EdgeT.A);
			if (EdgeT.B == IndexConstants::InvalidID)
			{
				return TPair<bool, bool>(true, true);
			}
			else if (Selected.Contains(EdgeT.B) != bA)
			{
				return TPair<bool, bool>(true, false);
			}
		}
		return TPair<bool, bool>(false, false);
	}


	/**
	 *  find set of tris just outside border of selection
	 */
	TArray<int> FindNeighbourTris() const
	{
		TArray<int> result;
		for (int tid : Selected)
		{
			FIndex3i nbr_tris = Mesh->GetTriNeighbourTris(tid);
			for (int j = 0; j < 3; ++j)
			{
				if (nbr_tris[j] != FDynamicMesh3::InvalidID && IsSelected(nbr_tris[j]) == false)
				{
					result.Add(nbr_tris[j]);
				}
			}
		}
		return result;
	}


	/**
	 *  find set of tris just inside border of selection
	 */
	TArray<int> FindBorderTris() const
	{
		TArray<int> result;
		for (int tid : Selected)
		{
			FIndex3i nbr_tris = Mesh->GetTriNeighbourTris(tid);
			if (IsSelected(nbr_tris.A) == false || IsSelected(nbr_tris.B) == false || IsSelected(nbr_tris.C) == false)
			{
				result.Add(tid);
			}
		}
		return result;
	}


	void ExpandToFaceNeighbours(const TUniqueFunction<bool(int)>& FilterF = nullptr)
	{
		TArray<int> ToAdd;

		for (int tid : Selected) { 
			FIndex3i nbr_tris = Mesh->GetTriNeighbourTris(tid);
			for (int j = 0; j < 3; ++j)
			{
				if (FilterF && FilterF(nbr_tris[j]) == false)
				{
					continue;
				}
				if (nbr_tris[j] != FDynamicMesh3::InvalidID && !IsSelected(nbr_tris[j]))
				{
					ToAdd.Add(nbr_tris[j]);
				}
			}
		}

		for (int ID : ToAdd)
		{
			add(ID);
		}
	}
	void ExpandToFaceNeighbours(int rounds, const TUniqueFunction<bool(int)>& FilterF = nullptr)
	{
		for (int k = 0; k < rounds; ++k)
		{
			ExpandToFaceNeighbours(FilterF);
		}
	}


	/**
	 *  Add all triangles in vertex one-rings of current selection to set.
	 *  On a large selection this is quite expensive as we don't know the boundary,
	 *  so we have to iterate over all selected triangles.
	 *  
	 *  Return false from FilterF to prevent triangles from being included.
	 */
	GEOMETRYCORE_API void ExpandToOneRingNeighbours(const TUniqueFunction<bool(int)>& FilterF = nullptr);

	/**
	 *  Expand selection by N vertex one-rings. This is *significantly* faster
	 *  than calling ExpandToOnering() multiple times, because we can track
	 *  the growing front and only check the new triangles.
	 *  
	 *  Return false from FilterF to prevent triangles from being included.
	 */
	GEOMETRYCORE_API void ExpandToOneRingNeighbours(int nRings, const TUniqueFunction<bool(int)>& FilterF = nullptr);

	/**
	 *  remove all triangles in vertex one-rings of current selection to set.
	 *  On a large selection this is quite expensive as we don't know the boundary,
	 *  so we have to iterate over all selected triangles.
	 *  
	 *  Return false from FilterF to prevent triangles from being deselected.
	 */
	GEOMETRYCORE_API void ContractBorderByOneRingNeighbours(int NumRings = 1, bool bContractFromMeshBoundary = false, const TUniqueFunction<bool(int)>& FilterF = nullptr);

	/**
	 *  Grow selection outwards from seed triangle, until it hits boundaries defined by triangle and edge filters.
	 *  Edge filter is not effective unless it (possibly combined w/ triangle filter) defines closed loops.
	 */
	void FloodFill(int tSeed, const TUniqueFunction<bool(int)>& TriFilterF = nullptr, const TUniqueFunction<bool(int)>& EdgeFilterF = nullptr)
	{
		TArray<int> Seeds = { tSeed };
		FloodFill(Seeds, TriFilterF, EdgeFilterF);
	}
	/**
	 *  Grow selection outwards from seed triangles, until it hits boundaries defined by triangle and edge filters.
	 *  Edge filter is not effective unless it (possibly combined w/ triangle filter) defines closed loops.
	 */
	void FloodFill(const TArray<int>& Seeds, const TUniqueFunction<bool(int)>& TriFilterF = nullptr, const TUniqueFunction<bool(int)>& EdgeFilterF = nullptr)
	{
		TDynamicVector<int> stack(Seeds);
		for (int Seed : Seeds)
		{
			add(Seed);
		}
		while (stack.Num() > 0)
		{
			int TID = stack.Back();
			stack.PopBack();

			FIndex3i nbrs = Mesh->GetTriNeighbourTris(TID);
			for (int j = 0; j < 3; ++j)
			{
				int nbr_tid = nbrs[j];
				if (nbr_tid == FDynamicMesh3::InvalidID || IsSelected(nbr_tid))
				{
					continue;
				}
				if (TriFilterF && TriFilterF(nbr_tid) == false)
				{
					continue;
				}
				int EID = Mesh->GetTriEdge(TID, j);
				if (EdgeFilterF && EdgeFilterF(EID) == false)
				{
					continue;
				}
				add(nbr_tid);

				stack.Add(nbr_tid);
			}
		}
	}



	// return true if we clipped something
	bool ClipFins(bool bClipLoners)
	{
		TArray<int> ToRemove; // temp array so we don't remove as we iterate the set
		for (int tid : Selected)
		{
			if (is_fin(tid, bClipLoners))
			{
				ToRemove.Add(tid);
			}
		}
		for (int tid : ToRemove)
		{
			remove(tid);
		}
		return ToRemove.Num() > 0;
	}


	// return true if we filled any ears.
	bool FillEars(bool bFillTinyHoles)
	{
		// [TODO] not efficient! checks each nbr 3 times !! ugh!!
		TArray<int> temp;
		for (int tid : Selected)
		{
			FIndex3i nbr_tris = Mesh->GetTriNeighbourTris(tid);
			for (int j = 0; j < 3; ++j)
			{
				int nbr_t = nbr_tris[j];
				if (IsSelected(nbr_t))
				{
					continue;
				}
				if (is_ear(nbr_t, bFillTinyHoles))
				{
					temp.Add(nbr_t);
				}
			}
		}
		for (int tid : temp)
		{
			add(tid);
		}
		return temp.Num() > 0;
	}

	// returns true if selection was modified
	bool LocalOptimize(bool bClipFins, bool bFillEars, bool bFillTinyHoles = true, bool bClipLoners = true, bool bRemoveBowties = false)
	{
		bool bModified = false;
		bool done = false;
		int count = 0;
		do
		{
			bool bDidClip = bClipFins && ClipFins(bClipLoners);
			bool bDidEars = bFillEars && FillEars(bFillTinyHoles);
			bool bDidBows = bRemoveBowties && remove_bowties();
			done = !(bDidClip || bDidEars || bDidBows); // only done if did nothing this pass
			bModified = bModified || !done;
		} while (!done && count++ < 25);
		if (bRemoveBowties)
		{
			remove_bowties();        // do a final pass of this because it is usually the most problematic...
		}
		return bModified;
	}

	bool LocalOptimize(bool bRemoveBowties = true)
	{
		return LocalOptimize(true, true, true, true, bRemoveBowties);
	}




	/**
	 *  Find any "bowtie" vertices - ie vertex v such taht there is multiple spans of triangles
	 *  selected in v's triangle one-ring - and deselect those one-rings.
	 *  Returns true if selection was modified.
	 */
	bool RemoveBowties() {
		return remove_bowties();
	}
	bool remove_bowties()
	{
		TSet<int> tempHash;
		bool bModified = false;
		bool done = false;
		TSet<int> vertices;
		while (!done)
		{
			done = true;
			vertices.Empty();
			for (int tid : Selected)
			{
				FIndex3i tv = Mesh->GetTriangle(tid);
				vertices.Add(tv.A); vertices.Add(tv.B); vertices.Add(tv.C);
			}

			for (int vid : vertices) 
			{
				if (is_bowtie_vtx(vid)) 
				{
					Mesh->EnumerateVertexTriangles(vid, [&](int TID)
					{
						Deselect(TID);
					});
					done = false;
				}
			}
			if (done == false)
			{
				bModified = true;
			}
		}
		return bModified;
	}

private:
	bool is_bowtie_vtx(int vid) const
	{
		int border_edges = 0;
		for (int eid : Mesh->VtxEdgesItr(vid)) 
		{
			FIndex2i et = Mesh->GetEdgeT(eid);
			if (et.B != FDynamicMesh3::InvalidID)
			{
				bool in_a = IsSelected(et.A);
				bool in_b = IsSelected(et.B);
				if (in_a != in_b)
				{
					border_edges++;
				}
			}
			else
			{
				if (IsSelected(et.A))
				{
					border_edges++;
				}
			}
		}
		return border_edges > 2;
	}

	void count_nbrs(int tid, int& nbr_in, int& nbr_out, int& bdry_e) const
	{
		FIndex3i nbr_tris = Mesh->GetTriNeighbourTris(tid);
		nbr_in = 0; nbr_out = 0; bdry_e = 0;
		for ( int j = 0; j < 3; ++j )
		{
			int nbr_t = nbr_tris[j];
			if (nbr_t == FDynamicMesh3::InvalidID)
			{
				bdry_e++;
			}
			else if (IsSelected(nbr_t) == true)
			{
				nbr_in++;
			}
			else
			{
				nbr_out++;
			}
		}
	}
	bool is_ear(int tid, bool include_tiny_holes) const
	{
		if (IsSelected(tid) == true)
		{
			return false;
		}
		int nbr_in, nbr_out, bdry_e;
		count_nbrs(tid, nbr_in, nbr_out, bdry_e);
		if (bdry_e == 2 && nbr_in == 1)
		{
			return true;        // unselected w/ 2 boundary edges, nbr is  in
		}
		else if (nbr_in == 2)
		{
			if (bdry_e == 1 || nbr_out == 1)
			{
				return true;        // unselected w/ 2 selected nbrs
			}
		}
		else if (include_tiny_holes && nbr_in == 3)
		{
			return true;
		}
		return false;
	}
	bool is_fin(int tid, bool include_loners) const
	{
		if (IsSelected(tid) == false)
		{
			return false;
		}
		int nbr_in, nbr_out, bdry_e;
		count_nbrs(tid, nbr_in, nbr_out, bdry_e);
		return (nbr_in == 1 && nbr_out == 2) ||
			   (include_loners == true && nbr_in == 0 && nbr_out == 3);
	}


};


} // end namespace UE::Geometry
} // end namespace UE
