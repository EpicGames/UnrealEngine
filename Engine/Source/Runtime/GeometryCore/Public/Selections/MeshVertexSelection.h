// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3sharp FMeshVertexSelection

#pragma once

#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
namespace Geometry
{


class FMeshFaceSelection;
class FMeshEdgeSelection;

class FMeshVertexSelection
{
private:
	const FDynamicMesh3* Mesh;

	TSet<int> Selected;

public:
	FMeshVertexSelection(const FDynamicMesh3* mesh)
	{
		Mesh = mesh;
	}

	// convert face selection to vertex selection. 
	GEOMETRYCORE_API FMeshVertexSelection(const FDynamicMesh3* mesh, const FMeshFaceSelection& convertT);

	// convert edge selection to vertex selection. 
	GEOMETRYCORE_API FMeshVertexSelection(const FDynamicMesh3* mesh, const FMeshEdgeSelection& convertE);


	TSet<int> AsSet() const
	{
		return Selected;
	}
	TArray<int> AsArray() const
	{
		return Selected.Array();
	}
	TBitArray<FDefaultBitArrayAllocator> AsBitArray() const
	{
		TBitArray<FDefaultBitArrayAllocator> Bitmap(false, Mesh->MaxVertexID());
		for (int tid : Selected)
		{
			Bitmap[tid] = true;
		}
		return Bitmap;
	}



public:
	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	TSet<int>::TRangedForIterator      begin() { return Selected.begin(); }
	TSet<int>::TRangedForConstIterator begin() const { return Selected.begin(); }
	TSet<int>::TRangedForIterator      end() { return Selected.end(); }
	TSet<int>::TRangedForConstIterator end() const { return Selected.end(); }

private:
	void add(int vID)
	{
		Selected.Add(vID);
	}
	void remove(int vID)
	{
		Selected.Remove(vID);
	}

public:
	int Num() const
	{
		return Selected.Num();
	}

	bool IsSelected(int vID) const
	{
		return Selected.Contains(vID);
	}

	void Select(int vID)
	{
		ensure(Mesh->IsVertex(vID));
		if (Mesh->IsVertex(vID))
		{
			add(vID);
		}
	}
	void Select(TArrayView<const int> Vertices)
	{
		for (int VID : Vertices)
		{
			if (Mesh->IsVertex(VID))
			{
				add(VID);
			}
		}
	}

	/**
	 * Select vertices where PredicteFunc(VertexID) == bSelectTrue
	 */
	template<typename PredicateFuncType>
	void SelectByVertexID(PredicateFuncType PredicateFunc, bool bSelectTrue = true)
	{
		int32 NumV = Mesh->MaxVertexID();
		for (int32 vid = 0; vid < NumV; ++vid)
		{
			if (Mesh->IsVertex(vid) && PredicateFunc(vid) == bSelectTrue)
			{
				add(vid);
			}
		}
	}

	/**
	 * Select vertices where PredicteFunc(VertexPosition) == bSelectTrue
	 */
	template<typename PredicateFuncType>
	void SelectByPosition(PredicateFuncType PredicateFunc, bool bSelectTrue = true)
	{
		int32 NumV = Mesh->MaxVertexID();
		for (int32 vid = 0; vid < NumV; ++vid)
		{
			if (Mesh->IsVertex(vid) && PredicateFunc(Mesh->GetVertex(vid)) == bSelectTrue)
			{
				add(vid);
			}
		}
	}


	void SelectTriangleVertices(TArrayView<const int> Triangles)
	{
		for (int TID : Triangles)
		{
			FIndex3i tri = Mesh->GetTriangle(TID);
			add(tri.A); add(tri.B); add(tri.C);
		}
	}
	GEOMETRYCORE_API void SelectTriangleVertices(const FMeshFaceSelection& Triangles);


	/**
	 *  for each vertex of input triangle set, select vertex if all
	 *  one-ring triangles are contained in triangle set (ie vertex is not on boundary of triangle set).
	 */
	GEOMETRYCORE_API void SelectInteriorVertices(const FMeshFaceSelection& triangles);


	/**
	 *  Select set of boundary vertices connected to vSeed.
	 */
	void SelectConnectedBoundaryV(int vSeed)
	{
		if (!ensureMsgf(Mesh->IsBoundaryVertex(vSeed), TEXT("MeshConnectedComponents.FindConnectedBoundaryV: vSeed is not a boundary vertex")))
		{
			return;
		}

		TSet<int> &found = Selected;
		found.Add(vSeed);
		TArray<int> queue;
		queue.Add(vSeed);
		while (queue.Num() > 0)
		{
			int vid = queue.Pop(EAllowShrinking::No);
			for (int nbrid : Mesh->VtxVerticesItr(vid))
			{
				if (Mesh->IsBoundaryVertex(nbrid) && found.Contains(nbrid) == false)
				{
					found.Add(nbrid);
					queue.Add(nbrid);
				}
			}
		}
	}


	void SelectEdgeVertices(TArrayView<const int> Edges)
	{
		for (int EID : Edges)
		{
			FIndex2i ev = Mesh->GetEdgeV(EID);
			add(ev.A); add(ev.B);
		}
	}


	void Deselect(int vID)
	{
		remove(vID);
	}
	void Deselect(TArrayView<const int> Vertices)
	{
		for (int VID : Vertices)
		{
			remove(VID);
		}
	}
	void DeselectEdge(int eid)
	{
		FIndex2i ev = Mesh->GetEdgeV(eid);
		remove(ev.A); remove(ev.B);
	}

	void DeselectEdges(TArrayView<const int> Edges) 
	{
		for (int EID : Edges)
		{
			FIndex2i ev = Mesh->GetEdgeV(EID);
			remove(ev.A); remove(ev.B);
		}
	}


	/**
	 *  Add all one-ring neighbours of current selection to set.
	 *  On a large selection this is quite expensive as we don't know the boundary,
	 *  so we have to iterate over all selected vertices.
	 *  
	 *  Return false from FilterF to prevent vertices from being included.
	 */
	void ExpandToOneRingNeighbours(const TUniqueFunction<bool(int)>& FilterF = nullptr)
	{
		TArray<int> temp;

		for (int vid : Selected) 
		{
			for (int nbr_vid : Mesh->VtxVerticesItr(vid))
			{
				if (FilterF && FilterF(nbr_vid) == false)
				{
					continue;
				}
				if (IsSelected(nbr_vid) == false)
				{
					temp.Add(nbr_vid);
				}
			}
		}

		for (int ID : temp)
		{
			add(ID);
		}
	}


	// [TODO] should do this more efficiently, like FMeshFaceSelection
	void ExpandToOneRingNeighbours(int nRings, const TUniqueFunction<bool(int)>& FilterF = nullptr)
	{
		for (int k = 0; k < nRings; ++k)
		{
			ExpandToOneRingNeighbours(FilterF);
		}
	}


	/**
	 * Remove all vertices in current selection set that have at least
	 * one neighbour vertex that is not selected (ie vertices are on border of selection)
	 */
	void ContractByBorderVertices(int32 nRings = 1)
	{
		// find set of boundary vertices
		TArray<int> BorderVertices;
		for (int32 k = 0; k < nRings; ++k)
		{
			BorderVertices.Reset();

			for (int vid : Selected) 
			{
				bool bAnyNeighbourDeselected = false;
				for (int nbr_vid : Mesh->VtxVerticesItr(vid))
				{
					if (IsSelected(nbr_vid) == false)
					{
						bAnyNeighbourDeselected = true;
						break;
					}
				}
				if (bAnyNeighbourDeselected)
				{
					BorderVertices.Add(vid);
				}
			}
			Deselect(BorderVertices);
		}
	}



	/**
	 *  Grow selection outwards from seed vertex, until it hits boundaries defined by vertex filter.
	 */
	void FloodFill(int vSeed, const TUniqueFunction<bool(int)>& VertIncludedF = nullptr)
	{
		TArray<int> Seeds = { vSeed };
		FloodFill(Seeds, VertIncludedF);
	}
	/**
	 *  Grow selection outwards from seed vertex, until it hits boundaries defined by vertex filter.
	 */
	void FloodFill(const TArray<int>& Seeds, const TUniqueFunction<bool(int)>& VertIncludedF = nullptr)
	{
		TDynamicVector<int> stack(Seeds);
		for (int Seed : Seeds)
		{
			add(Seed);
		}
		while (stack.Num() > 0)
		{
			int vID = stack.Back();
			stack.PopBack();

			for (int nbr_vid : Mesh->VtxVerticesItr(vID))
			{
				if (IsSelected(nbr_vid) == true || (VertIncludedF && VertIncludedF(nbr_vid) == false))
				{
					continue;
				}
				add(nbr_vid);
				stack.Add(nbr_vid);
			}
		}
	}


};


} // end namespace UE::Geometry
} // end namespace UE
