// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"


namespace UE
{
	namespace Solvers
	{
		using namespace UE::Geometry;

		/**
		 * Generic adapter for building a Sparse Matrix. Generally sparse-matrix assembly
		 * code emits matrix entries as tuples of form (i,j,Value).
		 */
		template<typename RealType>
		class TSparseMatrixAssembler
		{
		public:
			/** Hint to reserve space for at least this many entries */
			TUniqueFunction<void(int32)> ReserveEntriesFunc;

			/** Add matrix entry tuple (i,j,Value) */
			TUniqueFunction<void(int32, int32, RealType)> AddEntryFunc;

			struct FTupleData
			{
				FTupleData(int32 IIn, int32 JIn, RealType ValueIn)
					: I(IIn)
					, J(JIn)
					, Value(ValueIn)
				{}
				int32 I;
				int32 J;
				RealType Value;
			};

			/** Add matrix entries (i,j,Value) and size of tuples */
			TUniqueFunction<void(const FTupleData*, int32)> AddEntriesFunc = [this](const FTupleData* DataStart, int32 Num)
			{
				if (!AddEntryFunc)
				{
					return;
				}
				for (int32 i = 0; i < Num; ++i)
				{
					const FTupleData& Tuple = DataStart[Num];
					AddEntryFunc(Tuple.I, Tuple.J, Tuple.Value);
				}
			};
		};

		typedef TSparseMatrixAssembler<double> FSparseMatrixAssemblerd;
		typedef TSparseMatrixAssembler<float> FSparseMatrixAssemblerf;



		/**
		 * Basic position constraint
		 */
		struct FPositionConstraint
		{
			/** ID of constrained UV element */
			int32 ElementID = -1;

			/** Index/Identifier of constraint, defined by usage */
			int32 ConstraintIndex = -1;

			/** Constraint position */
			FVector3d Position = FVector3d::Zero();

			/** If bPostFix is true, this position constraint should be explicitly enforced after a solve */
			bool     bPostFix = false;

			/** Arbitrary weight */
			double Weight = 1.0;
		};



		/**
		 * Basic UV constraint
		 */
		struct FUVConstraint
		{
			/** ID of constrained UV element */
			int32 ElementID = -1;

			/** Index/Identifier of constraint, defined by usage */
			int32 ConstraintIndex = -1;

			/** Constraint position */
			FVector2d Position = FVector2d::Zero();

			/** If bPostFix is true, this position constraint should be explicitly enforced after a solve */
			bool bPostFix = false;

			/** Arbitrary weight */
			double Weight = 1.0;
		};


	}
}


