// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

namespace UE {
namespace Geometry {

/*
 * Solver for linear systems for tridiagonal matrices of the form
 *
 * [ B0  C0   0   0  ...   0    ]
 * [ A1  B1  C1   0  ...   0    ]
 * [  0  A2  B2  C2  ...   0    ]
 * [  0   0  A3  B3  ...   0    ]
 * [  .   .   .   .   .    Cn-2 ]
 * [  0   0   0   0  ...   Bn-1 ]
 *
 * using Thomas algorithm.
 */
template <typename T>
class TTridiagonalSolver
{
public:

	TTridiagonalSolver(const TArray<T>& InA, const TArray<T>& InB, const TArray<T>& InC)
	: A(InA)
	, B(InB)
	, C(InC)
	, N(A.Num())
	{
		check(B.Num() == N);
		check(C.Num() == N);
		D.SetNum(N-1);
		Prepare();
	}

	/*
     * Solve linear system A * X = Rhs
	 * 
	 * Note that Rhs is modified. 
	 */ 
	bool Solve(TArray<T>& Rhs, TArray<T>& X)
	{
		if (!bInvertible)
		{
			return false;
		}

		Rhs[0] /= B[0];
		for (size_t I=1; I<N; ++I)
		{
			const T Divisor = B[I] - A[I] * D[I-1];
			if (FMath::Abs(Divisor) < T(UE_SMALL_NUMBER))
			{
				return false;
			}
			Rhs[I] = (Rhs[I] - A[I] * Rhs[I-1]) / Divisor;
		}

		X[N-1] = Rhs[N-1];
		for (size_t I=N-2; ; --I)
		{
			X[I] = Rhs[I] - D[I] * X[I+1];
			if (I==0)
			{
				break;
			}
		}
		return true;
	}

private:

	void Prepare()
	{
		bInvertible = false;
		if (FMath::Abs(B[0]) < T(UE_SMALL_NUMBER))
		{
			return;
		}

		D[0] = C[0] / B[0];
		for (size_t I=1; I<N-1; ++I)
		{
			const T Divisor = B[I] - A[I] * D[I-1];
			if (FMath::Abs(Divisor) < T(UE_SMALL_NUMBER))
			{
				return;
			}
			D[I] = C[I] / Divisor;
		}
		bInvertible = true;
	}

	const TArray<T>& A, B, C; // matrix representation
	TArray<T>        D;       // forward pass temporary
	const size_t     N; 
	bool             bInvertible;
};

} // end namespace Geometry
} // end namespace UE