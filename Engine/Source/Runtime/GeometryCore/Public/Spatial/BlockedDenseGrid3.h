// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp BlockedDenseGrid3

#pragma once

#include "CoreMinimal.h"
#include "BlockedLayout3.h"
#include "BoxTypes.h"
#include "Containers/BitArray.h"
#include "Containers/StaticArray.h"
#include "Misc/ScopeLock.h"
#include "HAL/CriticalSection.h"


namespace UE
{
namespace Geometry
{

// Block Data Buffer, holds linear array of typed data
template <typename ElemType_, int32 BlockSize_>
class TBlockData3 : public TBlockData3Layout<BlockSize_>
{
public:
	typedef TBlockData3Layout<BlockSize_>                        DataLayout;
	typedef ElemType_                                            ElemType;
	typedef TStaticArray<ElemType, DataLayout::ElemCount>        LinearDataStorageType;

	TBlockData3()
	:Id(-1)
	{}

	TBlockData3(const ElemType& Value, const int32 ID)
	:Id(ID)
	{
		Reset(Value);
	}

	/** initialize all the data elements to the specified value*/
	void Reset(const ElemType& Value)
	{
		for (int32 k = 0, K = DataLayout::ElemCount; k < K; ++k)
		{
			Data[k] = Value;
		}
	}

	/** access the data by index into the underlying linear array*/
	ElemType& At(const int32 Index)
	{
		checkSlow(Index < DataLayout::ElemCount);
		return Data[Index];
	}
	/** access the data by index into the underlying linear array*/
	ElemType At(const int32 Index) const 
	{
		checkSlow(Index < DataLayout::ElemCount);
		return Data[Index];
	}

	template <typename FuncType>
	void ModifyValue(const int32 Index, FuncType Func)
	{
		ElemType& Value = Data[Index];
		Func(Value);
	}

	int32                        Id;          // an id associated with this block data.
	LinearDataStorageType        Data;        // payload

};

// Specialize the Block Data Buffer for bools by using a bitarray. 
template <int32 BlockSize_>
class TBlockData3<bool, BlockSize_> : public TBlockData3Layout<BlockSize_>
{
public:
	typedef TBlockData3Layout<BlockSize_>                       DataLayout;
	typedef bool                                                ElemType;
	typedef TInlineAllocator<DataLayout::ElemCount>             AllocatorType;
	typedef TBitArray<AllocatorType>                            BlockDataBitMask;
	typedef TConstSetBitIterator<AllocatorType>                 BitArrayConstIterator;

	TBlockData3(const ElemType& Value, const int32 ID)
		:Id(ID)
	{
		Reset(Value);
	}

	void Reset(const ElemType& Value)
	{
		BitArray.Init(Value, DataLayout::ElemCount);
	}

	FBitReference At(int32 LocalIndex)
	{
		checkSlow(LocalIndex < DataLayout::ElemCount);
		return BitArray[LocalIndex];
	}

	bool At(int32 LocalIndex) const
	{
		checkSlow(LocalIndex < DataLayout::ElemCount);
		return BitArray[LocalIndex];
	}

	template <typename FuncType>
	void ModifyValue(const int32 Index, FuncType Func)
	{
		bool Value = BitArray[Index];
		Func(Value);
		BitArray[Index] = Value;
	}

	// bit field operations can correspond to topological operations
	void TopologyUnion(const TBlockData3<bool, BlockSize_>& OtherBlockData)
	{
		const auto& OtherBitArray = OtherBlockData.BitArray;
		BitArray.CombineWithBitwiseOR(OtherBitArray, EBitwiseOperatorFlags::MaintainSize);
	}

	int32                        Id;
	BlockDataBitMask             BitArray;    
};


/**
* TBasicBlockedDenseGrid3 represents a 3D blocked uniform grid.
*
* The grid is allocated in BlockSize^3 blocks on-demand (BlockSize is a compile-time constant) allowing
* very large grids to be used without having to pre-allocate all the memory, eg for sparse/narrow-band use cases.
* 
* Block-level default values are returned when querying a grid cell in an unallocated block.
*
* For multi-threaded applications consider using the derived classes TBlockedDenseGrid3 which manages internal locks,
* or TBlockedGrid3 with your own lock provider.
*/
template<typename ElemType, int32 BlockSize_>
class TBasicBlockedDenseGrid3 : public TBlockedGrid3Layout<BlockSize_>
{
public:
	typedef TBlockedGrid3Layout<BlockSize_>                                   BlockedGrid3LayoutType;
	typedef TBlockData3<ElemType, BlockSize_>                                 BlockData3Type;


	static constexpr int32 BlockSize      = BlockSize_;                         // length of side of a block
	static constexpr int32 BlockElemCount = BlockSize * BlockSize * BlockSize;  // number of cells in each block
protected:

	/** The value to use to initialize new blocks */
	ElemType DefaultValue = (ElemType)0;


	// a default value and a pointer to a block that will be allocated on first write.
	struct FBlock3
	{
		// release any allocated block data and reset the default value.
		void Reset(const ElemType& DefaultIn = (ElemType)0)
		{
			BlockDataPtr.Reset();
			DefaultValue = DefaultIn;
		}

		/** @return the default value if the block has not been allocated, otherwise the indexed value in the block*/
		ElemType  GetValue(const int32& LocalIndex) const 
		{
			return (BlockDataPtr.IsValid()) ? BlockDataPtr->At(LocalIndex) : DefaultValue;
		}

		TUniquePtr<BlockData3Type>  BlockDataPtr;     // pointer to 'heavy' data
		ElemType                    DefaultValue;     // value to be used when BlockData is not allocated
	};


	TArray<FBlock3>          Blocks;                 // array of blocks that span the dimensions of the gird.
	

protected:

	// declare other templated instantiations as friends so that PreAllocateFromSourceGrid() can work
	template<class OtherElemType, int32 OtherBlockSize> friend class TBasicBlockedDenseGrid3;

	//friend class TBlockedDenseBitGrid3<BlockSize>;

#if UE_BUILD_DEBUG
	FBlock3& GetBlock(int32 Index) { return Blocks[Index]; }
	const FBlock3& GetBlock(int32 Index) const { return Blocks[Index]; }
#else
	// skip range checks in non-debug builds
	FBlock3& GetBlock(int32 Index) { return Blocks.GetData()[Index]; }
	const FBlock3& GetBlock(int32 Index) const { return Blocks.GetData()[Index]; }
#endif

	/** @return true if Block has an allocated block data buffer*/
	bool IsBlockAllocated(const FBlock3& Block) const
	{
		return Block.BlockDataPtr.IsValid();
	}
	/** @return true if the specified Block has an allocated block data buffer*/
	bool IsBlockAllocated(const int32 BlockIndex) const
	{
		const FBlock3& Block = GetBlock(BlockIndex);
		return IsBlockAllocated(Block);
	}

	/** return a reference to the block data buffer associated with the specified block, allocating if needed */
	BlockData3Type& TouchBlockData(const int32 BlockIndex)
	{
		FBlock3& Block = GetBlock(BlockIndex);
		if (!IsBlockAllocated(Block))
		{
			const ElemType& Value = Block.DefaultValue;
			Block.BlockDataPtr = MakeUnique<BlockData3Type>(Value, BlockIndex);
		}
		return *Block.BlockDataPtr;
	}

	// --- write methods
	template <typename FuncType>
	void WriteValue(const int32& BlockIndex, const int32& LocalIndex, FuncType Func)
	{
		BlockData3Type& BlockData = TouchBlockData(BlockIndex);
		BlockData.ModifyValue(LocalIndex, Func);
	}

	template<typename FuncType>
	void WriteValue(int32 I, int32 J, int32 K, FuncType Func)
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		WriteValue(BlockIndex, LocalIndex, Func);
	}
	
	template <typename FuncType>
	void WriteValueWithLock(const int32& BlockIndex, const int32& LocalIndex,  FuncType Func, FCriticalSection* CriticalSection)
	{
		FScopeLock Lock(CriticalSection);
		WriteValue(BlockIndex, LocalIndex, Func);
	}

	// --- read methods

	ElemType ReadValue(const int32& BlockIndex, const int32& LocalIndex) const 
	{
		return GetBlock(BlockIndex).GetValue(LocalIndex);
	}

	ElemType ReadValue(int32 I, int32 J, int32 K) const
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		return ReadValue(BlockIndex, LocalIndex);
	}

	ElemType ReadValueWithLock(const int32& BlockIndex, const int32& LocalIndex, FCriticalSection* CriticalSection)
	{
		FScopeLock Lock(CriticalSection);
		return ReadValue(BlockIndex, LocalIndex);
	}



public:
	/**
	* Create empty grid
	*/
	TBasicBlockedDenseGrid3() : BlockedGrid3LayoutType(0,0,0)
	{
	}

	/**
	* Create grid with specified domain.
	*/
	TBasicBlockedDenseGrid3(int32 DimI, int32 DimJ, int32 DimK, ElemType InitialValue)
	{
		DefaultValue = InitialValue;
		Resize(DimI, DimJ, DimK);
	}

	/**
	* Reconfigure the grid to have the target dimensions. This clears all the 
	* existing grid memory and resets the block-level default value.
	*/
	void Resize(int32 DimI, int32 DimJ, int32 DimK)
	{		
		BlockedGrid3LayoutType::Resize(DimI, DimJ, DimK);
		const int32 NumBlocks = GetNumBlocks();

		Blocks.Reset();
		Blocks.SetNum(NumBlocks);
		for (int32 k = 0; k < NumBlocks; ++k)
		{
			Blocks[k].Reset(DefaultValue);
		}
	}

	/**
	* Discard all allocated blocks but retain grid dimensions. This also resets the block-level default values.
	*/
	void Reset()
	{
		Reset(BlockedGrid3LayoutType::Dimensions.X, BlockedGrid3LayoutType::Dimensions.Y, BlockedGrid3LayoutType::Dimensions.Z, DefaultValue);
	}

	/**
	* Reconfigure the grid to have the target dimensions and default value. 
	* This clears all the existing grid memory, and resets default values to the specified InitialValue.
	*/
	void Reset(int32 DimI, int32 DimJ, int32 DimK, ElemType InitialValue)
	{
		DefaultValue = InitialValue;
		Resize(DimI, DimJ, DimK);
	}

	/**
	* @return the grid value at (I,J,K)
	*/
	ElemType GetValue(int32 I, int32 J, int32 K) const
	{
		return ReadValue(I, J, K);
	}

	/**
	* @return the grid value at CellIJK.  Note this is safe to call from multiple read threads (but not if simultaneously writing!)
	*/
	ElemType GetValue(const FVector3i& CellIJK) const
	{
		return ReadValue(CellIJK.X, CellIJK.Y, CellIJK.Z);
	}


	/**
	* Set the grid value at (I,J,K)
	*/
	void SetValue(int32 I, int32 J, int32 K, ElemType NewValue)
	{
		WriteValue(I, J, K, [NewValue](ElemType& GridValueInOut) { GridValueInOut = NewValue; });
	}

	/**
	* Set the grid value at CellIJK
	*/
	void SetValue(const FVector3i& CellIJK, ElemType NewValue)
	{
		WriteValue(CellIJK.X, CellIJK.Y, CellIJK.Z, [NewValue](ElemType& GridValueInOut) { GridValueInOut = NewValue; });
	}


	/**
	* Call an external lambda with a reference to the grid value at (I,J,K).
	* Called as Func(ElemType&), so the caller can both read and write the grid cell.  
	* 
	* Note: if doing pure reads prefer the GetValue() methods because the ProcessValue method will allocate the underlying data block. 
	*/
	template<typename ProcessFunc>
	void ProcessValue(int32 I, int32 J, int32 K, ProcessFunc Func)
	{
		WriteValue(I, J, K, Func);
	}

	/**
	* Call an external lambda with a reference to the grid value at CellIJK.
	* Called as Func(ElemType&), so the caller can both read and write the grid cell
	* 
	* Note: if doing pure reads prefer the GetValue() methods because the ProcessValue method will allocate the underlying data block. 
	*/
	template<typename ProcessFunc>
	void ProcessValue(const FVector3i& CellIJK, ProcessFunc Func)
	{
		WriteValue(CellIJK.X, CellIJK.Y, CellIJK.Z, Func);
	}


	/**
	* Pre-allocate the grid blocks that also exist in SourceGrid of the same dimensions.
	* This is useful in grid-processing algorithms where parallel grid operations can be
	* done without locking if the blocks are already allocated.
	* 
	* @return false on failure (e.g. different sized grids), otherwise true.
	*/
	template <typename OtherElemType>
	bool PreAllocateFromSourceGrid(const TBasicBlockedDenseGrid3<OtherElemType, BlockSize>& SourceGrid)
	{
		return AllocateTopologyUnionImpl(SourceGrid);
	}

	/** @return the total number of block units form the spatial decomposition of the grid.  Note these block may not all be allocated*/
	int32 GetNumBlocks() const
	{
		const FVector3i& BlockDims = BlockedGrid3LayoutType::GetBlockDimensions();
		return BlockDims.X * BlockDims.Y * BlockDims.Z;
	}

protected:
	template <typename OtherGridType >
	bool AllocateTopologyUnionImpl(const OtherGridType& OtherGrid)
	{

		// bound checking
		if ((BlockedGrid3LayoutType::GetDimensions() != OtherGrid.GetDimensions()) || (BlockedGrid3LayoutType::GetBlockDimensions() != OtherGrid.GetBlockDimensions()))
		{
			check(false);		// this is almost certainly a development-time error in calling code
			return false;
		}

		const int32 NumBlocks = GetNumBlocks();
		for (int32 k = 0; k < NumBlocks; ++k)
		{
			if (OtherGrid.IsBlockAllocated(k))
			{
				TouchBlockData(k);
			}
		}
		return true;
	}
};

/**
* TBlockedGrid3 represents a 3D uniform grid.
*
* The grid is allocated in BlockSize^3 blocks on-demand (BlockSize is a compile-time constant) allowing
* very large grids to be used without having to pre-allocate all the memory, eg for sparse/narrow-band use cases.
*
* Block-level default values are returned when querying a grid cell in an unallocated block., 
* 
* Block-level access is provided to facilitate parallel algorithms.  Additionally, 'Set' methods are provided that utilize 
* critical sections provided by the caller.  
*/
template<typename ElemType, int32 BlockSize_>
class TBlockedGrid3 : public TBasicBlockedDenseGrid3<ElemType, BlockSize_>
{
public:
	typedef TBasicBlockedDenseGrid3<ElemType, BlockSize_>                     BlockedDenseGridType;
	typedef TBlockedGrid3Layout<BlockSize_>                                   BlockedGrid3LayoutType;
	typedef TBlockData3<ElemType, BlockSize_>                                 BlockData3Type;


	static constexpr int32 BlockSize      = BlockSize_;                         // length of side of a block
	static constexpr int32 BlockElemCount = BlockSize * BlockSize * BlockSize;  // number of cells in each block


protected:

	typedef typename TBasicBlockedDenseGrid3<ElemType, BlockSize_>::FBlock3           FBlock3;


	template<typename FuncType>
	void WriteBlockDefaultValue(const int32 BlockIndex, FuncType Func, bool bDeallocateBock = true)
	{
		FBlock3& Block = BlockedDenseGridType::GetBlock(BlockIndex);
		Func(Block.DefaultValue);
		if (bDeallocateBock)
		{
			Block.BlockDataPtr.Reset();
		}
	}

	template <typename FuncType>
	void WriteBlockDefaultValue(const FVector3i& BlockIJK, FuncType Func, bool bDeallocateBock = true)
	{
		const int32 BlockIndex = BlockedGrid3LayoutType::BlockIJKToBlockIndex(BlockIJK);
		WriteBlockDefaultValue(BlockIndex, Func, bDeallocateBock);
	}

	template<typename FuncType>
	void WriteBlockDefaultValueWithLock(const int32 BlockIndex, FuncType Func, FCriticalSection* CriticalSection, bool bDeallocateBock = true)
	{
		FScopeLock Lock(CriticalSection);
		WriteBlockDefaultValue(BlockIndex, Func, bDeallocateBock);
	}


	// -- block-level default value read methods.

	ElemType ReadBlockDefaultValue(const int32& BlockIndex) const
	{
		return BlockedDenseGridType::GetBlock(BlockIndex).DefaultValue;
	}

	ElemType ReadBlockDefaultValue(const FVector3i& BlockIJK) const
	{
		const int32 BlockIndex = BlockedGrid3LayoutType::BlockIJKToBlockIndex(BlockIJK);
		return ReadBlockDefaultValue(BlockIndex);
	}

public:
	/**
	* Create empty grid
	*/
	TBlockedGrid3() :  BlockedDenseGridType()
	{
	}

	/**
	* Create grid with specified domain.
	*/
	TBlockedGrid3(int32 DimI, int32 DimJ, int32 DimK, ElemType InitialValue)
	: BlockedDenseGridType(DimI, DimJ, DimK, InitialValue)
	{
	}


	// -- methods to set values when using locks.

	/**
	* Set the grid value at (I,J,K), with user supplied locking in the form 
	* of a function of the form
	* 
	*		FCriticalSection& CriticalSectionProvider(id)
	* 
	* that provides a critical section for each block id.
	*/
	template <typename CriticalSectionProviderType>
	void SetValueWithLock(int32 I, int32 J, int32 K, ElemType NewValue, CriticalSectionProviderType& CriticalSectionProvider)
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		FCriticalSection& CriticalSection = CriticalSectionProvider(BlockIndex);
		BlockedDenseGridType::WriteValueWithLock(BlockIndex, LocalIndex, [&NewValue](ElemType& Value){Value = NewValue;}, &CriticalSection);
	}

	/**
	* Set the grid value at CellIJK,  with user supplied locking in the form
	* of a function of the form
	*
	*		FCriticalSection& CriticalSectionProvider(id)
	*
	* that provides a critical section for each block id.
	*/
	template <typename CriticalSectionProviderType>
	void SetValueWithLock(const FVector3i& CellIJK, ElemType NewValue,  CriticalSectionProviderType CriticalSectionProvider)
	{
		SetValueWithLock(CellIJK.X, CellIJK.Y, CellIJK.Z, NewValue, CriticalSectionProvider);
	}

	/**
	* Call an external lambda with a reference to the grid value at (I,J,K).
	* Called as Func(ElemType&), so the caller can both read and write the grid cell
	* 
	* note: if doing pure reads prefer the GetValue() methods because the ProcessValue method will allocate the underlying data block. 
	*/
	template<typename ProcessFunc, typename CriticalSectionProviderType>
	void ProcessValueWithLock(int32 I, int32 J, int32 K, ProcessFunc Func, CriticalSectionProviderType CriticalSectionProvider)
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		FCriticalSection& CriticalSection = CriticalSectionProvider(BlockIndex);
		BlockedDenseGridType::WriteValueWithLock(BlockIndex, LocalIndex, Func, &CriticalSection);
	}

	/**
	* Call an external lambda with a reference to the grid value at CellIJK.
	* Called as Func(ElemType&), so the caller can both read and write the grid cell
	*
	* note: if doing pure reads prefer the GetValue() methods because the ProcessValue method will allocate the underlying data block. 
	*/
	template<typename ProcessFunc, typename CriticalSectionProviderType>
	void ProcessValueWithLock(const FVector3i& CellIJK, ProcessFunc Func, CriticalSectionProviderType CriticalSectionProvider)
	{
		ProcessValueWithLock(CellIJK.X, CellIJK.Y, CellIJK.Z, Func, CriticalSectionProvider);
	}

	// -- block access methods ---/

	/** @return the total number of block units that covers the spatial dimensions of the grid.*/
	int32 GetNumBlocks() const
	{
		const FVector3i& BlockDims = BlockedGrid3LayoutType::GetBlockDimensions();
		return BlockDims.X * BlockDims.Y * BlockDims.Z;
	}

	/** @return array of pointers to the allocated blocks.  Note: the grid owns the blocks, so the calling code should not delete them. */
	TArray<const BlockData3Type*> GetAllocatedConstBlocks() const
	{
		TArray<const BlockData3Type*> Result;
		const int32 NumBlocks = BlockedDenseGridType::GetNumBlocks();
		for (int32 i = 0; i < NumBlocks; ++i)
		{
			if (BlockedDenseGridType::IsBlockAllocated(i))
			{
				Result.Add(BlockedDenseGridType::Blocks[i].BlockDataPtr.Get());
			}
		}
		return Result;
	}

	/** @return array of pointers to the allocated blocks.  Note: the grid owns the blocks, so the calling code should not delete them.*/
	TArray<BlockData3Type*> GetAllocatedBlocks()
	{
		TArray<BlockData3Type*> Result;
		const int32 NumBlocks = BlockedDenseGridType::GetNumBlocks();
		for (int32 i = 0; i < NumBlocks; ++i)
		{
			if (BlockedDenseGridType::IsBlockAllocated(i))
			{
				Result.Add(BlockedDenseGridType::Blocks[i].BlockDataPtr.Get());
			}
		}
		return Result;
	}

	/** @return array of pointers to the allocated blocks.  Note: the grid owns the blocks, so the calling code should not delete them.*/
	TArray<const BlockData3Type*> GetAllocatedBlocks() const
	{
		return GetAllocatedConstBlocks();
	}

	/** 
	* @return true if the specified block has been allocated.
	* @param BlockIJK  - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	*/
	bool IsBlockAllocated(const FVector3i& BlockIJK) const
	{
		return GetBlockData(BlockIJK).IsValid();
	}

	/** 
	* @return a references to the specified block, this will allocate the block if it doesn't already exists. 
	* @param BlockIJK  - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	*/
	BlockData3Type& TouchBlockData(const FVector3i& BlockIJK)
	{
		const int32 BlockIndex = BlockedGrid3LayoutType::BlockIJKToBlockIndex(BlockIJK);
		return BlockedDenseGridType::TouchBlockData(BlockIndex);
	}

	/**
	* @return reference to a unique pointer that points to a block that may or may not be allocated
	* @param BlockIJK  - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	*/
	TUniquePtr<BlockData3Type>& GetBlockData(const FVector3i& BlockIJK)
	{
		const int32 BlockIndex = BlockedGrid3LayoutType::BlockIJKToBlockIndex(BlockIJK);
		return GetBlockData(BlockIndex);
	}

	/**
	* @return reference to a unique pointer that points to a block that may or may not be allocated
	* @param BlockIJK  - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	*/
	const TUniquePtr<BlockData3Type>& GetBlockData(const FVector3i& BlockIJK) const
	{
		const int32 BlockIndex = BlockedGrid3LayoutType::BlockIJKToBlockIndex(BlockIJK);
		return GetBlockData(BlockIndex);
	}

	/**
	* @return reference to a unique pointer that points to a block that may or may not be allocated
	* @param BlockIndex  - the index of the block in an internal linearization of the blocks.
	* Note:   BlockIJK and BlockIndex  can be converted by means of ::BlockIJKToBlockIndex() and ::BlockIndexToBlockIJK()
	*/
	TUniquePtr<BlockData3Type>& GetBlockData(int32 BlockIndex)
	{
		return BlockedDenseGridType::GetBlock(BlockIndex).BlockDataPtr;
	}

	/**
	* @return reference to a unique pointer that points to a block that may or may not be allocated
	* @param BlockIndex  - the index of the block in an internal linearization of the blocks.
	* Note:   BlockIJK and BlockIndex  can be converted by means of ::BlockIJKToBlockIndex() and ::BlockIndexToBlockIJK()
	*/
	const TUniquePtr<BlockData3Type>& GetBlockData(int32 BlockIndex) const
	{
		return BlockedDenseGridType::GetBlock(BlockIndex).BlockDataPtr;
	}

	// --- methods on the block's default values.  
	// ---  The block 'default value' is returned when querying a cell in an unallocated block, and is used to initialize the block on allocation.

	/**
	* @return the default value for the specified block.
	* @param BlockIJK    - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	* 
	*/ 
	ElemType GetBlockDefaultValue(const FVector3i& BlockIJK) const
	{
		return ReadBlockDefaultValue(BlockIJK);
	}

	/**
	* Update the default value for the specified block.
	* @param BlockIJK    - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	* @param Value       - the new default value for this block
	* @param bPruneBlock - if true, any internal block allocation will be freed 
	*                      and the resulting deallocated block will just hold a single default value.
	*                      if false, the allocation of the block will no change.
	* 
	*/
	void SetBlockDefaultValue(const FVector3i& BlockIJK, const ElemType& Value, bool bPruneBlock = true)
	{
		WriteBlockDefaultValue(BlockIJK, [&Value](ElemType& oldValue){ oldValue = Value;}, bPruneBlock);
	}

	/**
	* Update the default value for the specified block using a functor
	* @param BlockIJK    - the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	* @param Func        - Func(ElemType& value){..} is called on the default value.
	* @param bPruneBlock - if true, any internal block allocation will be freed 
	*                      and the resulting deallocated block will just hold a single default value.
	*                      if false, the allocation of the block will no change.
	* 
	*/
	template<typename ProcessFunc>
	void ProcessBlockDefaultValue(const FVector3i& BlockIJK, ProcessFunc Func, bool bPruneBlock = true)
	{
		WriteBlockDefaultValue(BlockIJK, Func, bPruneBlock);
	}
};

// Note on the block size: 8 seems to be the sweet spot with vastly better performance in SDF generation
// perhaps because a float block can fit in the L1 cache with room to spare.
typedef TBlockedGrid3<float, 8>  FBlockedGrid3f;
typedef TBlockedGrid3<double, 8> FBlockedGrid3d;
typedef TBlockedGrid3<int, 8>    FBlockedGrid3i;

// note the bool grid uses a bitfield specialization
typedef TBlockedGrid3<bool, 8>   FBlockedGrid3b;

/**
* TBlockedDenseGrid adds thread-safe access functions to the 3D blocked uniform grid, TBasicBlockedDenseGrid3.
* Blocked grid is extended by adding internally owned locks for each data block.
* 
* The grid is allocated in BlockSize^3 blocks on-demand (BlockSize is a compile-time constant) allowing 
* very large grids to be used without having to pre-allocate all the memory, eg for sparse/narrow-band use cases.
*
* For multi-threaded applications where memory is a premium, consider using TBlockedGrid3 (above) as it gives better 
* control over the individual blocks and allows the caller to manage the lifetime of any locks used.
*/
template<typename ElemType>
class TBlockedDenseGrid3 : public TBasicBlockedDenseGrid3<ElemType, 32>
{
public:
	
	static constexpr int32 BlockSize = 32;
	static constexpr int32 BlockElemCount = BlockSize * BlockSize * BlockSize;
	
	typedef TBasicBlockedDenseGrid3<ElemType, BlockSize>                     BlockedDenseGridType;
	typedef TBlockedGrid3Layout<BlockSize>                                   BlockedGrid3LayoutType;
	typedef TBlockData3<ElemType, BlockSize>                                 BlockData3Type;


	

protected:

	TArray<FCriticalSection> BlockLocks;             // per-block level lock used for thread-safe methods

#if UE_BUILD_DEBUG

	FCriticalSection* GetBlockLock(int32 Index) { return &BlockLocks[Index]; }
#else
	// skip range checks in non-debug builds
	FCriticalSection* GetBlockLock(int32 Index) { return &BlockLocks.GetData()[Index]; }
#endif

	template<typename FuncType>
	void WriteValueThreadSafe(int32 I, int32 J, int32 K, FuncType Func)
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		BlockedDenseGridType::WriteValueWithLock(BlockIndex, LocalIndex, Func, GetBlockLock(BlockIndex));
	}


	ElemType ReadValueThreadSafe(int32 I, int32 J, int32 K)
	{
		int32 BlockIndex, LocalIndex;
		BlockedGrid3LayoutType::GetBlockAndLocalIndex(I, J, K, BlockIndex, LocalIndex);
		return BlockedDenseGridType::ReadValueWithLock(BlockIndex, LocalIndex, GetBlockLock(BlockIndex));
	}

public:

	/**
	* Create empty grid
	*/
	TBlockedDenseGrid3() : BlockedDenseGridType()
	{
	}

	/**
	* Create grid with specified domain.
	*/
	TBlockedDenseGrid3(int32 DimI, int32 DimJ, int32 DimK, ElemType InitialValue)
	{
		BlockedDenseGridType::DefaultValue = InitialValue;
		Resize(DimI, DimJ, DimK);
	}

	/**
	* Reconfigure the grid to have the target dimensions. This clears all the 
	* existing grid memory and resets the block-level default value.
	*/
	void Resize(int32 DimI, int32 DimJ, int32 DimK)
	{
		BlockedDenseGridType::Resize(DimI, DimJ, DimK);

		const int32 NumBlocks = BlockedDenseGridType::GetNumBlocks();
		BlockLocks.Reset();
		BlockLocks.SetNum(NumBlocks);
	}

	/**
	* Discard all allocated blocks but retain grid dimensions
	*/
	void Reset()
	{
		Reset(BlockedGrid3LayoutType::Dimensions.X, BlockedGrid3LayoutType::Dimensions.Y, BlockedGrid3LayoutType::Dimensions.Z, BlockedDenseGridType::DefaultValue);
	}

	/**
	* Reconfigure the grid to have the target dimensions and default value. 
	* This clears all the existing grid memory.
	*/
	void Reset(int32 DimI, int32 DimJ, int32 DimK, ElemType InitialValue)
	{
		BlockedDenseGridType::DefaultValue = InitialValue;
		Resize(DimI, DimJ, DimK);
	}

	/**
	* @return true if the specified cell IJK is contained within the grid bounds.
	*/ 
	bool IsValidIndex(FVector3i CellIJK) const
	{
		return BlockedGrid3LayoutType::IsValidIJK(CellIJK);
	}
	/**
	* @return the grid value at (I,J,K), with internal locking, so it is safe to call this from multiple read & write threads
	*/
	ElemType GetValueThreadSafe(int32 I, int32 J, int32 K)
	{
		return ReadValueThreadSafe(I, J, K);
	}

	/**
	* @return the grid value at CellIJK, with internal locking, so it is safe to call this from multiple read & write threads
	*/
	ElemType GetValueThreadSafe(const FVector3i& CellIJK)
	{
		return ReadValueThreadSafe(CellIJK.X, CellIJK.Y, CellIJK.Z);
	}

	/**
	* Set the grid value at (I,J,K), with internal locking, so it is safe to call this from multiple read & write threads
	*/
	void SetValueThreadSafe(int32 I, int32 J, int32 K, ElemType NewValue)
	{
		WriteValueThreadSafe(I, J, K, [NewValue](ElemType& GridValueInOut) { GridValueInOut = NewValue; });
	}

	/**
	* Set the grid value at CellIJK, with internal locking, so it is safe to call this from multiple read & write threads
	*/
	void SetValueThreadSafe(const FVector3i& CellIJK, ElemType NewValue)
	{
		WriteValueThreadSafe(CellIJK.X, CellIJK.Y, CellIJK.Z, [NewValue](ElemType& GridValueInOut) { GridValueInOut = NewValue; });
	}

	/**
	* Call an external lambda with a reference to the grid value at (I,J,K).
	* Called as Func(ElemType&), so the caller can both read and write the grid cell
	*
	* note: if doing pure reads prefer the GetValue() methods 
	* because the ProcessValueThreadSafe method will allocate the underlying data block. 
	*/
	template<typename ProcessFunc>
	void ProcessValueThreadSafe(int32 I, int32 J, int32 K, ProcessFunc Func)
	{
		WriteValueThreadSafe(I, J, K, Func);
	}

	/**
	* Call an external lambda with a reference to the grid value at CellIJK.
	* Called as Func(ElemType&), so the caller can both read and write the grid cell
	*/
	template<typename ProcessFunc>
	void ProcessValueThreadSafe(const FVector3i& CellIJK, ProcessFunc Func)
	{
		WriteValueThreadSafe(CellIJK.X, CellIJK.Y, CellIJK.Z, Func);
	}

	/** Convert a grid cell id to CellIJK coordinates*/
	FVector3i ToIndex(int64 LinearIndex) const
	{
		int32 x = (int32)(LinearIndex % (int64)BlockedGrid3LayoutType::Dimensions.X);
		int32 y = (int32)((LinearIndex / (int64)BlockedGrid3LayoutType::Dimensions.X) % (int64)BlockedGrid3LayoutType::Dimensions.Y);
		int32 z = (int32)(LinearIndex / ((int64)BlockedGrid3LayoutType::Dimensions.X * (int64)BlockedGrid3LayoutType::Dimensions.Y));
		return FVector3i(x, y, z);
	}
	/** Convert grid CellIJK coordinates to a linear ID, note this id is independent of the actual internal data storage.*/
	int64 ToLinear(int32 X, int32 Y, int32 Z) const
	{
		return (int64)X + (int64)BlockedGrid3LayoutType::Dimensions.X * ((int64)Y + (int64)BlockedGrid3LayoutType::Dimensions.Y * (int64)Z);
	}
	/** Convert grid CellIJK coordinates to a linear ID, note this id is independent of the actual internal data storage.*/
	int64 ToLinear(const FVector3i& IJK) const
	{
		return (int64)IJK.X + (int64)BlockedGrid3LayoutType::Dimensions.X * ((int64)IJK.Y + (int64)BlockedGrid3LayoutType::Dimensions.Y * (int64)IJK.Z);
	}


};

typedef TBlockedDenseGrid3<float>  FBlockedDenseGrid3f;
typedef TBlockedDenseGrid3<double> FBlockedDenseGrid3d;
typedef TBlockedDenseGrid3<int>    FBlockedDenseGrid3i;


} // end namespace UE::Geometry
} // end namespace UE
