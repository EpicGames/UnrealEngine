// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "IntBoxTypes.h"


namespace UE
{
namespace Geometry
{
// These are 3d grid space layout behaviors shared by the various blocked grids 
// ( c.f. BlockedDenseGrid3.h and other usages in SparseNarrowBandSDFMesh.h )


// Consistent translation between a linear array of size BlockSize to a 3d layout. 
template <int32 BlockSize_>
struct TBlockData3Layout
{
	static constexpr int32 BlockSize = BlockSize_;
	static constexpr int32 ElemCount = BlockSize_ * BlockSize_ * BlockSize_;

	/**
	* @return the 3d cell associated with the specified linear index.
	*/ 
	static FVector3i ToLocalIJK(int32 LocalLinear)
	{
		//inverse of  ToLinear()
		const int32 i = LocalLinear % BlockSize;
		const int32 j = (LocalLinear / BlockSize) % BlockSize;
		const int32 k = LocalLinear / (BlockSize * BlockSize);
		return FVector3i(i, j, k);
	}

	/**
	* @return linear index associated with the specified 3d cell.
	*/ 
	static inline int32 ToLinear(const int32& LocalI, const int32& LocalJ, const int32& LocalK)
	{
		//inverse of ToLocalIJK
		return LocalI + BlockSize * (LocalJ + BlockSize * LocalK);
	}
};

// Spatial decomposition of a region of 3d ijk space into 3d cubes of size BlockSize (per side).
// This class gives a consistent translation between locations in 3d lattice space  and blocks of a spatial decomposition;
// it is independent of any sort of data that you might want to associate with those locations.  See BlockedDenseGrid3.h for usage.
template <int32 BlockSize_>
class TBlockedGrid3Layout
{
public:
	static constexpr int32 BlockSize      = BlockSize_;                        // block is a cube in ijk space of length BlockSize           
	static constexpr int32 BlockElemCount = BlockSize * BlockSize * BlockSize; // no. of ijk coordinates (i.e. lattice points) in each block

	typedef TBlockData3Layout<BlockSize_>  FBlockData3Layout;

	TBlockedGrid3Layout()
		: BlockDimensions(FVector3i::Zero())
		, Dimensions(FVector3i::Zero())
	{}

	TBlockedGrid3Layout(int32 DimI, int32 DimJ, int32 DimK)
	{
		Resize(DimI, DimJ, DimK);
	}

	TBlockedGrid3Layout(FVector3i Dims)
	{
		Resize(Dims[0], Dims[1], Dims[2]);
	}

	/**
	* Recompute the block dimensions required to cover the requested cell dimensions.
	*/
	void Resize(int32 DimI, int32 DimJ, int32 DimK)
	{
		check((int64)DimI * (int64)DimJ * (int64)DimK < INT_MAX); // [todo] want a larger check here.. right now 1290^3 is near the upper lim 
		const int32 BlocksI = (DimI / BlockSize) + ( (DimI % BlockSize != 0) ? 1 : 0 );
		const int32 BlocksJ = (DimJ / BlockSize) + ( (DimJ % BlockSize != 0) ? 1 : 0 );
		const int32 BlocksK = (DimK / BlockSize) + ( (DimK % BlockSize != 0) ? 1 : 0 );

		BlockDimensions = FVector3i(BlocksI, BlocksJ, BlocksK);
		Dimensions = FVector3i(DimI, DimJ, DimK);
	}

	/**
	* @return the dimensions of this layout in terms of cell ijks.
	*/ 
	const FVector3i& GetDimensions() const
	{
		return Dimensions;
	}

	/**
	* @return the dimensions of this layout in terms of blocks.
	*/ 
	const FVector3i& GetBlockDimensions() const
	{
		return BlockDimensions;
	}

	/*
	* @return an axis aligned box that defines the bounds of this region in terms of cells. This bounds is [inclusive, exclusive) 
	*/
	FAxisAlignedBox3i Bounds() const
	{
		return FAxisAlignedBox3i({ 0, 0, 0 }, { Dimensions[0], Dimensions[1], Dimensions[2] });
	}

	/*
	* @return an axis aligned box that defines the bounds of this region in terms of cells. This bounds is [inclusive, inclusive] 
	*/
	FAxisAlignedBox3i BoundsInclusive() const
	{
		return FAxisAlignedBox3i({ 0, 0, 0 }, { Dimensions[0] - 1, Dimensions[1] - 1, Dimensions[2]- 1 });
	}

	/**
	* @return the number of ijk cells in contained within this grid layout.
	* note: because blocks are 3d pages of fixed size so the number of Blocks X Cells in Block
	*       might actually be a larger.
	*/ 
	int64 Size() const
	{
		return (int64)Dimensions[0] * (int64)Dimensions[1] * (int64)Dimensions[2];
	}

	/**
	* @return true if the specified cell IJK is within the bounds of the dimensions.
	*/
	bool IsValidIJK(const FVector3i& IJK) const
	{
		return IJK[0] >= 0 && IJK[1] >= 0 && IJK[2] >= 0
			&& IJK[0] < Dimensions[0] && IJK[1] < Dimensions[1] && IJK[2] < Dimensions[2];
	}

	/**
	* @return true if the specified block IJK is within the bounds of the block dimensions.
	*/
	bool IsValidBlockIJK(const FVector3i& BlockIJK)
	{
		return BlockIJK[0] >= 0 && BlockIJK[1] >= 0 && BlockIJK[2] >= 0
			&& BlockIJK[0] < BlockDimensions[0] && BlockIJK[1] < BlockDimensions[1] && BlockIJK[2] < BlockDimensions[2];
	}

	/**
	* @return the index of the block that holds the cell indicated by IJK.
	*/ 
	int32 IJKtoBlockIndex(const FVector3i& IJK) const
	{
		const FVector3i BlockIJK(IJK[0] / BlockSize, IJK[1]/BlockSize, IJK[2]/BlockSize);
		return BlockIJKToBlockIndex(BlockIJK);
	}

	/**
	* return the block index and local index of the cell at (I,J,K).
	*/ 
	void GetBlockAndLocalIndex(const FVector3i& IJK, int32& BlockIndexOut, int32& LocalIndexOut) const
	{
		BlockIndexOut = IJKtoBlockIndex(IJK);

		const int32 LocalI = IJK.X % BlockSize;
		const int32 LocalJ = IJK.Y % BlockSize;
		const int32 LocalK = IJK.Z % BlockSize;
		LocalIndexOut = FBlockData3Layout::ToLinear(LocalI, LocalJ, LocalK);
	}	
	void GetBlockAndLocalIndex(int32 I, int32 J, int32 K, int32& BlockIndexOut, int32& LocalIndexOut) const
	{
		return GetBlockAndLocalIndex(FVector3i(I, J, K), BlockIndexOut, LocalIndexOut);
	}

	/**
	* @return the 3-d coordinates of the block specified by an index in a linearization of the blocks.
	* BlockIJK is the lattice address of the block in terms of block units ( e.g. the location in a grid of blocks).
	*/ 
	FVector3i BlockIndexToBlockIJK(const int32 BlockIndex) const
	{

		const int32 BlockI = BlockIndex % BlockDimensions.X;
		const int32 BlockJ = (BlockIndex / BlockDimensions.X) % BlockDimensions.Y;
		const int32 BlockK = BlockIndex / (BlockDimensions.X * BlockDimensions.Y);

		return FVector3i(BlockI, BlockJ, BlockK);
	}

	/**
	* @return the block index for the specified block (BlockIJK) in a linearization of the 3d block space.
	*/ 
	int32 BlockIJKToBlockIndex(const FVector3i& BlockIJK) const 
	{
		return BlockIJK[0] + BlockDimensions.X * (BlockIJK[1] + BlockDimensions.Y * BlockIJK[2]);
	}

protected:

	FVector3i BlockDimensions; // dimensions in blocks per axis
	FVector3i Dimensions;      // dimensions in cells per axis
};

} // end namespace UE::Geometry
} // end namespace UE
