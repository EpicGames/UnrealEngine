// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IntBoxTypes.h"

#include "HAL/PlatformAtomics.h"

namespace UE::Geometry
{

/**
 * 2D dense grid of scalar values.
 */
template<typename ElemType>
class TDenseGrid2
{
protected:
	/** grid of allocated elements */
	TArray64<ElemType> Buffer;

	/** dimensions per axis */
	FVector2i Dimensions;

public:
	/**
	 * Create empty grid
	 */
	TDenseGrid2() : Dimensions(0, 0)
	{
	}

	TDenseGrid2(int32 DimX, int32 DimY)
		: Dimensions(DimX, DimY)
	{
		Buffer.SetNumUninitialized(Num());
	}

	TDenseGrid2(int32 DimX, int32 DimY, ElemType InitialValue)
		: Dimensions(DimX, DimY)
	{
		Buffer.Init(InitialValue, Num());
	}

	UE_DEPRECATED(5.8, "Please use Num instead.")
	int64 Size() const
	{
		return Num();
	}

	int64 Num() const
	{
		return static_cast<int64>(Dimensions.X) * static_cast<int64>(Dimensions.Y);
	}

	int32 Width() const
	{
		return Dimensions.X;
	}

	int32 Height() const
	{
		return Dimensions.Y;
	}

	const FVector2i& GetDimensions() const
	{
		return Dimensions;
	}

	void Resize(int32 DimX, int32 DimY, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		Dimensions = FVector2i(DimX, DimY);
		Buffer.SetNumUninitialized(Num(), AllowShrinking);
	}

	UE_DEPRECATED(5.8, "Please use Assign instead.")
	void AssignAll(ElemType Value)
	{
		Assign(Value);
	}

	void Assign(ElemType Value)
	{
		for (int64 Idx = 0, N = Num(); Idx < N; ++Idx)
		{
			Buffer[Idx] = Value;
		}
	}

	void SetMin(const FVector2i& XY, ElemType MinValue)
	{
		const int64 Idx = GetIndex(XY);
		if (MinValue < Buffer[Idx])
		{
			Buffer[Idx] = MinValue;
		}
	}

	void SetMax(const FVector2i& XY, ElemType MaxValue)
	{
		const int64 Idx = GetIndex(XY);
		if (MaxValue > Buffer[Idx])
		{
			Buffer[Idx] = MaxValue;
		}
	}

	bool IsValidIndex(int32 X, int32 Y) const
	{
		return X >= 0 && Y >= 0 && X < Dimensions.X && Y < Dimensions.Y;
	}

	bool IsValidIndex(const FVector2i& XY) const
	{
		return IsValidIndex(XY.X, XY.Y);
	}

	constexpr ElemType& operator[](int64 Index)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->operator[](Index));
	}

	constexpr const ElemType& operator[](int64 Index) const
	{
		return Buffer[Index];
	}

	constexpr ElemType& operator[](FVector2i XY)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->operator[](XY));
	}

	constexpr const ElemType& operator[](FVector2i XY) const
	{
		return Buffer[GetIndex(XY)];
	}

	constexpr ElemType& At(int32 X, int32 Y)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->At(X, Y));
	}

	constexpr const ElemType& At(int32 X, int32 Y) const
	{
		return Buffer[GetIndex(X, Y)];
	}

	constexpr ElemType& At(FVector2i XY)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->At(XY));
	}

	constexpr const ElemType& At(FVector2i XY) const
	{
		return Buffer[GetIndex(XY)];
	}

	UE_DEPRECATED(5.8, "Please use the int32 overloads instead.")
	constexpr ElemType& At(int64 X, int64 Y)
	{
		check(static_cast<int32>(X) <= MAX_int32);
		check(static_cast<int32>(Y) <= MAX_int32);
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->At(static_cast<int32>(X), static_cast<int32>(Y)));
	}

	UE_DEPRECATED(5.8, "Please use the int32 overloads instead.")
	constexpr const ElemType& At(int64 X, int64 Y) const
	{
		check(static_cast<int32>(X) <= MAX_int32);
		check(static_cast<int32>(Y) <= MAX_int32);
		return const_cast<ElemType&>(const_cast<const TDenseGrid2*>(this)->At(static_cast<int32>(X), static_cast<int32>(Y)));
	}

	TArray64<ElemType>& GridValues()
	{
		return Buffer;
	}
	const TArray64<ElemType>& GridValues() const
	{
		return Buffer;
	}

	UE_DEPRECATED(5.8, "Please use Process instead.")
	void Apply(TFunctionRef<ElemType(ElemType)> Func)
	{
		for (int64 Idx = 0, N = Num(); Idx < N; ++Idx)
		{
			Buffer[Idx] = Func(Buffer[Idx]);
		}
	}

	/**
	* Call a function on all values.
	*/
	template<typename ProcessFunc>
	void Process(ProcessFunc Func)
	{
		for (int64 Idx = 0, N = Num(); Idx < N; ++Idx)
		{
			Func(Buffer[Idx]);
		}
	}

	/**
	* Call a function on the value at (X,Y).
	*/
	template<typename ProcessFunc>
	void Process(int32 X, int32 Y, ProcessFunc Func)
	{
		Func(At(X, Y));
	}

	/**
	* Call a function on the value at (X,Y).
	*/
	template<typename ProcessFunc>
	void Process(const FVector2i& XY, ProcessFunc Func)
	{
		Func(At(XY.X, XY.Y));
	}

	void GetXPair(int32 X0, int32 Y, ElemType& AOut, ElemType& BOut) const
	{
		const int64 Offset = static_cast<int64>(Dimensions.X) * Y;
		AOut = Buffer[Offset + X0];
		BOut = Buffer[Offset + X0 + 1];
	}

	FAxisAlignedBox2i Bounds() const
	{
		return FAxisAlignedBox2i({0, 0}, {Dimensions.X, Dimensions.Y});
	}

	FAxisAlignedBox2i BoundsInclusive() const
	{
		return FAxisAlignedBox2i({0, 0}, {Dimensions.X - 1, Dimensions.Y - 1});
	}

	FVector2i GetCoords(int64 Index) const
	{
		checkSlow(Index >= 0);
		const int32 X = static_cast<int32>(Index % static_cast<int64>(Dimensions.X));
		const int32 Y = static_cast<int32>(Index / static_cast<int64>(Dimensions.X));
		return FVector2i(X, Y);
	}

	int64 GetIndex(int32 X, int32 Y) const
	{
		return X + static_cast<int64>(Dimensions.X) * Y;
	}

	int64 GetIndex(const FVector2i& XY) const
	{
		return GetIndex(XY.X, XY.Y);
	}
};


typedef TDenseGrid2<float> FDenseGrid2f;
typedef TDenseGrid2<double> FDenseGrid2d;
typedef TDenseGrid2<int32> FDenseGrid2i;


// additional utility functions
namespace DenseGrid
{

inline void AtomicIncrement(FDenseGrid2i& Grid, int32 i, int32 j)
{
	FPlatformAtomics::InterlockedIncrement(&Grid.At(i, j));
}

inline void AtomicDecrement(FDenseGrid2i& Grid, int32 i, int32 j)
{
	FPlatformAtomics::InterlockedDecrement(&Grid.At(i, j));
}

inline void AtomicIncDec(FDenseGrid2i& Grid, int32 i, int32 j, bool bDecrement = false)
{
	if (bDecrement)
	{
		FPlatformAtomics::InterlockedDecrement(&Grid.At(i, j));
	}
	else
	{
		FPlatformAtomics::InterlockedIncrement(&Grid.At(i, j));
	}
}
}

}
