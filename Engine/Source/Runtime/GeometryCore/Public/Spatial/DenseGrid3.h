// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IntBoxTypes.h"

#include "HAL/PlatformAtomics.h"

namespace UE::Geometry
{

/**
 * 3D dense grid of scalar values. 
 */
template<typename ElemType>
class TDenseGrid3
{
protected:
	/** grid of allocated elements */
	TArray64<ElemType> Buffer;

	/** dimensions per axis */
	FVector3i Dimensions;

public:
	/**
	 * Create empty grid
	 */
	TDenseGrid3() : Dimensions(0,0,0)
	{
	}

	TDenseGrid3(int32 DimX, int32 DimY, int32 DimZ)
		: Dimensions(DimX, DimY, DimZ)
	{
		Buffer.SetNumUninitialized(Num());
	}

	TDenseGrid3(int32 DimX, int32 DimY, int32 DimZ, ElemType InitialValue)
		: Dimensions(DimX, DimY, DimZ)
	{
		Buffer.Init(InitialValue, Num());
	}

	UE_DEPRECATED(5.8, "Please use Num instead.")
	int64 Size() const
	{
		return Num();
	}

	int64 Num() const
	{
		return static_cast<int64>(Dimensions.X) * static_cast<int64>(Dimensions.Y) * static_cast<int64>(Dimensions.Z);
	}

	int32 Width() const
	{
		return Dimensions.X;
	}

	int32 Height() const
	{
		return Dimensions.Y;
	}

	int32 Depth() const
	{
		return Dimensions.Z;
	}

	const FVector3i& GetDimensions() const
	{
		return Dimensions;
	}

	void Resize(int32 DimX, int32 DimY, int32 DimZ, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		Dimensions = FVector3i(DimX, DimY, DimZ);
		Buffer.SetNumUninitialized(Num(), AllowShrinking);
	}

	void Assign(ElemType Value)
	{
		for (int64 Idx = 0, N = Num(); Idx < N; ++Idx)
		{
			Buffer[Idx] = Value;
		}
	}

	void SetMin(const FVector3i& XYZ, ElemType MinValue)
	{
		const int64 Idx = GetIndex(XYZ);
		if (MinValue < Buffer[Idx])
		{
			Buffer[Idx] = MinValue;
		}
	}
	void SetMax(const FVector3i& XYZ, ElemType MaxValue)
	{
		const int64 Idx = GetIndex(XYZ);
		if (MaxValue > Buffer[Idx])
		{
			Buffer[Idx] = MaxValue;
		}
	}

	bool IsValidIndex(const FVector3i& XYZ) const
	{
		return IsValidIndex(XYZ.X, XYZ.Y, XYZ.Z);
	}

	bool IsValidIndex(int32 X, int32 Y, int32 Z) const
	{
		return X >= 0 && Y >= 0 && Z >= 0 && X < Dimensions.X && Y < Dimensions.Y && Z < Dimensions.Z;
	}

	constexpr ElemType& operator[](int64 Index)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid3*>(this)->operator[](Index));
	}

	constexpr const ElemType& operator[](int64 Index) const
	{
		return Buffer[Index];
	}

	constexpr ElemType& operator[](FVector3i XYZ)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid3*>(this)->operator[](XYZ));
	}

	constexpr const ElemType& operator[](FVector3i XYZ) const
	{
		return Buffer[GetIndex(XYZ.X, XYZ.Y, XYZ.Z)];
	}

	constexpr ElemType& At(int32 X, int32 Y, int32 Z)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid3*>(this)->At(X, Y, Z));
	}

	constexpr const ElemType& At(int32 X, int32 Y, int32 Z) const
	{
		return Buffer[GetIndex(X, Y, Z)];
	}

	constexpr ElemType& At(FVector3i XYZ)
	{
		return const_cast<ElemType&>(const_cast<const TDenseGrid3*>(this)->At(XYZ));
	}

	constexpr const ElemType& At(FVector3i XYZ) const
	{
		return Buffer[GetIndex(XYZ)];
	}

	TArray64<ElemType>& GridValues()
	{
		return Buffer;
	}
	const TArray64<ElemType>& GridValues() const
	{
		return Buffer;
	}

	ElemType GetValue(int32 X, int32 Y, int32 Z) const
	{
		return At(X, Y, Z);
	}

	ElemType GetValue(const FVector3i& XYZ) const
	{
		return At(XYZ);
	}

	void SetValue(int32 X, int32 Y, int32 Z, ElemType NewValue)
	{
		At(X, Y, Z) = NewValue;
	}

	void SetValue(const FVector3i& XYZ, ElemType NewValue)
	{
		At(XYZ) = NewValue;
	}

	template<typename ProcessFunc>
	UE_DEPRECATED(5.8, "Please use Process instead.")
	void ProcessValue(int32 X, int32 Y, int32 Z, ProcessFunc Func)
	{
		Process(X, Y, Z, Func);
	}

	template<typename ProcessFunc>
	UE_DEPRECATED(5.8, "Please use Process instead.")
	void ProcessValue(const FVector3i& XYZ, ProcessFunc Func)
	{
		Process(XYZ, Func);
	}

	/**
	* Call a function on all values.
	*/
	template<typename ProcessFunc>
	void Process(ProcessFunc Func)
	{
		for (int64 Idx = 0, N = Num(); Idx < N; ++Idx)
		{
			Func(Buffer[Idx]);
		}
	}

	/**
	* Call a function on the value at (X,Y,Z).
	*/
	template<typename ProcessFunc>
	void Process(int32 X, int32 Y, int32 Z, ProcessFunc Func)
	{
		Func(At(X, Y, Z));
	}

	/**
	* Call a function on the value at (X,Y,Z).
	*/
	template<typename ProcessFunc>
	void Process(const FVector3i& XYZ, ProcessFunc Func)
	{
		Func(At(XYZ));
	}

	void GetXPair(int32 X0, int32 Y, int32 Z, ElemType& AOut, ElemType& BOut) const
	{
		const int64 Offset = static_cast<int64>(Dimensions.X) * (static_cast<int64>(Y) + static_cast<int64>(Dimensions.Y) * static_cast<int64>(Z));
		AOut = Buffer[Offset + static_cast<int64>(X0)];
		BOut = Buffer[Offset + static_cast<int64>(X0) + 1];
	}

	FAxisAlignedBox3i Bounds() const
	{
		return FAxisAlignedBox3i({0, 0, 0}, {Dimensions.X, Dimensions.Y, Dimensions.Z});
	}

	FAxisAlignedBox3i BoundsInclusive() const
	{
		return FAxisAlignedBox3i({0, 0, 0}, {Dimensions.X - 1, Dimensions.Y - 1, Dimensions.Z - 1});
	}

	UE_DEPRECATED(5.8, "Please use GetCoords instead.")
	FVector3i ToIndex(int32 Index) const
	{
		return GetCoords(Index);
	}

	FVector3i GetCoords(int64 Index) const
	{
		checkSlow(Index >= 0);
		const int32 X = static_cast<int32>(Index % static_cast<int64>(Dimensions.X));
		const int32 Y = static_cast<int32>((Index / static_cast<int64>(Dimensions.X)) % static_cast<int64>(Dimensions.Y));
		const int32 Z = static_cast<int32>(Index / (static_cast<int64>(Dimensions.X) * static_cast<int64>(Dimensions.Y)));
		return FVector3i(X, Y, Z);
	}

	UE_DEPRECATED(5.8, "Please use GetIndex instead.")
	int32 ToLinear(int32 X, int32 Y, int32 Z) const
	{
		return GetIndex(X, Y, Z);
	}

	UE_DEPRECATED(5.8, "Please use GetIndex instead.")
	int32 ToLinear(const FVector3i& XYZ) const
	{
		return GetIndex(XYZ.X, XYZ.Y, XYZ.Z);
	}

	int64 GetIndex(int32 X, int32 Y, int32 Z) const
	{
		return X + static_cast<int64>(Dimensions.X) * (Y + static_cast<int64>(Dimensions.Y) * Z);
	}

	int64 GetIndex(const FVector3i& XYZ) const
	{
		return GetIndex(XYZ.X, XYZ.Y, XYZ.Z);
	}
};


typedef TDenseGrid3<float> FDenseGrid3f;
typedef TDenseGrid3<double> FDenseGrid3d;
typedef TDenseGrid3<int32> FDenseGrid3i;

// additional utility functions
namespace DenseGrid
{

inline void AtomicIncrement(FDenseGrid3i& Grid, int32 X, int32 Y, int32 Z)
{
	FPlatformAtomics::InterlockedIncrement(&Grid.At(X,Y,Z));
}

inline void AtomicDecrement(FDenseGrid3i& Grid, int32 X, int32 Y, int32 Z)
{
	FPlatformAtomics::InterlockedDecrement(&Grid.At(X, Y, Z));
}

inline void AtomicIncDec(FDenseGrid3i& Grid, int32 X, int32 Y, int32 Z, bool bDecrement = false)
{
	if (bDecrement)
	{
		FPlatformAtomics::InterlockedDecrement(&Grid.At(X, Y, Z));
	}
	else
	{
		FPlatformAtomics::InterlockedIncrement(&Grid.At(X, Y, Z));
	}
}
}

}
