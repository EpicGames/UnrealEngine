// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Spatial/MeshAABBTree3.h"
#include "Util/MeshCaches.h"
#include "MatrixTypes.h"
#include "Async/ParallelTransformReduce.h"

/**
 *  Formulas for triangle winding number approximation
 */
namespace FastTriWinding
{
	using namespace UE::Geometry;
	using FTriangle3d = UE::Geometry::FTriangle3d;
	using FMatrix3d = UE::Geometry::FMatrix3d;

	/**
	 *  precompute constant coefficients of triangle winding number approximation (serial implementation)
	 *  P: 'Center' of expansion for Triangles (area-weighted centroid avg)
	 *  R: max distance from P to Triangles
	 *  Order1: first-order vector coeff
	 *  Order2: second-order matrix coeff
	 *  TriCache: precomputed triangle centroid/normal/area
	 */
	template <class TriangleMeshType, class IterableTriangleIndices>
	void ComputeCoeffsSerial(const TriangleMeshType& Mesh,
							 const IterableTriangleIndices& Triangles,
							 const FMeshTriInfoCache& TriCache,
							 FVector3d& P, double& R,
							 FVector3d& Order1,
							 FMatrix3d& Order2)
	{
		P = FVector3d::Zero();
		Order1 = FVector3d::Zero();
		Order2 = FMatrix3d::Zero();
		R = 0;

		// compute area-weighted centroid of Triangles, we use this as the expansion point
		FVector3d P0 = FVector3d::Zero(), P1 = FVector3d::Zero(), P2 = FVector3d::Zero();
		double sum_area = 0;
		for (int tid : Triangles)
		{
			double Area = TriCache.Areas[tid];
			sum_area += Area;
			P += Area * TriCache.Centroids[tid];
		}
		P /= sum_area;

		// compute first and second-order coefficients of FWN taylor expansion, as well as
		// 'radius' value R, which is max dist from any tri vertex to P
		FVector3d n, c;
		double a = 0;
		double RSq = 0;
		for (int tid : Triangles)
		{
			Mesh.GetTriVertices(tid, P0, P1, P2);
			TriCache.GetTriInfo(tid, n, a, c);

			Order1 += a * n;

			FVector3d dcp = c - P;
			Order2 += a * FMatrix3d(dcp, n);

			// update max radius R (as squared value in loop)
			double MaxDistSq = FMath::Max3(DistanceSquared(P0,P), DistanceSquared(P1,P), DistanceSquared(P2,P));
			RSq = FMath::Max(RSq, MaxDistSq);
		}
		R = FMath::Sqrt(RSq);
	}

	/**
	 *  precompute constant coefficients of triangle winding number approximation (evaluated in parallel for large sets of triangles)
	 *  P: 'Center' of expansion for Triangles (area-weighted centroid avg)
	 *  R: max distance from P to Triangles
	 *  Order1: first-order vector coeff
	 *  Order2: second-order matrix coeff
	 *  TriCache: precomputed triangle centroid/normal/area
	 */
	template <class TriangleMeshType>
	void ComputeCoeffs(const TriangleMeshType& Mesh,
					   const TArray<int>& TriangleArray,
					   const FMeshTriInfoCache& TriCache,
					   FVector3d& P,
					   double& R,
					   FVector3d& Order1,
					   FMatrix3d& Order2,
					   int NumTasks = 16)
	{
		// If the data is small enough, don't bother trying to parallelize
		constexpr int ParallelThreshold = 3000;			// Benchmarking shows parallel starts to outperform serial at around 2500 triangles or so
		if (TriangleArray.Num() < ParallelThreshold)
		{
			ComputeCoeffsSerial(Mesh, TriangleArray, TriCache, P, R, Order1, Order2);
			return;
		}

		// First compute the area-weighted centroid

		struct PData
		{
			FVector3d P;
			double Area;
		};

		auto CentroidTransform = [&](int64 TriangleSubsetIndex) -> PData
		{
			int TID = TriangleArray[(int)TriangleSubsetIndex];

			PData DataOut;
			DataOut.Area = TriCache.Areas[TID];
			DataOut.P = DataOut.Area * TriCache.Centroids[TID];
			return DataOut;
		};

		auto CentroidReduce = [](const PData& A, const PData& B) -> PData
		{
			PData Out;
			Out.P = A.P + B.P;
			Out.Area = A.Area + B.Area;
			return Out;
		};

		PData CentroidData = ParallelTransformReduce(TriangleArray.Num(), PData{ FVector3d::Zero(), 0.0}, CentroidTransform, CentroidReduce, NumTasks);

		CentroidData.P /= CentroidData.Area;

		// Now compute the expansions

		struct OrderData
		{
			FVector3d Order1;
			FMatrix3d Order2;
			double RSq;
		};

		auto ExpansionTransform = [&, P = CentroidData.P](int32 TriangleSubsetIndex) -> OrderData
		{
			int tid = TriangleArray[(int)TriangleSubsetIndex];
			FVector3d P0, P1, P2;
			Mesh.GetTriVertices(tid, P0, P1, P2);

			FVector3d n, c;
			double a;
			TriCache.GetTriInfo(tid, n, a, c);

			OrderData DataOut;
			DataOut.Order1 = a * n;

			FVector3d dcp = c - P;
			DataOut.Order2 = a * FMatrix3d(dcp, n);

			// this is just for return value...
			double MaxDistSq = FMath::Max3(DistanceSquared(P0, P), DistanceSquared(P1, P), DistanceSquared(P2, P));
			DataOut.RSq = MaxDistSq;

			return DataOut;
		};

		auto ExpansionReduce = [](const OrderData& A, const OrderData& B) -> OrderData
		{
			OrderData Out;
			Out.Order1 = A.Order1 + B.Order1;
			Out.Order2 = A.Order2 + B.Order2;
			Out.RSq = FMath::Max(A.RSq, B.RSq);
			return Out;
		};

		OrderData Orders = ParallelTransformReduce(TriangleArray.Num(), 
												   OrderData{ FVector3d::Zero(), FMatrix3d::Zero(), 0.0 }, 
												   ExpansionTransform, 
												   ExpansionReduce, 
												   NumTasks);

		// Set out values

		P = CentroidData.P;
		R = FMath::Sqrt(Orders.RSq);
		Order1 = Orders.Order1;
		Order2 = Orders.Order2;
	}

	/**
	 *  precompute constant coefficients of triangle winding number approximation (evaluated in parallel for large sets of triangles)
	 *  P: 'Center' of expansion for Triangles (area-weighted centroid avg)
	 *  R: max distance from P to Triangles
	 *  Order1: first-order vector coeff
	 *  Order2: second-order matrix coeff
	 *  TriCache: precomputed triangle centroid/normal/area
	 */
	 template <class TriangleMeshType>
	 UE_DEPRECATED(5.7, "Use the above version of ComputeCoeffs, which takes a TArray of triangle indices instead")
	 void ComputeCoeffs(const TriangleMeshType& Mesh,
		 const TSet<int>& TriangleSet,
		 const FMeshTriInfoCache& TriCache,
		 FVector3d& P,
		 double& R,
		 FVector3d& Order1,
		 FMatrix3d& Order2,
		 int NumTasks = 16)
	 {
		 // If the data is small enough, don't bother trying to parallelize
		 constexpr int ParallelThreshold = 3000;			// Benchmarking shows parallel starts to outperform serial at around 2500 triangles or so
		 if (TriangleSet.Num() < ParallelThreshold)
		 {
			 ComputeCoeffsSerial(Mesh, TriangleSet, TriCache, P, R, Order1, Order2);
			 return;
		 }

		 TArray<int> TriangleArray = TriangleSet.Array();
		 ComputeCoeffs(Mesh, TriangleArray, TriCache, P, R, Order1, Order2, NumTasks);

	 }


	/**
	 *  Evaluate first-order FWN approximation at point Q, relative to Center c
	 */
	inline double EvaluateOrder1Approx(const FVector3d& Center, const FVector3d& Order1Coeff, const FVector3d& Q)
	{
		FVector3d dpq = (Center - Q);
		double len = dpq.Length();

		return (1.0 / FMathd::FourPi) * Order1Coeff.Dot(dpq / (len * len * len));
	}

	/**
	 *  Evaluate second-order FWN approximation at point Q, relative to Center c
	 */
	inline double EvaluateOrder2Approx(const FVector3d& Center, const FVector3d& Order1Coeff, const FMatrix3d& Order2Coeff, const FVector3d& Q)
	{
		FVector3d dpq = (Center - Q);
		double len = dpq.Length();
		double len3 = len * len * len;
		double fourPi_len3 = 1.0 / (FMathd::FourPi * len3);

		double Order1 = fourPi_len3 * Order1Coeff.Dot(dpq);

		// second-order hessian \grad^2(G)
		double c = -3.0 / (FMathd::FourPi * len3 * len * len);

		// expanded-out version below avoids extra constructors
		//FMatrix3d xqxq(dpq, dpq);
		//FMatrix3d hessian(fourPi_len3, fourPi_len3, fourPi_len3) - c * xqxq;
		FMatrix3d hessian(
			fourPi_len3 + c * dpq.X * dpq.X, c * dpq.X * dpq.Y, c * dpq.X * dpq.Z,
			c * dpq.Y * dpq.X, fourPi_len3 + c * dpq.Y * dpq.Y, c * dpq.Y * dpq.Z,
			c * dpq.Z * dpq.X, c * dpq.Z * dpq.Y, fourPi_len3 + c * dpq.Z * dpq.Z);

		double Order2 = Order2Coeff.InnerProduct(hessian);

		return Order1 + Order2;
	}

	// triangle-winding-number first-order approximation.
	// T is triangle, P is 'Center' of cluster of dipoles, Q is evaluation point
	// (This is really just for testing)
	inline double Order1Approx(const FTriangle3d& T, const FVector3d& P, const FVector3d& XN, double XA, const FVector3d& Q)
	{
		FVector3d at0 = XA * XN;

		FVector3d dpq = (P - Q);
		double len = dpq.Length();
		double len3 = len * len * len;

		return (1.0 / FMathd::FourPi) * at0.Dot(dpq / (len * len * len));
	}

	// triangle-winding-number second-order approximation
	// T is triangle, P is 'Center' of cluster of dipoles, Q is evaluation point
	// (This is really just for testing)
	inline double Order2Approx(const FTriangle3d& T, const FVector3d& P, const FVector3d& XN, double XA, const FVector3d& Q)
	{
		FVector3d dpq = (P - Q);

		double len = dpq.Length();
		double len3 = len * len * len;

		// first-order approximation - integrated_normal_area * \grad(G)
		double Order1 = (XA / FMathd::FourPi) * XN.Dot(dpq / len3);

		// second-order hessian \grad^2(G)
		FMatrix3d xqxq(dpq, dpq);
		xqxq *= 3.0 / (FMathd::FourPi * len3 * len * len);
		double diag = 1 / (FMathd::FourPi * len3);
		FMatrix3d hessian = FMatrix3d(diag, diag, diag) - xqxq;

		// second-order LHS - integrated second-order area matrix (formula 26)
		FVector3d centroid(
			(T.V[0].X + T.V[1].X + T.V[2].X) / 3.0, (T.V[0].Y + T.V[1].Y + T.V[2].Y) / 3.0, (T.V[0].Z + T.V[1].Z + T.V[2].Z) / 3.0);
		FVector3d dcp = centroid - P;
		FMatrix3d o2_lhs(dcp, XN);
		double Order2 = XA * o2_lhs.InnerProduct(hessian);

		return Order1 + Order2;
	}
} // namespace FastTriWinding



namespace UE
{
namespace Geometry
{

/**
 * Fast Mesh Winding Number extension to a TMeshAABBTree3.
 * This class is an "add-on" to the AABBTree, that can compute the Fast Mesh Winding Number.
 * This calculation requires a precomputation pass where information is cached at each tree node.
 */
template <class TriangleMeshType>
class TFastWindingTree
{
	TMeshAABBTree3<TriangleMeshType>* Tree;

	struct FWNInfo
	{
		FVector3d Center;
		double R;
		FVector3d Order1Vec;
		FMatrix3d Order2Mat;
		bool bValid = false;
	};

	TArray<FWNInfo> FastWindingCache;
	int32 FastWindingCacheOffset = 0;
	uint64 FastWindingCacheMeshChangeStamp = 0;

public:
	/**
	 * FWN beta parameter - is 2.0 in paper
	 */
	double FWNBeta = 2.0;

	/**
	 * FWN approximation order. Must be 1 or 2. 2 is more accurate, obviously.
	 */
	int FWNApproxOrder = 2;

	TFastWindingTree(TMeshAABBTree3<TriangleMeshType>* TreeToRef, bool bAutoBuild = true)
	{
		SetTree(TreeToRef, bAutoBuild);
	}

	void SetTree(TMeshAABBTree3<TriangleMeshType>* TreeToRef, bool bAutoBuild = true)
	{
		this->Tree = TreeToRef;
		if (bAutoBuild)
		{
			Build(true);
		}
	}

	TMeshAABBTree3<TriangleMeshType>* GetTree() const
	{
		return Tree;
	}

	void Build(bool bForceRebuild = true)
	{
		check(Tree);
		if ( Tree->IsValid(false) == false )
		{
			Tree->Build();
		}
		if (bForceRebuild || FastWindingCacheMeshChangeStamp != Tree->MeshChangeStamp)
		{
			build_fast_winding_cache();
			FastWindingCacheMeshChangeStamp = Tree->MeshChangeStamp;
		}
	}

	bool IsBuilt() const
	{
		return Tree->IsValid(false) && FastWindingCacheMeshChangeStamp == Tree->MeshChangeStamp;
	}

	/**
	 * Fast approximation of winding number using far-field approximations.
	 * On a closed mesh the winding number will be 1 or more inside (depending on number of "winds").
	 * Outside a closed mesh the winding number will be zero.
	 * On an open mesh, the above holds near the mesh but in the "hole" areas the value will smoothly blend from 1 to 0 over a band of width dependent on the hole extent
	 */
	double FastWindingNumber(const FVector3d& P)
	{
		Build(false);
		double sum = branch_fast_winding_num(Tree->RootIndex, P);
		return sum;
	}

	/**
	 * Const version does not auto-build on query
	 */
	double FastWindingNumber(const FVector3d& P) const
	{
		checkSlow(IsBuilt());
		double sum = branch_fast_winding_num(Tree->RootIndex, P);
		return sum;
	}

	/**
	 * @return true if fast winding number at point P is greater than winding threshold (default 0.5)
	 */
	bool IsInside(const FVector3d& P, double WindingIsoThreshold = 0.5) const
	{
		return FastWindingNumber(P) > WindingIsoThreshold;
	}

private:
	// evaluate winding number contribution for all Triangles below IBox
	double branch_fast_winding_num(int IBox, FVector3d P) const
	{
		double branch_sum = 0;

		int idx = Tree->BoxToIndex[IBox];
		if (idx < Tree->TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			int num_tris = Tree->IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				FVector3d a, b, c;
				int ti = Tree->IndexList[idx + i];
				Tree->Mesh->GetTriVertices(ti, a, b, c);
				double angle = VectorUtil::TriSolidAngle(a, b, c, P);
				branch_sum += angle;
			}
			branch_sum /= FMathd::FourPi;
		}
		else
		{ // internal node, either 1 or 2 child boxes
			int iChild1 = Tree->IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;

				// if we have winding cache, we can more efficiently compute contribution of all Triangles
				// below this box. Otherwise, recursively descend Tree.
				bool contained = Tree->box_contains(iChild1, P);
				if (contained == false && can_use_fast_winding_cache(iChild1, P))
				{
					branch_sum += evaluate_box_fast_winding_cache(iChild1, P);
				}
				else
				{
					branch_sum += branch_fast_winding_num(iChild1, P);
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = Tree->IndexList[idx + 1] - 1;

				bool contained1 = Tree->box_contains(iChild1, P);
				if (contained1 == false && can_use_fast_winding_cache(iChild1, P))
				{
					branch_sum += evaluate_box_fast_winding_cache(iChild1, P);
				}
				else
				{
					branch_sum += branch_fast_winding_num(iChild1, P);
				}

				bool contained2 = Tree->box_contains(iChild2, P);
				if (contained2 == false && can_use_fast_winding_cache(iChild2, P))
				{
					branch_sum += evaluate_box_fast_winding_cache(iChild2, P);
				}
				else
				{
					branch_sum += branch_fast_winding_num(iChild2, P);
				}
			}
		}

		return branch_sum;
	}

	void build_fast_winding_cache()
	{
		// set this to a larger number to ignore caches if number of Triangles is too small.
		// (seems to be no benefit to doing this...is holdover from Tree-decomposition FWN code)
		constexpr int WINDING_CACHE_THRESH = 1;

		//FMeshTriInfoCache TriCache = null;
		FMeshTriInfoCache TriCache = FMeshTriInfoCache::BuildTriInfoCache(*Tree->Mesh);

		// Fast winding cache can only exist for non-leaf boxes, so find the offset where the cache-able boxes start
		for (FastWindingCacheOffset = 0; FastWindingCacheOffset < Tree->BoxToIndex.Num(); ++FastWindingCacheOffset)
		{
			if (Tree->BoxToIndex[FastWindingCacheOffset] >= Tree->TrianglesEnd)
			{
				break;
			}
		}
		int32 CacheSize = Tree->BoxToIndex.Num() - FastWindingCacheOffset;
		FastWindingCache.Empty(CacheSize);
		FastWindingCache.Init({ .bValid = false }, CacheSize);
		TOptional<TArray<int>> root_hash;
		build_fast_winding_cache(Tree->RootIndex, 0, WINDING_CACHE_THRESH, root_hash, TriCache);
	}
	int build_fast_winding_cache(int IBox, int Depth, int TriCountThresh, TOptional<TArray<int>>& TriArray, const FMeshTriInfoCache& TriCache)
	{
		TriArray.Reset();

		int idx = Tree->BoxToIndex[IBox];
		if (idx < Tree->TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			int num_tris = Tree->IndexList[idx];
			return num_tris;
		}
		else
		{ // internal node, either 1 or 2 child boxes
			int iChild1 = Tree->IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				int num_child_tris = build_fast_winding_cache(iChild1, Depth + 1, TriCountThresh, TriArray, TriCache);

				// if count in child is large enough, we already built a cache at lower node
				return num_child_tris;
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = Tree->IndexList[idx + 1] - 1;

				// let each child build its own cache if it wants. If so, it will return the
				// list of its child tris
				TOptional<TArray<int>> child2_array;
				int num_tris_1 = build_fast_winding_cache(iChild1, Depth + 1, TriCountThresh, TriArray, TriCache);
				int num_tris_2 = build_fast_winding_cache(iChild2, Depth + 1, TriCountThresh, child2_array, TriCache);
				bool build_cache = (num_tris_1 + num_tris_2 > TriCountThresh);

				if (Depth == 0)
				{
					return num_tris_1 + num_tris_2; // cannot build cache at level 0...
				}

				// collect up the Triangles we need. there are various cases depending on what children already did
				if (TriArray.IsSet() || child2_array.IsSet() || build_cache)
				{
					if (!TriArray.IsSet() && child2_array.IsSet())
					{
						collect_triangles(iChild1, child2_array.GetValue());
						TriArray = MoveTemp(child2_array); // Note child2_array is not used after this
					}
					else
					{
						if (!TriArray.IsSet())
						{
							TriArray.Emplace();
							collect_triangles(iChild1, TriArray.GetValue());
						}
						if (!child2_array.IsSet())
						{
							collect_triangles(iChild2, TriArray.GetValue());
						}
						else
						{
							TriArray->Append(child2_array.GetValue());
						}
					}
				}
				if (build_cache)
				{
					check(TriArray.IsSet());
					make_box_fast_winding_cache(IBox, TriArray.GetValue(), TriCache);
				}

				return (num_tris_1 + num_tris_2);
			}
		}
	}

	// check if value is in cache and far enough away from Q that we can use cached value
	bool can_use_fast_winding_cache(int IBox, const FVector3d& Q) const
	{
		int32 CacheIdx = IBox - FastWindingCacheOffset;
		if (CacheIdx < 0)
		{
			return false;
		}

		const FWNInfo& CacheInfo = FastWindingCache[CacheIdx];
		if (!CacheInfo.bValid)
		{
			return false;
		}

		double DistQP_Sq = DistanceSquared(CacheInfo.Center, Q);
		double DistThreshold = FWNBeta * CacheInfo.R;
		return DistQP_Sq > DistThreshold * DistThreshold;
	}

	// compute FWN cache for all Triangles underneath this box
	void make_box_fast_winding_cache(int IBox, const TArray<int>& Triangles, const FMeshTriInfoCache& TriCache)
	{
		check(!FastWindingCache[IBox - FastWindingCacheOffset].bValid);

		// construct cache
		FWNInfo cacheInfo;
		FastTriWinding::ComputeCoeffs(*Tree->Mesh, Triangles, TriCache, cacheInfo.Center, cacheInfo.R, cacheInfo.Order1Vec, cacheInfo.Order2Mat);
		cacheInfo.bValid = true;

		FastWindingCache[IBox - FastWindingCacheOffset] = cacheInfo;
	}

	// evaluate the FWN cache for IBox
	double evaluate_box_fast_winding_cache(int IBox, const FVector3d& Q) const
	{
		const FWNInfo& cacheInfo = FastWindingCache[IBox - FastWindingCacheOffset];

		if (FWNApproxOrder == 2)
		{
			return FastTriWinding::EvaluateOrder2Approx(cacheInfo.Center, cacheInfo.Order1Vec, cacheInfo.Order2Mat, Q);
		}
		else
		{
			return FastTriWinding::EvaluateOrder1Approx(cacheInfo.Center, cacheInfo.Order1Vec, Q);
		}
	}

	// collect all the Triangles below IBox
	void collect_triangles(int IBox, TArray<int>& Triangles)
	{
		int idx = Tree->BoxToIndex[IBox];
		if (idx < Tree->TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			int num_tris = Tree->IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				Triangles.Add(Tree->IndexList[idx + i]);
			}
		}
		else
		{
			int iChild1 = Tree->IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				collect_triangles((-iChild1) - 1, Triangles);
			}
			else
			{ // 2 children, descend closest first
				collect_triangles(iChild1 - 1, Triangles);
				collect_triangles(Tree->IndexList[idx + 1] - 1, Triangles);
			}
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE