// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Math/Ray.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Polyline3.h"
#include "Templates/Function.h"
#include "Templates/Invoke.h"

namespace UE
{
namespace Geometry
{

/**
 * FGeometrySet3 stores a set of 3D Points and Polyline curves,
 * and supports spatial queries against these sets. 
 * 
 * Since Points and Curves have no area to hit, hit-tests are done via nearest-point-on-ray.
 */
class FGeometrySet3
{
public:
	/**
	 * @param bPoints if true, discard all points
	 * @param bCurves if true, discard all polycurves
	 */
	GEOMETRYCORE_API void Reset(bool bPoints = true, bool bCurves = true);

	/** Add a point with given PointID at the given Position*/
	GEOMETRYCORE_API void AddPoint(int PointID, const FVector3d& Position);
	/** Add a polycurve with given CurveID and the give Polyline */
	GEOMETRYCORE_API void AddCurve(int CurveID, const FPolyline3d& Polyline);

	/** Remove a point with given PointID. */
	GEOMETRYCORE_API void RemovePoint(int PointID);
	/** Remove a curve with given CurveID. */
	GEOMETRYCORE_API void RemoveCurve(int CurveID);

	/** Update the Position of previously-added PointID */
	GEOMETRYCORE_API void UpdatePoint(int PointID, const FVector3d& Position);
	/** Update the Polyline of previously-added CurveID */
	GEOMETRYCORE_API void UpdateCurve(int CurveID, const FPolyline3d& Polyline);

	/**
	 * FNearest is returned by nearest-point queries
	 */
	struct FNearest
	{
		/** ID of point or curve */
		int ID;
		/** true for point, false for polyline curve*/
		bool bIsPoint;

		/** Nearest point on ray */
		FVector3d NearestRayPoint;
		/** Nearest point on geometry (ie the point, or point on curve)*/
		FVector3d NearestGeoPoint;

		/** parameter of nearest point on ray (equivalent to NearestRayPoint) */
		double RayParam;

		/** if bIsPoint=false, index of nearest segment on polyline curve */
		int PolySegmentIdx;
		/** if bIsPoint=false, parameter of NearestGeoPoint along segment defined by PolySegmentIdx */
		double PolySegmentParam;
	};


	/**
	 * @param Ray query ray
	 * @param ResultOut populated with information about successful nearest point result
	 * @param PointWithinToleranceTest should return true if two 3D points are "close enough" to be considered a hit
	 * @return true if the nearest point on Ray to some point in the set passed the PointWithinToleranceTest.
	 * @warning PointWithinToleranceTest is called in parallel and hence must be thread-safe/re-entrant!
	 */
	GEOMETRYCORE_API bool FindNearestPointToRay(const FRay3d& Ray, FNearest& ResultOut,
		TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const;

	/**
	 * Like FindNearestPointToRay, but gives all elements within tolerance, rather than just the closest.
	 *
	 * @param Ray query ray
	 * @param ResultsOut populated with information about successful nearest point results. Not cleared in advance.
	 * @param PointWithinToleranceTest should return true if two 3D points are "close enough" to be considered a hit
	 * @return true if at least one result was added (ie, passed PointWithinToleranceTest).
	 * @warning PointWithinToleranceTest is called in parallel and hence must be thread-safe/re-entrant!
	 */
	GEOMETRYCORE_API bool CollectPointsNearRay(const FRay3d& Ray, TArray<FNearest>& ResultsOut,
		TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const;

	/**
	 * @param Ray query ray
	 * @param ResultOut populated with information about successful nearest point result
	 * @param PointWithinToleranceTest should return true if two 3D points are "close enough" to be considered a hit
	 * @return true if the nearest point on Ray to some curve in the set passed the PointWithinToleranceTest.
	 * @warning PointWithinToleranceTest is called in parallel and hence must be thread-safe/re-entrant!
	 */
	GEOMETRYCORE_API bool FindNearestCurveToRay(const FRay3d& Ray, FNearest& ResultOut,
		TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const;

	/**
	 * Like FindNearestCurveToRay, but gives all elements within tolerance, rather than just the closest.
	 *
	 * @param Ray query ray
	 * @param ResultsOut populated with information about successful nearest point results. Not cleared in advance.
	 * @param PointWithinToleranceTest should return true if two 3D points are "close enough" to be considered a hit
	 * @return true if at least one result was added (ie, passed PointWithinToleranceTest).
	 * @warning PointWithinToleranceTest is called in parallel and hence must be thread-safe/re-entrant!
	 */
	GEOMETRYCORE_API bool CollectCurvesNearRay(const FRay3d& Ray, TArray<FNearest>& ResultsOut,
		TFunction<bool(const FVector3d&, const FVector3d&)> PointWithinToleranceTest) const;

	/**
	 * Fills PointIDsOut with point IDs of points that satisfy the given predicate.
	 *
	 * @param Predicate A lambda or function that takes in a const FVector3d& and returns
	 *  true when the point's ID should be added to PointIDsOut.
	 * @param PointIDsOut Output list or set. Must have an Add(int) method.
	 */
	template <typename PredicateType, typename IntContainerType>
	bool FindAllPointsSatisfying(PredicateType Predicate, IntContainerType& PointIDsOut) const
	{
		PointIDsOut.Reset();
		for (const FPoint& Point : Points)
		{
			if (Invoke(Predicate, Point.Position))
			{
				PointIDsOut.Add(Point.ID);
			}
		}
		return !PointIDsOut.IsEmpty();
	}

	/**
	 * Like FindAllPointsSatisfying, but parallel, so predicate must be safe to call in parallel.
	 */
	template <typename PredicateType, typename IntContainerType>
	bool ParallelFindAllPointsSatisfying(PredicateType Predicate, IntContainerType& PointIDsOut) const
	{
		TArray<bool> Flags;
		Flags.SetNumZeroed(Points.Num());
		ParallelFor(Points.Num(), [this, &Predicate, &Flags](int32 i)
			{
				if (Invoke(Predicate, Points[i].Position))
				{
					Flags[i] = true;
				}
			});

		for (int32 i = 0; i < Points.Num(); ++i)
		{
			if (Flags[i])
			{
				PointIDsOut.Add(Points[i].ID);
			}
		}
		return !PointIDsOut.IsEmpty();
	}

	/**
	 * Fills CurveIDsOut with IDs of curves that satisfy the given predicate.
	 *
	 * @param Predicate A lambda or function that takes in a const FPolyline3d& and returns
	 *  true when the curve's ID should be added to PointIDsOut.
	 * @param CurveIDsOut Output list or set. Must have an Add(int) method.
	 */
	template <typename PredicateType, typename IntContainerType>
	bool FindAllCurvesSatisfying(PredicateType Predicate, IntContainerType& CurveIDsOut) const
	{
		CurveIDsOut.Reset();
		for (const FCurve& Curve : Curves)
		{
			if (Invoke(Predicate, Curve.Geometry))
			{
				CurveIDsOut.Add(Curve.ID);
			}
		}
		return !CurveIDsOut.IsEmpty();
	}

	/**
	 * Like FindAllCurvesSatisfying, but parallel, so predicate must be safe to call in parallel.
	 */
	template <typename PredicateType, typename IntContainerType>
	bool ParallelFindAllCurvesSatisfying(PredicateType Predicate, IntContainerType& CurveIDsOut) const
	{
		TArray<bool> Flags;
		Flags.SetNumZeroed(Curves.Num());
		ParallelFor(Curves.Num(), [this, &Predicate, &Flags](int32 i)
		{
			if (Invoke(Predicate, Curves[i].Geometry))
			{
				Flags[i] = true;
			}
		});

		for (int32 i = 0; i < Curves.Num(); ++i)
		{
			if (Flags[i])
			{
				CurveIDsOut.Add(Curves[i].ID);
			}
		}
		return !CurveIDsOut.IsEmpty();
	}

protected:

	struct FPoint
	{
		int ID;
		FVector3d Position;
	};
	TArray<FPoint> Points;
	TMap<int, int> PointIDToIndex;

	struct FCurve
	{
		int ID;
		FPolyline3d Geometry;

		FAxisAlignedBox3d Bounds;
	};
	TArray<FCurve> Curves;
	TMap<int, int> CurveIDToIndex;

};


} // end namespace UE::Geometry
} // end namespace UE
