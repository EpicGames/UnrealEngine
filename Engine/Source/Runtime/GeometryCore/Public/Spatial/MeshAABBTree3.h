// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp TMeshAABBTree3

#pragma once

#include "Util/DynamicVector.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "Intersection/IntrTriangle3Triangle3.h"
#include "Intersection/IntersectionUtil.h"
#include "MeshQueries.h"
#include "Spatial/SpatialInterfaces.h"
#include "Distance/DistTriangle3Triangle3.h"


namespace MeshIntersection
{
	using namespace UE::Geometry;

	/**
	 * Intersection query result types for triangle mesh intersections
	 */
	struct FPointIntersection
	{
		int TriangleID[2];
		FVector3d Point;
	};
	struct FSegmentIntersection
	{
		int TriangleID[2];
		FVector3d Point[2];
	};
	struct FPolygonIntersection
	{
		int TriangleID[2];
		FVector3d Point[6]; // Coplanar tri-tri intersection forms a polygon of at most six points
		int Quantity; // number of points actually used
	};
	struct FIntersectionsQueryResult
	{
		TArray<FPointIntersection> Points;
		TArray<FSegmentIntersection> Segments;
		TArray<FPolygonIntersection> Polygons;
	};
}


namespace UE
{
namespace Geometry
{

using namespace UE::Math;

template <class TriangleMeshType>
class TFastWindingTree;


template <class TriangleMeshType>
class TMeshAABBTree3 : public IMeshSpatial
{
	friend class TFastWindingTree<TriangleMeshType>;

	constexpr static bool bValidatePerAccess = (bool)DO_GUARD_SLOW;

public:
	using MeshType = TriangleMeshType;
	using GetSplitAxisFunc = TUniqueFunction<int(int Depth, const FAxisAlignedBox3d& Box)>;

protected:
	const TriangleMeshType* Mesh;
	uint64 MeshChangeStamp = 0;
	int TopDownLeafMaxTriCount = 3;

	static GetSplitAxisFunc MakeDefaultSplitAxisFunc()
	{
		return [](int Depth, const FAxisAlignedBox3d&)
		{
			return Depth % 3;
		};
	}
	
	GetSplitAxisFunc GetSplitAxis = MakeDefaultSplitAxisFunc();

public:
	static constexpr double DOUBLE_MAX = TNumericLimits<double>::Max();

	TMeshAABBTree3()
	{
		Mesh = nullptr;
	}

	TMeshAABBTree3(const TriangleMeshType* SourceMesh, bool bAutoBuild = true)
	{
		SetMesh(SourceMesh, bAutoBuild);
	}

	void SetMesh(const TriangleMeshType* SourceMesh, bool bAutoBuild = true)
	{
		Mesh = SourceMesh;
		MeshChangeStamp = 0;

		if (bAutoBuild)
		{
			Build();
		}
	}

	const TriangleMeshType* GetMesh() const
	{
		return Mesh;
	}

	/**
	 * @param bAllowUnsafeModifiedMeshQueries if true, then a built tree will still be considered valid even if the mesh change stamp has been modified
	 * @return true if tree is built and mesh is unchanged (identified based on change stamp)
	 */
	bool IsValid(bool bAllowUnsafeModifiedMeshQueries) const 
	{
		checkSlow(RootIndex >= 0);
		if (RootIndex < 0)
		{
			return false;
		}
		if (! bAllowUnsafeModifiedMeshQueries)
		{
			return (MeshChangeStamp == Mesh->GetChangeStamp());
		}
		return true;
	}

	void SetBuildOptions(int32 MaxBoxTriCount, GetSplitAxisFunc&& GetSplitAxisIn = MakeDefaultSplitAxisFunc())
	{
		TopDownLeafMaxTriCount = MaxBoxTriCount;
		GetSplitAxis = MoveTemp(GetSplitAxisIn);
	}

	void Build()
	{
		BuildTopDown(false);
		MeshChangeStamp = Mesh->GetChangeStamp();
	}

	void Build(const TArray<int32>& TriangleList)
	{
		BuildTopDown(false, TriangleList, TriangleList.Num());
		MeshChangeStamp = Mesh->GetChangeStamp();
	}

	virtual bool SupportsNearestTriangle() const override
	{
		return true;
	}

	/**
	 * Find the triangle closest to P, and distance to it, within distance MaxDist, or return InvalidID
	 * Use MeshQueries.TriangleDistance() to get more information
	 */
	virtual int FindNearestTriangle(
		const FVector3d& P, double& NearestDistSqr,
		const FQueryOptions& Options = FQueryOptions()
	) const override
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return IndexConstants::InvalidID;
			}
		}

		NearestDistSqr = (Options.MaxDistance < DOUBLE_MAX) ? Options.MaxDistance * Options.MaxDistance : DOUBLE_MAX;
		int tNearID = IndexConstants::InvalidID;
		find_nearest_tri(RootIndex, P, NearestDistSqr, tNearID, Options);
		return tNearID;
	}

	/**
	 * Get the overall bounding box of the whole tree
	 */
	FAxisAlignedBox3d GetBoundingBox() const
	{
		if (!ensure(RootIndex >= 0))
		{
			return FAxisAlignedBox3d::Empty();
		}
		else
		{
			return GetBox(RootIndex);
		}
	}

	/**
	 * Convenience function that calls FindNearestTriangle and then finds nearest point
	 * @return nearest point to Point, or Point itself if a nearest point was not found
	 */
	virtual FVector3d FindNearestPoint(
		const FVector3d& Point, const FQueryOptions& Options = FQueryOptions()
	) const
	{
		double NearestDistSqr;
		int32 NearTriID = FindNearestTriangle(Point, NearestDistSqr, Options);
		if (NearTriID >= 0)
		{
			FDistPoint3Triangle3d Query = TMeshQueries<TriangleMeshType>::TriangleDistance(*Mesh, NearTriID, Point);
			return Query.ClosestTrianglePoint;
		}
		return Point;
	}

	/**
	 * Test whether there is any triangle closer than sqrt(ThresholdDistanceSqr) to Point
	 * Note that FQueryOptions::MaxDistance is ignored by this query, as ThresholdDistanceSqr controls the query distance instead
	 * 
	 * @param OutTriangleID  The ID of a triangle (not necessarily the closest) within the distance threshold, or InvalidID if none found.
	 */
	virtual bool IsWithinDistanceSquared(
		const FVector3d& Point, double ThresholdDistanceSqr, int& OutTriangleID, const FQueryOptions& Options = FQueryOptions()
	) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		OutTriangleID = IndexConstants::InvalidID;
		find_nearest_tri<true>(RootIndex, Point, ThresholdDistanceSqr, OutTriangleID, Options);
		return OutTriangleID != IndexConstants::InvalidID;
	}

protected:
	// bEarlyStop causes the traversal to stop as soon as any triangle closer than NearestDistSqr is found
	// @return true if an early stop triangle was found
	template<bool bEarlyStop = false>
	bool find_nearest_tri(int IBox, const FVector3d& P, double& NearestDistSqr, int& TID, const FQueryOptions& Options) const
	{
		int idx = BoxToIndex[IBox];
		if (idx < TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}
				double fTriDistSqr = TMeshQueries<TriangleMeshType>::TriDistanceSqr(*Mesh, ti, P);
				if (fTriDistSqr < NearestDistSqr)
				{
					NearestDistSqr = fTriDistSqr;
					TID = ti;
					if constexpr (bEarlyStop)
					{
						return true;
					}
				}
			}
		}
		else
		{ // internal node, either 1 or 2 child boxes
			int iChild1 = IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1DistSqr = BoxDistanceSqr(iChild1, P);
				if (fChild1DistSqr <= NearestDistSqr)
				{
					bool bFoundEarly = find_nearest_tri<bEarlyStop>(iChild1, P, NearestDistSqr, TID, Options);
					if (bEarlyStop && bFoundEarly)
					{
						return true;
					}
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				double fChild1DistSqr = BoxDistanceSqr(iChild1, P);
				double fChild2DistSqr = BoxDistanceSqr(iChild2, P);
				if (fChild1DistSqr < fChild2DistSqr)
				{
					if (fChild1DistSqr < NearestDistSqr)
					{
						bool bFoundEarly1 = find_nearest_tri<bEarlyStop>(iChild1, P, NearestDistSqr, TID, Options);
						if (bEarlyStop && bFoundEarly1)
						{
							return true;
						}
						if (fChild2DistSqr < NearestDistSqr)
						{
							bool bFoundEarly2 = find_nearest_tri<bEarlyStop>(iChild2, P, NearestDistSqr, TID, Options);
							if (bEarlyStop && bFoundEarly2)
							{
								return true;
							}
						}
					}
				}
				else
				{
					if (fChild2DistSqr < NearestDistSqr)
					{
						bool bFoundEarly1 = find_nearest_tri<bEarlyStop>(iChild2, P, NearestDistSqr, TID, Options);
						if (bEarlyStop && bFoundEarly1)
						{
							return true;
						}
						if (fChild1DistSqr < NearestDistSqr)
						{
							bool bFoundEarly2 = find_nearest_tri<bEarlyStop>(iChild1, P, NearestDistSqr, TID, Options);
							if (bEarlyStop && bFoundEarly2)
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}




public:
	/**
	 * Find the Vertex closest to P, and distance to it, within distance MaxDist, or return InvalidID
	 */
	virtual int FindNearestVertex(const FVector3d& P, double& NearestDistSqr,
		double MaxDist = TNumericLimits<double>::Max(), const FQueryOptions& Options = FQueryOptions())
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return IndexConstants::InvalidID;
			}
		}

		NearestDistSqr = (MaxDist < DOUBLE_MAX) ? MaxDist * MaxDist : DOUBLE_MAX;
		int NearestVertexID = IndexConstants::InvalidID;
		find_nearest_vertex(RootIndex, P, NearestDistSqr, NearestVertexID, Options);
		return NearestVertexID;
	}


protected:
	void find_nearest_vertex(int IBox, const FVector3d& P, double& NearestDistSqr,
		int& NearestVertexID, const FQueryOptions& Options)
	{
		int idx = BoxToIndex[IBox];
		if (idx < TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}
				FIndex3i Triangle = Mesh->GetTriangle(ti);
				for (int j = 0; j < 3; ++j) 
				{
					double VertexDistSqr = DistanceSquared((FVector3d)Mesh->GetVertex(Triangle[j]), P);
					if (VertexDistSqr < NearestDistSqr) 
					{
						NearestDistSqr = VertexDistSqr;
						NearestVertexID = Triangle[j];
					}
				}
			}
		}
		else
		{ // internal node, either 1 or 2 child boxes
			int iChild1 = IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1DistSqr = BoxDistanceSqr(iChild1, P);
				if (fChild1DistSqr <= NearestDistSqr)
				{
					find_nearest_vertex(iChild1, P, NearestDistSqr, NearestVertexID, Options);
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				double fChild1DistSqr = BoxDistanceSqr(iChild1, P);
				double fChild2DistSqr = BoxDistanceSqr(iChild2, P);
				if (fChild1DistSqr < fChild2DistSqr)
				{
					if (fChild1DistSqr < NearestDistSqr)
					{
						find_nearest_vertex(iChild1, P, NearestDistSqr, NearestVertexID, Options);
						if (fChild2DistSqr < NearestDistSqr)
						{
							find_nearest_vertex(iChild2, P, NearestDistSqr, NearestVertexID, Options);
						}
					}
				}
				else
				{
					if (fChild2DistSqr < NearestDistSqr)
					{
						find_nearest_vertex(iChild2, P, NearestDistSqr, NearestVertexID, Options);
						if (fChild1DistSqr < NearestDistSqr)
						{
							find_nearest_vertex(iChild1, P, NearestDistSqr, NearestVertexID, Options);
						}
					}
				}
			}
		}
	}





public:
	virtual bool SupportsTriangleRayIntersection() const override
	{
		return true;
	}

	inline virtual int FindNearestHitTriangle(
		const FRay3d& Ray, const FQueryOptions& Options = FQueryOptions()) const override
	{
		double NearestT;
		int TID;
		FVector3d BaryCoords;
		FindNearestHitTriangleImpl<FRay3d>(Ray, NearestT, TID, BaryCoords, Options);
		return TID;
	}

	virtual bool FindNearestHitTriangle(const FRay3d& Ray, double& NearestT, int& TID, const FQueryOptions& Options = FQueryOptions()) const override
	{
		FVector3d BaryCoords;
		return FindNearestHitTriangleImpl<FRay3d>(Ray, NearestT, TID, BaryCoords, Options);
	}

	virtual bool FindNearestHitTriangle(
		const FRay3d& Ray, double& NearestT, int& TID, FVector3d& BaryCoords,
		const FQueryOptions& Options = FQueryOptions()) const override
	{
		return FindNearestHitTriangleImpl<FRay3d>(Ray, NearestT, TID, BaryCoords, Options);
	}

	bool FindNearestHitTriangle(
		const FWatertightRay3d& Ray, double& NearestT, int& TID, FVector3d& BaryCoords,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		return FindNearestHitTriangleImpl<FWatertightRay3d>(Ray, NearestT, TID, BaryCoords, Options);
	}
	
	virtual bool FindAllHitTriangles(
		const FRay3d& Ray, TArray<MeshIntersection::FHitIntersectionResult>& OutHits,
		const FQueryOptions& Options = FQueryOptions()) const override
	{
		return FindAllHitTrianglesImpl<FRay3d>(Ray, OutHits, Options);
	}

	bool FindAllHitTriangles(
		const FWatertightRay3d& Ray, TArray<MeshIntersection::FHitIntersectionResult>& OutHits,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		return FindAllHitTrianglesImpl<FWatertightRay3d>(Ray, OutHits, Options);
	}
private:

	template<typename RayType>
	bool FindNearestHitTriangleImpl(
		const RayType& Ray, double& NearestT, int& TID, FVector3d& BaryCoords,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		TID = IndexConstants::InvalidID;
		BaryCoords = FVector3d::Zero();

		// Note: using TNumericLimits<float>::Max() here because we need to use <= to compare Box hit
		//   to NearestT, and Box hit returns TNumericLimits<double>::Max() on no-hit. So, if we set
		//   nearestT to TNumericLimits<double>::Max(), then we will test all boxes (!)
		NearestT = (Options.MaxDistance < TNumericLimits<float>::Max()) ? Options.MaxDistance : TNumericLimits<float>::Max();

		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		FindHitTriangle(RootIndex, Ray, NearestT, TID, BaryCoords, Options);
		return TID != IndexConstants::InvalidID;
	}

	template<typename RayType>
	bool FindAllHitTrianglesImpl(
		const RayType& Ray, TArray<MeshIntersection::FHitIntersectionResult>& OutHits,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		using HitResult = MeshIntersection::FHitIntersectionResult;
		
		FindHitTriangles(RootIndex, Ray, OutHits, Options);

		if (OutHits.IsEmpty())
		{
			return false;
		}

		// sort all hits by distance		
		OutHits.Sort([](const HitResult& Hit0, const HitResult& Hit1)
		{
			return Hit0.Distance < Hit1.Distance;
		});
		
		return true;
	}

protected:
	template<typename RayType = FRay3d>
	void FindHitTriangle(
		int IBox, const RayType& Ray, double& NearestT, int& TID, FVector3d& BaryCoords,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		int idx = BoxToIndex[IBox];
		if (idx < TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			FTriangle3d Triangle;
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}

				Mesh->GetTriVertices(ti, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
				TIntrRay3Triangle3<double, RayType> Query(Ray, Triangle);
				if (Query.Find())
				{
					if (Query.RayParameter < NearestT)
					{
						NearestT = Query.RayParameter;
						TID = ti;
						BaryCoords = Query.TriangleBaryCoords;
					}
				}
			}
		}
		else
		{ // internal node, either 1 or 2 child boxes
			double e = FMathd::ZeroTolerance;

			int iChild1 = IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				if (fChild1T <= NearestT + e)
				{
					FindHitTriangle(iChild1, Ray, NearestT, TID, BaryCoords, Options);
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				double fChild2T = box_ray_intersect_t(iChild2, Ray);
				if (fChild1T < fChild2T)
				{
					if (fChild1T <= NearestT + e)
					{
						FindHitTriangle(iChild1, Ray, NearestT, TID, BaryCoords, Options);
						if (fChild2T <= NearestT + e)
						{
							FindHitTriangle(iChild2, Ray, NearestT, TID, BaryCoords, Options);
						}
					}
				}
				else
				{
					if (fChild2T <= NearestT + e)
					{
						FindHitTriangle(iChild2, Ray, NearestT, TID, BaryCoords, Options);
						if (fChild1T <= NearestT + e)
						{
							FindHitTriangle(iChild1, Ray, NearestT, TID, BaryCoords, Options);
						}
					}
				}
			}
		}
	}

	template<typename RayType = FRay3d>
	void FindHitTriangles(
		int IBox, const RayType& Ray, TArray<MeshIntersection::FHitIntersectionResult>& Intersections,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		// Note: using TNumericLimits<float>::Max() here because we need to use <= to compare Box hit
        //   to MaxDistance, and Box hit returns TNumericLimits<double>::Max() on no-hit. So, if we set
        //   MaxDistance to TNumericLimits<double>::Max(), then we will test all boxes (!)
        const double MaxDistance = (Options.MaxDistance < TNumericLimits<float>::Max()) ? Options.MaxDistance : TNumericLimits<float>::Max();
		static constexpr double e = FMathd::ZeroTolerance;
		
		int idx = BoxToIndex[IBox];
		if (idx < TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			FTriangle3d Triangle;
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}

				Mesh->GetTriVertices(ti, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
				TIntrRay3Triangle3<double, RayType> Query(Ray, Triangle);
				if (Query.Find() && Query.RayParameter <= MaxDistance + e)
				{
					MeshIntersection::FHitIntersectionResult NewIntersection({ti, Query.RayParameter, Query.TriangleBaryCoords});
					Intersections.Emplace(MoveTemp(NewIntersection));
				}
			}
		}
		else
		{ // internal node, either 1 or 2 child boxes
			int iChild1 = IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				if (fChild1T <= MaxDistance + e)
				{
					FindHitTriangles(iChild1, Ray, Intersections, Options);
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				double fChild2T = box_ray_intersect_t(iChild2, Ray);
				if (fChild1T < fChild2T)
				{
					if (fChild1T <= MaxDistance + e)
					{
						FindHitTriangles(iChild1, Ray, Intersections, Options);
						if (fChild2T <= MaxDistance + e)
						{
							FindHitTriangles(iChild2, Ray, Intersections, Options);
						}
					}
				}
				else
				{
					if (fChild2T <= MaxDistance + e)
					{
						FindHitTriangles(iChild2, Ray, Intersections, Options);
						if (fChild1T <= MaxDistance + e)
						{
							FindHitTriangles(iChild1, Ray, Intersections, Options);
						}
					}
				}
			}
		}
	}

public:
	/**
	 * @return true if ray hits the mesh at some point (doesn't necessarily find the nearest hit).
	 */
	virtual bool TestAnyHitTriangle(const FRay3d& Ray, const FQueryOptions& Options = FQueryOptions()) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		double MaxDistance = (Options.MaxDistance < TNumericLimits<float>::Max()) ? Options.MaxDistance : TNumericLimits<float>::Max();
		int32 HitTID = IndexConstants::InvalidID;
		bool bFoundHit = TestAnyHitTriangle(RootIndex, Ray, HitTID, MaxDistance, Options);
		return bFoundHit;
	}

protected:
	bool TestAnyHitTriangle(
		int IBox, const FRay3d& Ray, int32& HitTIDOut, double MaxDistance,
		const FQueryOptions& Options = FQueryOptions()) const
	{
		int idx = BoxToIndex[IBox];
		if (idx < TrianglesEnd)
		{ // triangle-list case, array is [N t1 t2 ... tN]
			FVector3d A, B, C;
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}
				Mesh->GetTriVertices(ti, A, B, C);
				// TODO: RayTriangleTest does most of the work of Query.Find(); test if it's faster to just do Query.Find() directly.
				if ( IntersectionUtil::RayTriangleTest(Ray.Origin, Ray.Direction, A,B,C) ) 
				{
					FIntrRay3Triangle3d Query = FIntrRay3Triangle3d(Ray, FTriangle3d(A,B,C));
					if (Query.Find() && Query.RayParameter < Options.MaxDistance)
					{
						return true;
					}
				}
			}
		}
		else
		{ // internal node, either 1 or 2 child boxes
			double e = FMathd::ZeroTolerance;
			int iChild1 = IndexList[idx];
			if (iChild1 < 0)
			{ // 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				if (fChild1T <= MaxDistance + e)
				{
					if (TestAnyHitTriangle(iChild1, Ray, HitTIDOut, MaxDistance, Options))
					{
						return true;
					}
				}
			}
			else
			{ // 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				double fChild1T = box_ray_intersect_t(iChild1, Ray);
				double fChild2T = box_ray_intersect_t(iChild2, Ray);
				if (fChild1T < fChild2T)
				{
					if (fChild1T <= MaxDistance + e)
					{
						if (TestAnyHitTriangle(iChild1, Ray, HitTIDOut, MaxDistance, Options))
						{
							return true;
						}
						if (fChild2T <= MaxDistance + e)
						{
							if (TestAnyHitTriangle(iChild2, Ray, HitTIDOut, MaxDistance, Options))
							{
								return true;
							}
						}
					}
				}
				else
				{
					if (fChild2T <= MaxDistance + e)
					{
						if (TestAnyHitTriangle(iChild2, Ray, HitTIDOut, MaxDistance, Options))
						{
							return true;
						}
						if (fChild1T <= MaxDistance + e)
						{
							if (TestAnyHitTriangle(iChild1, Ray, HitTIDOut, MaxDistance, Options))
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}




public:
	/**
	 * Find nearest pair of triangles on this tree with OtherTree, within Options.MaxDistance.
	 * TransformF transforms vertices of OtherTree into our coordinates. can be null.
	 * returns triangle-id pair (my_tri,other_tri), or FIndex2i::Invalid if not found within max_dist
	 * Use MeshQueries.TrianglesDistance() to get more information
	 * Note: Only uses MaxDistance from Options; OtherTreeOptions.MaxDistance is not used
	 */
	virtual FIndex2i FindNearestTriangles(
		TMeshAABBTree3& OtherTree, const TFunction<FVector3d(const FVector3d&)>& TransformF,
		double& Distance, const FQueryOptions& Options = FQueryOptions(), const FQueryOptions& OtherTreeOptions = FQueryOptions()
	)
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return FIndex2i::Invalid();
			}
		}

		double NearestSqr = FMathd::MaxReal;
		if (Options.MaxDistance < FMathd::MaxReal)
		{
			NearestSqr = Options.MaxDistance * Options.MaxDistance;
		}
		FIndex2i NearestPair = FIndex2i::Invalid();

		find_nearest_triangles(RootIndex, OtherTree, TransformF, OtherTree.RootIndex, 0, NearestSqr, NearestPair, Options, OtherTreeOptions);
		Distance = (NearestSqr < FMathd::MaxReal) ? FMathd::Sqrt(NearestSqr) : FMathd::MaxReal;
		return NearestPair;
	}



	virtual bool SupportsPointContainment() const override
	{
		return false;
	}

	virtual bool IsInside(const FVector3d& P) const override
	{
		return false;
	}

	class FTreeTraversal
	{
	public:
		// The traversal is terminated at the current node if this function returns false
		TFunction<bool(const FAxisAlignedBox3d&, int)> NextBoxF = [](const FAxisAlignedBox3d& Box, int Depth) { return true; };

		// These functions are called in sequence if NextBoxF returned true and the current node is a leaf
		TFunction<void(int, int)> BeginBoxTrianglesF = [](int BoxID, int Depth) {};
		TFunction<void(int)> NextTriangleF = [](int TriangleID) {}; // Call may be skipped by TriangleFilterF
		TFunction<void(int)> EndBoxTrianglesF = [](int BoxID) {};
	};

	/**
	 * Hierarchically descend through the tree Nodes, calling the FTreeTraversal functions at each level
	 */
	virtual void DoTraversal(FTreeTraversal& Traversal, const FQueryOptions& Options = FQueryOptions()) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return;
			}
		}

		if (Traversal.NextBoxF(GetBox(RootIndex), 0))
		{
			TreeTraversalImpl(RootIndex, 0, Traversal, Options);
		}
	}

protected:
	// Traversal implementation. you can override to customize this if necessary.
	virtual void TreeTraversalImpl(
		int IBox, int Depth, FTreeTraversal& Traversal, const FQueryOptions& Options
	) const
	{
		int idx = BoxToIndex[IBox];

		if (idx < TrianglesEnd)
		{
			Traversal.BeginBoxTrianglesF(IBox, Depth);

			// triangle-list case, array is [N t1 t2 ... tN]
			int n = IndexList[idx];
			for (int i = 1; i <= n; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}
				Traversal.NextTriangleF(ti);
			}

			Traversal.EndBoxTrianglesF(IBox);
		}
		else
		{
			int i0 = IndexList[idx];
			if (i0 < 0)
			{
				// negative index means we only have one 'child' Box to descend into
				i0 = (-i0) - 1;
				if (Traversal.NextBoxF(GetBox(i0), Depth + 1))
				{
					TreeTraversalImpl(i0, Depth + 1, Traversal, Options);
				}
			}
			else
			{
				// positive index, two sequential child Box indices to descend into
				i0 = i0 - 1;
				if (Traversal.NextBoxF(GetBox(i0), Depth + 1))
				{
					TreeTraversalImpl(i0, Depth + 1, Traversal, Options);
				}
				
				int i1 = IndexList[idx + 1] - 1;
				if (Traversal.NextBoxF(GetBox(i1), Depth + 1))
				{
					TreeTraversalImpl(i1, Depth + 1, Traversal, Options);
				}
			}
		}
	}


public:
	/**
	 * return true if *any* triangle of TestMesh intersects with our tree.
	 * If TestMeshBounds is not empty, only test collision if the provided bounding box intersects the root AABB box
	 * Use TransformF to transform vertices of TestMesh into space of this tree.
	 */
	virtual bool TestIntersection(
		const TriangleMeshType* TestMesh, FAxisAlignedBox3d TestMeshBounds = FAxisAlignedBox3d::Empty(),
		const TFunction<FVector3d(const FVector3d&)>& TransformF = nullptr,
		const FQueryOptions& Options = FQueryOptions()
	) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		if (!TestMeshBounds.IsEmpty())
		{
			if (TransformF != nullptr)
			{
				TestMeshBounds = FAxisAlignedBox3d(TestMeshBounds,
					[&](const FVector3d& P) { return TransformF(P); });
			}
			if (box_box_intersect(RootIndex, TestMeshBounds) == false)
			{
				return false;
			}
		}

		FTriangle3d TestTri;
		for (int TID = 0, N = TestMesh->MaxTriangleID(); TID < N; TID++)
		{
			if (TransformF != nullptr)
			{
				FIndex3i Tri = TestMesh->GetTriangle(TID);
				TestTri.V[0] = TransformF((FVector3d)TestMesh->GetVertex(Tri.A));
				TestTri.V[1] = TransformF((FVector3d)TestMesh->GetVertex(Tri.B));
				TestTri.V[2] = TransformF((FVector3d)TestMesh->GetVertex(Tri.C));
			}
			else
			{
				TestMesh->GetTriVertices(TID, TestTri.V[0], TestTri.V[1], TestTri.V[2]);
			}
			if (TestIntersection(TestTri, Options))
			{
				return true;
			}
		}
		return false;
	}


	/**
	 * Returns true if there is *any* intersection between our mesh and 'other' mesh.
	 * TransformF takes vertices of OtherTree into our tree - can be null if in same coord space
	 */
	virtual bool TestIntersection(
		const TMeshAABBTree3& OtherTree,
		const TFunction<FVector3d(const FVector3d&)>& TransformF = nullptr,
		const FQueryOptions& Options = FQueryOptions(), const FQueryOptions& OtherTreeOptions = FQueryOptions()
	) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		if (find_any_intersection(RootIndex, OtherTree, TransformF, OtherTree.RootIndex, 0,
			[this](const FTriangle3d& A, const FTriangle3d& B)
			{
				return TMeshAABBTree3<TriangleMeshType>::TriangleIntersectionFilter(A, B);
			}, Options, OtherTreeOptions))
		{
			return true;
		}

		return false;
	}

	/**
	 * Returns true if triangle intersects any triangle of our mesh
	 */
	virtual bool TestIntersection(
		const FTriangle3d& Triangle,
		const FQueryOptions& Options = FQueryOptions()
	) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		FAxisAlignedBox3d triBounds(Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		int interTri = find_any_intersection(RootIndex, Triangle, triBounds,
			[](const FTriangle3d& A, const FTriangle3d& B)
			{
				return TMeshAABBTree3<TriangleMeshType>::TriangleIntersectionFilter(A, B);
			},
		Options);
		return (interTri >= 0);
	}


	/**
	 * Compute all intersections between two meshes.
	 * TransformF argument transforms vertices of OtherTree to our tree (can be null if in same coord space)
	 * Returns pairs of intersecting triangles, which could intersect in either point or segment
	 * Currently *does not* return coplanar intersections.
	 */
	virtual MeshIntersection::FIntersectionsQueryResult FindAllIntersections(
		const TMeshAABBTree3& OtherTree, const TFunction<FVector3d(const FVector3d&)>& TransformF = nullptr,
		const FQueryOptions& Options = FQueryOptions(), const FQueryOptions& OtherTreeOptions = FQueryOptions(),
		TFunction<bool(FIntrTriangle3Triangle3d&)> IntersectionFn = nullptr
	) const
	{
		if (!IntersectionFn)
		{
			IntersectionFn = [](FIntrTriangle3Triangle3d& Intr)
			{
				return TMeshAABBTree3<TriangleMeshType>::TriangleIntersection(Intr);
			};
		}

		MeshIntersection::FIntersectionsQueryResult result;

		if ( ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false )
		{
			return result;
		}

		find_intersections(RootIndex, OtherTree, TransformF, OtherTree.RootIndex, 0, result, 
						   IntersectionFn, Options, OtherTreeOptions);

		return result;
	}

	/**
	 * Compute self intersections on our mesh.
	 * Returns pairs of intersecting triangles, which could intersect in either point or segment
	 * Currently *does not* return coplanar intersections.
	 *
	 * @param bIgnoreTopoConnected Ignore intersections between triangles that share a vertex (if false a lot of triangles that simply share an edge will be counted as self intersecting)
	 */
	virtual MeshIntersection::FIntersectionsQueryResult FindAllSelfIntersections(
		bool bIgnoreTopoConnected = true,
		const FQueryOptions& Options = FQueryOptions(),
		TFunction<bool(FIntrTriangle3Triangle3d&)> IntersectionFn = nullptr
	) const
	{
		if (!IntersectionFn)
		{
			IntersectionFn = [](FIntrTriangle3Triangle3d& Intr)
			{
				return TMeshAABBTree3<TriangleMeshType>::TriangleIntersection(Intr);
			};
		}

		MeshIntersection::FIntersectionsQueryResult Result;

		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return Result;
			}
		}

		find_self_intersections(&Result, bIgnoreTopoConnected, IntersectionFn, Options);

		return Result;
	}

	virtual bool TestSelfIntersection(bool bIgnoreTopoConnected = true, const FQueryOptions& Options = FQueryOptions()) const
	{
		if constexpr (bValidatePerAccess)
		{
			if (ensure(IsValid(Options.bAllowUnsafeModifiedMeshQueries)) == false)
			{
				return false;
			}
		}

		return find_self_intersections(nullptr, bIgnoreTopoConnected, 
			[](FIntrTriangle3Triangle3d& Intr)
			{
				// only compute the boolean test value, and manually set result, to skip computing the actual intersection
				bool bIsIntersecting = TMeshAABBTree3<TriangleMeshType>::TriangleIntersectionFilter(Intr.GetTriangle0(), Intr.GetTriangle1());
				Intr.SetResult(bIsIntersecting);
				return bIsIntersecting;
			},
			Options);
	}


protected:
	//
	// Internals - data structures, construction, etc
	//

	FAxisAlignedBox3d GetBox(int IBox) const
	{
		const FVector3d& c = BoxCenters[IBox];
		const FVector3d& e = BoxExtents[IBox];
		FVector3d Min = c - e, Max = c + e;
		return FAxisAlignedBox3d(Min, Max);
	}
	FAxisAlignedBox3d GetBox(int iBox, const TFunction<FVector3d(const FVector3d&)>& TransformF) const
	{
		if (TransformF != nullptr)
		{
			FAxisAlignedBox3d box = GetBox(iBox);
			return FAxisAlignedBox3d(box, 
				[&](const FVector3d& P) { return TransformF(P); });
		}
		else
		{
			return GetBox(iBox);
		}
	}

	FAxisAlignedBox3d GetBoxEps(int IBox, double Epsilon = FMathd::ZeroTolerance) const
	{
		const FVector3d& c = BoxCenters[IBox];
		FVector3d e = BoxExtents[IBox];
		e[0] += Epsilon;
		e[1] += Epsilon;
		e[2] += Epsilon;
		FVector3d Min = c - e, Max = c + e;
		return FAxisAlignedBox3d(Min, Max);
	}

protected:
	// TODO: move BoxEps to IMeshSpatial::FQueryOptions
	double BoxEps = FMathd::ZeroTolerance;
public:

	/**
	 * Sets the box intersection tolerance
	 * TODO: move into the IMeshSpatial::FQueryOptions and delete this function
	 */
	void SetTolerance(double Tolerance)
	{
		BoxEps = Tolerance;
	}

	double BoxDistanceSqr(int IBox, const FVector3d& V) const
	{
		const FVector3d& c = BoxCenters[IBox];
		const FVector3d& e = BoxExtents[IBox];

		// per-axis delta is max(abs(P-c) - e, 0)... ?
		double dx = FMath::Max(fabs(V.X - c.X) - e.X, 0.0);
		double dy = FMath::Max(fabs(V.Y - c.Y) - e.Y, 0.0);
		double dz = FMath::Max(fabs(V.Z - c.Z) - e.Z, 0.0);
		return dx * dx + dy * dy + dz * dz;
	}

	bool box_contains(int IBox, const FVector3d& P) const
	{
		FAxisAlignedBox3d Box = GetBoxEps(IBox, BoxEps);
		return Box.Contains(P);
	}

	template<typename RayType>
	double box_ray_intersect_t(int IBox, const RayType& Ray) const
	{
		const FVector3d& c = BoxCenters[IBox];
		FVector3d e = BoxExtents[IBox] + BoxEps;
		FAxisAlignedBox3d Box(c - e, c + e);

		double ray_t = TNumericLimits<double>::Max();
		if (FIntrRay3AxisAlignedBox3d::FindIntersection(Ray.Origin, Ray.Direction, Box, ray_t))
		{
			return ray_t;
		}
		else
		{
			return TNumericLimits<double>::Max();
		}
	}

	bool box_box_intersect(int IBox, const FAxisAlignedBox3d& TestBox) const
	{
		// [TODO] could compute this w/o constructing box
		FAxisAlignedBox3d Box = GetBoxEps(IBox, BoxEps);

		return Box.Intersects(TestBox);
	}

	SIZE_T GetByteCount() const
	{
		return BoxToIndex.GetByteCount() + BoxCenters.GetByteCount() + BoxExtents.GetByteCount() + IndexList.GetByteCount();
	}

	// storage for Box Nodes.
	//   - BoxToIndex is a pointer into IndexList
	//   - BoxCenters and BoxExtents are the Centers/extents of the bounding boxes
	TDynamicVector<int> BoxToIndex;
	TDynamicVector<FVector3d> BoxCenters;
	TDynamicVector<FVector3d> BoxExtents;

	// list of indices for a given Box. There is *no* marker/sentinel between
	// boxes, you have to get the starting index from BoxToIndex[]
	//
	// There are three kinds of records:
	//   - if i < TrianglesEnd, then the list is a number of Triangles,
	//       stored as [N t1 t2 t3 ... tN]
	//   - if i > TrianglesEnd and IndexList[i] < 0, this is a single-child
	//       internal Box, with index (-IndexList[i])-1     (shift-by-one in case actual value is 0!)
	//   - if i > TrianglesEnd and IndexList[i] > 0, this is a two-child
	//       internal Box, with indices IndexList[i]-1 and IndexList[i+1]-1
	TDynamicVector<int> IndexList;

	// IndexList[i] for i < TrianglesEnd is a triangle-index list, otherwise Box-index pair/single
	int TrianglesEnd = -1;

	// BoxToIndex[RootIndex] is the root node of the tree
	int RootIndex = -1;

	struct FBoxesSet
	{
		TDynamicVector<int> BoxToIndex;
		TDynamicVector<FVector3d> BoxCenters;
		TDynamicVector<FVector3d> BoxExtents;
		TDynamicVector<int> IndexList;
		int IBoxCur;
		int IIndicesCur;
		FBoxesSet()
		{
			IBoxCur = 0;
			IIndicesCur = 0;
		}
	};


	void BuildTopDown(bool bSorted)
	{
		// build list of valid Triangles & Centers. We skip any
		// Triangles that have infinite/garbage vertices...
		TArray<int> Triangles;
		Triangles.Reserve(Mesh->TriangleCount());
		TArray<FVector3d> Centers;
		Centers.Reserve(Mesh->TriangleCount());
		for (int ti = 0; ti < Mesh->MaxTriangleID(); ti++)
		{
			if (Mesh->IsTriangle(ti) == false)
			{
				continue;
			}
			FVector3d centroid = TMeshQueries<TriangleMeshType>::GetTriCentroid(*Mesh, ti);
			double d2 = centroid.SquaredLength();
			bool bInvalid = FMathd::IsNaN(d2) || (FMathd::IsFinite(d2) == false);
			if (bInvalid == false)
			{
				Triangles.Add(ti);
				Centers.Add(TMeshQueries<TriangleMeshType>::GetTriCentroid(*Mesh, ti));
			} // otherwise skip this Tri
		}
		checkSlow(Triangles.Num() == Centers.Num());

		BuildTopDown(Triangles, Centers, Triangles.Num());
	}


	template<typename TriIndexEnumerable>
	void BuildTopDown(bool bSorted, TriIndexEnumerable TriangleList, int32 NumTriangles)
	{
		// build list of valid Triangles & Centers. We skip any
		// Triangles that have infinite/garbage vertices...
		TArray<int32> Triangles;
		Triangles.Reserve(NumTriangles);
		TArray<FVector3d> Centers;
		Centers.Reserve(NumTriangles);
		for (int32 ti : TriangleList)
		{
			if ( Mesh->IsTriangle(ti) == false)
			{
				continue;
			}
			FVector3d centroid = TMeshQueries<TriangleMeshType>::GetTriCentroid(*Mesh, ti);
			double d2 = centroid.SquaredLength();
			bool bInvalid = FMathd::IsNaN(d2) || (FMathd::IsFinite(d2) == false);
			if (bInvalid == false)
			{
				Triangles.Add(ti);
				Centers.Add(TMeshQueries<TriangleMeshType>::GetTriCentroid(*Mesh, ti));
			} // otherwise skip this Tri
		}
		checkSlow(Triangles.Num() == Centers.Num());

		BuildTopDown(Triangles, Centers, Triangles.Num());
	}


	void BuildTopDown(TArray<int>& Triangles, TArray<FVector3d>& Centers, int32 NumTriangles)
	{
		FBoxesSet Tris;
		FBoxesSet Nodes;
		FAxisAlignedBox3d rootBox;
		int rootnode =
			//(bSorted) ? split_tri_set_sorted(Triangles, Centers, 0, NumTriangles, 0, TopDownLeafMaxTriCount, Tris, Nodes, out rootBox) :
			SplitTriSetMidpoint(Triangles, Centers, 0, NumTriangles, 0, TopDownLeafMaxTriCount, Tris, Nodes, rootBox);

		BoxToIndex = Tris.BoxToIndex;
		BoxCenters = Tris.BoxCenters;
		BoxExtents = Tris.BoxExtents;
		IndexList = Tris.IndexList;
		TrianglesEnd = Tris.IIndicesCur;
		int iIndexShift = TrianglesEnd;
		int iBoxShift = Tris.IBoxCur;

		// ok now append internal node boxes & index ptrs
		for (int32 i = 0; i < Nodes.IBoxCur; ++i)
		{
			FVector3d NodeBoxCenter = Nodes.BoxCenters[i];		// cannot pass as argument in case a resize happens
			BoxCenters.InsertAt(NodeBoxCenter, iBoxShift + i);
			FVector3d NodeBoxExtents = Nodes.BoxExtents[i];
			BoxExtents.InsertAt(NodeBoxExtents, iBoxShift + i);
			// internal node indices are shifted
			int NodeBoxIndex = Nodes.BoxToIndex[i];
			BoxToIndex.InsertAt(iIndexShift + NodeBoxIndex, iBoxShift + i);
		}

		// now append index list
		for (int32 i = 0; i < Nodes.IIndicesCur; ++i)
		{
			int child_box = Nodes.IndexList[i];
			if (child_box < 0)
			{ // this is a Triangles Box
				child_box = (-child_box) - 1;
			}
			else
			{
				child_box += iBoxShift;
			}
			child_box = child_box + 1;
			IndexList.InsertAt(child_box, iIndexShift + i);
		}

		RootIndex = rootnode + iBoxShift;
	}


	int SplitTriSetMidpoint(
		TArray<int>& Triangles,
		TArray<FVector3d>& Centers,
		int IStart, int ICount, int Depth, int MinTriCount,
		FBoxesSet& Tris, FBoxesSet& Nodes, FAxisAlignedBox3d& Box)
	{
		Box = (Triangles.Num() > 0) ?
			FAxisAlignedBox3d::Empty() : FAxisAlignedBox3d(FVector3d::Zero(), 0.0);
		int IBox = -1;

		if (ICount <= MinTriCount)
		{
			// append new Triangles Box
			IBox = Tris.IBoxCur++;
			Tris.BoxToIndex.InsertAt(Tris.IIndicesCur, IBox);

			Tris.IndexList.InsertAt(ICount, Tris.IIndicesCur++);
			for (int i = 0; i < ICount; ++i)
			{
				Tris.IndexList.InsertAt(Triangles[IStart + i], Tris.IIndicesCur++);
				Box.Contain(TMeshQueries<TriangleMeshType>::GetTriBounds(*Mesh, Triangles[IStart + i]));
			}

			Tris.BoxCenters.InsertAt(Box.Center(), IBox);
			Tris.BoxExtents.InsertAt(Box.Extents(), IBox);

			return -(IBox + 1);
		}

		//compute interval along an axis and find midpoint
		int axis = GetSplitAxis(Depth, Box);
		FInterval1d interval = FInterval1d::Empty();
		for (int i = 0; i < ICount; ++i)
		{
			interval.Contain(Centers[IStart + i][axis]);
		}
		double midpoint = interval.Center();

		int n0, n1;
		if (interval.Length() > FMathd::ZeroTolerance)
		{
			// we have to re-sort the Centers & Triangles lists so that Centers < midpoint
			// are first, so that we can recurse on the two subsets. We walk in from each side,
			// until we find two out-of-order locations, then we swap them.
			int l = 0;
			int r = ICount - 1;
			while (l < r)
			{
				// Float rounding can collapse midpoint onto an extremum, in which case
				// every value matches one of the conditions; the scans must be bounded.
				while (l < ICount && Centers[IStart + l][axis] <= midpoint)
				{
					l++;
				}
				while (r >= 0 && Centers[IStart + r][axis] > midpoint)
				{
					r--;
				}
				if (l >= r)
				{
					break; //done!
						   //swap
				}
				FVector3d tmpc = Centers[IStart + l];
				Centers[IStart + l] = Centers[IStart + r];
				Centers[IStart + r] = tmpc;
				int tmpt = Triangles[IStart + l];
				Triangles[IStart + l] = Triangles[IStart + r];
				Triangles[IStart + r] = tmpt;
			}

			n0 = l;
			n1 = ICount - n0;
			if (n0 == 0 || n0 == ICount)
			{
				// Partition degenerated to a single side; fall back to half-split.
				n0 = ICount / 2;
				n1 = ICount - n0;
			}
			checkSlow(n0 >= 1 && n1 >= 1);
		}
		else
		{
			// interval is near-empty, so no point trying to do sorting, just split half and half
			n0 = ICount / 2;
			n1 = ICount - n0;
		}

		// create child boxes
		FAxisAlignedBox3d box1;
		int child0 = SplitTriSetMidpoint(Triangles, Centers, IStart, n0, Depth + 1, MinTriCount, Tris, Nodes, Box);
		int child1 = SplitTriSetMidpoint(Triangles, Centers, IStart + n0, n1, Depth + 1, MinTriCount, Tris, Nodes, box1);
		Box.Contain(box1);

		// append new Box
		IBox = Nodes.IBoxCur++;
		Nodes.BoxToIndex.InsertAt(Nodes.IIndicesCur, IBox);

		Nodes.IndexList.InsertAt(child0, Nodes.IIndicesCur++);
		Nodes.IndexList.InsertAt(child1, Nodes.IIndicesCur++);

		Nodes.BoxCenters.InsertAt(Box.Center(), IBox);
		Nodes.BoxExtents.InsertAt(Box.Extents(), IBox);

		return IBox;
	}


	void find_nearest_triangles(
		int iBox, TMeshAABBTree3& OtherTree, const TFunction<FVector3d(const FVector3d&)>& TransformF,
		int oBox, int depth, double &nearest_sqr, FIndex2i &nearest_pair,
		const FQueryOptions& Options, const FQueryOptions& OtherTreeOptions
	) const
	{
		int idx = BoxToIndex[iBox];
		int odx = OtherTree.BoxToIndex[oBox];

		if (idx < TrianglesEnd && odx < OtherTree.TrianglesEnd)
		{
			// ok we are at triangles for both trees, do triangle-level testing
			FTriangle3d Tri, otri;
			int num_tris = IndexList[idx], onum_tris = OtherTree.IndexList[odx];

			FDistTriangle3Triangle3d dist;

			// outer iteration is "other" tris that need to be transformed (more expensive)
			for (int j = 1; j <= onum_tris; ++j)
			{
				int tj = OtherTree.IndexList[odx + j];
				if (OtherTreeOptions.TriangleFilterF != nullptr && OtherTreeOptions.TriangleFilterF(tj) == false)
				{
					continue;
				}
				OtherTree.Mesh->GetTriVertices(tj, otri.V[0], otri.V[1], otri.V[2]);
				if (TransformF != nullptr)
				{
					otri.V[0] = TransformF(otri.V[0]);
					otri.V[1] = TransformF(otri.V[1]);
					otri.V[2] = TransformF(otri.V[2]);
				}
				dist.SetTriangle(0, otri);

				// inner iteration over "our" triangles
				for (int i = 1; i <= num_tris; ++i)
				{
					int ti = IndexList[idx + i];
					if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
					{
						continue;
					}
					Mesh->GetTriVertices(ti, Tri.V[0], Tri.V[1], Tri.V[2]);
					dist.SetTriangle(1, Tri);
					double dist_sqr = dist.GetSquared();
					if (dist_sqr < nearest_sqr)
					{
						nearest_sqr = dist_sqr;
						nearest_pair = FIndex2i(ti, tj);
					}
				}
			}

			return;
		}

		// we either descend "our" tree or the other tree
		//   - if we have hit triangles on "our" tree, we have to descend other
		//   - if we hit triangles on "other", we have to descend ours
		//   - otherwise, we alternate at each depth. This produces wider
		//     branching but is significantly faster (~10x) for both hits and misses
		bool bDescendOther = (idx < TrianglesEnd || depth % 2 == 0);
		if (bDescendOther && odx < OtherTree.TrianglesEnd)
		{
			bDescendOther = false;      // can't
		}

		if (bDescendOther)
		{
			// ok we reached triangles on our side but we need to still reach triangles on
			// the other side, so we descend "their" children
			FAxisAlignedBox3d bounds = GetBox(iBox);

			int oChild1 = OtherTree.IndexList[odx];
			if (oChild1 < 0)		// 1 child, descend if nearer than cur min-dist
			{
				oChild1 = (-oChild1) - 1;
				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				if (oChild1Box.DistanceSquared(bounds) < nearest_sqr)
				{
					find_nearest_triangles(iBox, OtherTree, TransformF, oChild1, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
				}
			}
			else                            // 2 children
			{
				oChild1 = oChild1 - 1;
				int oChild2 = OtherTree.IndexList[odx + 1] - 1;

				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				FAxisAlignedBox3d oChild2Box = OtherTree.GetBox(oChild2, TransformF);

				// descend closer box first
				double d1Sqr = oChild1Box.DistanceSquared(bounds);
				double d2Sqr = oChild2Box.DistanceSquared(bounds);
				if (d2Sqr < d1Sqr)
				{
					if (d2Sqr < nearest_sqr)
					{
						find_nearest_triangles(iBox, OtherTree, TransformF, oChild2, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
					if (d1Sqr < nearest_sqr)
					{
						find_nearest_triangles(iBox, OtherTree, TransformF, oChild1, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
				}
				else
				{
					if (d1Sqr < nearest_sqr)
					{
						find_nearest_triangles(iBox, OtherTree, TransformF, oChild1, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
					if (d2Sqr < nearest_sqr)
					{
						find_nearest_triangles(iBox, OtherTree, TransformF, oChild2, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
				}
			}
		}
		else
		{
			// descend our tree nodes if they intersect w/ current bounds of other tree
			FAxisAlignedBox3d oBounds = OtherTree.GetBox(oBox, TransformF);

			int iChild1 = IndexList[idx];
			if (iChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				iChild1 = (-iChild1) - 1;
				if (box_box_distsqr(iChild1, oBounds) < nearest_sqr)
				{
					find_nearest_triangles(iChild1, OtherTree, TransformF, oBox, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
				}
			}
			else                             // 2 children
			{
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				// descend closer box first
				double d1Sqr = box_box_distsqr(iChild1, oBounds);
				double d2Sqr = box_box_distsqr(iChild2, oBounds);
				if (d2Sqr < d1Sqr)
				{
					if (d2Sqr < nearest_sqr)
					{
						find_nearest_triangles(iChild2, OtherTree, TransformF, oBox, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
					if (d1Sqr < nearest_sqr)
					{
						find_nearest_triangles(iChild1, OtherTree, TransformF, oBox, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
				}
				else
				{
					if (d1Sqr < nearest_sqr)
					{
						find_nearest_triangles(iChild1, OtherTree, TransformF, oBox, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
					if (d2Sqr < nearest_sqr)
					{
						find_nearest_triangles(iChild2, OtherTree, TransformF, oBox, depth + 1, nearest_sqr, nearest_pair, Options, OtherTreeOptions);
					}
				}
			}
		}
	}

	double box_box_distsqr(int iBox, const FAxisAlignedBox3d& testBox) const
	{
		// [TODO] could compute this w/o constructing box
		FAxisAlignedBox3d box = GetBoxEps(iBox, BoxEps);
		return box.DistanceSquared(testBox);
	}



	/**
	 * Standard tri tri intersection test (without computing intersection geometry)
	 */
	static bool TriangleIntersectionFilter(const FTriangle3d& A, const FTriangle3d& B)
	{
		return FIntrTriangle3Triangle3d::Intersects(A, B);
	}
	/**
	 * Standard tri tri intersection test (with computing intersection geometry)
	 * @param Intr Stores intersection problem; after call, should be set with intersection result
	 * @return true if triangles were intersecting
	 */
	static bool TriangleIntersection(FIntrTriangle3Triangle3d& Intr)
	{
		// Note: Test() is much faster than Find() so it makes sense to call it first, as most
		// triangles will not intersect (right?  TODO: actually test this somehow, though it is very data dependent)
		bool bFound = Intr.Test();
		if (bFound)
		{
			Intr.Find();
			return Intr.Result == EIntersectionResult::Intersects;
		}
		else
		{
			return false;
		}
	}


	int find_any_intersection(
		int iBox, const FTriangle3d& Triangle, const FAxisAlignedBox3d& triBounds, 
		TFunctionRef<bool(const FTriangle3d& A, const FTriangle3d& B)> TriangleIntersectionTest,
		const FQueryOptions& Options
	) const
	{
		int idx = BoxToIndex[iBox];
		if (idx < TrianglesEnd)             // triangle-list case, array is [N t1 t2 ... tN]
		{
			FTriangle3d box_tri;
			int num_tris = IndexList[idx];
			for (int i = 1; i <= num_tris; ++i)
			{
				int ti = IndexList[idx + i];
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
				{
					continue;
				}
				Mesh->GetTriVertices(ti, box_tri.V[0], box_tri.V[1], box_tri.V[2]);
				if (TriangleIntersectionTest(Triangle, box_tri))
				{
					return ti;
				}
			}
		}
		else                                 // internal node, either 1 or 2 child boxes
		{
			int iChild1 = IndexList[idx];
			if (iChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				iChild1 = (-iChild1) - 1;
				if (box_box_intersect(iChild1, triBounds))
				{
					return find_any_intersection(iChild1, Triangle, triBounds, TriangleIntersectionTest, Options);
				}

			}
			else                             // 2 children, descend closest first
			{
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[idx + 1] - 1;

				int interTri = -1;
				if (box_box_intersect(iChild1, triBounds))
				{
					interTri = find_any_intersection(iChild1, Triangle, triBounds, TriangleIntersectionTest, Options);
				}
				if (interTri == -1 && box_box_intersect(iChild2, triBounds))
				{
					interTri = find_any_intersection(iChild2, Triangle, triBounds, TriangleIntersectionTest, Options);
				}
				return interTri;
			}
		}

		return -1;
	}


	bool find_any_intersection(
		int iBox, const TMeshAABBTree3& OtherTree, const TFunction<FVector3d(const FVector3d&)>& TransformF, int oBox, int depth,
		TFunctionRef<bool(const FTriangle3d & A, const FTriangle3d & B)> TriangleIntersectionTest,
		const FQueryOptions& Options, const FQueryOptions& OtherTreeOptions
	) const
	{
		int idx = BoxToIndex[iBox];
		int odx = OtherTree.BoxToIndex[oBox];

		if (idx < TrianglesEnd && odx < OtherTree.TrianglesEnd)
		{
			// ok we are at triangles for both trees, do triangle-level testing
			FTriangle3d Tri, otri;
			int num_tris = IndexList[idx], onum_tris = OtherTree.IndexList[odx];

			// outer iteration is "other" tris that need to be transformed (more expensive)
			for (int j = 1; j <= onum_tris; ++j)
			{
				int tj = OtherTree.IndexList[odx + j];
				if (OtherTreeOptions.TriangleFilterF != nullptr && OtherTreeOptions.TriangleFilterF(tj) == false)
				{
					continue;
				}
				OtherTree.Mesh->GetTriVertices(tj, otri.V[0], otri.V[1], otri.V[2]);
				if (TransformF != nullptr)
				{
					otri.V[0] = TransformF(otri.V[0]);
					otri.V[1] = TransformF(otri.V[1]);
					otri.V[2] = TransformF(otri.V[2]);
				}

				// inner iteration over "our" triangles
				for (int i = 1; i <= num_tris; ++i)
				{
					int ti = IndexList[idx + i];
					if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
					{
						continue;
					}
					Mesh->GetTriVertices(ti, Tri.V[0], Tri.V[1], Tri.V[2]);
					if (TriangleIntersectionTest(otri, Tri))
					{
						return true;
					}
				}
			}
			return false;
		}

		// we either descend "our" tree or the other tree
		//   - if we have hit triangles on "our" tree, we have to descend other
		//   - if we hit triangles on "other", we have to descend ours
		//   - otherwise, we alternate at each depth. This produces wider
		//     branching but is significantly faster (~10x) for both hits and misses
		bool bDescendOther = (idx < TrianglesEnd || depth % 2 == 0);
		if (bDescendOther && odx < OtherTree.TrianglesEnd)
		{
			bDescendOther = false;      // can't
		}

		if (bDescendOther)
		{
			// ok we hit triangles on our side but we need to still reach triangles on
			// the other side, so we descend "their" children

			// [TODO] could we do efficient box.intersects(transform(box)) test?
			//   ( Contains() on each xformed point? )
			FAxisAlignedBox3d bounds = GetBox(iBox);

			int oChild1 = OtherTree.IndexList[odx];
			if (oChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				oChild1 = (-oChild1) - 1;
				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				if (oChild1Box.Intersects(bounds))
				{
					return find_any_intersection(iBox, OtherTree, TransformF, oBox, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
				}

			}
			else                             // 2 children
			{
				oChild1 = oChild1 - 1;          // [TODO] could descend one w/ larger overlap volume first??
				int oChild2 = OtherTree.IndexList[odx + 1] - 1;

				bool intersects = false;
				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				if (oChild1Box.Intersects(bounds))
				{
					intersects = find_any_intersection(iBox, OtherTree, TransformF, oChild1, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
				}

				if (intersects == false)
				{
					FAxisAlignedBox3d oChild2Box = OtherTree.GetBox(oChild2, TransformF);
					if (oChild2Box.Intersects(bounds))
					{
						intersects = find_any_intersection(iBox, OtherTree, TransformF, oChild2, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
					}
				}
				return intersects;
			}
		}
		else
		{
			// descend our tree nodes if they intersect w/ current bounds of other tree
			FAxisAlignedBox3d oBounds = OtherTree.GetBox(oBox, TransformF);

			int iChild1 = IndexList[idx];
			if (iChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				iChild1 = (-iChild1) - 1;
				if (box_box_intersect(iChild1, oBounds))
				{
					return find_any_intersection(iChild1, OtherTree, TransformF, oBox, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
				}
			}
			else                             // 2 children
			{
				iChild1 = iChild1 - 1;          // [TODO] could descend one w/ larger overlap volume first??
				int iChild2 = IndexList[idx + 1] - 1;

				bool intersects = false;
				if (box_box_intersect(iChild1, oBounds))
				{
					intersects = find_any_intersection(iChild1, OtherTree, TransformF, oBox, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
				}
				if (intersects == false && box_box_intersect(iChild2, oBounds))
				{
					intersects = find_any_intersection(iChild2, OtherTree, TransformF, oBox, depth + 1, TriangleIntersectionTest, Options, OtherTreeOptions);
				}
				return intersects;
			}

		}
		return false;
	}

	
	// helper for find_self_intersections that checks intersections between triangles from separate boxes
	bool find_self_intersections_acrossboxes(
		int Box1, int Box2, MeshIntersection::FIntersectionsQueryResult* Result,
		bool bIgnoreTopoConnected, int depth,
		TFunctionRef<bool(FIntrTriangle3Triangle3d&)> IntersectFn,
		const FQueryOptions& Options
	) const
	{
		bool bFound = false;
		if (Box1 < 0)
		{
			return false;
		}

		int Box1Idx = BoxToIndex[Box1];
		int Box2Idx = Box2 > -1 ? BoxToIndex[Box2] : -1; // Box2Idx allowed to be invalid

		// at leaf
		if (Box1Idx < TrianglesEnd && Box2Idx < TrianglesEnd)
		{
			if (Box2 == -1)
			{
				return false;
			}

			for (int IdxA = Box1Idx + 1, Box1End = Box1Idx + 1 + IndexList[Box1Idx]; IdxA < Box1End; IdxA++)
			{
				int TID_A = IndexList[IdxA];
				bFound = find_tri_tri_intersections(TID_A, Box2Idx + 1, Box2Idx + 1 + IndexList[Box2Idx], Result, bIgnoreTopoConnected, IntersectFn, Options) || bFound;
				if (!Result && bFound)
				{
					return true;
				}
			}
			return bFound;
		}

		// alternate which box we descend, while still making sure Box1 is not a leaf
		// (the alternating part is just a heuristic that I guess may help performance; it's not necessary)
		if (Box1Idx < TrianglesEnd || (Box2Idx >= TrianglesEnd && depth % 2 == 1))
		{
			Swap(Box1, Box2);
			Swap(Box1Idx, Box2Idx);
		}
		
		
		// if Box2 is present, we *only* care about the intersection of Box1's children w/ Box2
		// if Box2 is not present, we handle all the other cases (descending into the children of box1, and checking if the children intersect)
		if (Box2 > -1)
		{
			FAxisAlignedBox3d Box2Box = GetBox(Box2);

			// Descend through Box1
			int iChild1 = IndexList[Box1Idx];
			if (iChild1 < 0)                  // 1 child
			{
				iChild1 = (-iChild1) - 1;

				if (Box2 > -1 && box_box_intersect(iChild1, Box2Box))
				{
					bFound = find_self_intersections_acrossboxes(iChild1, Box2, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
					if (!Result && bFound)
					{
						return true;
					}
				}
			}
			else                             // 2 children
			{
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[Box1Idx + 1] - 1;

				if (Box2 > -1 && box_box_intersect(iChild1, Box2Box))
				{
					bFound = find_self_intersections_acrossboxes(iChild1, Box2, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
					if (!Result && bFound)
					{
						return true;
					}
				}
				if (Box2 > -1 && box_box_intersect(iChild2, Box2Box))
				{
					bFound = find_self_intersections_acrossboxes(iChild2, Box2, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
					if (!Result && bFound)
					{
						return true;
					}
				}
			}
		}
		else // Box2 is not present
		{
			// Descend through Box1
			int iChild1 = IndexList[Box1Idx];
			if (iChild1 < 0)                  // 1 child
			{
				iChild1 = (-iChild1) - 1;

				bFound = find_self_intersections_acrossboxes(iChild1, -1, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
				if (!Result && bFound)
				{
					return true;
				}
			}
			else                             // 2 children
			{
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[Box1Idx + 1] - 1;

				bFound = find_self_intersections_acrossboxes(iChild1, -1, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
				if (!Result && bFound)
				{
					return true;
				}
				bFound = find_self_intersections_acrossboxes(iChild2, -1, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
				if (!Result && bFound)
				{
					return true;
				}

				FAxisAlignedBox3d Child2Box = GetBox(iChild2);

				if (box_box_intersect(iChild1, Child2Box))
				{
					bFound = find_self_intersections_acrossboxes(iChild1, iChild2, Result, bIgnoreTopoConnected, depth + 1, IntersectFn, Options) || bFound;
					if (!Result && bFound)
					{
						return true;
					}
				}
			}
		}

		return bFound;
	}

	// helper for find_self_intersections that intersects a single triangle vs a range of triangles
	bool find_tri_tri_intersections(int TID_A, int IdxRangeStart, int IdxRangeEnd, 
		MeshIntersection::FIntersectionsQueryResult* Result,
		bool bIgnoreTopoConnected,
		TFunctionRef<bool(FIntrTriangle3Triangle3d&)> IntersectFn,
		const FQueryOptions& Options
	) const
	{
		if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(TID_A) == false)
		{
			return false;
		}

		bool bFound = false;
		FTriangle3d TriA, TriB;
		Mesh->GetTriVertices(TID_A, TriA.V[0], TriA.V[1], TriA.V[2]);
		FIntrTriangle3Triangle3d Intr;
		Intr.SetTriangle0(TriA);
		FIndex3i TriA_VID = Mesh->GetTriangle(TID_A);
		
		for (int IdxB = IdxRangeStart; IdxB < IdxRangeEnd; IdxB++)
		{
			int TID_B = IndexList[IdxB];
			if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(TID_B) == false)
			{
				continue;
			}

			FIndex3i TriB_VID = Mesh->GetTriangle(TID_B);
			if (bIgnoreTopoConnected)
			{
				// ignore triangles that share a vertex
				bool bTopoConnected = false;
				for (int AIdx = 0; AIdx < 3 && !bTopoConnected; AIdx++)
				{
					int VID_A = TriA_VID[AIdx];
					for (int BIdx = 0; BIdx < 3; BIdx++)
					{
						if (VID_A == TriB_VID[BIdx])
						{
							bTopoConnected = true;
							break;
						}
					}
				}
				if (bTopoConnected)
				{
					continue;
				}
			}

			Mesh->GetTriVertices(TID_B, TriB.V[0], TriB.V[1], TriB.V[2]);
			Intr.SetTriangle1(TriB);
			bFound = IntersectFn(Intr);
			if (bFound)
			{
				if (!Result)
				{
					return true;
				}
				else
				{
					AddTriTriIntersectionResult(Intr, TID_A, TID_B, *Result);
				}
			}
		}

		return bFound;
	}

	/**
	 * Helper to add a triangle-triangle intersection result from to a intersection result struct
	 */
	static void AddTriTriIntersectionResult(const FIntrTriangle3Triangle3d& Intr, int TID_A, int TID_B, MeshIntersection::FIntersectionsQueryResult& Result)
	{
		if (Intr.Quantity == 1)
		{
			Result.Points.Add(MeshIntersection::FPointIntersection{ {TID_A, TID_B}, Intr.Points[0] });
		}
		else if (Intr.Quantity == 2)
		{
			Result.Segments.Add(MeshIntersection::FSegmentIntersection{ {TID_A, TID_B}, {Intr.Points[0], Intr.Points[1]} });
		}
		else if (Intr.Quantity > 2)
		{
			if (Intr.Type == EIntersectionType::MultiSegment)
			{
				Result.Segments.Add(MeshIntersection::FSegmentIntersection{ {TID_A, TID_B}, {Intr.Points[0], Intr.Points[1]} });
				Result.Segments.Add(MeshIntersection::FSegmentIntersection{ {TID_A, TID_B}, {Intr.Points[2], Intr.Points[3]} });
				if (Intr.Quantity > 4)
				{
					Result.Segments.Add(MeshIntersection::FSegmentIntersection{ {TID_A, TID_B}, {Intr.Points[4], Intr.Points[5]} });
				}
			}
			else
			{
				Result.Polygons.Add(MeshIntersection::FPolygonIntersection{ {TID_A, TID_B},
					{Intr.Points[0],Intr.Points[1],Intr.Points[2],Intr.Points[3],Intr.Points[4],Intr.Points[5]}, Intr.Quantity });
			}
		}
	}


	bool find_self_intersections(
		MeshIntersection::FIntersectionsQueryResult* Result, bool bIgnoreTopoConnected,
		TFunctionRef<bool(FIntrTriangle3Triangle3d&)> IntersectFn,
		const FQueryOptions & Options
	) const
	{
		// Check each leaf-box for intersecting triangles within the same box
		bool bFound = false;
		for (int StartIdx = 0; StartIdx < TrianglesEnd; )
		{
			int NumTris = IndexList[StartIdx];
			int EndIdx = StartIdx + NumTris + 1;
			for (int IdxA = StartIdx + 1; IdxA + 1 < EndIdx; IdxA++)
			{
				int TID_A = IndexList[IdxA];
				bFound = find_tri_tri_intersections(TID_A, IdxA + 1, EndIdx, Result, bIgnoreTopoConnected, IntersectFn, Options) || bFound;
				if (!Result && bFound) // early out if not filling Result
				{
					return bFound;
				}
			}
			StartIdx = EndIdx;
		}

		// Recursively check across boxes
		return find_self_intersections_acrossboxes(RootIndex, -1, Result, bIgnoreTopoConnected, 0, IntersectFn, Options) || bFound;
	}


	void find_intersections(
		int iBox, const TMeshAABBTree3& OtherTree, const TFunction<FVector3d(const FVector3d&)>& TransformF,
		int oBox, int depth, MeshIntersection::FIntersectionsQueryResult& result,
		TFunctionRef<bool(FIntrTriangle3Triangle3d&)> IntersectFn,
		const FQueryOptions& Options, const FQueryOptions& OtherTreeOptions
	) const
	{
		int idx = BoxToIndex[iBox];
		int odx = OtherTree.BoxToIndex[oBox];
		FIntrTriangle3Triangle3d Intr;

		if (idx < TrianglesEnd && odx < OtherTree.TrianglesEnd)
		{
			// ok we are at triangles for both trees, do triangle-level testing
			FTriangle3d Tri, otri;
			int num_tris = IndexList[idx], onum_tris = OtherTree.IndexList[odx];

			// outer iteration is "other" tris that need to be transformed (more expensive)
			for (int j = 1; j <= onum_tris; ++j)
			{
				int tj = OtherTree.IndexList[odx + j];
				if (OtherTreeOptions.TriangleFilterF != nullptr && OtherTreeOptions.TriangleFilterF(tj) == false)
				{
					continue;
				}
				OtherTree.Mesh->GetTriVertices(tj, otri.V[0], otri.V[1], otri.V[2]);
				if (TransformF != nullptr)
				{
					otri.V[0] = TransformF(otri.V[0]);
					otri.V[1] = TransformF(otri.V[1]);
					otri.V[2] = TransformF(otri.V[2]);
				}
				Intr.SetTriangle0(otri);

				// inner iteration over "our" triangles
				for (int i = 1; i <= num_tris; ++i)
				{
					int ti = IndexList[idx + i];
					if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(ti) == false)
					{
						continue;
					}
					Mesh->GetTriVertices(ti, Tri.V[0], Tri.V[1], Tri.V[2]);
					Intr.SetTriangle1(Tri);


					if (IntersectFn(Intr))
					{
						AddTriTriIntersectionResult(Intr, ti, tj, result);
					}
				}
			}

			// done these nodes
			return;
		}

		// we either descend "our" tree or the other tree
		//   - if we have hit triangles on "our" tree, we have to descend other
		//   - if we hit triangles on "other", we have to descend ours
		//   - otherwise, we alternate at each depth. This produces wider
		//     branching but is significantly faster (~10x) for both hits and misses
		bool bDescendOther = (idx < TrianglesEnd || depth % 2 == 0);
		if (bDescendOther && odx < OtherTree.TrianglesEnd)
			bDescendOther = false;      // can't

		if (bDescendOther) {
			// ok we hit triangles on our side but we need to still reach triangles on
			// the other side, so we descend "their" children

			// [TODO] could we do efficient box.intersects(transform(box)) test?
			//   ( Contains() on each xformed point? )
			FAxisAlignedBox3d bounds = GetBoxEps(iBox, BoxEps);

			int oChild1 = OtherTree.IndexList[odx];
			if (oChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				oChild1 = (-oChild1) - 1;
				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				if (oChild1Box.Intersects(bounds))
					find_intersections(iBox, OtherTree, TransformF, oChild1, depth + 1, result, IntersectFn, Options, OtherTreeOptions);

			}
			else                             // 2 children
			{
				oChild1 = oChild1 - 1;

				FAxisAlignedBox3d oChild1Box = OtherTree.GetBox(oChild1, TransformF);
				if (oChild1Box.Intersects(bounds))
				{
					find_intersections(iBox, OtherTree, TransformF, oChild1, depth + 1, result, IntersectFn, Options, OtherTreeOptions);
				}

				int oChild2 = OtherTree.IndexList[odx + 1] - 1;
				FAxisAlignedBox3d oChild2Box = OtherTree.GetBox(oChild2, TransformF);
				if (oChild2Box.Intersects(bounds))
				{
					find_intersections(iBox, OtherTree, TransformF, oChild2, depth + 1, result, IntersectFn, Options, OtherTreeOptions);
				}
			}

		}
		else
		{
			// descend our tree nodes if they intersect w/ current bounds of other tree
			FAxisAlignedBox3d oBounds = OtherTree.GetBox(oBox, TransformF);

			int iChild1 = IndexList[idx];
			if (iChild1 < 0)                  // 1 child, descend if nearer than cur min-dist
			{
				iChild1 = (-iChild1) - 1;
				if (box_box_intersect(iChild1, oBounds))
				{
					find_intersections(iChild1, OtherTree, TransformF, oBox, depth + 1, result, IntersectFn, Options, OtherTreeOptions);
				}

			}
			else                             // 2 children
			{
				iChild1 = iChild1 - 1;
				if (box_box_intersect(iChild1, oBounds))
				{
					find_intersections(iChild1, OtherTree, TransformF, oBox, depth + 1, result, IntersectFn, Options, OtherTreeOptions);
				}

				int iChild2 = IndexList[idx + 1] - 1;
				if (box_box_intersect(iChild2, oBounds))
				{
					find_intersections(iChild2, OtherTree, TransformF, oBox, depth + 1, result, IntersectFn, Options, OtherTreeOptions);
				}
			}

		}
	}




public:
	// 1) make sure we can reach every Tri in Mesh through tree (also demo of how to traverse tree...)
	// 2) make sure that Triangles are contained in parent boxes
	void TestCoverage()
	{
		TArray<int> tri_counts;
		tri_counts.SetNumZeroed(Mesh->MaxTriangleID());

		TArray<int> parent_indices;
		parent_indices.SetNumZeroed(BoxToIndex.GetLength());

		test_coverage(tri_counts, parent_indices, RootIndex);

		for (int ti = 0; ti < Mesh->MaxTriangleID(); ti++)
		{
			if (!Mesh->IsTriangle(ti))
			{
				continue;
			}
			checkSlow(tri_counts[ti] == 1);
		}
	}

	/**
	 * Total sum of volumes of all boxes in the tree. Mainly useful to evaluate tree quality.
	 */
	double TotalVolume()
	{
		double volSum = 0;
		FTreeTraversal t;
		t.NextBoxF = [&](const FAxisAlignedBox3d& Box, int)
		{
			volSum += Box.Volume();
			return true;
		};
		DoTraversal(t);
		return volSum;
	}

private:
	// accumulate triangle counts and track each Box-parent index.
	// also checks that Triangles are contained in boxes
	void test_coverage(TArray<int>& tri_counts, TArray<int>& parent_indices, int IBox)
	{
		int idx = BoxToIndex[IBox];

		debug_check_child_tris_in_box(IBox);

		if (idx < TrianglesEnd)
		{
			// triangle-list case, array is [N t1 t2 ... tN]
			int n = IndexList[idx];
			FAxisAlignedBox3d Box = GetBoxEps(IBox);
			for (int i = 1; i <= n; ++i)
			{
				int ti = IndexList[idx + i];
				tri_counts[ti]++;

				FIndex3i tv = Mesh->GetTriangle(ti);
				for (int j = 0; j < 3; ++j)
				{
					FVector3d V = Mesh->GetVertex(tv[j]);
					checkSlow(Box.Contains(V));
				}
			}
		}
		else
		{
			int i0 = IndexList[idx];
			if (i0 < 0)
			{
				// negative index means we only have one 'child' Box to descend into
				i0 = (-i0) - 1;
				parent_indices[i0] = IBox;
				test_coverage(tri_counts, parent_indices, i0);
			}
			else
			{
				// positive index, two sequential child Box indices to descend into
				i0 = i0 - 1;
				parent_indices[i0] = IBox;
				test_coverage(tri_counts, parent_indices, i0);
				int i1 = IndexList[idx + 1];
				i1 = i1 - 1;
				parent_indices[i1] = IBox;
				test_coverage(tri_counts, parent_indices, i1);
			}
		}
	}
	// do full tree Traversal below IBox and make sure that all Triangles are further
	// than Box-distance-sqr
	void debug_check_child_tri_distances(int IBox, const FVector3d& P)
	{
		double fBoxDistSqr = BoxDistanceSqr(IBox, P);

		// [TODO]
		FTreeTraversal t;
		t.NextTriangleF = [&](int TID)
		{
			double fTriDistSqr = TMeshQueries<TriangleMeshType>::TriDistanceSqr(*Mesh, TID, P);
			if (fTriDistSqr < fBoxDistSqr)
			{
				checkSlow(fabs(fTriDistSqr - fBoxDistSqr) <= FMathd::ZeroTolerance * 100);
			}
		};
		TreeTraversalImpl(IBox, 0, t, FQueryOptions());
	}

	// do full tree Traversal below IBox to make sure that all child Triangles are contained
	void debug_check_child_tris_in_box(int IBox)
	{
		FAxisAlignedBox3d Box = GetBoxEps(IBox);
		FTreeTraversal t;
		t.NextTriangleF = [&](int TID)
		{
			FIndex3i tv = Mesh->GetTriangle(TID);
			for (int j = 0; j < 3; ++j)
			{
				FVector3d V = Mesh->GetVertex(tv[j]);
				checkSlow(Box.Contains(V));
			}
		};
		TreeTraversalImpl(IBox, 0, t, FQueryOptions());
	}
};


} // end namespace UE::Geometry
} // end namespace UE