// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Spatial/FastWinding.h"

namespace UE
{
namespace Geometry
{


/**
 * Sorts a list meshes into a list of "Nests" where each Nest has an outer shell mesh and a list of contained-inside meshes
 *
 * For example, if you have a "Russian nesting doll" arrangement of meshes [A,B,C,D] where A holds B holds C holds D:
 *
 *  - If NestingMethod is InSmallestParent, the output will be:
 *     Nest 1: Outer: A, Inner: B, Parent: Invalid
 *     Nest 2: Outer; C, Inner: D, Parent: B
 *    Nest 1 conceptually corresponds to the outer and inner walls of the first doll, and Nest 2 the same of the second doll. 
 *    (Note if the orientation-related booleans are set to true, this would only hold if the mesh orientations are also alternating)
 *
 *  - If NestingMethod is InLargestParent, the output will be:
 *     Nest 1: Outer: A; Inner: B, C, D; Parent: Invalid
 *    In this case Nest 1 conceptually corresponds to there being a single outer structure, and the inner structure is left unsorted
 *    (Note for InLargestParent mode, you typically want bOnlyNestNegativeVolumes to be false)
 *
 * For similar functionality in 2D, see the TPlanarComplex class
 */
template <class TriangleMeshType>
class TMeshSpatialSort
{
public:

	/**
	 * Meshes to be spatially sorted
	 */
	UE_DEPRECATED(5.8, "This will no longer stored be on the class; instead an accessor should be passed in per-call")
	TArrayView<const TriangleMeshType> InputMeshes;

	/**
	 * Optional array of bounding boxes for each mesh
	 */
	TArray<FAxisAlignedBox3d> MeshBounds;

	enum class ENestingMethod
	{
		InLargestParent, // meshes nest flatly in the largest mesh that contains them (simplest structure, fastest to compute, will never have nests inside nests)
		InSmallestParent, // meshes nest in the smallest mesh that contains them, capturing nests inside nests
	};

	/**
	 * Which algorithm is used to determine nesting
	 */
	ENestingMethod NestingMethod = ENestingMethod::InLargestParent;

	/**
	 * If true, only negative-volume meshes can be nested (i.e., can be included in an FMeshNesting InnerIndices array)
	 */
	bool bOnlyNestNegativeVolumes = false;

	/**
	 * If true, only positive-volume meshes can be assigned the "OuterIndex" of an FMeshNesting
	 * Leads to more logical nesting, but note: if true, a negative volume that isn't inside any positive volume mesh will not show up in any nests at all
	 */
	bool bOnlyParentPostiveVolumes = true;

	/**
	 * If true, we assume meshes don't intersect, so we can check if a whole mesh is inside another by testing a single vertex
	 * If false, we only consider a mesh inside another if *every* vertex of the smaller is inside the larger
	 */
	bool bOnlyCheckSingleVertexForNesting = true;


	/// Outputs

	struct FMeshNesting
	{
		int OuterIndex; // index of the outer (positive volume) shell
		int ParentIndex = -1; // only used if NestingMethod is InSmallestParent. Index of the mesh that contains the outer index mesh, or -1 if there is no such mesh.
		TArray<int> InnerIndices; // indices of all meshes contained inside outer shell (unless they are recursively nested inside a child mesh)
	};

	/**
	 * Computed nests
	 */
	TArray<FMeshNesting> Nests;

	/**
	 * Indices of meshes that are not included in the "Nests" output, e.g. negative-volume meshes that are not inside any positive-volume mesh if bOnlyParentPostiveVolumes
	 */
	TArray<int> SkippedMeshIndices;

	TMeshSpatialSort() {}

	UE_DEPRECATED(5.8, "Pass meshes and bounds to the Compute() method directly")
	TMeshSpatialSort(TArrayView<const TriangleMeshType> InputMeshesIn, TArrayView<const FAxisAlignedBox3d> MeshBoundsIn = TArrayView<const FAxisAlignedBox3d>())
		: InputMeshes(InputMeshesIn)
	{
		ensure(false);
	}

	UE_DEPRECATED(5.8, "Use a version of Compute() that takes the input meshes directly")
	void Compute()
	{
		ensure(false);
		Compute(InputMeshes);
	}

	void Compute(TArrayView<const TriangleMeshType> InInputMeshes, TArrayView<const FAxisAlignedBox3d> MeshBoundsIn = TArrayView<const FAxisAlignedBox3d>())
	{
		Compute([&InInputMeshes](int32 Idx) { return &InInputMeshes[Idx]; }, InInputMeshes.Num(), MeshBoundsIn);
	}

	void Compute(TArrayView<const TriangleMeshType*> InInputMeshes, TArrayView<const FAxisAlignedBox3d> MeshBoundsIn = TArrayView<const FAxisAlignedBox3d>())
	{
		Compute([&InInputMeshes](int32 Idx) { return InInputMeshes[Idx]; }, InInputMeshes.Num(), MeshBoundsIn);
	}

	void Compute(TArrayView<const TUniquePtr<TriangleMeshType>> InInputMeshes, TArrayView<const FAxisAlignedBox3d> MeshBoundsIn = TArrayView<const FAxisAlignedBox3d>())
	{
		Compute([&InInputMeshes](int32 Idx) { return InInputMeshes[Idx].Get(); }, InInputMeshes.Num(), MeshBoundsIn);
	}

	void Compute(TFunctionRef<const TriangleMeshType*(int32)> GetInputMesh, int32 NumInputMeshes, TArrayView<const FAxisAlignedBox3d> MeshBoundsIn = TArrayView<const FAxisAlignedBox3d>())
	{
		MeshBounds.Reset();

		if (MeshBoundsIn.Num() == NumInputMeshes)
		{
			MeshBounds.Append(MeshBoundsIn);
		}
		else
		{
			InitBounds(GetInputMesh, NumInputMeshes);
		}

		FAxisAlignedBox3d CombinedBounds;
		for (const FAxisAlignedBox3d& Bounds : MeshBounds)
		{
			CombinedBounds.Contain(Bounds);
		}

		Nests.Empty();
		SkippedMeshIndices.Empty();

		int32 N = NumInputMeshes;
		TArray<int> MeshIndices;
		MeshIndices.SetNum(N);
		for (int i = 0; i < N; i++)
		{
			MeshIndices[i] = i;
		}

		// precomputed useful stats on input meshes
		struct FMeshInfo
		{
			double Volume;
			bool bPositiveVolume;
		};
		TArray<FMeshInfo> Info;
		Info.SetNum(N);
		double DimScaleFactor = 1.0 / (CombinedBounds.MaxDim() + KINDA_SMALL_NUMBER);
		for (int i = 0; i < N; i++)
		{
			double SignedVolume = TMeshQueries<TriangleMeshType>::GetVolumeNonWatertight(*GetInputMesh(i), DimScaleFactor);
			Info[i].bPositiveVolume = SignedVolume >= 0;
			Info[i].Volume = FMathd::Abs(SignedVolume);
		}

		MeshIndices.Sort([&Info](int Ind1, int Ind2)
			{
				return Info[Ind1].Volume > Info[Ind2].Volume;
			}
		);

		TArray<int> Parent;
		Parent.Init(-1, N);

		// From largest mesh to smallest, greedily assign parents where possible
		// (If not nesting in largest: 
		//   If a mesh already has a parent, but a smaller containing parent is found, switch to the new parent)
		for (int ParentCand = 0; ParentCand + 1 < N; ParentCand++)
		{
			int PIdx = MeshIndices[ParentCand];
			const FMeshInfo& PInfo = Info[PIdx];
			const TriangleMeshType& PMesh = *GetInputMesh(PIdx);
			if (NestingMethod == ENestingMethod::InLargestParent && Parent[ParentCand] != -1)
			{
				continue; // mesh was already contained in a larger mesh
			}

			if (!PInfo.bPositiveVolume && NestingMethod != ENestingMethod::InSmallestParent && bOnlyParentPostiveVolumes)
			{
				// negative volume meshes can't be an 'outer shell,' so we can skip unless we're looking for nests-in-nests
				continue;
			}

			double WindingTestSign = PInfo.bPositiveVolume ? 1 : -1;
			TMeshAABBTree3<TriangleMeshType> Spatial(GetInputMesh(PIdx));
			TFastWindingTree<TriangleMeshType> Winding(&Spatial);
			for (int ChildCand = ParentCand + 1; ChildCand < N; ChildCand++)
			{
				int CIdx = MeshIndices[ChildCand];
				const FMeshInfo& CInfo = Info[CIdx];
				const TriangleMeshType& CMesh = *GetInputMesh(CIdx);
				if (CMesh.VertexCount() == 0)
				{
					continue;
				}
				if (NestingMethod == ENestingMethod::InLargestParent)
				{
					// InLargestParent nesting can't have recursive nests or reassign to smaller parents, so we can skip some tests
					if ((bOnlyNestNegativeVolumes && CInfo.bPositiveVolume) || Parent[ChildCand] != -1)
					{
						continue;
					}
				}
				if (bOnlyCheckSingleVertexForNesting)
				{
					FVector3d Vert;
					
					for (int VID = 0; VID < CMesh.MaxVertexID(); VID++)
					{
						if (CMesh.IsVertex(VID))
						{
							Vert = CMesh.GetVertex(VID);
							break;
						}
					}

					double WindingNumber = Winding.FastWindingNumber(Vert);
					if (WindingNumber*WindingTestSign > .5)
					{
						Parent[ChildCand] = ParentCand;
					}
				}
				else
				{
					bool bHasOutsideSample = false;
					for (int VID = 0; VID < CMesh.MaxVertexID(); VID++)
					{
						if (CMesh.IsVertex(VID))
						{
							double WindingNumber = Winding.FastWindingNumber(CMesh.GetVertex(VID));
							if (WindingNumber * WindingTestSign < .5)
							{
								bHasOutsideSample = true;
								break;
							}
						}
					}
					if (!bHasOutsideSample)
					{
						Parent[ChildCand] = ParentCand;
					}
				}
			}
		}

		// Build nests from parent relationship
		TArray<bool> Taken;
		Taken.SetNumZeroed(N);
		for (int NestCand = 0; NestCand < N; NestCand++)
		{
			int NIdx = MeshIndices[NestCand];
			if (Taken[NestCand])
			{
				continue;
			}
			if (bOnlyParentPostiveVolumes && !Info[NIdx].bPositiveVolume)
			{
				SkippedMeshIndices.Add(NIdx);
				continue;
			}
			FMeshNesting& Nest = Nests.Emplace_GetRef();
			Nest.OuterIndex = NIdx;

			if (NestingMethod == ENestingMethod::InSmallestParent)
			{
				int ParentCand = Parent[NestCand];
				if (ParentCand > -1)
				{
					Nest.ParentIndex = MeshIndices[ParentCand];
				}
			}
			for (int HoleCand = NestCand + 1; HoleCand < N; HoleCand++)
			{
				int MIdx = MeshIndices[HoleCand];
				if (Parent[HoleCand] == NestCand && (!Info[MIdx].bPositiveVolume || !bOnlyNestNegativeVolumes))
				{
					Nest.InnerIndices.Add(MIdx);
					Taken[HoleCand] = true;
				}
			}
		}
	}

private:

	void InitBounds(TFunctionRef<const TriangleMeshType*(int32)> GetInputMesh, int32 NumInputMeshes)
	{
		MeshBounds.Reset();
		MeshBounds.Reserve(NumInputMeshes);
		for (int32 Idx = 0; Idx < NumInputMeshes; ++Idx)
		{
			MeshBounds.Add(TMeshQueries<TriangleMeshType>::GetBounds(*GetInputMesh(Idx)));
		}
	}
};


} // end namespace UE::Geometry
} // end namespace UE