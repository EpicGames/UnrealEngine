// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshWindingNumberGrid

#pragma once

#include "MathUtil.h"
#include "MeshQueries.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "Implicit/SweepingMeshSDF.h"
#include "Util/MeshCaches.h"
#include "MatrixTypes.h"


namespace UE
{
namespace Geometry
{

/**
 * Sample mesh winding number (MWN) on a discrete grid. Can sample full grid, or
 * compute MWN values along a specific iso-contour and then fill in rest of grid
 * with correctly-signed values via fast sweeping (this is the default)
 * 
 * TODO: 
 *   - I think we are over-exploring the grid most of the time. eg along an x-ray that
 *     intersects the surface, we only need at most 2 cells, but we are computing at least 3,
 *     and possibly 5. 
 *   - it may be better to use something like bloomenthal polygonizer continuation? where we 
 *     are keeping track of active edges instead of active cells?
 */
template <class TriangleMeshType>
class TMeshWindingNumberGrid
{
public:
	const TriangleMeshType* Mesh;
	TFastWindingTree<TriangleMeshType>* FastWinding;

	// size of cubes in the grid
	double CellSize;

	// how many cells around border should we keep
	int BufferCells = 1;

	// Should we compute MWN at all grid cells (expensive!!) or only in narrow band.
	// In narrow-band mode, we guess rest of MWN values by propagating along x-rows
	enum EComputeModes
	{
		FullGrid = 0,
		NarrowBand = 1
	};
	EComputeModes ComputeMode = EComputeModes::NarrowBand;

	// in narrow-band mode, if mesh is not closed, we will explore space around
	// this MWN iso-value
	float WindingIsoValue = 0.5f;

	// in NarrowBand mode, we compute mesh SDF grid; if true then it can be accessed
	// via SDFGrid function after Compute()
	bool bWantMeshSDFGrid = true;

	/** if this function returns true, we should abort calculation  */
	TFunction<bool()> CancelF = [](){ return false; };

	// computed results
	FVector3f GridOrigin;
	FDenseGrid3f WindingGrid;

	// SDF grid we compute in narrow-band mode
	TSweepingMeshSDF<TriangleMeshType> MeshSDF;


	TMeshWindingNumberGrid(const TriangleMeshType* Mesh, TFastWindingTree<TriangleMeshType>* FastWinding, double CellSize)
		: Mesh(Mesh), FastWinding(FastWinding), CellSize(CellSize), MeshSDF(Mesh, (float)CellSize, FastWinding->GetTree())
	{
	}


	void Compute()
	{
		// figure out origin & dimensions
		FAxisAlignedBox3d bounds = FastWinding->GetTree()->GetBoundingBox();

		if (bounds.IsEmpty())
		{
			return;
		}

		float fBufferWidth = (float)BufferCells * (float)CellSize;
		GridOrigin = (FVector3f)bounds.Min - fBufferWidth * FVector3f::One();
		FVector3f max = (FVector3f)bounds.Max + fBufferWidth * FVector3f::One();
		int ni = (int)((max.X - GridOrigin.X) / (float)CellSize) + 1;
		int nj = (int)((max.Y - GridOrigin.Y) / (float)CellSize) + 1;
		int nk = (int)((max.Z - GridOrigin.Z) / (float)CellSize) + 1;

		if (ni >= 0 && nj >= 0 && nk >= 0)
		{
			if (ComputeMode == EComputeModes::FullGrid)
			{
				make_grid_dense(GridOrigin, (float)CellSize, ni, nj, nk, WindingGrid);
			}
			else
			{
				make_grid(GridOrigin, (float)CellSize, ni, nj, nk, WindingGrid);
			}
		}

		if (!bWantMeshSDFGrid)
		{
			MeshSDF.Empty();
		}
	}



	FVector3i Dimensions() const
	{
		return WindingGrid.GetDimensions();
	}

	constexpr const float& At(int I, int J, int K) const
	{
		return WindingGrid.At(I, J, K);
	}

	float GetValue(const FVector3i& IJK) const // TTriLinearGridInterpolant interface
	{
		return WindingGrid[IJK];
	}

	FVector3f CellCenter(int I, int J, int K)
	{
		return FVector3f((float)I * CellSize + GridOrigin.X,
			(float)J * CellSize + GridOrigin.Y,
			(float)K * CellSize + GridOrigin.Z);
	}

private:

	void make_grid(FVector3f origin, float dx, int ni, int nj, int nk, FDenseGrid3f& winding)
	{
		winding.Resize(ni, nj, nk);
		winding.Assign(FMathf::MaxReal); // sentinel

		// seed MWN cache
		FastWinding->Build(false);

		// Ok, because the whole idea is that the surface might have holes, we are going to 
		// compute MWN along known triangles and then propagate the computed region outwards
		// until any MWN iso-sign-change is surrounded.
		// To seed propagation, we compute unsigned SDF and then compute MWN for any voxels
		// containing surface (ie w/ distance smaller than cellsize)

		// compute unsigned SDF
		MeshSDF.bComputeSigns = false;
		MeshSDF.CancelF = CancelF;
		MeshSDF.Compute(FastWinding->GetTree()->GetBoundingBox());
		if (CancelF())
		{
			return;
		}

		FDenseGrid3f& distances = MeshSDF.Grid;

		// compute MWN at surface voxels
		double ox = (double)origin[0], oy = (double)origin[1], oz = (double)origin[2];
		ParallelFor(nj*nk, [this, origin, dx, ni, nj, nk, &winding, &distances](int LinearJK)
		{
			if (CancelF())
			{
				return;
			}
			int J = LinearJK % nj, K = LinearJK / nj;
			int StartIdx = ni * (J + nj * K);
			for (int I = 0; I < ni; ++I)
			{
				int LinearIdx = StartIdx + I;
				float dist = distances[LinearIdx];
				// this could be tighter? but I don't think it matters...
				if (dist < CellSize)
				{
					FVector3d gx((float)I * dx + origin[0], (float)J * dx + origin[1], (float)K * dx + origin[2]);
					winding[LinearIdx] = (float)FastWinding->FastWindingNumber(gx);
				}
			}
		});
		if (CancelF())
		{
			return;
		}

		// Now propagate outwards from computed voxels.
		// Current procedure is to check 26-neighbours around each 'front' voxel,
		// and if there are any MWN sign changes, that neighbour is added to front.
		// Front is initialized w/ all voxels we computed above

		FAxisAlignedBox3i bounds = winding.Bounds();
		bounds.Max -= FVector3i::One();

		// since we will be computing new MWN values as necessary, we cannot use
		// winding grid to track whether a voxel is 'new' or not. 
		// So, using 3D bitmap instead - is updated at end of each pass.
		TArray<bool> bits;
		check(winding.Num() <= MAX_int32);
		bits.SetNumZeroed(static_cast<int32>(winding.Num()));

		TArray<int> cur_front;
		for (int32 LinearIdx = 0; LinearIdx < bits.Num(); LinearIdx++)
		{
			if (winding[LinearIdx] != FMathf::MaxReal)
			{
				cur_front.Add(LinearIdx);
				bits[LinearIdx] = true;
			}
		}
		if (CancelF())
		{
			return;
		}

		// Unique set of 'new' voxels to compute in next iteration.
		TSet<int> queue;
		FCriticalSection QueueSection;

		while (true)
		{
			if (CancelF())
			{
				return;
			}

			// can process front voxels in parallel
			bool abort = false;
			ParallelFor(cur_front.Num(), [this, origin, dx, ni, nj, nk, &winding, &distances, &bounds, &bits, &cur_front, &queue, &QueueSection, &abort](int FrontIdx)
			{
				if (FrontIdx % 100 == 0)
				{
					abort = CancelF();
				}
				if (abort)
				{
					return;
				}

				FVector3i ijk = winding.GetCoords(cur_front[FrontIdx]);
				float val = winding[ijk];

				// check 26-neighbours to see if we have a crossing in any direction
				for (int K = 0; K < 26; ++K) {
					FVector3i nijk = ijk + IndexUtil::GridOffsets26[K];
					if (bounds.Contains(nijk) == false)
					{
						continue;
					}
					checkSlow(winding.GetIndex(nijk) <= MAX_int32);
					int LinearNIJK = static_cast<int32>(winding.GetIndex(nijk));
					float val2 = winding[LinearNIJK];
					if (val2 == FMathf::MaxReal) {
						FVector3d gx((float)nijk.X * dx + origin[0], (float)nijk.Y * dx + origin[1], (float)nijk.Z * dx + origin[2]);
						val2 = (float)FastWinding->FastWindingNumber(gx);
						winding[LinearNIJK] = val2;
					}
					if (bits[LinearNIJK] == false) {
						// this is a 'new' voxel this round.
						// If we have a MWN-iso-crossing, add it to the front next round
						bool crossing = (val <= WindingIsoValue && val2 > WindingIsoValue) ||
							(val > WindingIsoValue && val2 <= WindingIsoValue);
						if (crossing)
						{
							FScopeLock QueueLock(&QueueSection);
							queue.Add(LinearNIJK);
						}
					}
				}
			});
			if (queue.Num() == 0)
			{
				break;
			}

			// update known-voxels list and create front for next iteration
			for (int LinearIdx : queue)
			{
				bits[LinearIdx] = true;
			}
			cur_front.Reset();
			for (int idx : queue)
			{
				cur_front.Add(idx);
			}
			queue.Reset();
		}

		if (CancelF())
		{
			return;
		}

		// fill in the rest of the grid by propagating know MWN values
		fill_spans(ni, nj, nk, winding);
	}



	void make_grid_dense(FVector3f origin, float dx, int ni, int nj, int nk, FDenseGrid3f& winding)
	{
		winding.Resize(ni, nj, nk);

		// seed MWN cache
		FastWinding->Build(false);

		bool abort = false;
		check(winding.Num() <= MAX_int32);
		ParallelFor(static_cast<int32>(winding.Num()), [this, origin, dx, &winding, &abort](int LinearIdx)
		{
			if (LinearIdx % 100 == 0)
			{
				abort = CancelF();
			}
			if (abort)
			{
				return;
			}
			FVector3i ijk = winding.GetCoords(LinearIdx);
			FVector3d gx((double)ijk.X * dx + origin[0], (double)ijk.Y * dx + origin[1], (double)ijk.Z * dx + origin[2]);
			winding[LinearIdx] = (float)FastWinding->FastWindingNumber(gx);
		});

	}


	void fill_spans(int ni, int nj, int nk, FDenseGrid3f& winding)
	{
		ParallelFor(nj*nk, [this, ni, nj, nk, &winding](int LinearJK)
		{
			int J = LinearJK % nj, K = LinearJK / nj;
			int StartIdx = ni * (J + nj * K);
			float last = winding[StartIdx];
			if (last == FMathf::MaxReal)
			{
				last = 0;
			}
			for (int I = 0; I < ni; ++I) {
				int LinearIdx = I + StartIdx;
				if (winding[LinearIdx] == FMathf::MaxReal)
				{
					winding[LinearIdx] = last;
				}
				else
				{
					last = winding[LinearIdx];
					if (last < WindingIsoValue)   // propagate zeros on outside
					{
						last = 0;
					}
				}
			}
		});
	}

};

} // end namespace UE::Geometry
} // end namespace UE