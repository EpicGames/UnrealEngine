// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DenseGrid3.h"
#include "FastWinding.h"
#include "MeshAABBTree3.h"
#include "MeshWindingNumberGrid.h"
#include "Algo/Count.h"

namespace UE::Geometry
{

/**
 * Sample mesh occupancy in a voxel grid by using the mesh's winding number, on voxel corners, to determine whether the
 * voxel is fully contained within the interior of the mesh, touches the mesh's boundary, or is fully contained on the 
 * exterior of the mesh.
 */
struct FOccupancyGrid3
{
	/// An enum representing a voxel's classification.
	enum class EDomain : int32
	{
		Exterior,		/// The point in the occupancy grid is completely on the exterior of the given mesh. 
		Boundary,		/// The point in the occupancy grid includes a mesh boundary.
		Interior		/// The point in the occupancy grid is completely on the interior of the given mesh.
	};

	template<typename MeshType>
	FOccupancyGrid3(
		const MeshType& InMesh,
		const int32 InVoxelResolution
		)
	{
		// Compute a voxel grid 
		TMeshAABBTree3<MeshType> Spatial(&InMesh);
		TFastWindingTree FastWinding(&Spatial);
		const FAxisAlignedBox3d Bounds = Spatial.GetBoundingBox();
		CellSize = float(Bounds.MaxDim() / InVoxelResolution);
		CellMidPoint = {CellSize / 2.0f, CellSize / 2.0f, CellSize / 2.0f};

		TMeshWindingNumberGrid WindingGrid(&InMesh, &FastWinding, CellSize);
	
		WindingGrid.Compute();

		// Our occupancy grid is computed on the winding number grid's cell centers.
		const FVector3i WindingDims = WindingGrid.Dimensions();
		Occupancy = {WindingDims.X - 1, WindingDims.Y - 1, WindingDims.Z - 1, EDomain::Exterior};

		GridOrigin = WindingGrid.GridOrigin + CellMidPoint;

		static constexpr FVector3i CornerOffsets[] = {
			FVector3i(0, 0, 0),
			FVector3i(0, 0, 1),
			FVector3i(0, 1, 0),
			FVector3i(0, 1, 1),
			FVector3i(1, 0, 0),
			FVector3i(1, 0, 1),
			FVector3i(1, 1, 0),
			FVector3i(1, 1, 1),
		};

		if (!ensure(Occupancy.Num() <= (int64)MAX_int32))
		{
			// TODO: support occupancy grids of larger sizes
			return;
		}
		ParallelFor(static_cast<int32>(Occupancy.Num()), [&](const int32 OccupancyId)
		{
			const FVector3i OccupancyIndex(Occupancy.GetCoords(OccupancyId));
			const SIZE_T Count = Algo::CountIf(CornerOffsets, [&WindingGrid, OccupancyIndex](FVector3i CornerOffset)
			{
				const FVector3i CornerIndex(OccupancyIndex + CornerOffset);
				return WindingGrid.GetValue(CornerIndex) >= WindingGrid.WindingIsoValue;
			});

			if (Count == 8)
			{
				Occupancy[OccupancyIndex] = EDomain::Interior;
			}
			else if (Count > 0)
			{
				Occupancy[OccupancyIndex] = EDomain::Boundary;
			}
		});

		// Make sure we include all the vertices of the mesh as a part of the boundary, if
		// the vertex areas are marked as being exterior.
		for (int32 VertexIdx = 0; VertexIdx < InMesh.VertexCount(); VertexIdx++)
		{
			const FVector3d& Pos = InMesh.GetVertex(VertexIdx);
			const FVector3i OccupancyIndex = GetCellIndexFromPoint(FVector(Pos));

			if (ensure(Occupancy.IsValidIndex(OccupancyIndex)))
			{
				if (Occupancy[OccupancyIndex] == EDomain::Exterior)
				{
					Occupancy[OccupancyIndex] = EDomain::Boundary;
				}
			}
		}
	}

	/// Given a point in space, return a computed index into the occupancy grid. If the point is outside of the mesh's
	/// bounding box, the given index will be outside the range of the occupancy grid. To check for validity, test
	/// against the occupancy grid's extents as returned by GetOccupancy.
	FVector3i GetCellIndexFromPoint(const FVector &InPoint) const
	{
		FVector3f PP(InPoint);
		PP -= GridOrigin;
		
		return { FMath::FloorToInt(PP.X / CellSize),
				FMath::FloorToInt(PP.Y / CellSize),
				FMath::FloorToInt(PP.Z / CellSize) };
	}

	/// Given an index into the occupancy grid, returns the midpoint of the bbox that represents the matching cell 
	/// in the winding number grid, from which the occupancy got computed.
	FVector3f GetCellCenterFromIndex(const FVector3i &Index) const
	{
		return {float(Index.X) * CellSize + GridOrigin.X + CellMidPoint.X,
				float(Index.Y) * CellSize + GridOrigin.Y + CellMidPoint.Y,
				float(Index.Z) * CellSize + GridOrigin.Z + CellMidPoint.Z};
	}

	/// Given an index into the occupancy grid, returns the bbox that represents the cell in the winding number
	/// grid, from which the occupancy got computed.
	FBox3f GetCellBoxFromIndex(const FVector3i &Index) const
	{
		const FVector3f P = GetCellCenterFromIndex(Index);
		return {P - CellMidPoint, P + CellMidPoint};
	}

	/// Returns a reference to the occupancy grid.  
	const TDenseGrid3<EDomain>& GetOccupancyStateGrid() const
	{
		return Occupancy;
	}

	// Returns the size of the cell. The cell size is uniform on each axis. 
	float GetCellSize() const
	{
		return CellSize;
	}
	
private:
	// The dense grid that contains the occupancy state of the mesh within each voxel. 
	TDenseGrid3<EDomain> Occupancy;

	// The size of each cell, applies to all dimensions.
	float CellSize;

	// The origin of the occupancy grid. Since the occupancy of a cell is defined at the winding number grid's cell
	// centers, this is offset by half the cell size, compared to the winding number grid's origin (same as the
	// mesh's origin, which defined by the lowest coordinate value).
	FVector3f GridOrigin;

	// This value is the offset from the lowest coordinate of a winding number grid's cell's box, to its mid-point.    
	FVector3f CellMidPoint;
};



}
