// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp PointHashGrid2

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "Util/GridIndexing2.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Hash Grid for values associated with 2D points.
 *
 * This class addresses the situation where you have a list of (point, point_data) and you
 * would like to be able to do efficient proximity queries, i.e. find the nearest point_data
 * for a given query point.
 *
 * We don't store copies of the 2D points. You provide a point_data type. This could just be the
 * integer index into your list for example, a pointer to something more complex, etc.
 * Insert and Remove functions require you to pass in the 2D point for the point_data.
 * To Update a point you need to know its old and new 2D coordinates.
 */
template<typename PointDataType, typename RealType>
class TPointHashGrid2
{
private:
	TMultiMap<FVector2i, PointDataType> Hash;
	FCriticalSection CriticalSection;
	TScaleGridIndexer2<RealType> Indexer;
	PointDataType InvalidValue;

public:

	/**
	 * Construct 2D hash grid
	 * @param cellSize size of grid cells
	 * @param InvalidValue this value will be returned by queries if no valid result is found (e.g. bounded-distance query)
	 */
	TPointHashGrid2(RealType cellSize, PointDataType InvalidValue) : Indexer(cellSize), InvalidValue(InvalidValue)
	{
	}

	/**
	 * Reserve space in the underlying hash map
	 * @param Num amount of elements to reserve
	 */
	void Reserve(int32 Num)
	{
		Hash.Reserve(Num);
	}

	/** Invalid grid value */
	PointDataType GetInvalidValue() const
	{
		return InvalidValue;
	}

	/**
	 * Insert at given position. This function is thread-safe.
	 * @param Value the point/value to insert
	 * @param Position the position associated with this value
	 */
	void InsertPoint(const PointDataType& Value, const TVector2<RealType>& Position)
	{
		FVector2i idx = Indexer.ToGrid(Position);
		{
			FScopeLock Lock(&CriticalSection);
			Hash.Add(idx, Value);
		}
	}

	/**
	 * Insert at given position, without locking / thread-safety
	 * @param Value the point/value to insert
	 * @param Position the position associated with this value
	 */
	void InsertPointUnsafe(const PointDataType& Value, const TVector2<RealType>& Position)
	{
		FVector2i idx = Indexer.ToGrid(Position);
		Hash.Add(idx, Value);
	}


	/**
	 * Remove at given position. This function is thread-safe.
	 * @param Value the point/value to remove
	 * @param Position the position associated with this value
	 * @return true if the value existed at this position
	 */
	bool RemovePoint(const PointDataType& Value, const TVector2<RealType>& Position)
	{
		FVector2i idx = Indexer.ToGrid(Position);
		{
			FScopeLock Lock(&CriticalSection);
			return Hash.RemoveSingle(idx, Value) > 0;
		}
	}

	/**
	 * Remove at given position, without locking / thread-safety
	 * @param Value the point/value to remove
	 * @param Position the position associated with this value
	 * @return true if the value existed at this position
	 */
	bool RemovePointUnsafe(const PointDataType& Value, const TVector2<RealType>& Position)
	{
		FVector2i idx = Indexer.ToGrid(Position);
		return Hash.RemoveSingle(idx, Value) > 0;
	}


	/**
	 * Move value from old to new position. This function is thread-safe.
	 * @param Value the point/value to update
	 * @param OldPosition the current position associated with this value
	 * @param NewPosition the new position for this value
	 */
	void UpdatePoint(const PointDataType& Value, const TVector2<RealType>& OldPosition, const TVector2<RealType>& NewPosition)
	{
		FVector2i old_idx = Indexer.ToGrid(OldPosition);
		FVector2i new_idx = Indexer.ToGrid(NewPosition);
		if (old_idx == new_idx)
		{
			return;
		}
		bool bWasAtOldPos;
		{
			FScopeLock Lock(&CriticalSection);
			bWasAtOldPos = Hash.RemoveSingle(old_idx, Value) > 0;
		}
		check(bWasAtOldPos);
		{
			FScopeLock Lock(&CriticalSection);
			Hash.Add(new_idx, Value);
		}
		return;
	}


	/**
	 * Move value from old to new position, without locking / thread-safety
	 * @param Value the point/value to update
	 * @param OldPosition the current position associated with this value
	 * @param NewPosition the new position for this value
	 */
	void UpdatePointUnsafe(const PointDataType& Value, const TVector2<RealType>& OldPosition, const TVector2<RealType>& NewPosition)
	{
		FVector2i old_idx = Indexer.ToGrid(OldPosition);
		FVector2i new_idx = Indexer.ToGrid(NewPosition);
		if (old_idx == new_idx)
		{
			return;
		}
		bool bWasAtOldPos = Hash.RemoveSingle(old_idx, Value) > 0;
		check(bWasAtOldPos);
		Hash.Add(new_idx, Value);
		return;
	}


	/**
	 * Find nearest point in grid, within a given sphere, without locking / thread-safety.
	 * @param QueryPoint the center of the query sphere
	 * @param Radius the radius of the query sphere
	 * @param DistanceFunc Function you provide which measures the distance between QueryPoint and a Value
	 * @param IgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 * @return the found pair (Value,DistanceFunc(Value)), or (InvalidValue,MaxDouble) if not found
	 */
	TPair<PointDataType, RealType> FindNearestInRadius(
		const TVector2<RealType>& QueryPoint, RealType Radius, 
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc = [](const PointDataType& data) { return false; }) const
	{
		if (!Hash.Num())
		{
			return TPair<PointDataType, RealType>(GetInvalidValue(), TNumericLimits<RealType>::Max());
		}

		FVector2i min_idx = Indexer.ToGrid(QueryPoint - Radius * TVector2<RealType>::One());
		FVector2i max_idx = Indexer.ToGrid(QueryPoint + Radius * TVector2<RealType>::One());

		RealType min_distsq = TNumericLimits<RealType>::Max();
		PointDataType nearest = GetInvalidValue();
		RealType RadiusSquared = Radius * Radius;

		TArray<PointDataType> Values;
		for (int yi = min_idx.Y; yi <= max_idx.Y; yi++) 
		{
			for (int xi = min_idx.X; xi <= max_idx.X; xi++) 
			{
				FVector2i idx(xi, yi);
				Values.Reset();
				Hash.MultiFind(idx, Values);
				for (PointDataType Value : Values) 
				{
					if (IgnoreFunc(Value))
					{
						continue;
					}
					RealType distsq = DistanceSqFunc(Value);
					if (distsq < RadiusSquared && distsq < min_distsq)
					{
						nearest = Value;
						min_distsq = distsq;
					}
				}
			}
		}

		return TPair<PointDataType, RealType>(nearest, min_distsq);
	}
};

template <typename PointDataType> using TPointHashGrid2d = TPointHashGrid2<PointDataType, double>;
template <typename PointDataType> using TPointHashGrid2f = TPointHashGrid2<PointDataType, float>;

} // end namespace UE::Geometry
} // end namespace UE
