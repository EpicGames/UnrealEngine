// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp PointHashGrid3

#pragma once

#include "CoreMinimal.h"
#include "Async/TransactionallySafeMutex.h"
#include "Misc/ScopeLock.h"
#include "Util/GridIndexing3.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Hash Grid for values associated with 3D points.
 *
 * This class addresses the situation where you have a list of (point, point_data) and you
 * would like to be able to do efficient proximity queries, i.e. find the nearest point_data
 * for a given query point.
 *
 * We don't store copies of the 3D points. You provide a point_data type. This could just be the
 * integer index into your list for example, a pointer to something more complex, etc.
 * Insert and Remove functions require you to pass in the 3D point for the point_data.
 * To Update a point you need to know its old and new 3D coordinates.
 */
template<typename PointDataType, typename RealType>
class TPointHashGrid3
{
private:
	TMultiMap<FVector3i, PointDataType> Hash;
	FTransactionallySafeMutex Mutex;
	TWrapAroundGridIndexer3<RealType> Indexer;
	PointDataType InvalidValue;

public:

	/**
	 * Construct 3D hash grid
	 * @param CellSize size of grid cells. Clamped to at least a small tolerance size.
	 * @param InvalidValue this value will be returned by queries if no valid result is found (e.g. bounded-distance query)
	 */
	TPointHashGrid3(RealType CellSize, PointDataType InvalidValue) : Indexer(FMath::Max(TMathUtil<RealType>::ZeroTolerance, CellSize)), InvalidValue(InvalidValue)
	{
	}

	/**
	 * Reserve space in the underlying hash map
	 * @param Num amount of elements to reserve
	 */
	void Reserve(int32 Num)
	{
		Hash.Reserve(Num);
	}

	/**
	 * Remove all entries in the underlying hash map, without resizing or releasing any allocations, and reset the cell size.
	 * @param NewCellSize The size of grid cells. Clamped to at least a small tolerance size.
	 */
	void Reset(RealType NewCellSize)
	{
		Hash.Reset();
		Indexer.CellSize = FMath::Max(TMathUtil<RealType>::ZeroTolerance, NewCellSize);
	}

	/** Invalid grid value */
	PointDataType GetInvalidValue() const
	{
		return InvalidValue;
	}

	/**
	 * Insert at given position. This function is thread-safe.
	 * @param Value the point/value to insert
	 * @param Position the position associated with this value
	 */
	void InsertPoint(const PointDataType& Value, const TVector<RealType>& Position)
	{
		FVector3i idx = Indexer.ToGrid(Position);
		{
			UE::TScopeLock Lock(Mutex);
			Hash.Add(idx, Value);
		}
	}

	/**
	 * Insert at given position, without locking / thread-safety
	 * @param Value the point/value to insert
	 * @param Position the position associated with this value
	 */
	void InsertPointUnsafe(const PointDataType& Value, const TVector<RealType>& Position)
	{
		FVector3i idx = Indexer.ToGrid(Position);
		Hash.Add(idx, Value);
	}


	/**
	 * Remove at given position. This function is thread-safe.
	 * @param Value the point/value to remove
	 * @param Position the position associated with this value
	 * @return true if the value existed at this position
	 */
	bool RemovePoint(const PointDataType& Value, const TVector<RealType>& Position)
	{
		FVector3i idx = Indexer.ToGrid(Position);
		{
			UE::TScopeLock Lock(Mutex);
			return Hash.RemoveSingle(idx, Value) > 0;
		}
	}

	/**
	 * Remove at given position, without locking / thread-safety
	 * @param Value the point/value to remove
	 * @param Position the position associated with this value
	 * @return true if the value existed at this position
	 */
	bool RemovePointUnsafe(const PointDataType& Value, const TVector<RealType>& Position)
	{
		FVector3i idx = Indexer.ToGrid(Position);
		return Hash.RemoveSingle(idx, Value) > 0;
	}


	/**
	 * Test if the cell containing Position is empty. This function is thread-safe.
	 * Can be used to skip a more expensive range search, in some cases.
	 * 
	 * @return true if the cell containing Position is empty
	 */
	bool IsCellEmpty(const TVector<RealType>& Position)
	{
		FVector3i Idx = Indexer.ToGrid(Position);
		{
			UE::TScopeLock Lock(Mutex);
			return !Hash.Contains(Idx);
		}
	}


	/**
	 * Test if the whole cell containing Position is empty, without locking / thread-safety
	 * Can be used to skip a more expensive range search, in some cases.
	 * 
	 * @return true if the cell containing Position is empty
	 */
	bool IsCellEmptyUnsafe(const TVector<RealType>& Position)
	{
		FVector3i Idx = Indexer.ToGrid(Position);
		return !Hash.Contains(Idx);
	}


	/**
	 * Move value from old to new position. This function is thread-safe.
	 * @param Value the point/value to update
	 * @param OldPosition the current position associated with this value
	 * @param NewPosition the new position for this value
	 */
	void UpdatePoint(const PointDataType& Value, const TVector<RealType>& OldPosition, const TVector<RealType>& NewPosition)
	{
		FVector3i old_idx = Indexer.ToGrid(OldPosition);
		FVector3i new_idx = Indexer.ToGrid(NewPosition);
		if (old_idx == new_idx)
		{
			return;
		}
		bool bWasAtOldPos;
		{
			UE::TScopeLock Lock(Mutex);
			bWasAtOldPos = Hash.RemoveSingle(old_idx, Value) > 0;
		}
		check(bWasAtOldPos);
		{
			UE::TScopeLock Lock(Mutex);
			Hash.Add(new_idx, Value);
		}
		return;
	}


	/**
	 * Move value from old to new position, without locking / thread-safety
	 * @param Value the point/value to update
	 * @param OldPosition the current position associated with this value
	 * @param NewPosition the new position for this value
	 */
	void UpdatePointUnsafe(const PointDataType& Value, const TVector<RealType>& OldPosition, const TVector<RealType>& NewPosition)
	{
		FVector3i old_idx = Indexer.ToGrid(OldPosition);
		FVector3i new_idx = Indexer.ToGrid(NewPosition);
		if (old_idx == new_idx)
		{
			return;
		}
		bool bWasAtOldPos = Hash.RemoveSingle(old_idx, Value) > 0;
		check(bWasAtOldPos);
		Hash.Add(new_idx, Value);
		return;
	}

	/**
	 * Find nearest point within radius.
	 * Note: Not thread-safe to update, remove or insert points during this query.
	 * @param QueryPoint the center of the query sphere
	 * @param Radius the radius of the query sphere
	 * @param DistanceSqFunc Function you provide which measures the squared distance between QueryPoint and a Value
	 * @param IgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 * @return the found pair (Value,DistanceSqFunc(Value)), or (InvalidValue,MaxDouble) if not found
	 */
	TPair<PointDataType, RealType> FindNearestInRadius(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc) const
	{
		return FindInRadiusHelper<false>(QueryPoint, Radius, DistanceSqFunc, IgnoreFunc);
	}

	TPair<PointDataType, RealType> FindNearestInRadius(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc) const
	{
		return FindInRadiusHelper<false>(QueryPoint, Radius, DistanceSqFunc, [](const PointDataType& data) { return false; });
	}

	/**
	 * Find any point within radius.
	 * Note: Not thread-safe to update, remove or insert points during this query.
	 * @param QueryPoint the center of the query sphere
	 * @param Radius the radius of the query sphere
	 * @param DistanceSqFunc Function you provide which measures the squared distance between QueryPoint and a Value
	 * @param IgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 * @return the found pair (Value,DistanceSqFunc(Value)), or (InvalidValue,MaxDouble) if not found
	 */
	TPair<PointDataType, RealType> FindAnyInRadius(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc) const
	{
		return FindInRadiusHelper<true>(QueryPoint, Radius, DistanceSqFunc, IgnoreFunc);
	}

	TPair<PointDataType, RealType> FindAnyInRadius(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc) const
	{
		return FindInRadiusHelper<true>(QueryPoint, Radius, DistanceSqFunc, [](const PointDataType& data) { return false; });
	}


	/**
	 * Find all points in grid within a given sphere. 
	 * Note: Not thread-safe to update, remove or insert points during this query.
	 * @param QueryPoint the center of the query sphere
	 * @param Radius the radius of the query sphere
	 * @param DistanceSqFunc Function you provide which measures the squared distance between QueryPoint and a Value
	 * @param ResultsOut Array that points in sphere will be added to
	 * @param IgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 * @return the number of found points
	 */
	int FindPointsInBall(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TArray<PointDataType>& ResultsOut,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc = [](const PointDataType& data) { return false; }) const
	{
		if (!Hash.Num())
		{
			return 0;
		}
		int32 InitialNum = ResultsOut.Num();

		FVector3i min_idx = Indexer.ToGrid(QueryPoint - Radius * TVector<RealType>::One());
		FVector3i max_idx = Indexer.ToGrid(QueryPoint + Radius * TVector<RealType>::One());

		RealType RadiusSquared = Radius * Radius;

		for (int zi = min_idx.Z; zi <= max_idx.Z; zi++)
		{
			for (int yi = min_idx.Y; yi <= max_idx.Y; yi++)
			{
				for (int xi = min_idx.X; xi <= max_idx.X; xi++)
				{
					FVector3i idx(xi, yi, zi);
					for (typename TMultiMap<FVector3i, PointDataType>::TConstKeyIterator It = Hash.CreateConstKeyIterator(idx); It; ++It)
					{
						const PointDataType& Value = It.Value();
						if (IgnoreFunc(Value))
						{
							continue;
						}
						RealType distsq = DistanceSqFunc(Value);
						if (distsq < RadiusSquared)
						{
							ResultsOut.Add(Value);
						}
					}
				}
			}
		}

		return ResultsOut.Num() - InitialNum;
	}

	/**
	 * Call ProcessPointFunc on all points in grid within a given sphere, or until the function returns false
	 * Note: Not thread-safe to update, remove or insert points during enumeration.
	 * @param QueryPoint the center of the query sphere
	 * @param Radius the radius of the query sphere
	 * @param DistanceSqFunc Function you provide which measures the squared distance between QueryPoint and a Value
	 * @param ProcessPointFunc Function you provide to process each found point. Takes point data and the distance squared; returns true to continue iteration, or false to stop.
	 * @param IgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 */
	void EnumeratePointsInBall(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TFunctionRef<bool(PointDataType, double)> ProcessPointFunc,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc = [](const PointDataType& data) { return false; }) const
	{
		if (!Hash.Num())
		{
			return;
		}

		FVector3i min_idx = Indexer.ToGrid(QueryPoint - Radius * TVector<RealType>::One());
		FVector3i max_idx = Indexer.ToGrid(QueryPoint + Radius * TVector<RealType>::One());

		RealType RadiusSquared = Radius * Radius;

		for (int zi = min_idx.Z; zi <= max_idx.Z; zi++)
		{
			for (int yi = min_idx.Y; yi <= max_idx.Y; yi++)
			{
				for (int xi = min_idx.X; xi <= max_idx.X; xi++)
				{
					FVector3i idx(xi, yi, zi);
					for (typename TMultiMap<FVector3i, PointDataType>::TConstKeyIterator It = Hash.CreateConstKeyIterator(idx); It; ++It)
					{
						const PointDataType& Value = It.Value();
						if (IgnoreFunc(Value))
						{
							continue;
						}
						RealType DistSq = DistanceSqFunc(Value);
						if (DistSq <= RadiusSquared)
						{
							if (!ProcessPointFunc(Value, DistSq))
							{
								return;
							}
						}
					}
				}
			}
		}
	}

private:
	template<bool bEarlyOut = false>
	TPair<PointDataType, RealType> FindInRadiusHelper(
		const TVector<RealType>& QueryPoint, RealType Radius,
		TFunctionRef<RealType(const PointDataType&)> DistanceSqFunc,
		TFunctionRef<bool(const PointDataType&)> IgnoreFunc) const
	{
		if (!Hash.Num())
		{
			return TPair<PointDataType, RealType>(GetInvalidValue(), TNumericLimits<RealType>::Max());
		}

		RealType MinDistSq = Radius * Radius;
		PointDataType Nearest = GetInvalidValue();

		auto SearchCell = [this, &Nearest, &MinDistSq, &DistanceSqFunc, &IgnoreFunc](FVector3i CellIdx)
		{
			bool bFound = false;
			for (typename TMultiMap<FVector3i, PointDataType>::TConstKeyIterator It = Hash.CreateConstKeyIterator(CellIdx); It; ++It)
			{
				const PointDataType& Value = It.Value();
				if (IgnoreFunc(Value))
				{
					continue;
				}
				RealType DistSq = DistanceSqFunc(Value);
				if (DistSq < MinDistSq)
				{
					Nearest = Value;
					MinDistSq = DistSq;
					if (bEarlyOut) {
						return true;
					}
					bFound = true;
				}
			}
			return bFound;
		};

		FVector3i CenterIdx = Indexer.ToGrid(QueryPoint);
		RealType SearchRadius = Radius;

		if (SearchCell(CenterIdx))
		{
			if (bEarlyOut)
			{
				return TPair<PointDataType, RealType>(Nearest, MinDistSq);
			}
			SearchRadius = FMath::Sqrt(MinDistSq);
		}

		Indexer.IterateAcrossBounds(
			QueryPoint - SearchRadius * TVector<RealType>::One(),
			QueryPoint + SearchRadius * TVector<RealType>::One(), [&CenterIdx, &SearchCell](const FVector3i& Idx)
		{
			if (Idx == CenterIdx)
			{
				return true; // continue
			}
			bool bFound = SearchCell(Idx);
			return !(bEarlyOut && bFound);
		});

		if (Nearest == GetInvalidValue())
		{
			MinDistSq = TNumericLimits<RealType>::Max();
		}
		return TPair<PointDataType, RealType>(Nearest, MinDistSq);
	}

};

template <typename PointDataType> using TPointHashGrid3d = TPointHashGrid3<PointDataType, double>;
template <typename PointDataType> using TPointHashGrid3f = TPointHashGrid3<PointDataType, float>;

} // end namespace UE::Geometry
} // end namespace UE
