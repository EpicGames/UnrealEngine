// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp PointSetHashTable

#pragma once

#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "PointSetAdapter.h"
#include "Spatial/SparseGrid3.h"
#include "Util/GridIndexing3.h"

namespace UE
{
namespace Geometry
{

/**
 * FPointSetHashTable builds a spatial data structure that supports efficient
 * range queries on a point set (in FPointSetAdapterd form). The spatial data
 * structure is currently a uniform sparse 3D grid.
 *
 * @todo support larger search radius than cellsize
 */
class FPointSetHashtable
{
protected:
	typedef TArray<int> PointList;

	/** Input point set */
	FPointSetAdapterd* Points;
	/** Sparse grid, each voxel contains lists of contained points */
	TSparseGrid3<PointList> Grid;
	/** index mapping object */
	FShiftGridIndexer3d GridIndexer;

	/** World origin of sparse grid */
	FVector3d Origin;
	/** Cell size of grid */
	double CellSize;

public:
	FPointSetHashtable(FPointSetAdapterd* PointSetIn)
	{
		Points = PointSetIn;
	}

	/**
	 * Construct the spatial data structure for the current point set
	 * @param CellSize the size of the cells/voxels in the grid.
	 * @param Origin World origin of the grid (not strictly necessary since grid is sparse, can set to origin for example)
	 */
	GEOMETRYCORE_API void Build(double CellSize, const FVector3d& Origin);

	/**
	 * Find all points within given query distance from query point.
	 * Note that in current implementation the distance must be less than the CellSize used in construction,
	 * ie at most the directly adjacent neighbours next to the cell containing QueryPt can be searched
	 * @param QueryPt center of search sphere/ball
	 * @param QueryRadius radius of search sphere/ball. Points within this distance of QueryPt are returned.
	 * @param ResultOut indices of discovered points are stored in this list
	 * @return true on success
	 */
	GEOMETRYCORE_API bool FindPointsInBall(const FVector3d& QueryPt, double QueryRadius, TArray<int>& ResultOut);

};


} // end namespace UE::Geometry
} // end namespace UE
