// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Spatial/PointHashGrid3.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Generate a "priority ordering" for a point set
 *
 * Either order by descending ImportanceWeight, or try to additionally un-clump the points --
 * such that if we only look at the first N points, they will cover the overall space as well as possible
 * while still favoring higher ImportanceWeight points in each local region.
 */
struct FPriorityOrderPoints
{

	/// Inputs

	// Octree levels to populate incrementally when spacing out points via ComputeUniformSpaced
	// We will consider a maximum of (2^SpatialLevels)^3 "cells" at the finest granularity
	int32 SpatialLevels = 10;


	/// Outputs

	TArray<int32> Order;

public:

	FPriorityOrderPoints(int32 SpatialLevels = 10) : SpatialLevels(SpatialLevels)
	{
	}

	/**
	 * Compute an ordering for the input points that attempts to keep points 'well spaced' / un-clumped
	 * 
	 * @param Points				Points to re-order
	 * @param ImportanceWeights		Points with a higher ImportanceWeight in their local region of space will appear earlier in the ordering
	 * @param EarlyStop				If >= 0, stop re-ordering once we've sorted this many points; beyond this the ordering will be arbitrary
	 * @param OffsetResFactor		Factor that controls how closely the points are allowed to space, with higher values -> more clumping allowed
	 */
	GEOMETRYCORE_API void ComputeUniformSpaced(TArrayView<const FVector3d> Points, TArrayView<const float> ImportanceWeights, int32 EarlyStop = -1, int32 OffsetResFactor = 4);
	GEOMETRYCORE_API void ComputeUniformSpaced(TArrayView<const FVector3f> Points, TArrayView<const float> ImportanceWeights, int32 EarlyStop = -1, int32 OffsetResFactor = 4);

	/**
	 * Compute an ordering for the input points that attempts to keep points 'well spaced' / un-clumped
	 *
	 * @param Points					Points to re-order
	 * @param ImportanceWeights			Points with a higher ImportanceWeight in their local region of space will appear earlier in the ordering
	 * @param SecondImportanceWeights	A second set of importance weights; we may pick an additional point from each local region to also sample points by this metric
	 * @param EarlyStop					If >= 0, stop re-ordering once we've sorted this many points; beyond this the ordering will be arbitrary
	 * @param OffsetResFactor			Factor that controls how closely the points are allowed to space, with higher values -> more clumping allowed
	 */
	GEOMETRYCORE_API void ComputeUniformSpaced(TArrayView<const FVector3d> Points, TArrayView<const float> ImportanceWeights, TArrayView<const float> SecondImportanceWeights, int32 EarlyStop = -1, int32 OffsetResFactor = 4);
	GEOMETRYCORE_API void ComputeUniformSpaced(TArrayView<const FVector3f> Points, TArrayView<const float> ImportanceWeights, TArrayView<const float> SecondImportanceWeights, int32 EarlyStop = -1, int32 OffsetResFactor = 4);

	/**
	 * Compute an ordering that only sorts by descending ImportanceWeights, and does not attempt to spatially distribute the points
	 * 
	 * @param ImportanceWeights		Points with a higher ImportanceWeight will appear earlier in the ordering
	 */
	GEOMETRYCORE_API void ComputeDescendingImportance(TArrayView<const float> ImportanceWeights);

};

} // end namespace UE::Geometry
} // end namespace UE
