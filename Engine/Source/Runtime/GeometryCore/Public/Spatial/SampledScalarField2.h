// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DSparseGrid3

#pragma once

#include "CoreMinimal.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "DenseGrid2.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TSampledScalarField2 implements a generic 2D grid of values that can be interpolated in various ways.
 * The grid is treated as a set of sample points in 2D space, IE a grid origin and x/y point-spacing is part of this class.
 * 
 * The class is templated on two types:
 *    RealType: the real type used for spatial calculations, ie 2D grid origin, cell dimensions, and sample positions
 *    ValueType: the type of value stored in the grid. Could be real or vector-typed, needs to support multiplication by RealType (for interpolation)
 */
template<typename RealType, typename ValueType>
class TSampledScalarField2
{
public:
	TDenseGrid2<ValueType> GridValues;

	TVector2<RealType> GridOrigin;
	TVector2<RealType> CellDimensions;

	/**
	 * Create empty grid, defaults to 2x2 grid of whatever default value of ValueType is
	 */
	TSampledScalarField2()
	{
		GridValues.Resize(2, 2);
		GridOrigin = TVector2<RealType>::Zero();
		CellDimensions = TVector2<RealType>::One();
	}

	void CopyConfiguration(const TSampledScalarField2<RealType,ValueType>& OtherField)
	{
		GridValues.Resize(OtherField.GridValues.Width(), OtherField.GridValues.Height());
		GridOrigin = OtherField.GridOrigin;
		CellDimensions = OtherField.CellDimensions;
	}

	/**
	 * Resize the grid to given Width/Height, and initialize w/ given InitValue
	 */
	void Resize(int64 Width, int64 Height, const ValueType& InitValue)
	{
		GridValues.Resize(Width, Height);
		GridValues.Assign(InitValue);
	}

	int32 Width() const
	{
		return (int32)GridValues.Width();
	}

	int32 Height() const
	{
		return (int32)GridValues.Height();
	}

	int64 Num() const
	{
		return GridValues.Num();
	}

	/**
	 * Set the 2D origin of the grid
	 */
	void SetPosition(const TVector2<RealType>& Origin)
	{
		GridOrigin = Origin;
	}
	
	/**
	 * Set the size of the grid cells to uniform CellSize
	 */
	void SetCellSize(RealType CellSize)
	{
		CellDimensions.X = CellDimensions.Y = CellSize;
	}

	/**
	 * Sample scalar field with bilinear interpolation at given Position
	 * @param Position sample point relative to grid origin/dimensions
	 * @return interpolated value at this position
	 */
	ValueType BilinearSampleClamped(const TVector2<RealType>& Position) const
	{
		// transform Position into grid coordinates
		TVector2<RealType> GridPoint(
			((Position.X - GridOrigin.X) / CellDimensions.X),
			((Position.Y - GridOrigin.Y) / CellDimensions.Y));

		// compute integer grid coordinates
		int64 x0 = (int64)GridPoint.X;
		int64 x1 = x0 + 1;
		int64 y0 = (int64)GridPoint.Y;
		int64 y1 = y0 + 1;

		// clamp to valid range
		int64 Width = GridValues.Width(), Height = GridValues.Height();
		x0 = FMath::Clamp(x0, (int64)0, Width - 1);
		x1 = FMath::Clamp(x1, (int64)0, Width - 1);
		y0 = FMath::Clamp(y0, (int64)0, Height - 1);
		y1 = FMath::Clamp(y1, (int64)0, Height - 1);

		// convert real-valued grid coords to [0,1] range
		RealType fAx = FMath::Clamp(GridPoint.X - (RealType)x0, (RealType)0, (RealType)1);
		RealType fAy = FMath::Clamp(GridPoint.Y - (RealType)y0, (RealType)0, (RealType)1);
		RealType OneMinusfAx = (RealType)1 - fAx;
		RealType OneMinusfAy = (RealType)1 - fAy;

		// fV## is grid cell corner index
		const ValueType& fV00 = GridValues[y0*Width + x0];
		const ValueType& fV10 = GridValues[y0*Width + x1];
		const ValueType& fV01 = GridValues[y1*Width + x0];
		const ValueType& fV11 = GridValues[y1*Width + x1];

		return
			(OneMinusfAx * OneMinusfAy) * fV00 +
			(OneMinusfAx * fAy)         * fV01 +
			(fAx         * OneMinusfAy) * fV10 +
			(fAx         * fAy)         * fV11;
	}




	/**
	 * Sample scalar field gradient with bilinear interpolation at given Position
	 * @param Position sample point relative to grid origin/dimensions
	 * @return interpolated value at this position
	 */
	void BilinearSampleGradientClamped(const TVector2<RealType>& Position, ValueType& GradXOut, ValueType& GradYOut) const
	{
		// transform Position into grid coordinates
		TVector2<RealType> GridPoint(
			((Position.X - GridOrigin.X) / CellDimensions.X),
			((Position.Y - GridOrigin.Y) / CellDimensions.Y));

		// compute integer grid coordinates
		int64 x0 = (int64)GridPoint.X;
		int64 x1 = x0 + 1;
		int64 y0 = (int64)GridPoint.Y;
		int64 y1 = y0 + 1;

		// clamp to valid range
		int64 Width = GridValues.Width(), Height = GridValues.Height();
		x0 = FMath::Clamp(x0, (int64)0, Width - 1);
		x1 = FMath::Clamp(x1, (int64)0, Width - 1);
		y0 = FMath::Clamp(y0, (int64)0, Height - 1);
		y1 = FMath::Clamp(y1, (int64)0, Height - 1);

		// convert real-valued grid coords to [0,1] range
		RealType fAx = FMath::Clamp(GridPoint.X - (RealType)x0, (RealType)0, (RealType)1);
		RealType fAy = FMath::Clamp(GridPoint.Y - (RealType)y0, (RealType)0, (RealType)1);
		RealType OneMinusfAx = (RealType)1 - fAx;
		RealType OneMinusfAy = (RealType)1 - fAy;

		// fV## is grid cell corner index
		const ValueType& fV00 = GridValues[y0 * Width + x0];
		const ValueType& fV10 = GridValues[y0 * Width + x1];
		const ValueType& fV01 = GridValues[y1 * Width + x0];
		const ValueType& fV11 = GridValues[y1 * Width + x1];

		GradXOut =
			-fV00 * (OneMinusfAy) +
			-fV01 * (fAy) +
			fV10 * (OneMinusfAy) +
			fV11 * (fAy);

		GradYOut =
			-fV00 * (OneMinusfAx) +
			fV01 * (OneMinusfAx)  +
			-fV10 * (fAx) +
			fV11 * (fAx);
	}

};

typedef TSampledScalarField2<double, double> FSampledScalarField2d;
typedef TSampledScalarField2<float, float> FSampledScalarField2f;

} // end namespace UE::Geometry
} // end namespace UE
