// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/SpatialInterfaces.h"
#include "Util/DynamicVector.h"
#include "SegmentTypes.h"
#include "BoxTypes.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "Intersection/IntersectionUtil.h"
#include "Distance/DistRay3Segment3.h"


namespace UE
{
namespace Geometry
{



/**
 * FSegmentTree3 is a spatial data structure for a set of 3D line segments.
 * The line segments are provided externally, and each segment can have
 * an arbitrary ID. The line segment geometry (ie endpoints) are stored directly
 * by FSegmentTree3, so the class does not hold onto any reference to the source 
 * geometry (ie unlike FMeshAABBTree3).
 * 
 * Available queries:
 *   - FindNearestSegment(Point) - finds the nearest segment to the 3D point
 *   - FindNearestVisibleSegmentHitByRay(Ray) - finds the nearest segment
 *        that is "hit" by the ray under a tolerance check (see comments below).
 *        This is mainly intended for UI hit testing.
 * 
 */
class FSegmentTree3
{
public:
	using GetSplitAxisFunc = TUniqueFunction<int(int Depth, const FAxisAlignedBox3d& Box)>;

public:

	/**
	 * FSegment is a 3D line segment with an external identifier
	 */
	struct FSegment
	{
		int32 ID;
		FSegment3d Segment;
	};

	/**
	 * Build the segment tree based on the given Enumerable (ie something that supports a range-based for loop
	 * over a set of integer IDs) and a function GetSegmentForID that returns the 3D line segment for a given ID.
	 * Gaps/etc are allowed but all enumerated IDs must be valid. 
	 */
	template<typename SegmentIDEnumerable, typename GetSegmentFunc>
	void Build(SegmentIDEnumerable Enumerable, GetSegmentFunc GetSegmentForID, int32 NumSegmentsHint = 0)
	{
		SegmentList.Reserve(NumSegmentsHint);

		for (int32 SegmentID : Enumerable)
		{
			FSegment3d Segment = GetSegmentForID(SegmentID);
			SegmentList.Add( FSegment{SegmentID, Segment} );
		}

		// Make explicit list of linear indices and centers for the build step, which
		// will sort/swap these lists in-place. 
		// Centers could possibly be removed as centers can be fetched from the SegmentList...
		TArray<int> SegmentIndices;
		SegmentIndices.Reserve(SegmentList.Num());
		TArray<FVector3d> Centers;
		Centers.Reserve(SegmentList.Num());
		for (int32 k = 0; k < SegmentList.Num(); ++k )
		{
			SegmentIndices.Add(k);
			Centers.Add( SegmentList[k].Segment.Center );
		}
		BuildTopDown(SegmentIndices, Centers, SegmentList.Num());
	}

	void Build(const TArray<FSegment3d>& Segments)
	{
		SegmentList.SetNumUninitialized(Segments.Num());
		for (int32 SegmentID = 0; SegmentID < Segments.Num(); ++SegmentID)
		{
			SegmentList[SegmentID].ID = SegmentID;
			SegmentList[SegmentID].Segment = Segments[SegmentID];
		}

		// Make explicit list of linear indices and centers for the build step, which
		// will sort/swap these lists in-place. 
		// Centers could possibly be removed as centers can be fetched from the SegmentList...
		TArray<int> SegmentIndices;
		SegmentIndices.Reserve(SegmentList.Num());
		TArray<FVector3d> Centers;
		Centers.Reserve(SegmentList.Num());
		for (int32 k = 0; k < SegmentList.Num(); ++k)
		{
			SegmentIndices.Add(k);
			Centers.Add(SegmentList[k].Segment.Center);
		}
		BuildTopDown(SegmentIndices, Centers, SegmentList.Num());
	}

	/**
	 * Find the nearest segment to query point P, and return it in NearestSegmentOut
	 */
	bool FindNearestSegment(
		const FVector3d& P, FSegment& NearestSegmentOut,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()
	) const
	{
		double NearestDistSqr = (Options.MaxDistance < TNumericLimits<double>::Max()) ? 
			Options.MaxDistance * Options.MaxDistance : TNumericLimits<double>::Max();
		int NearestSegmentIdx = IndexConstants::InvalidID;
		FindNearestSegmentInternal(RootBoxIndex, P, NearestDistSqr, NearestSegmentIdx, Options);
		if (NearestSegmentIdx != IndexConstants::InvalidID)
		{
			NearestSegmentOut = SegmentList[NearestSegmentIdx];
			return true;
		}
		return false;
	}

	/**
	 * Information on a nearest-segment returned by query functions
	 */
	struct FRayNearestSegmentInfo
	{
		/** Parameter at point on ray closest to identified nearest segment */
		double RayParam;
		/** Parameter at point on nearest segment closest to the ray */
		double SegmentParam;
		/** Distance between ray point at RayParam and segment point at SegmentParam */
		double SegmentDist;
		/** distance metric at nearest points */
		double Metric;
	};

	/**
	 * Find the segment that is hit by a 3D ray under a function WithinToleranceCheck,
	 * which will be called with the Ray and Segment points. For example if this is a 3D
	 * distance check against a fixed radius, this function essentially raycasts against 3D capsules.
	 * 
	 * The function compares potential hits (ie that pass the tolerance) with a sort of 
	 * "distance" metric that tries to balance distance from the ray origin (ie "close to eye") 
	 * and distance from the hit segment (ie "on the line"). There is no correct way to do
	 * this, currently the code uses a metric based on the opening angle between ray 
	 * and segment points. 
	 */
	bool FindNearestVisibleSegmentHitByRay(
		const FRay3d& Ray,
		TFunctionRef<bool(int32, const FVector3d&, const FVector3d&)> WithinToleranceCheck,
		FSegment& NearestHitSegmentOut, 
		FRayNearestSegmentInfo& NearestInfo,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const
	{
		// Note: using TNumericLimits<float>::Max() here because we need to use <= to compare Box hit
		//   to NearestT, and Box hit returns TNumericLimits<double>::Max() on no-hit. So, if we set
		//   nearestT to TNumericLimits<double>::Max(), then we will test all boxes (!)
		NearestInfo = FRayNearestSegmentInfo();
		NearestInfo.RayParam = (Options.MaxDistance < TNumericLimits<float>::Max()) ? Options.MaxDistance : TNumericLimits<float>::Max();
		NearestInfo.SegmentParam = 0.5;
		NearestInfo.SegmentDist = TNumericLimits<double>::Max();
		NearestInfo.Metric = TNumericLimits<double>::Max();

		int NearestHitSegmentIdx = IndexConstants::InvalidID;
		FindNearestVisibleSegmentHitByRayInternal(RootBoxIndex, Ray, WithinToleranceCheck, NearestHitSegmentIdx, NearestInfo, Options);
		if (NearestHitSegmentIdx != IndexConstants::InvalidID)
		{
			NearestHitSegmentOut = SegmentList[NearestHitSegmentIdx];
			return true;
		}
		return false;
	}



protected:


	int TopDownLeafMaxSegmentCount = 8;

	static GetSplitAxisFunc MakeDefaultSplitAxisFunc()
	{
		return [](int Depth, const FAxisAlignedBox3d&)
		{
			return Depth % 3;
		};
	}

	GetSplitAxisFunc GetSplitAxis = MakeDefaultSplitAxisFunc();

	// list of segments
	TArray<FSegment> SegmentList;

	// storage for Box Nodes.
	struct FTreeBox
	{
		int BoxToIndex;			// index into IndexList
		FVector3d Center;
		FVector3d Extents;
	};
	TArray<FTreeBox> TreeBoxes;

	// list of indices for a given Box. There is *no* marker/sentinel between
	// the per-box lists, you have to get the starting index from TreeBox.BoxToIndex
	//
	// There are two sections, first are the lists of triangle indices in the leaf boxes,
	// and then the lists for the internal-node boxes that have one or two child boxes.
	// So there are three kinds of records:
	//   - if i < SegmentsEnd, then the list is a number of Segments,
	//       stored as [N t1 t2 t3 ... tN]
	//   - if i > SegmentsEnd and IndexList[i] < 0, this is a single-child
	//       internal Box, with index (-IndexList[i])-1     (shift-by-one in case actual value is 0!)
	//   - if i > SegmentsEnd and IndexList[i] > 0, this is a two-child
	//       internal Box, with indices IndexList[i]-1 and IndexList[i+1]-1
	TArray<int> IndexList;

	// IndexList[i] for i < SegmentsEnd is a segment-index list, ie a leaf-box list
	int SegmentsEnd = -1;

	// RootBoxIndex is the index of the root box of the tree, index into TreeBoxes
	int RootBoxIndex = -1;



	// this is a temporary data structure used in tree building
	struct FBoxesSet
	{
		TDynamicVector<FTreeBox> Boxes;
		TDynamicVector<int> IndexList;
		int IBoxCur;
		int IIndicesCur;
		FBoxesSet()
		{
			IBoxCur = 0;
			IIndicesCur = 0;
		}
	};

	void BuildTopDown(TArray<int>& SegmentIndices, TArray<FVector3d>& SegmentCenters, int32 NumSegments)
	{
		FBoxesSet Tris;
		FBoxesSet Nodes;
		FAxisAlignedBox3d RootBox;
		int SplitsRootNode =
			SplitSegmentSetMidpoints(SegmentIndices, SegmentCenters, 0, NumSegments, 0, TopDownLeafMaxSegmentCount, Tris, Nodes, RootBox);

		SegmentsEnd = Tris.IIndicesCur;
		int IndexShift = SegmentsEnd;
		int BoxShift = Tris.IBoxCur;

		// append internal node boxes & index ptrs
		TDynamicVector<FTreeBox>& UseBoxes = Tris.Boxes;
		for (int32 i = 0; i < Nodes.IBoxCur; ++i)
		{
			FVector3d NodeBoxCenter = Nodes.Boxes[i].Center;		// cannot pass as argument in case a resize happens
			FVector3d NodeBoxExtents = Nodes.Boxes[i].Extents;
			int NodeBoxIndex = Nodes.Boxes[i].BoxToIndex;

			// internal node indices are shifted
			UseBoxes.InsertAt(
				FTreeBox{ IndexShift + NodeBoxIndex, NodeBoxCenter, NodeBoxExtents }, BoxShift + i);
		}

		// copy to final boxes list
		int32 NumBoxes = UseBoxes.Num();
		TreeBoxes.SetNum(NumBoxes);
		for (int32 k = 0; k < NumBoxes; ++k)
		{
			TreeBoxes[k] = UseBoxes[k];
		}

		// now append index list
		TDynamicVector<int>& UseIndexList = Tris.IndexList;
		for (int32 i = 0; i < Nodes.IIndicesCur; ++i)
		{
			int ChildBox = Nodes.IndexList[i];
			if (ChildBox < 0)
			{ 
				// this is a Segments Box
				ChildBox = (-ChildBox) - 1;
			}
			else
			{
				ChildBox += BoxShift;
			}
			ChildBox = ChildBox + 1;
			UseIndexList.InsertAt(ChildBox, IndexShift + i);
		}

		int32 NumIndices = UseIndexList.Num();
		IndexList.SetNum(NumIndices);
		for (int32 k = 0; k < NumIndices; ++k)
		{
			IndexList[k] = UseIndexList[k];
		}

		RootBoxIndex = SplitsRootNode + BoxShift;
	}


	// TODO: should not actually need SegmentCenters here, can get from Indices...

	int SplitSegmentSetMidpoints(
		TArray<int>& SegmentIndices,
		TArray<FVector3d>& SegmentCenters,
		int StartIdx, int Count, int Depth, int MinSegmentCount,
		FBoxesSet& SegmentBoxes, FBoxesSet& Nodes, FAxisAlignedBox3d& Box)
	{
		Box = (SegmentIndices.Num() > 0) ?
			FAxisAlignedBox3d::Empty() : FAxisAlignedBox3d(FVector3d::Zero(), 0.0);
		int BoxIndex = -1;

		if (Count <= MinSegmentCount)
		{
			// append new Segments Box
			BoxIndex = SegmentBoxes.IBoxCur++;
			int32 IndexCur = SegmentBoxes.IIndicesCur;

			SegmentBoxes.IndexList.InsertAt(Count, SegmentBoxes.IIndicesCur++);
			for (int i = 0; i < Count; ++i)
			{
				int32 SegmentIdx = SegmentIndices[StartIdx + i];
				SegmentBoxes.IndexList.InsertAt(SegmentIdx, SegmentBoxes.IIndicesCur++);
				FAxisAlignedBox3d SegmentBounds = SegmentList[SegmentIdx].Segment.GetBounds();
				Box.Contain(SegmentBounds);
			}

			SegmentBoxes.Boxes.InsertAt( FTreeBox{IndexCur, Box.Center(), Box.Extents()}, BoxIndex );

			return -(BoxIndex + 1);
		}

		//compute interval along an axis and find midpoint
		int SplitAxis = GetSplitAxis(Depth, Box);
		FInterval1d Interval = FInterval1d::Empty();
		for (int i = 0; i < Count; ++i)
		{
			Interval.Contain(SegmentCenters[StartIdx + i][SplitAxis]);
		}
		double Midpoint = Interval.Center();

		int Count0, Count1;
		if (Interval.Length() > FMathd::ZeroTolerance)
		{
			// we have to re-sort the Centers & Segments lists so that Centers < midpoint
			// are first, so that we can recurse on the two subsets. We walk in from each side,
			// until we find two out-of-order locations, then we swap them.
			int Left = 0;
			int Right = Count - 1;
			while (Left < Right)
			{
				// TODO: is <= right here? if V.axis == midpoint, then this loop
				//   can get stuck unless one of these has an equality test. But
				//   I did not think enough about if this is the right thing to do...
				while (SegmentCenters[StartIdx + Left][SplitAxis] <= Midpoint)
				{
					Left++;
				}
				while (SegmentCenters[StartIdx + Right][SplitAxis] > Midpoint)
				{
					Right--;
				}
				if (Left >= Right)
				{
					break; //done!
						   //swap
				}
				Swap(SegmentCenters[StartIdx + Left], SegmentCenters[StartIdx + Right]);
				Swap(SegmentIndices[StartIdx + Left], SegmentIndices[StartIdx + Right]);
			}

			Count0 = Left;
			Count1 = Count - Count0;
			checkSlow(Count0 >= 1 && Count1 >= 1);
		}
		else
		{
			// interval is near-empty, so no point trying to do sorting, just split half and half
			Count0 = Count / 2;
			Count1 = Count - Count0;
		}

		// create child boxes
		FAxisAlignedBox3d Child1Box;
		int Child0 = SplitSegmentSetMidpoints(SegmentIndices, SegmentCenters, StartIdx, Count0, Depth + 1, MinSegmentCount, SegmentBoxes, Nodes, Box);
		int Child1 = SplitSegmentSetMidpoints(SegmentIndices, SegmentCenters, StartIdx + Count0, Count1, Depth + 1, MinSegmentCount, SegmentBoxes, Nodes, Child1Box);
		Box.Contain(Child1Box);

		// append new Box
		BoxIndex = Nodes.IBoxCur++;
		Nodes.Boxes.InsertAt( FTreeBox{Nodes.IIndicesCur, Box.Center(), Box.Extents()}, BoxIndex);

		Nodes.IndexList.InsertAt(Child0, Nodes.IIndicesCur++);
		Nodes.IndexList.InsertAt(Child1, Nodes.IIndicesCur++);

		return BoxIndex;
	}


public:
	/**
	* Sets the box intersection tolerance
	* TODO: move into the IMeshSpatial::FQueryOptions and delete this function
	*/
	void SetTolerance(double Tolerance)
	{
		BoxEps = Tolerance;
	}

protected:
	// TODO: move BoxEps to IMeshSpatial::FQueryOptions
	double BoxEps = FMathd::ZeroTolerance;

	double GetBoxDistanceSqr(int BoxIndex, const FVector3d& V) const
	{
		const FTreeBox& Box = TreeBoxes[BoxIndex];

		// This appears to be more accurate, but I think the distance computation is more expensive (branches)
		// (if we preferred accuracy, we could store box in [Min,Max] instead of [Center,Extent]...)
		//FAxisAlignedBox3d Temp(Box.Center - Box.Extents, Box.Center + Box.Extents);
		//double DistSqr1 = Temp.DistanceSquared(V);

		// per-axis delta is max(abs(P-c) - e, 0)... ?
		double dx = FMath::Max(FMathd::Abs(V.X - Box.Center.X) - Box.Extents.X, 0.0);
		double dy = FMath::Max(FMathd::Abs(V.Y - Box.Center.Y) - Box.Extents.Y, 0.0);
		double dz = FMath::Max(FMathd::Abs(V.Z - Box.Center.Z) - Box.Extents.Z, 0.0);
		return dx * dx + dy * dy + dz * dz;
	}

	double GetRayBoxIntersectionParam(int BoxIndex, const FRay3d& Ray, double Radius = 0) const
	{
		const FTreeBox& TreeBox = TreeBoxes[BoxIndex];
		FVector3d Extents = TreeBox.Extents + (Radius + BoxEps);
		FAxisAlignedBox3d Box(TreeBox.Center - Extents, TreeBox.Center + Extents);

		double RayParameter = TNumericLimits<double>::Max();
		if (FIntrRay3AxisAlignedBox3d::FindIntersection(Ray, Box, RayParameter))
		{
			return RayParameter;
		}
		else
		{
			return TNumericLimits<double>::Max();
		}
	}



protected:
	void FindNearestSegmentInternal(int BoxIndex, const FVector3d& P, double& NearestDistSqr, int& NearSegmentIdx, const IMeshSpatial::FQueryOptions& Options) const
	{
		const FTreeBox& TreeBox = TreeBoxes[BoxIndex];
		if (TreeBox.BoxToIndex < SegmentsEnd)
		{ 
			// segment-list case, array is [N t1 t2 ... tN]
			int NumSegments = IndexList[TreeBox.BoxToIndex];
			for (int i = 1; i <= NumSegments; ++i)
			{
				int SegmentIdx = IndexList[TreeBox.BoxToIndex + i];
				
				if (Options.TriangleFilterF != nullptr && Options.TriangleFilterF(SegmentIdx) == false)
				{
					continue;
				}

				double SegmentDistSqr = SegmentList[SegmentIdx].Segment.DistanceSquared(P);
				if (SegmentDistSqr < NearestDistSqr)
				{
					NearestDistSqr = SegmentDistSqr;
					NearSegmentIdx = SegmentIdx;
				}
			}
		}
		else
		{ 
			// internal node, either 1 or 2 child boxes
			int iChild1 = IndexList[TreeBox.BoxToIndex];
			if (iChild1 < 0)
			{ 
				// 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1DistSqr = GetBoxDistanceSqr(iChild1, P);
				if (fChild1DistSqr <= NearestDistSqr)
				{
					FindNearestSegmentInternal(iChild1, P, NearestDistSqr, NearSegmentIdx, Options);
				}
			}
			else
			{ 
				// 2 children, descend closest first
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[TreeBox.BoxToIndex + 1] - 1;

				double fChild1DistSqr = GetBoxDistanceSqr(iChild1, P);
				double fChild2DistSqr = GetBoxDistanceSqr(iChild2, P);
				if (fChild1DistSqr < fChild2DistSqr)
				{
					if (fChild1DistSqr < NearestDistSqr)
					{
						FindNearestSegmentInternal(iChild1, P, NearestDistSqr, NearSegmentIdx, Options);
						if (fChild2DistSqr < NearestDistSqr)
						{
							FindNearestSegmentInternal(iChild2, P, NearestDistSqr, NearSegmentIdx, Options);
						}
					}
				}
				else
				{
					if (fChild2DistSqr < NearestDistSqr)
					{
						FindNearestSegmentInternal(iChild2, P, NearestDistSqr, NearSegmentIdx, Options);
						if (fChild1DistSqr < NearestDistSqr)
						{
							FindNearestSegmentInternal(iChild1, P, NearestDistSqr, NearSegmentIdx, Options);
						}
					}
				}
			}
		}
	}




	// This function tries to find the segment "hit" by the ray, where the definition
	// of "hit" tries to balance distance from the ray origin (ie "close to eye") and
	// distance from the hit segment (ie "on the line"). There is no correct way to do
	// this, currently the code uses a metric based on the opening angle between ray
	// and segment points. 
	void FindNearestVisibleSegmentHitByRayInternal(
		int BoxIndex, const FRay3d& Ray, 
		TFunctionRef<bool(int32, const FVector3d&, const FVector3d&)> WithinToleranceCheck,
		int& NearSegmentIdx, FRayNearestSegmentInfo& NearestInfo,
		const IMeshSpatial::FQueryOptions& Options) const
	{
		const FTreeBox& TreeBox = TreeBoxes[BoxIndex];
		if (TreeBox.BoxToIndex < SegmentsEnd)
		{
			// segment-list case, array is [N t1 t2 ... tN]
			int NumSegments = IndexList[TreeBox.BoxToIndex];
			for (int i = 1; i <= NumSegments; ++i)
			{
				int SegmentIdx = IndexList[TreeBox.BoxToIndex + i];

				const FSegment3d& Segment = SegmentList[SegmentIdx].Segment;
				double SegRayParam; double SegSegParam;
				double SegDistSqr = FDistRay3Segment3d::SquaredDistance(Ray, Segment, SegRayParam, SegSegParam);
				FVector3d RayPoint = Ray.PointAt(SegRayParam), SegPoint = Segment.PointAt(SegSegParam);
				if (WithinToleranceCheck( SegmentList[SegmentIdx].ID, RayPoint, SegPoint) )
				{
					// try to balance preference for closer-to-ray-origin and closer-to-segment
					double DistanceMetric = FMathd::Abs(VectorUtil::Area(Ray.Origin, SegPoint, RayPoint));

					if (DistanceMetric < NearestInfo.Metric)
					{
						NearSegmentIdx = SegmentIdx;
						NearestInfo.RayParam = SegRayParam;
						NearestInfo.SegmentParam = SegSegParam;
						NearestInfo.SegmentDist = FMathd::Sqrt(SegDistSqr);
						NearestInfo.Metric = DistanceMetric;
					}

				}
			}
		}
		else
		{
			// expand boxes by this amount to try to ensure that we hit sufficient radius...
			double UseBoxRadius = FMathd::Min( 2*NearestInfo.SegmentDist, (double)TNumericLimits<float>::Max() );

			// This code may need some work. Since we might prefer a "further" hit segment
			// that is closer to the ray, have to descend any hit box. We need to grow the
			// box by the maximum hit radius that would have any effect. However we don't
			// easily know that, in particular because we don't know what WithinToleranceCheck()
			// is based on. It could be a world-space distance but in some usage it is based
			// on a projection to 2D, or a visual-angle. In this case it is quite hard to
			// determine how much the bounding-box needs to be expanded. One option would be
			// to do the WithinToleranceCheck() on the box corners, or alternate the caller
			// could provide a function to compute the tolerance radius for any 3D point
			// (which may be quite expensive too)...

			// internal node, either 1 or 2 child boxes
			int iChild1 = IndexList[TreeBox.BoxToIndex];
			if (iChild1 < 0)
			{
				// 1 child, descend if nearer than cur min-dist
				iChild1 = (-iChild1) - 1;
				double fChild1T = GetRayBoxIntersectionParam(iChild1, Ray, UseBoxRadius);
				if (fChild1T < TNumericLimits<double>::Max())
				{
					FindNearestVisibleSegmentHitByRayInternal(iChild1, Ray, WithinToleranceCheck, NearSegmentIdx, NearestInfo, Options);
				}
			}
			else
			{
				// 2 children. No sorting of descent here because we are only checking hit/no-hit
				iChild1 = iChild1 - 1;
				int iChild2 = IndexList[TreeBox.BoxToIndex + 1] - 1;

				double fChild1T = GetRayBoxIntersectionParam(iChild1, Ray, UseBoxRadius);
				if (fChild1T < TNumericLimits<double>::Max())
				{
					FindNearestVisibleSegmentHitByRayInternal(iChild1, Ray, WithinToleranceCheck, NearSegmentIdx, NearestInfo, Options);
				}

				double fChild2T = GetRayBoxIntersectionParam(iChild2, Ray, UseBoxRadius);
				if (fChild2T < TNumericLimits<double>::Max())
				{
					FindNearestVisibleSegmentHitByRayInternal(iChild2, Ray, WithinToleranceCheck, NearSegmentIdx, NearestInfo, Options);
				}
			}
		}
	}


};


} // end namespace UE::Geometry
} // end namespace UE