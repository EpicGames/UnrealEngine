// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Async/ParallelFor.h"
#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GeometryTypes.h"
#include "HAL/PlatformCrt.h"
#include "IntVectorTypes.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "Math/NumericLimits.h"
#include "Math/Ray.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "MathUtil.h"
#include "Misc/AssertionMacros.h"
#include "Spatial/SparseGrid3.h"
#include "Util/DynamicVector.h"
#include "Util/RefCountVector.h"
#include "Util/SmallListSet.h"

template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Geometry
{

/**
 * Utility class that allows for get/set of a flag for each integer ID, where
 * the flag set automatically grows to contain whatever integer ID is passed
 */
class FDynamicFlagArray
{
public:
	TBitArray<> BitArray;
	uint32 MaxIndex = 0;
	static constexpr int32 GrowChunkSize = 0xFFF;

	FDynamicFlagArray()
	{
		BitArray.Init(false, GrowChunkSize);
		MaxIndex = BitArray.Num();
	}

	void Set(uint32 BitIndex, bool bValue)
	{
		if (bValue && BitIndex >= MaxIndex)
		{
			int32 ToAdd = (BitIndex - MaxIndex + 1) | GrowChunkSize;
			BitArray.Add(false, ToAdd);
			MaxIndex = BitArray.Num();
			BitArray[BitIndex] = bValue;
		}
		else if (BitIndex < MaxIndex)
		{
			BitArray[BitIndex] = bValue;
		}
		// don't need to set anything for false bValue with BitIndex beyond bounds; Get() will return false beyond bounds
	}

	inline bool Get(uint32 BitIndex) const
	{
		return (BitIndex < MaxIndex) ? BitArray[BitIndex] : false;
	}
};




/**
 * FSparseOctreeCell is a Node in a SparseDynamicOctree3. 
 */
struct FSparseOctreeCell
{
	static constexpr uint32 InvalidID = TNumericLimits<uint32>::Max();;
	static constexpr uint8 InvalidLevel = TNumericLimits<uint8>::Max();

	/** ID of cell (index into cell list) */
	uint32 CellID;

	/** Level of cell in octree */
	uint8 Level;

	/** i,j,k index of cell in level, relative to origin */
	FVector3i Index;

	/** CellID of each child, or InvalidID if that child does not exist */
	TStaticArray<uint32, 8> Children;

	FSparseOctreeCell()
		: CellID(InvalidID), Level(0), Index(FVector3i::Zero())
	{
		Children[0] = Children[1] = Children[2] = Children[3] = InvalidID;
		Children[4] = Children[5] = Children[6] = Children[7] = InvalidID;
	}

	FSparseOctreeCell(uint8 LevelIn, const FVector3i& IndexIn)
		: CellID(InvalidID), Level(LevelIn), Index(IndexIn)
	{
		Children[0] = Children[1] = Children[2] = Children[3] = InvalidID;
		Children[4] = Children[5] = Children[6] = Children[7] = InvalidID;
	}

	inline bool IsExistingCell() const
	{
		return CellID != InvalidID;
	}

	inline bool HasChild(int ChildIndex) const
	{
		return Children[ChildIndex] != InvalidID;
	}

	inline uint32 GetChildCellID(int ChildIndex) const
	{
		return Children[ChildIndex];
	}

	inline FSparseOctreeCell MakeChildCell(int ChildIndex)
	{
		FVector3i IndexOffset(
			((ChildIndex & 1) != 0) ? 1 : 0,
			((ChildIndex & 2) != 0) ? 1 : 0,
			((ChildIndex & 4) != 0) ? 1 : 0);
		return FSparseOctreeCell(Level + 1, 2*Index + IndexOffset);
	}

	inline void SetChild(uint32 ChildIndex, const FSparseOctreeCell& ChildCell)
	{
		Children[ChildIndex] = ChildCell.CellID;
	}

};


/**
 * FSparseDynamicOctree3 sorts objects with axis-aligned bounding boxes into a dynamic
 * sparse octree of axis-aligned uniform grid cells. At the top level we have an infinite
 * grid of "root cells" of size RootDimension, which then contain 8 children, and so on.
 * (So in fact each cell is a separate octree, and we have a uniform grid of octrees)
 * 
 * The objects and their bounding-boxes are not stored in the tree. You must have an
 * integer identifier (ObjectID) for each object, and call Insert(ObjectID, BoundingBox).
 * Some query functions will require you to provide a lambda/etc that can be called to
 * retrieve the bounding box for a given ObjectID.
 * 
 * Objects are currently inserted at the maximum possible depth, ie smallest cell that
 * will contain them, or MaxTreeDepth. The tree boxes are expanded by MaxExpandFactor
 * to allow for deeper insertion. If MaxExpandFactor > 0 then the tree does not strictly
 * partition space, IE adjacent cells overlap.
 * 
 * The octree is dynamic. Objects can be removed and re-inserted.
 * 
 */
class FSparseDynamicOctree3
{
	// potential optimizations/improvements
	//    - Cell sizes are known at each level...can keep a lookup table? will this improve performance?
	//    - Store cells for each level in separate TDynamicVectors. CellID is then [Level:8 | Index:24].
	//      This would allow level-grids to be processed separately / in-parallel (for example a cut at given level would be much faster)
	//    - Currently insertion is max-depth but we do not dynamically expand more than once. So early
	//      insertions end up in very large buckets. When a child expands we should check if any of its parents would fit.
	//    - Currently insertion is max-depth so we end up with a huge number of single-object cells. Should only go down a level
	//      if enough objects exist in current cell. Can do this in a greedy fashion, less optimal but still acceptable...
	//    - Store an expand-factor for each cell? or an actual AABB for each cell? this would allow for tighter bounds but
	//      requires accumulating expansion "up" the tree...
	//    - get rid of ValidObjectIDs, I don't think we need it?

public:

	//
	// Tree configuration parameters. It is not safe to change these after tree initialization!
	// 

	/**
	 * Size of the Root cells of the octree. Objects that don't fit in a Root cell are added to a "Spill set"
	 */
	double RootDimension = 1000.0;

	/**
	 * Fraction we expand the dimension of any cell, to allow extra space to fit objects.
	 */
	double MaxExpandFactor = 0.25;

	static constexpr uint32 MaxSupportedTreeDepth = 0x1F;

	int GetMaxTreeDepth()
	{
		return MaxTreeDepth;
	}
	/**
	 * Sets max tree depth w/ protection against setting beyond supported max
	 */
	void SetMaxTreeDepth(int MaxTreeDepthIn)
	{
		if (!ensure((uint32)MaxTreeDepthIn <= MaxSupportedTreeDepth))
		{
			MaxTreeDepthIn = (int32)MaxSupportedTreeDepth;
		}

		MaxTreeDepth = MaxTreeDepthIn;
	}
protected:
	/**
	 * Objects will not be inserted more than this many levels deep from a Root cell
	 */
	int MaxTreeDepth = 10;


public:

	/**
	 * Test if an object is stored in the tree
	 * @param ObjectID ID of the object
	 * @return true if ObjectID is stored in this octree
	 */
	GEOMETRYCORE_API bool ContainsObject(int32 ObjectID) const;

	/**
	 * Insert ObjectID into the Octree
	 * @param ObjectID ID of the object to insert
	 * @param Bounds bounding box of the object
	 *
	 */
	GEOMETRYCORE_API void InsertObject(int32 ObjectID, const FAxisAlignedBox3d& Bounds);

	/**
	 * Remove an object from the octree
	 * @param ObjectID ID of the object
	 * @return true if the object was in the tree and removed
	 */
	GEOMETRYCORE_API bool RemoveObject(int32 ObjectID);

	/**
	 * Update the position of an object in the octree. This is more efficient than doing a remove+insert
	 * @param ObjectID ID of the object
	 * @param NewBounds new bounding box of the object
	 * @return true if object was reinserted, false if the object was fine in it's existing cell
	 */
	GEOMETRYCORE_API bool ReinsertObject(int32 ObjectID, const FAxisAlignedBox3d& NewBounds, uint32 CellIDHint = InvalidCellID);

	/**
	 * Check if the object needs to be reinserted, if it has NewBounds. 
	 * @param ObjectID ID of the object
	 * @param NewBounds new bounding box of the object
	 * @param CellIDOut returned CellID of the object, if it is in the tree. This can be passed to ReinsertObject() to save some computation.
	 * @return false if ObjectID is already in the tree, and NewBounds fits within the cell that currently contains ObjectID
	 */
	GEOMETRYCORE_API bool CheckIfObjectNeedsReinsert(int32 ObjectID, const FAxisAlignedBox3d& NewBounds, uint32& CellIDOut) const;


	/**
	 * Find nearest ray-hit point with objects in tree
	 * @param Ray the ray 
	 * @param GetObjectBoundsFunc function that returns bounding box of object identified by ObjectID
	 * @param HitObjectDistFunc function that returns distance along ray to hit-point on object identified by ObjectID (or TNumericLimits<double>::Max() on miss)
	 * @param MaxDistance maximum hit distance
	 * @return ObjectID of hit object, or -1 on miss
	 */
	GEOMETRYCORE_API int32 FindNearestHitObject(const FRay3d& Ray,
		TFunctionRef<FAxisAlignedBox3d(int)> GetObjectBoundsFunc,
		TFunctionRef<double(int, const FRay3d&)> HitObjectDistFunc,
		double MaxDistance = TNumericLimits<double>::Max()) const;

	/**
	 * Process ObjectIDs from all the cells with bounding boxes that contain query point
	 * @param Point query point
	 * @param ObjectIDFunc this function is called for each ObjectID
	 */
	GEOMETRYCORE_API void ContainmentQuery(const FVector3d& Point,
		TFunctionRef<void(int)> ObjectIDFunc) const;

	/**
	 * Process ObjectIDs from all the cells with bounding boxes that contain query point
	 * @param Point query point
	 * @param ObjectIDFunc this function is called for each ObjectID. Returns true to continue query and false to abort
	 * @return true if query finished, false if it exited
	 */
	GEOMETRYCORE_API bool ContainmentQueryCancellable(const FVector3d& Point,
		TFunctionRef<bool(int)> ObjectIDFunc) const;

	/**
	 * Process ObjectIDs from all the cells with bounding boxes that intersect Bounds
	 * @param Bounds query box
	 * @param ObjectIDFunc this function is called for each ObjectID
	 */
	GEOMETRYCORE_API void RangeQuery(const FAxisAlignedBox3d& Bounds,
		TFunctionRef<void(int)> ObjectIDFunc) const;

	/**
	 * Collect ObjectIDs from all the cells with bounding boxes that intersect Bounds
	 * @param Bounds query box
	 * @param ObjectIDsOut collected ObjectIDs are stored here 
	 */
	GEOMETRYCORE_API void RangeQuery(const FAxisAlignedBox3d& Bounds,
		TArray<int>& ObjectIDsOut ) const;

	/**
	 * Collect ObjectIDs from all the cells with bounding boxes that intersect Bounds.
	 * Current implementation creates a separate TArray for each thread.
	 * @param Bounds query box
	 * @param ObjectIDsOut collected ObjectIDs are stored here
	 */
	GEOMETRYCORE_API void ParallelRangeQuery(const FAxisAlignedBox3d& Bounds,
		TArray<int>& ObjectIDsOut) const;

	/**
	 * Collect ObjectIDs from all the leaf cells that have bounding boxes that pass BoundsOverlapFn
	 * @param RangeBoundsHint A box to potentially limit the iteration range across cells. Note that
	 *  this might be ignored if the allocation of cells is such that trying to check for cells in that
	 *  range would take more time than just iterating over all allocated cells.
	 * @param BoundsOverlapFn Must be thread-safe. Used to determine if the cell is part
	 *  of the desired range.
	 * @param ObjectIDsOut collected ObjectIDs are stored here
	 */
	GEOMETRYCORE_API void ParallelRangeQuery(
		const FAxisAlignedBox3d& RangeBoundsHint,
		TFunctionRef<bool (const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn,
		TArray<int32>& ObjectIDs) const;

	/**
	 * Find any overlap between a caller-defined query and any object ID. Returns the first overlap it finds.
	 * @param ShapeBounds Overall bounds of the custom query shape
	 * @param ObjectOverlapFn Custom function that indicates if the given ObjectID overlaps the query shape
	 * @param BoundsOverlapFn Optional custom function that indicates if the given bounding box overlaps the query shape, used to filter the octree search.
	 *                        Note the bounds overlap with ShapeBounds will already be called before calling this; only provide this if a more accurate overlap can be quickly checked.
	 * @return index of overlapping ObjectID or INDEX_NONE if no overlaps are found
	 */
	GEOMETRYCORE_API int ParallelOverlapAnyQuery(const FAxisAlignedBox3d& ShapeBounds,
		TFunctionRef<bool(int32)> ObjectOverlapFn, 
		TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn) const;

	/**
	 * Find any overlap between a caller-defined query and any object ID. Returns the first overlap it finds.
	 * @param ShapeBounds Overall bounds of the custom query shape
	 * @param ObjectOverlapFn Custom function that indicates if the given ObjectID overlaps the query shape
	 * @return index of overlapping ObjectID or INDEX_NONE if no overlaps are found
	 */
	int ParallelOverlapAnyQuery(const FAxisAlignedBox3d& ShapeBounds,
		TFunctionRef<bool(int32)> ObjectOverlapFn) const
	{
		return ParallelOverlapAnyQuery(ShapeBounds, ObjectOverlapFn, 
			// Note a bounds vs ShapeBounds test will always be performed automatically;
			// the optional bounds overlap fn here can just return true to indicate no additional filtering is performed
			[](const FAxisAlignedBox3d&) {return true;}
		);
	}


	/**
	 * Check that the octree is internally valid
	 * @param IsValidObjectIDFunc function that returns true if given ObjectID is valid
	 * @param GetObjectBoundSFunc function that returns bounding box of object identified by ObjectID
	 * @param FailMode how should validity checks fail
	 * @param bVerbose if true, print some debug info via UE_LOG
	 * @param bFailOnMissingObjects if true, assume ObjectIDs are dense and that all ObjectIDs must be in the tree
	 */
	GEOMETRYCORE_API void CheckValidity(
		TFunctionRef<bool(int)> IsValidObjectIDFunc,
		TFunctionRef<FAxisAlignedBox3d(int)> GetObjectBoundsFunc,
		EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check,
		bool bVerbose = false,
		bool bFailOnMissingObjects = false) const;

	/**
	 * statistics about internal structure of the octree
	 */
	struct FStatistics
	{
		int32 Levels;
		TArray<int32> LevelBoxCounts;
		TArray<int32> LevelObjCounts;
		int32 SpillObjCount;
		FString ToString() const;
	};

	/**
	 * Populate given FStatistics with info about the octree
	 */
	GEOMETRYCORE_API void ComputeStatistics(FStatistics& StatsOut) const;

protected:
	// this identifier is used for unknown cells
	static constexpr uint32 InvalidCellID = FSparseOctreeCell::InvalidID;

	// if an object is in the spill cell, that means it didn't fit in the tree
	static constexpr uint32 SpillCellID = InvalidCellID - 1;

	// reference counts for Cells list. We don't actually need reference counts here, but we need a free
	// list and iterators, and FRefCountVector provides this
	FRefCountVector CellRefCounts;

	// list of cells. Note that some cells may be unused, depending on CellRefCounts
	TDynamicVector<FSparseOctreeCell> Cells;

	// TODO: Consider switching FSmallListSet to TSparseListSet<int32> for more reliable performance in cases where we have more than 8 objects in many cells
	FSmallListSet CellObjectLists;			// per-cell object ID lists
	TSet<int32> SpillObjectSet;				// list of object IDs for objects that didn't fit in a root cell

	TDynamicVector<uint32> ObjectIDToCellMap;	// map from external Object IDs to which cell the object is in (or spill cell, or invalid)
	FDynamicFlagArray ValidObjectIDs;			// set of ObjectIDs in the tree. This is perhaps not necessary...couldn't we rely on ObjectIDToCellMap?

	// RootCells are the top-level cells of the octree, of size RootDimension. 
	// So the elements of this sparse grid are CellIDs
	TSparseGrid3<uint32> RootCells;


	// calculate the base width of a cell at a given level
	inline double GetCellWidth(uint32 Level) const
	{
		checkSlow(Level <= MaxSupportedTreeDepth);
		double Divisor = (double)( (uint64)1 << (Level & MaxSupportedTreeDepth) );
		double CellWidth = RootDimension / Divisor;
		return CellWidth;
	}


	FAxisAlignedBox3d GetBox(uint32 Level, const FVector3i& Index, double ExpandFactor) const
	{
		double CellWidth = GetCellWidth(Level);
		double ExpandDelta = CellWidth * ExpandFactor;
		double MinX = (CellWidth * (double)Index.X) - ExpandDelta;
		double MinY = (CellWidth * (double)Index.Y) - ExpandDelta;
		double MinZ = (CellWidth * (double)Index.Z) - ExpandDelta;
		CellWidth += 2.0 * ExpandDelta;
		return FAxisAlignedBox3d(
			FVector3d(MinX, MinY, MinZ),
			FVector3d(MinX + CellWidth, MinY + CellWidth, MinZ + CellWidth));
	}
	inline FAxisAlignedBox3d GetCellBox(const FSparseOctreeCell& Cell, double ExpandFactor = 0) const
	{
		return GetBox(Cell.Level, Cell.Index, ExpandFactor);
	}
	FVector3d GetCellCenter(const FSparseOctreeCell& Cell) const
	{
		double CellWidth = GetCellWidth(Cell.Level);
		double MinX = CellWidth * (double)Cell.Index.X;
		double MinY = CellWidth * (double)Cell.Index.Y;
		double MinZ = CellWidth * (double)Cell.Index.Z;
		CellWidth *= 0.5;
		return FVector3d(MinX + CellWidth, MinY + CellWidth, MinZ + CellWidth);
	}


	// warning: result here appears to be unstable (due to optimization?) if any of the position values are on the border of a cell
	// (in testing, pragma-optimization-disabled code produced off-by-one result from calling this function)
	FVector3i PointToIndex(uint32 Level, const FVector3d& Position) const
	{
		double CellWidth = GetCellWidth(Level);

		// Converts to an int32 in such a way that values outside of the representable
		//  range ends up at the min and max values of the representable range.
		auto SafeDoubleToInt32Floor = [](double Value)
		{
			return static_cast<int32>(FMath::Clamp(
				FMathd::Floor(Value),
				static_cast<double>(TNumericLimits<int32>::Lowest()), 
				static_cast<double>(TNumericLimits<int32>::Max())));
		};

		int32 i = SafeDoubleToInt32Floor(Position.X / CellWidth);
		int32 j = SafeDoubleToInt32Floor(Position.Y / CellWidth);
		int32 k = SafeDoubleToInt32Floor(Position.Z / CellWidth);
		return FVector3i(i, j, k);
	}


	int ToChildCellIndex(const FSparseOctreeCell& Cell, const FVector3d& Position) const
	{
		FVector3d Center = GetCellCenter(Cell);
		int ChildIndex =
			((Position.X < Center.X) ? 0 : 1) +
			((Position.Y < Center.Y) ? 0 : 2) +
			((Position.Z < Center.Z) ? 0 : 4);
		return ChildIndex;
	}

	bool CanFit(const FSparseOctreeCell& Cell, const FAxisAlignedBox3d& Bounds) const
	{
		FAxisAlignedBox3d CellBox = GetCellBox(Cell, MaxExpandFactor);
		return CellBox.Contains(Bounds);
	}

	uint32 GetCellForObject(int32 ObjectID) const
	{
		if (ObjectID >= 0 && static_cast<size_t>(ObjectID) < ObjectIDToCellMap.Num())
		{
			return ObjectIDToCellMap[ObjectID];
		}
		return InvalidCellID;
	}

	GEOMETRYCORE_API FSparseOctreeCell FindCurrentContainingCell(const FAxisAlignedBox3d& Bounds) const;


	GEOMETRYCORE_API void Insert_Spill(int32 ObjectID, const FAxisAlignedBox3d& Bounds);
	GEOMETRYCORE_API void Insert_NewRoot(int32 ObjectID, const FAxisAlignedBox3d& Bounds, FSparseOctreeCell NewRootCell);
	GEOMETRYCORE_API void Insert_ToCell(int32 ObjectID, const FAxisAlignedBox3d& Bounds, const FSparseOctreeCell& ExistingCell);
	GEOMETRYCORE_API void Insert_NewChildCell(int32 ObjectID, const FAxisAlignedBox3d& Bounds, int ParentCellID, FSparseOctreeCell NewChildCell, int ChildIdx);

	GEOMETRYCORE_API double FindNearestRayCellIntersection(const FSparseOctreeCell& Cell, const FRay3d& Ray) const;


	GEOMETRYCORE_API void BranchRangeQuery(const FSparseOctreeCell* ParentCell,
		const FAxisAlignedBox3d& Bounds,
		TArray<int>& ObjectIDs) const;

private:
	// Gather objects in all cells that pass BoundsOverlapFn, starting from a set of root cells
	void ParallelCollectObjectsInOverlappingCells(
		TArray<const FSparseOctreeCell*, TInlineAllocator<32>> RelevantRootCells,
		TFunctionRef<bool(const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn,
		TArray<int32>& ObjectIDs) const;
	// Go down subtree of ParentCell and collect all objects in cells that pass BoundsOverlapFn.
	void BranchRangeQuery(const FSparseOctreeCell* ParentCell,
		TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn,
		TArray<int>& ObjectIDs) const;

	int BranchCustomOverlapAnyQuery(const FSparseOctreeCell* ParentCell,
		const FAxisAlignedBox3d& Bounds,
		TFunctionRef<bool(int32)> ObjectOverlapFn, TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsOverlapFn) const;

	/**
	 * Helper to find root-level cells that satisfy BoundsOverlapFn
	 * 
	 * @param RangeBounds If the allocation of cells is such that it is worth doing, only cells in these
	 *   bounds are looked for and iterated over. May be ignored if the allocation is such that it is
	 *   faster to iterate over all allocated root cells than to check for allocation in a big range.
	 * @param BoundsOverlapFn Function for determining whether cell should be included in output queue
	 * @param bCheckOverlapFnInRangeBounds If true, then BoundsOverlapFn is still checked when iterating
	 *   over RangeBounds (i.e., RangeBounds is an optimization hint, but not sufficient for determining
	 *   actual overlap).
	 */
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> InitializeQueryQueue(
		const FAxisAlignedBox3d& RangeBounds, 
		TFunctionRef<bool(const FAxisAlignedBox3d& CellBounds)> BoundsOverlapFn, 
		bool bCheckOverlapFnInRangeBounds = true) const;

	// helper to find the root-level cells that could intersect with a given query object, specialized for point containment queries
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> InitializeQueryQueue(const FVector3d& Point) const
	{
		return InitializeQueryQueue(FAxisAlignedBox3d(Point, Point),
			[&Point](const FAxisAlignedBox3d& CellBounds) { return CellBounds.Contains(Point); }, 
			false);
	}

	// helper to find the root-level cells that could intersect with a given query object, specialized for box queries
	TArray<const FSparseOctreeCell*, TInlineAllocator<32>> InitializeQueryQueue(const FAxisAlignedBox3d& Bounds) const
	{
		return InitializeQueryQueue(Bounds,
			[&Bounds](const FAxisAlignedBox3d& CellBounds) { return CellBounds.Intersects(Bounds); },
			false);
	}
};


} // end namespace UE::Geometry
} // end namespace UE
