// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "GeometryTypes.h"
#include "BoxTypes.h"
#include "Util/DynamicVector.h"
#include "Util/RefCountVector.h"
#include "Util/SparseListSet.h"
#include "Spatial/SparseGrid3.h"
#include "Intersection/IntrRay3AxisAlignedBox3.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{


/**
 * FSparsePointOctreeCell is a Node in a SparseDynamicOctree3. 
 */
struct FSparsePointOctreeCell
{
	static constexpr uint32 InvalidID = TNumericLimits<uint32>::Max();;
	static constexpr uint8 InvalidLevel = TNumericLimits<uint8>::Max();

	/** ID of cell (index into cell list) */
	uint32 CellID;

	/** Level of cell in octree */
	uint8 Level;

	/** i,j,k index of cell in level, relative to origin */
	FVector3i Index;

	/** CellID of each child, or InvalidID if that child does not exist */
	TStaticArray<uint32, 8> Children;

	FSparsePointOctreeCell()
		: CellID(InvalidID), Level(0), Index(FVector3i::Zero())
	{
		Children[0] = Children[1] = Children[2] = Children[3] = InvalidID;
		Children[4] = Children[5] = Children[6] = Children[7] = InvalidID;
	}

	FSparsePointOctreeCell(uint8 LevelIn, const FVector3i& IndexIn)
		: CellID(InvalidID), Level(LevelIn), Index(IndexIn)
	{
		Children[0] = Children[1] = Children[2] = Children[3] = InvalidID;
		Children[4] = Children[5] = Children[6] = Children[7] = InvalidID;
	}

	inline bool IsExistingCell() const
	{
		return CellID != InvalidID;
	}

	inline bool HasChild(int ChildIndex) const
	{
		return Children[ChildIndex] != InvalidID;
	}

	inline uint32 GetChildCellID(int ChildIndex) const
	{
		return Children[ChildIndex];
	}

	inline FSparsePointOctreeCell MakeChildCell(int ChildIndex) const
	{
		FVector3i IndexOffset(
			((ChildIndex & 1) != 0) ? 1 : 0,
			((ChildIndex & 2) != 0) ? 1 : 0,
			((ChildIndex & 4) != 0) ? 1 : 0);
		return FSparsePointOctreeCell(Level + 1, 2*Index + IndexOffset);
	}

	inline void SetChild(uint32 ChildIndex, const FSparsePointOctreeCell& ChildCell)
	{
		Children[ChildIndex] = ChildCell.CellID;
	}

};


/**
 * FSparseDynamicPointOctree3 sorts Points with axis-aligned bounding boxes into a dynamic
 * sparse octree of axis-aligned uniform grid cells. At the top level we have an infinite
 * grid of "root cells" of size RootDimension, which then contain 8 children, and so on.
 * (So in fact each cell is a separate octree, and we have a uniform grid of octrees)
 * 
 * The Points and their bounding-boxes are not stored in the tree. You must have an
 * integer identifier (PointID) for each Point, and call Insert(PointID, BoundingBox).
 * Some query functions will require you to provide a lambda/etc that can be called to
 * retrieve the bounding box for a given PointID.
 * 
 * By default Points are currently inserted at the maximum possible depth, ie smallest cell that
 * will contain them, or MaxTreeDepth. 
 * 
 * The octree is dynamic. Points can be removed and re-inserted.
 * 
 */
class FSparseDynamicPointOctree3
{
	// potential optimizations/improvements
	//    - Cell sizes are known at each level...can keep a lookup table? will this improve performance?
	//    - Store cells for each level in separate TDynamicVectors. CellID is then [Level:8 | Index:24].
	//      This would allow level-grids to be processed separately / in-parallel (for example a cut at given level would be much faster)
	//    - Currently insertion is max-depth but we do not dynamically expand more than once. So early
	//      insertions end up in very large buckets. When a child expands we should check if any of its parents would fit.
	//    - Currently insertion is max-depth so we end up with a huge number of single-Point cells. Should only go down a level
	//      if enough Points exist in current cell. Can do this in a greedy fashion, less optimal but still acceptable...

public:

	//
	// Tree configuration parameters. It is not safe to change these after tree initialization!
	// 

	/**
	 * Size of the Root cells of the octree. 
	 */
	double RootDimension = 1000.0;

	/**
	 * Points will not be inserted more than this many levels deep from a Root cell
	 */
	int MaxTreeDepth = 10;

	/**
	 * if using the InsertPoint_DynamicExpand() insertion path, then when a cell gets
	 * to this many points, and has depth < MaxTreeDepth, it will be expanded and its points moved down
	 */
	int MaxPointsPerCell = 1000;

public:

	/**
	 * Do a rough ballpark configuration of the RootDimension, MaxTreeDepth, and MaxPointsPerCell tree config values,
	 * given a bounding-box dimension and estimate of the total number of points that will be inserted.
	 * This is primarily intended to be used with ParallelInsertDensePointSet()
	 */
	inline void ConfigureFromPointCountEstimate(double MaxBoundsDimension, int CountEstimate);

	/**
	 * Test if an Point is stored in the tree
	 * @param PointID ID of the Point
	 * @return true if PointID is stored in this octree
	 */
	inline bool ContainsPoint(int32 PointID) const;

	/**
	 * Insert PointID into the Octree at maximum depth. 
	 * Currently only expands at most one depth-level per insertion. This is sub-optimal.
	 * @param PointID ID of the Point to insert
	 * @param Position position of the Point
	 *
	 */
	inline void InsertPoint(int32 PointID, const FVector3d& Position);

	/**
	 * Insert PointID into the Octree. Dynamically expands cells that have more than MaxPointsPerCell
	 * @param PointID ID of the Point to insert
	 * @param GetPositionFunc function that returns position of point (Required to reinsert points on expand)
	 */
	inline void InsertPoint_DynamicExpand(int32 PointID, TFunctionRef<FVector3d(int)> GetPositionFunc);

	/**
	 * Insert a set of dense points with IDs in range [0, MaxPointID-1], in parallel.
	 * The points are only inserted in leaf nodes, at MaxTreeDepth level, so that value should
	 * be set conservatively. Parallel insertion is across root cells, so if the RootDimension
	 * is larger than the point set bounds, this will be slower than incremental construction.
	 * ConfigureFromPointCountEstimate() provides reasonable values for large point sets.
	 * 
	 * @param GetPositionFunc function that returns position of point (Required to reinsert points on expand)
	 */
	inline void ParallelInsertDensePointSet(int32 MaxPointID, TFunctionRef<FVector3d(int)> GetPositionFunc);


	/**
	 * Remove a Point from the octree
	 * @param PointID ID of the Point
	 * @return true if the Point was in the tree and removed
	 */
	inline bool RemovePoint(int32 PointID);

	/**
	 * Remove a Point from the octree. This function must only be called with PointIDs that are certain to be in the tree.
	 * @param PointID ID of the Point. 
	 */
	inline void RemovePointUnsafe(int32 PointID);

	/**
	 * Update the position of an Point in the octree. This is more efficient than doing a remove+insert
	 * @param PointID ID of the Point
	 * @param NewBounds enw bounding box of the Point
	 */
	inline void ReinsertPoint(int32 PointID, const FVector3d& NewPosition);

	/**
	 * Find nearest ray-hit point with Points in tree
	 * @param Ray the ray 
	 * @param HitPointDistFunc function that returns distance along ray to hit-point on Point identified by PointID (or TNumericLimits<double>::Max() on miss)
	 * @param MaxDistance maximum hit distance
	 * @return PointID of hit Point, or -1 on miss
	 */
	inline int32 FindNearestHitPoint(const FRay3d& Ray,
		TFunctionRef<double(int, const FRay3d&)> HitPointDistFunc,
		double MaxDistance = TNumericLimits<double>::Max()) const;

	/**
	 * Find closest point in the tree, relative to a query point, within a search distance
	 * 
	 * @param QueryPt Find the point closest to this query point
	 * @param DistanceThreshold Limit the search to this distance
	 * @param PredicateFunc Return true if point should be included (i.e., this is a filter)
	 * @param DistSqFunc User-provided squared distance between the point with the given ID and the QueryPt
	 * @param TempBuffer Optional temporary buffer to store a queue of cells (to avoid memory allocations)
	 * @return The ID of the closest point, or INDEX_NONE if no point was found
	 */
	GEOMETRYCORE_API int32 FindClosestPoint(FVector3d QueryPt, double DistanceThreshold,
		TFunctionRef<bool(int32)> PredicateFunc, TFunctionRef<double(int32)> DistSqFunc,
		TArray<const FSparsePointOctreeCell*>* TempBuffer = nullptr) const;

	/**
	 * Find the K closest point in the tree, relative to a query point, within a search distance
	 *
	 * @param QueryPt Find the points closest to this query point
	 * @param DistanceThreshold Limit the search to this distance
	 * @param NumToFind The maximum number of closest points to find
	 * @param FoundPoints The found closest points will be added to this array
	 * @param PredicateFunc Return true if point should be included (i.e., this is a filter)
	 * @param DistSqFunc User-provided squared distance between the point with the given ID and the QueryPt
	 * @param TempBuffer Optional temporary buffer to store a queue of cells (to avoid memory allocations)
	 */
	GEOMETRYCORE_API void FindKClosestPoints(FVector3d QueryPt, double DistanceThreshold, int32 NumToFind,
		TArray<TPair<int32, double>>& FoundPoints, TFunctionRef<bool(int32)> PredicateFunc, TFunctionRef<double(int32)> DistSqFunc,
		TArray<const FSparsePointOctreeCell*>* TempBuffer = nullptr) const;

	/**
	 * Collect PointIDs from all the cells with bounding boxes that intersect Bounds,
	 * where PredicateFunc passes
	 * @param Bounds query box
	 * @param PredicateFunc return true if point should be included (ie this is a filter)
	 * @param PointIDsOut collected PointIDs are stored here 
	 * @param TempBuffer optional temporary buffer to store a queue of cells (to avoid memory allocations)
	 */
	inline void RangeQuery(const FAxisAlignedBox3d& Bounds,
		TFunctionRef<bool(int)> PredicateFunc,
		TArray<int>& PointIDsOut,
		TArray<const FSparsePointOctreeCell*>* TempBuffer = nullptr) const;

	/**
	 * Collect PointIDs from all the cells with bounding boxes that intersect Bounds, where PredicateFunc passes
	 * Query is parallelized across Root cells. So if there is only one Root cell, it's actually slower
	 * than RangeQuery() due to internal temp buffers.
	 * @param Bounds query box
	 * @param PredicateFunc return true if point should be included (ie this is a filter)
	 * @param PointIDsOut collected PointIDs are stored here
	 * @param TempBuffer optional temporary buffer to store a queue of cells (to avoid memory allocations)
	 */
	inline void ParallelRangeQuery(const FAxisAlignedBox3d& Bounds,
		TFunctionRef<bool(int)> PredicateFunc,
		TArray<int>& PointIDsOut,
		TArray<const FSparsePointOctreeCell*>* TempBuffer = nullptr) const;

	inline void ParallelRangeQuery(const FAxisAlignedBox3d& Bounds,
		TFunctionRef<bool(const FAxisAlignedBox3d& Bounds)> BoundsPredicateFunc,
		TFunctionRef<bool(int)> PredicateFunc,
		TArray<int>& PointIDsOut,
		TArray<const FSparsePointOctreeCell*>* TempBuffer = nullptr) const;



	/**
	 * Check that the octree is internally valid
	 * @param IsValidPointIDFunc function that returns true if given PointID is valid
	 * @param GetPointFunc function that returns point position identified by PointID
	 * @param FailMode how should validity checks fail
	 * @param bVerbose if true, print some debug info via UE_LOG
	 * @param bFailOnMissingPoints if true, assume PointIDs are dense and that all PointIDs must be in the tree
	 */
	inline void CheckValidity(
		TFunctionRef<bool(int)> IsValidPointIDFunc,
		TFunctionRef<FVector3d(int)> GetPointFunc,
		EValidityCheckFailMode FailMode = EValidityCheckFailMode::Check,
		bool bVerbose = false,
		bool bFailOnMissingPoints = false) const;

	/**
	 * statistics about internal structure of the octree
	 */
	struct FStatistics
	{
		int32 Levels;
		TArray<int32> LevelBoxCounts;
		TArray<int32> LevelObjCounts;
		TArray<int32> LevelMaxObjCounts;
		TArray<float> LevelCellSizes;
		inline FString ToString() const;
	};

	/**
	 * Populate given FStatistics with info about the octree
	 */
	inline void ComputeStatistics(FStatistics& StatsOut) const;

protected:
	// this identifier is used for unknown cells
	static constexpr uint32 InvalidCellID = FSparsePointOctreeCell::InvalidID;

	// reference counts for Cells list. We don't actually need reference counts here, but we need a free
	// list and iterators, and FRefCountVector provides this
	FRefCountVector CellRefCounts;

	// list of cells. Note that some cells may be unused, depending on CellRefCounts
	TDynamicVector<FSparsePointOctreeCell> Cells;

	TSparseListSet<int32> CellPointLists;		// per-cell Point ID lists

	TDynamicVector<uint32> PointIDToCellMap;	// map from external Point IDs to which cell the Point is in (or InvalidCellID)

	// RootCells are the top-level cells of the octree, of size RootDimension. 
	// So the elements of this sparse grid are CellIDs
	TSparseGrid3<uint32> RootCells;


	// calculate the base width of a cell at a given level
	inline double GetCellWidth(uint32 Level) const
	{
		double Divisor = (double)( (uint64)1 << (Level & 0x1F) );
		double CellWidth = RootDimension / Divisor;
		return CellWidth;
	}


	inline FAxisAlignedBox3d GetCellBox(uint32 Level, const FVector3i& Index) const
	{
		double CellWidth = GetCellWidth(Level);
		double MinX = (CellWidth * (double)Index.X);
		double MinY = (CellWidth * (double)Index.Y);
		double MinZ = (CellWidth * (double)Index.Z);
		return FAxisAlignedBox3d(
			FVector3d(MinX, MinY, MinZ),
			FVector3d(MinX + CellWidth, MinY + CellWidth, MinZ + CellWidth));
	}
	inline FAxisAlignedBox3d GetCellBox(const FSparsePointOctreeCell& Cell) const
	{
		return GetCellBox(Cell.Level, Cell.Index);
	}

	inline FVector3d GetCellCenter(uint32 Level, const FVector3i& Index) const
	{
		double CellWidth = GetCellWidth(Level);
		double MinX = CellWidth * (double)Index.X;
		double MinY = CellWidth * (double)Index.Y;
		double MinZ = CellWidth * (double)Index.Z;
		CellWidth *= 0.5;
		return FVector3d(MinX + CellWidth, MinY + CellWidth, MinZ + CellWidth);
	}
	inline FVector3d GetCellCenter(const FSparsePointOctreeCell& Cell) const
	{
		return GetCellCenter(Cell.Level, Cell.Index);
	}


	inline FVector3i PointToIndex(uint32 Level, const FVector3d& Position) const
	{
		double CellWidth = GetCellWidth(Level);

		// Converts to an int32 in such a way that values outside of the representable
		//  range ends up at the min and max values of the representable range.
		auto SafeDoubleToInt32Floor = [](double Value)
		{
			return static_cast<int32>(FMath::Clamp(
				FMathd::Floor(Value),
				static_cast<double>(TNumericLimits<int32>::Lowest()),
				static_cast<double>(TNumericLimits<int32>::Max())));
		};
		
		int32 i = SafeDoubleToInt32Floor(Position.X / CellWidth);
		int32 j = SafeDoubleToInt32Floor(Position.Y / CellWidth);
		int32 k = SafeDoubleToInt32Floor(Position.Z / CellWidth);
		return FVector3i(i, j, k);
	}

	int ToChildCellIndex(uint32 Level, const FVector3i& Index, const FVector3d& Position) const
	{
		FVector3d Center = GetCellCenter(Level, Index);
		int ChildIndex =
			((Position.X < Center.X) ? 0 : 1) +
			((Position.Y < Center.Y) ? 0 : 2) +
			((Position.Z < Center.Z) ? 0 : 4);
		return ChildIndex;
	}
	int ToChildCellIndex(const FSparsePointOctreeCell& Cell, const FVector3d& Position) const
	{
		return ToChildCellIndex(Cell.Level, Cell.Index, Position);
	}

	bool CellContains(const FSparsePointOctreeCell& Cell, const FVector3d& Position) const
	{
		// Note: Uses PointToIndex for consistency; GetCellBox(Cell).Contains(Position) may disagree due to floating point error
		return PointToIndex(Cell.Level, Position) == Cell.Index;
	}

	uint32 GetCellForPoint(int32 PointID) const
	{
		if (PointID >= 0 && size_t(PointID) < PointIDToCellMap.Num())
		{
			return PointIDToCellMap[PointID];
		}
		return InvalidCellID;
	}

	inline FSparsePointOctreeCell FindCurrentContainingCell(const FVector3d& Position) const;

	inline uint32 CreateNewRootCell(FSparsePointOctreeCell NewRootCell, bool bInitializeCellPointList);
	inline void Insert_NewRoot(int32 PointID, const FVector3d& Position, FSparsePointOctreeCell NewRootCell);
	inline void Insert_ToCell(int32 PointID, const FVector3d& Position, const FSparsePointOctreeCell& ExistingCell);
	inline void Insert_NewChildCell(int32 PointID, const FVector3d& Position, int ParentCellID, FSparsePointOctreeCell NewChildCell, int ChildIdx);

	inline double FindNearestRayCellIntersection(const FSparsePointOctreeCell& Cell, const FRay3d& Ray) const;

private:
	// Helper to iterate over root cells in a query range. 
	// Calls IsInRangeFn on each candidate cell, and then ProcessCellFn on the cell if IsInRangeFn returns true.
	template<typename IsInRangeFn, typename ProcessCellFn>
	void ForEachRootCellInBounds(const FAxisAlignedBox3d& Bounds, const IsInRangeFn& IsInRange, const ProcessCellFn& ProcessCell) const;


	// Helper to iterate over the root cells within a radius. Note the RootFn can return a refined search radius squared, which will let the iteration visit fewer root cells.
	// RootFn is passed the current squared search radius, and returns the (possibly smaller) squared search radius.
	void IterateRootsInRadius(FVector3d QueryPt, double InitialRadius, TFunctionRef<double(double RadSq, const FSparsePointOctreeCell& Root)> RootFn) const;

	// Helper to gather relevant root cells for a range query
	template<typename ArrayType>
	inline void GatherRootCellsForRangeQuery(const FAxisAlignedBox3d& Bounds, ArrayType& OutRootCells) const;
};



void FSparseDynamicPointOctree3::ConfigureFromPointCountEstimate(double MaxBoundsDimension, int CountEstimate)
{
	// These are some rough values collected from some basic profiling. At some point it will
	// probably be a good idea to do a more comprehensive study and perhaps come up with some
	// regression formulas. 

	this->MaxTreeDepth = 2;
	int RootCellDivisions = 6;
	if (CountEstimate > 500000)
	{
		this->MaxTreeDepth = 3;
		RootCellDivisions = 8;
	}
	if (CountEstimate > 5000000)
	{
		this->MaxTreeDepth = 4;
		RootCellDivisions = 10;
	}
	if (CountEstimate > 50000000)
	{
		this->MaxTreeDepth = 5;
		RootCellDivisions = 12;
	}
	this->RootDimension = MaxBoundsDimension / (double)RootCellDivisions;

	// maybe not necessarily a good estimate? 
	this->MaxPointsPerCell = 250;
}


bool FSparseDynamicPointOctree3::ContainsPoint(int32 PointID) const
{
	checkSlow(PointID >= 0);
	return (PointID < PointIDToCellMap.Num() && PointIDToCellMap[PointID] != InvalidCellID);
}


void FSparseDynamicPointOctree3::InsertPoint(int32 PointID, const FVector3d& Position)
{
	if (ContainsPoint(PointID))
	{
		checkSlow(false);
		return;
	}

	FSparsePointOctreeCell CurrentCell = FindCurrentContainingCell(Position);
	checkSlow(CurrentCell.Level != FSparsePointOctreeCell::InvalidLevel);

	// if we found a containing root cell but it doesn't exist, create it and insert
	if (CurrentCell.Level == 0 && CurrentCell.IsExistingCell() == false)
	{
		Insert_NewRoot(PointID, Position, CurrentCell);
		return;
	}

	int PotentialChildIdx = ToChildCellIndex(CurrentCell, Position);
	// if current cell does not have this child we might fit there so try it
	if (CurrentCell.HasChild(PotentialChildIdx) == false)
	{
		// todo can we do a fast check based on level and dimensions??
		FSparsePointOctreeCell NewChildCell = CurrentCell.MakeChildCell(PotentialChildIdx);
		if (NewChildCell.Level <= MaxTreeDepth && CellContains(NewChildCell, Position))
		{
			Insert_NewChildCell(PointID, Position, CurrentCell.CellID, NewChildCell, PotentialChildIdx);
			return;
		}
	}

	// insert into current cell if
	//   1) child cell exists (in which case we didn't fit or FindCurrentContainingCell would have returned)
	//   2) we tried to fit in child cell and failed
	Insert_ToCell(PointID, Position, CurrentCell);
}


void FSparseDynamicPointOctree3::Insert_NewChildCell(int32 PointID, const FVector3d& Position,
	int ParentCellID, FSparsePointOctreeCell NewChildCell, int ChildIdx)
{
	FSparsePointOctreeCell& OrigParentCell = Cells[ParentCellID];
	checkSlow(OrigParentCell.HasChild(ChildIdx) == false);

	NewChildCell.CellID = CellRefCounts.Allocate();
	Cells.InsertAt(NewChildCell, NewChildCell.CellID);

	PointIDToCellMap.InsertAt(NewChildCell.CellID, PointID, InvalidCellID);

	CellPointLists.AllocateAt(NewChildCell.CellID);
	CellPointLists.Insert(NewChildCell.CellID, PointID);

	OrigParentCell.SetChild(ChildIdx, NewChildCell);

	checkSlow(CellContains(NewChildCell, Position));
	checkSlow(PointToIndex(NewChildCell.Level, Position) == NewChildCell.Index);
}



void FSparseDynamicPointOctree3::Insert_ToCell(int32 PointID, const FVector3d& Position, const FSparsePointOctreeCell& ExistingCell)
{
	check(ExistingCell.CellID != InvalidCellID);
	checkSlow(CellRefCounts.IsValid(ExistingCell.CellID));

	PointIDToCellMap.InsertAt(ExistingCell.CellID, PointID, InvalidCellID);

	CellPointLists.Insert(ExistingCell.CellID, PointID);

	checkSlow(CellContains(ExistingCell, Position));
}



void FSparseDynamicPointOctree3::Insert_NewRoot(int32 PointID, const FVector3d& Position, FSparsePointOctreeCell NewRootCell)
{
	uint32 NewRootCellID = CreateNewRootCell(NewRootCell, true);

	PointIDToCellMap.InsertAt(NewRootCellID, PointID, InvalidCellID);
	CellPointLists.Insert(NewRootCellID, PointID);
}

uint32 FSparseDynamicPointOctree3::CreateNewRootCell(FSparsePointOctreeCell NewRootCell, bool bInitializeCellPointList)
{
	checkSlow(RootCells.Has(NewRootCell.Index) == false);

	NewRootCell.CellID = CellRefCounts.Allocate();
	Cells.InsertAt(NewRootCell, NewRootCell.CellID);
	uint32* RootCellElem = RootCells.Get(NewRootCell.Index, true);
	*RootCellElem = NewRootCell.CellID;

	if (bInitializeCellPointList)
	{
		CellPointLists.AllocateAt(NewRootCell.CellID);
	}

	return NewRootCell.CellID;
}



void FSparseDynamicPointOctree3::ParallelInsertDensePointSet(int32 MaxPointID, TFunctionRef<FVector3d(int)> GetPositionFunc)
{
	// TODO: this current implementation could be adapted to handle non-dense point sets

	// ParallelInsertDensePointSet can currently only be called if the tree is empty
	if (ensure(Cells.IsEmpty()) == false)
	{
		return;
	}

	// information for an input point
	struct FPointInfo
	{
		int32 PointID;			// ID of this point
		FVector3i RootCell;		// index of Root cell this point is in
		uint32 LeafCellIndex;	// index of Leaf Cell this point should go into. This is an index into the RootCell-specific LevelCells array used inside the ParallelFor below.
	};

	// populate initial list of points
	TArray<FPointInfo> PointInfos;
	PointInfos.SetNumUninitialized(MaxPointID);
	ParallelFor(MaxPointID, [&](int32 k)
	{
		PointInfos[k] = FPointInfo{ k, PointToIndex(0, GetPositionFunc(k)), InvalidCellID };
	});
	
	// build a linear list of root cells, and sort all points into a list for each root cell
	TMap<FVector3i, int> RootCellLinearIndices;
	TArray<FVector3i> RootCellIndexes;
	TArray< TUniquePtr<TDynamicVector<int>> > RootCellPointIndices;
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(InitializeRootCellLists);
		int LinearIndexCounter = 0;
		for (int32 k = 0; k < MaxPointID; ++k)
		{
			int UseLinearIndex = 0;
			const int* FoundLinearIndex = RootCellLinearIndices.Find(PointInfos[k].RootCell);
			if (FoundLinearIndex == nullptr)
			{
				UseLinearIndex = LinearIndexCounter++;
				RootCellLinearIndices.Add(PointInfos[k].RootCell, UseLinearIndex);
				RootCellPointIndices.Add( MakeUnique<TDynamicVector<int>>() );
				RootCellIndexes.Add(PointInfos[k].RootCell);
			}
			else
			{
				UseLinearIndex = *FoundLinearIndex;
			}

			// try doing these Adds in a second per-root-cell ParallelFor? (parallel-scan seems wasteful but might still be faster...)
			RootCellPointIndices[UseLinearIndex]->Add(k);
		}
	}

	// create all the Root Cells
	TArray<FSparsePointOctreeCell*> RootCellPtrs;
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(CreateRootCells);
		// Note RootCellPtrs holds pointers to elements of Cells; this should be safe
		// thanks to the block storage of the TDynamicVector used for Cells
		RootCellPtrs.SetNumUninitialized(RootCellIndexes.Num());
		for (int32 Index = 0; Index < RootCellIndexes.Num(); ++Index)
		{
			FVector3i RootIndex = RootCellIndexes[Index];
			FSparsePointOctreeCell NewRootCell(0, RootIndex);
			int32 CellsIndex = CreateNewRootCell(NewRootCell, false);
			RootCellPtrs[Index] = &Cells[CellsIndex];
		}
		RootCellIndexes.Empty(); // discard this index array; we will only use the pointer array below
	}

	// make sure the PointIDToCellMap is large enough for all the PointIDs, then we never have to insert/resize
	PointIDToCellMap.SetNum(MaxPointID);

	// FParentChildCell represents a child-cell-of-a-parent that needs to be created. We will make lists of these below.
	// This strategy is maybe over-complicated, and possibly we can just create all the leaf cells and then figure out
	// the necessary parents that need to exist above it?
	struct FParentChildCell
	{
		FVector3i ParentCellIndex;		// XYZ index of parent cell
		FVector3i ChildCellIndex;		// XYZ index of child cell that needs to exist (in it's layer)
		int ChildNum;					// index of child cell in parent cell's 8-children list
		int NumPoints;					// number of points that will be inserted in this cell (only set for leaf cells)
		uint32 NewCellID;				// ID of cell after it is created. This is set in the code below
		bool operator==(const FParentChildCell& OtherCell) const { return ChildCellIndex == OtherCell.ChildCellIndex; }
	};

	// Parallel iteration over the Root cells. For each Root cell, we iterate over its points, and for each point, 
	// descend down to the max depth to figure out which cells need to be created. Then we create those cells, and 
	// then we insert the points into the leaf cells. 
	FCriticalSection SharedDataLock, CellPointListsLock;
	ParallelFor(RootCellPtrs.Num(), [&](int32 Index)
	{
		FSparsePointOctreeCell& RootCell = *RootCellPtrs[Index];
		
		// array of new child cells at each level
		TArray<TArray<FParentChildCell>> LevelCells;
		LevelCells.SetNum(MaxTreeDepth);

		// descend the non-existent octree cells mathematically, to determine which cells need to be instantiated
		TDynamicVector<int>& PointIndices = *RootCellPointIndices[Index];
		for ( int32 PointIndex : PointIndices)
		{
			int32 PointID = PointInfos[PointIndex].PointID;
			FVector3d Position = GetPositionFunc(PointID);

			int32 CurLevel = 0;
			int32 LastInsertIndex = 0;
			FSparsePointOctreeCell CurrentCell = RootCell;
			while (CurLevel < MaxTreeDepth)
			{
				int ChildIndex = ToChildCellIndex(CurrentCell, Position);	
				FSparsePointOctreeCell NextLevelCell = CurrentCell.MakeChildCell(ChildIndex);
				LastInsertIndex = LevelCells[CurLevel].AddUnique( FParentChildCell{ CurrentCell.Index, NextLevelCell.Index, ChildIndex, 0, InvalidCellID } );
				CurLevel++;
				CurrentCell = NextLevelCell;
			}
			// keep track of which leaf cell this point needs to be inserted into, since we computed it in the last iteration of the above loop
			PointInfos[PointIndex].LeafCellIndex = LastInsertIndex;
			LevelCells[CurLevel-1][LastInsertIndex].NumPoints++;
		}

		// sorted CellIDs of all the Leaf Cells we create in the loop below
		TArray<uint32> LeafCellIDs; 

		// instantiate all the cells at each level. This is a bit messy because at level/depth N+1 we 
		// do not know our parent Cells/CellIDs, only their indices, and that our parent must be one 
		// of the cells that were created at level N. So that takes a linear search. Unclear how to
		// track that mapping, except maybe a TMap?  (note that in profiling this is not remotely a bottleneck...)
		TArray<FSparsePointOctreeCell*> LastParents, NextParents;
		LastParents.Add(&RootCell);
		for (int32 Level = 0; Level < MaxTreeDepth; ++Level)
		{
			for (FParentChildCell& ParentChild : LevelCells[Level])
			{
				FSparsePointOctreeCell** ParentCell = LastParents.FindByPredicate( [&](FSparsePointOctreeCell* Cell) { return Cell->Index == ParentChild.ParentCellIndex; } );
				checkSlow(ParentCell != nullptr);
				checkSlow( (*ParentCell)->HasChild(ParentChild.ChildNum) == false );
				FSparsePointOctreeCell NewChildCell = (*ParentCell)->MakeChildCell( ParentChild.ChildNum );
				checkSlow( NewChildCell.Index == ParentChild.ChildCellIndex );

				// only this block accesses data structures shared between root cells
				SharedDataLock.Lock();
				NewChildCell.CellID = CellRefCounts.Allocate();
				Cells.InsertAt(NewChildCell, NewChildCell.CellID);
				if ( Level == MaxTreeDepth-1 )
				{
					LeafCellIDs.Add(NewChildCell.CellID);
				}
				FSparsePointOctreeCell* NewCellPtr = &Cells[NewChildCell.CellID];
				SharedDataLock.Unlock();

				(*ParentCell)->SetChild(ParentChild.ChildNum, NewChildCell);
				ParentChild.NewCellID = NewChildCell.CellID;

				NextParents.Add(NewCellPtr);
			}
			Swap(LastParents, NextParents);
			NextParents.Reset();
		}

		TArray<FParentChildCell>& LeafCells = LevelCells[MaxTreeDepth-1];
		int NumLeafCells = LeafCells.Num();
		TArray<TSparseListSet<int32>::FListHandle> LeafCellLists;
		LeafCellLists.SetNum(NumLeafCells);

		// initialize the cell-point-lists for each new leaf cell. This has to be thread-safe, but
		// it will return handles to the lists that can be used in parallel below 
		// (remember we are in a big ParallelFor over root cells here, that are accessing CellPointLists simultaneously)
		CellPointListsLock.Lock();
		for (int32 k = 0; k < NumLeafCells; ++k)
		{
			LeafCellLists[k] = CellPointLists.AllocateAt(LeafCellIDs[k]);
		}
		CellPointListsLock.Unlock();

		// It is faster to batch-insert points into the leaf cell lists, even though we have 
		// to build up new arrays for it. So first we collect per-leaf-cell lists, 
		// then we send those lists to the leaf cells
		TArray<TArray<int32>> LeafCellPointLists;		// todo could convert to a single array with offsets?
		LeafCellPointLists.SetNum(NumLeafCells);
		for (int32 k = 0; k < NumLeafCells; ++k)
		{
			LeafCellPointLists.Reserve( LeafCells[k].NumPoints );
		}
		for ( int32 PointIndex : PointIndices)
		{
			int PointID = PointInfos[PointIndex].PointID;
			int LeafIndex = PointInfos[PointIndex].LeafCellIndex;
			LeafCellPointLists[LeafIndex].Add(PointID);
			PointIDToCellMap[PointID] = LeafCells[LeafIndex].NewCellID;
		}
		for ( int32 k = 0; k < NumLeafCells; ++k)
		{
			// TODO: doing this step in parallel did not help, but perhaps the entire above block could be done in parallel (with repeat iterations over PointIndices)
			CellPointLists.SetValues(LeafCellLists[k], LeafCellPointLists[k]);
		}

	}, EParallelForFlags::Unbalanced);

}


void FSparseDynamicPointOctree3::InsertPoint_DynamicExpand(
	int32 PointID, 
	TFunctionRef<FVector3d(int)> GetPositionFunc)
{
	if (ContainsPoint(PointID))
	{
		checkSlow(false);
		return;
	}

	FVector3d Position = GetPositionFunc(PointID);

	FSparsePointOctreeCell CurrentCell = FindCurrentContainingCell(Position);
	checkSlow(CurrentCell.Level != FSparsePointOctreeCell::InvalidLevel);

	// if we found a containing root cell but it doesn't exist, create it and insert
	if (CurrentCell.Level == 0 && CurrentCell.IsExistingCell() == false)
	{
		Insert_NewRoot(PointID, Position, CurrentCell);
		return;
	}

	// insert into current cell if there is space, or if we hit max depth
	int NumPointsInCell = CellPointLists.GetCount(CurrentCell.CellID);
	if (NumPointsInCell + 1 < MaxPointsPerCell || CurrentCell.Level == MaxTreeDepth)
	{
		Insert_ToCell(PointID, Position, CurrentCell);
		return;
	}

	const FSparsePointOctreeCell* ParentCell = &Cells[CurrentCell.CellID];

	// this function inserts a point into one of the child cells of CurrentCell
	auto InsertToChildLocalFunc = [this,&GetPositionFunc,ParentCell](int InsertPointID)
	{
		FVector3d MovePosition = GetPositionFunc(InsertPointID);
		int ChildIdx = this->ToChildCellIndex(*ParentCell, MovePosition);
		if (ParentCell->HasChild(ChildIdx))
		{
			uint32 ChildCellID = ParentCell->GetChildCellID(ChildIdx);
			const FSparsePointOctreeCell* ChildCell = &Cells[ChildCellID];
			this->Insert_ToCell(InsertPointID, MovePosition, *ChildCell);
		}
		else
		{
			FSparsePointOctreeCell NewChildCell = ParentCell->MakeChildCell(ChildIdx);
			this->Insert_NewChildCell(InsertPointID, MovePosition, ParentCell->CellID, NewChildCell, ChildIdx);
		}
	};

	// otherwise current cell got too big, so move points to child cells
	CellPointLists.Enumerate(ParentCell->CellID, [&](int MovePointID)
	{
		InsertToChildLocalFunc(MovePointID);
	});

	// we removed all the points from ParentCell
	CellPointLists.Clear(ParentCell->CellID);

	// now insert ourself into one of these child cells
	InsertToChildLocalFunc(PointID);
}





bool FSparseDynamicPointOctree3::RemovePoint(int32 PointID)
{
	if (ContainsPoint(PointID) == false)
	{
		return false;
	}

	uint32 CellID = GetCellForPoint(PointID);
	if (CellID == InvalidCellID)
	{
		return false;
	}

	PointIDToCellMap[PointID] = InvalidCellID;

	bool bInList = CellPointLists.Remove(CellID, PointID);
	checkSlow(bInList);
	return true;
}


void FSparseDynamicPointOctree3::RemovePointUnsafe(int32 PointID)
{
	uint32 CellID = PointIDToCellMap[PointID];
	if (CellID != InvalidCellID)
	{
		PointIDToCellMap[PointID] = InvalidCellID;
		CellPointLists.Remove(CellID, PointID);
	}
}




void FSparseDynamicPointOctree3::ReinsertPoint(int32 PointID, const FVector3d& NewPosition)
{
	if (ContainsPoint(PointID))
	{
		uint32 CellID = GetCellForPoint(PointID);
		if (CellID != InvalidCellID)
		{
			FSparsePointOctreeCell& CurrentCell = Cells[CellID];
			if (CellContains(CurrentCell, NewPosition))
			{
				return;		// everything is fine
			}

		}
	}

	RemovePoint(PointID);
	InsertPoint(PointID, NewPosition);
}






double FSparseDynamicPointOctree3::FindNearestRayCellIntersection(const FSparsePointOctreeCell& Cell, const FRay3d& Ray) const
{
	FAxisAlignedBox3d Box = GetCellBox(Cell);
	double ray_t = TNumericLimits<double>::Max();
	if (FIntrRay3AxisAlignedBox3d::FindIntersection(Ray, Box, ray_t))
	{
		return ray_t;
	}
	else
	{
		return TNumericLimits<double>::Max();
	}
}




int32 FSparseDynamicPointOctree3::FindNearestHitPoint(const FRay3d& Ray,
	TFunctionRef<double(int, const FRay3d&)> HitPointDistFunc,
	double MaxDistance) const
{
	// this should take advantage of raster!

	// we use queue instead of recursion
	TArray<const FSparsePointOctreeCell*> Queue;
	Queue.Reserve(64);

	// push all root cells onto queue if they are hit by ray
	RootCells.AllocatedIteration([&](const uint32* RootCellID)
	{
		const FSparsePointOctreeCell* RootCell = &Cells[*RootCellID];
		double RayHitParam = FindNearestRayCellIntersection(*RootCell, Ray);
		if (RayHitParam < MaxDistance)
		{
			Queue.Add(&Cells[*RootCellID]);
		}
	});

	// test cells until the queue is empty
	int32 HitPointID = -1;
	while (Queue.Num() > 0)
	{
		const FSparsePointOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);
		
		// process elements
		CellPointLists.Enumerate(CurCell->CellID, [&](int32 PointID)
		{
			double HitDist = HitPointDistFunc(PointID, Ray);
			if (HitDist < MaxDistance)
			{
				MaxDistance = HitDist;
				HitPointID = PointID;
			}
		});

		// descend to child cells
		// sort by distance? use DDA?
		for (int k = 0; k < 8; ++k) 
		{
			if (CurCell->HasChild(k))
			{
				const FSparsePointOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				double RayHitParam = FindNearestRayCellIntersection(*ChildCell, Ray);
				if (RayHitParam < MaxDistance)
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}

	return HitPointID;
}







template<typename IsInRangeFn, typename ProcessCellFn>
void FSparseDynamicPointOctree3::ForEachRootCellInBounds(
	const FAxisAlignedBox3d& Bounds,
	const IsInRangeFn& IsInRange,
	const ProcessCellFn& ProcessCell) const
{
	FVector3i RootMinIndex = PointToIndex(0, Bounds.Min);
	FVector3i RootMaxIndex = PointToIndex(0, Bounds.Max);

	// Clamp to the range of allocated root cells
	FAxisAlignedBox3i IndexBounds = RootCells.GetBoundsInclusive();
	RootMinIndex = Max(RootMinIndex, IndexBounds.Min);
	RootMaxIndex = Min(RootMaxIndex, IndexBounds.Max);

	// Early out if query is outside allocated cell bounds
	for (int i = 0; i < 3; ++i)
	{
		if (RootMinIndex[i] > RootMaxIndex[i])
		{
			return;
		}
	}

	// If the clamped range would iterate more coordinates than there are allocated cells,
	// fall back to iterating only allocated cells. Use double to avoid int32 overflow.
	FVector3d QuerySize = FVector3d(RootMaxIndex) - FVector3d(RootMinIndex) + FVector3d::One();
	if (static_cast<double>(RootCells.GetCount()) < QuerySize.X * QuerySize.Y * QuerySize.Z)
	{
		RootCells.AllocatedIteration([this, &IsInRange, &ProcessCell](const uint32* RootCellID)
		{
			const FSparsePointOctreeCell& RootCell = Cells[*RootCellID];
			if (IsInRange(RootCell))
			{
				ProcessCell(RootCell);
			}
		});
	}
	else
	{
		RootCells.RangeIteration(RootMinIndex, RootMaxIndex, [this, &IsInRange, &ProcessCell](uint32 RootCellID)
		{
			const FSparsePointOctreeCell& RootCell = Cells[RootCellID];
			if (IsInRange(RootCell))
			{
				ProcessCell(RootCell);
			}
		});
	}
}


template<typename ArrayType>
void FSparseDynamicPointOctree3::GatherRootCellsForRangeQuery(
	const FAxisAlignedBox3d& Bounds,
	ArrayType& OutRootCells) const
{
	ForEachRootCellInBounds(Bounds,
		[this, &Bounds](const FSparsePointOctreeCell& Cell) { return GetCellBox(Cell).Intersects(Bounds); },
		[&OutRootCells](const FSparsePointOctreeCell& Cell) { OutRootCells.Add(&Cell); });
}


void FSparseDynamicPointOctree3::RangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TFunctionRef<bool(int)> PredicateFunc,
	TArray<int>& PointIDs,
	TArray<const FSparsePointOctreeCell*>* TempBuffer) const
{
	TArray<const FSparsePointOctreeCell*> InternalBuffer;
	TArray<const FSparsePointOctreeCell*>& Queue =
		(TempBuffer == nullptr) ? InternalBuffer : *TempBuffer;
	Queue.Reset();
	Queue.Reserve(128);

	GatherRootCellsForRangeQuery(Bounds, Queue);

	while (Queue.Num() > 0)
	{
		const FSparsePointOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

		// process elements
		if (CellPointLists.IsAllocated(CurCell->CellID))
		{
			CellPointLists.Enumerate(CurCell->CellID, [&](int32 PointID)
			{
				if (PredicateFunc(PointID))
				{
					PointIDs.Add(PointID);
				}
			});
		}

		for (int k = 0; k < 8; ++k)
		{
			if (CurCell->HasChild(k))
			{
				const FSparsePointOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
				if (GetCellBox(*ChildCell).Intersects(Bounds))
				{
					Queue.Add(ChildCell);
				}
			}
		}
	}
}






void FSparseDynamicPointOctree3::ParallelRangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TFunctionRef<bool(int)> PredicateFunc,
	TArray<int>& PointIDs,
	TArray<const FSparsePointOctreeCell*>* TempBuffer) const
{
	ParallelRangeQuery(Bounds, [](const FAxisAlignedBox3d&) {return true;},
		PredicateFunc, PointIDs, TempBuffer);
}

void FSparseDynamicPointOctree3::ParallelRangeQuery(
	const FAxisAlignedBox3d& Bounds,
	TFunctionRef<bool(const FAxisAlignedBox3d&)> BoundsPredicateFunc,
	TFunctionRef<bool(int)> PredicateFunc,
	TArray<int>& PointIDs,
	TArray<const FSparsePointOctreeCell*>* TempBuffer) const
{
	TArray<const FSparsePointOctreeCell*> InternalBuffer;
	TArray<const FSparsePointOctreeCell*>& ProcessRootCells =
		(TempBuffer == nullptr) ? InternalBuffer : *TempBuffer;
	ProcessRootCells.Reset();
	ProcessRootCells.Reserve(128);

	FCriticalSection OutputLock;

	ForEachRootCellInBounds(Bounds,
		[this, &Bounds, BoundsPredicateFunc](const FSparsePointOctreeCell& Cell) 
		{
			FAxisAlignedBox3d CellBox = GetCellBox(Cell);
			return CellBox.Intersects(Bounds) && BoundsPredicateFunc(CellBox);
		},
		[&ProcessRootCells](const FSparsePointOctreeCell& Cell) { ProcessRootCells.Add(&Cell); });

	ParallelFor(ProcessRootCells.Num(), [&](int idx)
	{
		const FSparsePointOctreeCell* RootCell = ProcessRootCells[idx];
		TArray<const FSparsePointOctreeCell*, TInlineAllocator<128>> Queue;
		Queue.Add(RootCell);

		TArray<int, TInlineAllocator<256>> LocalPointIDs;

		while (Queue.Num() > 0)
		{
			const FSparsePointOctreeCell* CurCell = Queue.Pop(EAllowShrinking::No);

			// process elements
			CellPointLists.Enumerate(CurCell->CellID, [&](int PointID)
			{
				if (PredicateFunc(PointID))
				{
					LocalPointIDs.Add(PointID);
				}
			});

			for (int k = 0; k < 8; ++k)
			{
				if (CurCell->HasChild(k))
				{
					const FSparsePointOctreeCell* ChildCell = &Cells[CurCell->GetChildCellID(k)];
					FAxisAlignedBox3d ChildCellBox = GetCellBox(*ChildCell);
					if (ChildCellBox.Intersects(Bounds) && BoundsPredicateFunc(ChildCellBox))
					{
						Queue.Add(ChildCell);
					}
				}
			}
		}

		if (LocalPointIDs.Num() > 0)
		{
			OutputLock.Lock();
			for (int ObjID : LocalPointIDs )
			{
				PointIDs.Add(ObjID);
			}
			OutputLock.Unlock();
		}
	});


	
}






FSparsePointOctreeCell FSparseDynamicPointOctree3::FindCurrentContainingCell(const FVector3d& Position) const
{
	// look up root cell, which may not exist
	FVector3i RootIndex = PointToIndex(0, Position);
	const uint32* RootCellID = RootCells.Get(RootIndex);
	if (RootCellID == nullptr)
	{
		return FSparsePointOctreeCell(0, RootIndex);
	}
	checkSlow(CellRefCounts.IsValid(*RootCellID));

	// check if point is contained in root cell
	// (should we do this before checking for existence? we can...)
	const FSparsePointOctreeCell* RootCell = &Cells[*RootCellID];
	if (CellContains(*RootCell, Position) == false)
	{
		return FSparsePointOctreeCell(FSparsePointOctreeCell::InvalidLevel, FVector3i::Zero());
	}

	const FSparsePointOctreeCell* CurrentCell = RootCell;
	do
	{
		int ChildIdx = ToChildCellIndex(*CurrentCell, Position);
		if (CurrentCell->HasChild(ChildIdx))
		{
			int32 ChildCellID = CurrentCell->GetChildCellID(ChildIdx);
			checkSlow(CellRefCounts.IsValid(ChildCellID));
			const FSparsePointOctreeCell* ChildCell = &Cells[ChildCellID];
			if (CellContains(*ChildCell, Position))
			{
				CurrentCell = ChildCell;
				continue;
			}
		}

		return *CurrentCell;

	} while (true);		// loop will always terminate

	return FSparsePointOctreeCell(FSparsePointOctreeCell::InvalidLevel, FVector3i::Zero());
}



void FSparseDynamicPointOctree3::CheckValidity(
	TFunctionRef<bool(int)> IsValidPointIDFunc,
	TFunctionRef<FVector3d(int)> GetPointFunc,
	EValidityCheckFailMode FailMode,
	bool bVerbose,
	bool bFailOnMissingPoints) const
{
	bool is_ok = true;
	TFunction<void(bool)> CheckOrFailF = [&](bool b)
	{
		is_ok = is_ok && b;
	};
	if (FailMode == EValidityCheckFailMode::Check)
	{
		CheckOrFailF = [&](bool b)
		{
			checkf(b, TEXT("FSparseDynamicPointOctree3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}
	else if (FailMode == EValidityCheckFailMode::Ensure)
	{
		CheckOrFailF = [&](bool b)
		{
			ensureMsgf(b, TEXT("FSparseDynamicPointOctree3::CheckValidity failed!"));
			is_ok = is_ok && b;
		};
	}

	TArray<int> CellsAtLevels, PointsAtLevel;
	CellsAtLevels.Init(0, 32);
	PointsAtLevel.Init(0, 32);
	uint32 MissingPointCount = 0;
	uint8 MaxLevel = 0;

	// check that all Point IDs in per-cell Point lists is valid
	for (int32 CellID : CellRefCounts.Indices())
	{
		CellPointLists.Enumerate(CellID, [&](int32 PointID)
		{
			CheckOrFailF(IsValidPointIDFunc(PointID));
		});
	}

	uint32 NumPointIDs = PointIDToCellMap.Num();
	for (uint32 PointID = 0; PointID < NumPointIDs; PointID++)
	{
		bool IsValidPointID = IsValidPointIDFunc(PointID);
		if (IsValidPointID)
		{
			FVector3d Position = GetPointFunc(PointID);

			CheckOrFailF(PointID < PointIDToCellMap.Num());
			uint32 CellID = PointIDToCellMap[PointID];

			if (bFailOnMissingPoints)
			{
				CheckOrFailF(CellID != InvalidCellID);
			}

			if (CellID == InvalidCellID)
			{
				MissingPointCount++;
			}
			else
			{
				CheckOrFailF(CellRefCounts.IsValid(CellID));
				FSparsePointOctreeCell Cell = Cells[CellID];
				CheckOrFailF(CellContains(Cell, Position));
				CheckOrFailF(CellPointLists.Contains(CellID, PointID));

				PointsAtLevel[Cell.Level]++;
			}
		}
	}


	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparsePointOctreeCell& Cell = Cells[CellID];
		CellsAtLevels[Cell.Level]++;
		MaxLevel = FMath::Max(MaxLevel, Cell.Level);
	}

	if (bVerbose)
	{
		UE_LOGF(LogGeometry, Warning, "FSparseDynamicPointOctree3::CheckValidity: MaxLevel %d  MissingCount %d", MaxLevel, MissingPointCount);
		for (uint32 k = 0; k <= MaxLevel; ++k)
		{
			UE_LOGF(LogGeometry, Warning, "    Level %4d  Cells %4d  Points %4d", k, CellsAtLevels[k], PointsAtLevel[k]);
		}
	}
}




void FSparseDynamicPointOctree3::ComputeStatistics(FStatistics& StatsOut) const
{
	StatsOut.Levels = 0;
	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparsePointOctreeCell& Cell = Cells[CellID];
		StatsOut.Levels = FMath::Max(StatsOut.Levels, (int32)Cell.Level);
	}
	StatsOut.Levels++;
	StatsOut.LevelObjCounts.Init(0, StatsOut.Levels);
	StatsOut.LevelMaxObjCounts.Init(0, StatsOut.Levels);
	StatsOut.LevelBoxCounts.Init(0, StatsOut.Levels);
	StatsOut.LevelCellSizes.Init(0, StatsOut.Levels);
	for (int32 CellID : CellRefCounts.Indices())
	{
		const FSparsePointOctreeCell& Cell = Cells[CellID];
		StatsOut.LevelBoxCounts[Cell.Level]++;
		int ObjCount = CellPointLists.IsAllocated(CellID) ? CellPointLists.GetCount(CellID) : 0;
		StatsOut.LevelObjCounts[Cell.Level] += ObjCount;
		StatsOut.LevelMaxObjCounts[Cell.Level] = FMath::Max(StatsOut.LevelMaxObjCounts[Cell.Level], ObjCount);
		StatsOut.LevelCellSizes[Cell.Level] = float(GetCellWidth(Cell.Level));
	}
}


FString FSparseDynamicPointOctree3::FStatistics::ToString() const
{
	FString Result = FString::Printf(
		TEXT("Levels %2d\r\n"), Levels);
	for (int k = 0; k < Levels; ++k)
	{
		Result += FString::Printf(TEXT("  Level %2d:  Cells %8d  Objs %8d  AvgObjs %5.3f  MaxObjs %8d  Dimension %5.5f\r\n"), 
			k, LevelBoxCounts[k], LevelObjCounts[k], ((float)LevelObjCounts[k] / (float)LevelBoxCounts[k]), LevelMaxObjCounts[k], LevelCellSizes[k] );
	}
	return Result;
}

} // end namespace UE::Geometry
} // end namespace UE
