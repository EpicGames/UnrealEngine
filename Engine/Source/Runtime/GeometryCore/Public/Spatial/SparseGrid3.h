// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DSparseGrid3

#pragma once

#include "CoreMinimal.h"
#include "BoxTypes.h"
#include "IntBoxTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * Dynamic sparse 3D grid. Useful in cases where we have grid of some type of non-trivial 
 * object and we don't want to pre-allocate full grid of them. So we allocate on-demand.
 * This can be used to implement multi-grid schemes, eg for example the ElemType
 * could be sub-grid of fixed dimension.
 *
 * @todo support pooling/re-use of allocated elements? currently not required by any use cases.
 * @todo port AllocatedIndices() and Allocated() iterators 
 */
template<typename ElemType>
class TSparseGrid3
{
protected:
	/** sparse grid of allocated elements */
	TMap<FVector3i, ElemType*> Elements;
	/** accumulated bounds of all indices inserted into Elements. Not currently used internally. */
	FAxisAlignedBox3i Bounds;

public:

	/**
	 * Create empty grid
	 */
	TSparseGrid3()
	{
		Bounds = FAxisAlignedBox3i::Empty();
	}

	/**
	 * Deletes all grid elements
	 */
	~TSparseGrid3()
	{
		FreeAll();
	}


	TSparseGrid3(const TSparseGrid3& Other) = delete;
	TSparseGrid3& operator=(const TSparseGrid3& Other) = delete;
	TSparseGrid3(TSparseGrid3&& Other) noexcept : Elements(MoveTemp(Other.Elements)), Bounds(MoveTemp(Other.Bounds))
	{
		Other.Elements.Reset();
	}
	TSparseGrid3& operator=(TSparseGrid3&& Other) noexcept
	{
		if (this != &Other)
		{
			Elements = MoveTemp(Other.Elements);
			Bounds = MoveTemp(Other.Bounds);
			Other.Elements.Reset();
		}
		return *this;
	}

	/**
	 * @param Index an integer grid index
	 * @return true if there is an allocated element at this index
	 */
	bool Has(const FVector3i& Index) const
	{
		return Elements.Contains(Index);
	}


	/**
	 * Get the grid element at this index
	 * @param Index integer grid index
	 * @return pointer to ElemType instance, or nullptr if element doesn't exist
	 */
	const ElemType* Get(const FVector3i& Index) const
	{
		return Elements.FindRef(Index);
	}


	/**
	 * Get the grid element at this index, and optionally allocate it if it doesn't exist
	 * @param Index integer grid index
	 * @param bAllocateIfMissing if the element at this index is null, allocate a new one
	 * @return pointer to ElemType instance, or nullptr if element doesn't exist
	 */
	ElemType* Get(const FVector3i& Index, bool bAllocateIfMissing)
	{
		ElemType* result = Elements.FindRef(Index);
		if (result != nullptr)
		{
			return result;
		}
		if (bAllocateIfMissing == true)
		{
			return Allocate(Index);
		}
		return nullptr;
	}

	/**
	 * Delete an element in the grid
	 * @param Index integer grid index
	 * @return true if the element existed, false if it did not
	 */
	bool Free(const FVector3i& Index)
	{
		if (Elements.Contains(Index))
		{
			delete Elements[Index];
			Elements.Remove(Index);
			return true;
		}
		return false;
	}

	/**
	 * Delete all elements in the grid
	 */
	void FreeAll()
	{
		for (auto pair : Elements)
		{
			delete pair.Value;
		}
		Elements.Reset();
	}

	/**
	 * @return number of allocated elements in the grid
	 */
	int GetCount() const
	{
		return Elements.Num();
	}

	/**
	 * @return ratio of allocated elements to total number of possible cells for current bounds
	 */
	float GetDensity() const
	{
		return (float)Elements.Num() / (float)Bounds.Volume();
	}

	/** 
	 * @return integer range of valid grid indices [min,max] (inclusive) 
	 */
	FAxisAlignedBox3i GetBoundsInclusive() const
	{
		return Bounds;
	}

	/** 
	 * @return dimensions of grid along each axis 
	 */
	FVector3i GetDimensions() const
	{
		return Bounds.Diagonal() + FVector3i::One();
	}


	/**
	 * Iterate over existing elements and apply ElementFunc(ElementType) to each of them
	 */
	template<typename Func>
	void AllocatedIteration(Func ElementFunc) const
	{
		for (auto Pair : Elements)
		{
			ElementFunc(Pair.Value);
		}
	}


	/**
	 * Iterate over existing allocated elements within the integer bounds defined 
	 * by Min and Max (inclusive), and apply ElementFunc(ElementType) to each of them
	 */
	template<typename Func>
	void RangeIteration(FVector3i MinIndex, FVector3i MaxIndex, Func ElementFunc) const
	{
		for (int32 zi = MinIndex.Z; zi <= MaxIndex.Z; ++zi)
		{
			for (int32 yi = MinIndex.Y; yi <= MaxIndex.Y; ++yi)
			{
				for (int32 xi = MinIndex.X; xi <= MaxIndex.X; ++xi)
				{
					const ElemType* Found = Elements.FindRef(FVector3i(xi,yi,zi));
					if (Found != nullptr)
					{
						ElementFunc(*Found);
					}
				}
			}
		}
	}



protected:

	ElemType* Allocate(const FVector3i& index)
	{
		ElemType* NewElem = new ElemType();
		Elements.Add(index, NewElem);
		Bounds.Contain(index);
		return NewElem;
	}
};

} // end namespace UE::Geometry
} // end namespace UE