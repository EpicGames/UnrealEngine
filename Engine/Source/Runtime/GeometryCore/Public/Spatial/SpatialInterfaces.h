// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp SpatialInterfaces

#pragma once

#include "Math/Ray.h"
#include "VectorTypes.h"

namespace MeshIntersection
{
	struct FHitIntersectionResult
	{
		int TriangleId;
		double Distance;
		FVector3d BaryCoords;
	};
}

namespace UE
{
namespace Geometry
{


/**
 * ISpatial is a base interface for spatial queries
 */
class ISpatial
{
public:
	virtual ~ISpatial() = default;

	/**
	 * @return true if this object supports point-containment inside/outside queries
	 */
	virtual bool SupportsPointContainment() const = 0;

	/**
	 * @return true if the query point is inside the object
	 */
	virtual bool IsInside(const FVector3d & Point) const = 0;
};



/**
 * IMeshSpatial is an extension of ISpatial specifically for meshes
 */
class IMeshSpatial : public ISpatial
{
public:
	virtual ~IMeshSpatial() = default;

	// standard shared options for all mesh spatial queries
	struct FQueryOptions
	{
		/**
		 * Maximum search distance / hit distance, where applicable
		 */
		double MaxDistance = TNumericLimits<double>::Max();

		/**
		 * If non-null, only triangle IDs that pass this filter (i.e. filter is true) are considered
		 */
		TFunction<bool(int)> TriangleFilterF = nullptr;

		// If true, then the IMeshSpatial may allow queries even when the underlying mesh has been
		// modified without updating the queried structure. This may be useful, for instance, to 
		// run queries against a mesh as it is being interactively modified, but it requires the 
		// caller to pass an appropriate TriangleFilterF to make sure that modified triangles do
		// not affect the query.
		bool bAllowUnsafeModifiedMeshQueries = false;

		FQueryOptions() {}
		FQueryOptions(TFunction<bool(int)> TriangleFilterF) : TriangleFilterF(TriangleFilterF) {}
		FQueryOptions(double MaxDistance, TFunction<bool(int)> TriangleFilterF = nullptr) : MaxDistance(MaxDistance), TriangleFilterF(TriangleFilterF) {}
	};

	/**
	 * @return true if this object supports nearest-triangle queries
	 */
	virtual bool SupportsNearestTriangle() const = 0;

	/**
	 * @param Query point
	 * @param NearestDistSqrOut returned nearest squared distance, if triangle is found
	 * @param Options Query options (ex. max distance)
	 * @return ID of triangle nearest to Point within MaxDistance, or InvalidID if not found
	 */
	virtual int FindNearestTriangle(const FVector3d& Point, double& NearestDistSqrOut, const FQueryOptions& Options = FQueryOptions()) const = 0;


	/**
	 * @return true if this object supports ray-triangle intersection queries
	 */
	virtual bool SupportsTriangleRayIntersection() const = 0;

	/**
	 * @param Ray query ray
	 * @param Options Query options (ex. max distance)
	 * @return ID of triangle intersected by ray within MaxDistance, or InvalidID if not found
	 */
	virtual int FindNearestHitTriangle(const FRay3d& Ray, const FQueryOptions& Options = FQueryOptions()) const
	{
		double NearestT;
		int TID;
		FindNearestHitTriangle(Ray, NearestT, TID, Options);
		return TID;
	}

	/**
	 * Find nearest triangle from the given ray
	 * @param Ray query ray
	 * @param NearestT returned-by-reference parameter of the nearest hit
	 * @param TID returned-by-reference ID of triangle intersected by ray within MaxDistance, or InvalidID if not found
	 * @param Options Query options (ex. max distance)
	 * @return true if hit, false if no hit found
	 */
	virtual bool FindNearestHitTriangle(const FRay3d& Ray, double& NearestT, int& TID, const FQueryOptions& Options = FQueryOptions()) const
	{
		FVector3d BaryCoords;
		return FindNearestHitTriangle(Ray, NearestT, TID, BaryCoords, Options);
	}

	/**
	 * Find nearest triangle from the given ray
	 * @param Ray query ray
	 * @param NearestT returned-by-reference parameter of the nearest hit
	 * @param TID returned-by-reference ID of triangle intersected by ray within MaxDistance, or InvalidID if not found
	 * @param BaryCoords returned-by-reference Barycentric coordinates of the triangle intersected by ray within MaxDistance, or FVector3d::Zero if not found.
	 * @param Options Query options (ex. max distance)
	 * @return true if hit, false if no hit found
	 */
	virtual bool FindNearestHitTriangle(const FRay3d& Ray, double& NearestT, int& TID, FVector3d& BaryCoords, const FQueryOptions& Options = FQueryOptions()) const = 0;

	/**
	 * Find all triangles intersected by the given ray sorted by distance
	 * @param Ray query ray
	 * @param OutHits returned-by-reference hit infos sorted by distance
	 * @param Options Query options (ex. max distance)
	 * @return true if hit, false if no hit found
	 */
	virtual bool FindAllHitTriangles(const FRay3d& Ray, TArray<MeshIntersection::FHitIntersectionResult>& OutHits, const FQueryOptions& Options = FQueryOptions()) const = 0;
};



/**
 * IProjectionTarget is an object that supports projecting a 3D point onto it
 */
class IProjectionTarget
{
public:
	virtual ~IProjectionTarget() = default;

	/**
	 * @param Point the point to project onto the target
	 * @param Identifier client-defined integer identifier of the point (may not be used)
	 * @return position of Point projected onto the target
	 */
	virtual FVector3d Project(const FVector3d& Point, int Identifier = -1) = 0;
};



/**
 * IOrientedProjectionTarget is a projection target that can return a normal in addition to the projected point
 */
class IOrientedProjectionTarget : public IProjectionTarget
{
public:
	virtual ~IOrientedProjectionTarget() = default;

	/**
	 * @param Point the point to project onto the target
	 * @param Identifier client-defined integer identifier of the point (may not be used)
	 * @return position of Point projected onto the target
	 */
	virtual FVector3d Project(const FVector3d& Point, int Identifier = -1) override = 0;

	/**
	 * @param Point the point to project onto the target
	 * @param ProjectNormalOut the normal at the projection point 
	 * @param Identifier client-defined integer identifier of the point (may not be used)
	 * @return position of Point projected onto the target
	 */
	virtual FVector3d Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier = -1) = 0;
};



/**
 * IIntersectionTarget is an object that can be intersected with a ray
 */
class IIntersectionTarget
{
public:
	virtual ~IIntersectionTarget() = default;

	/**
	 * @return true if RayIntersect will return a normal
	 */
	virtual bool HasNormal() = 0;

	/**
	 * @param Ray query ray
	 * @param HitOut returned hit point
	 * @param HitNormalOut returned hit point normal
	 * @return true if ray hit the object
	 */
	virtual bool RayIntersect(const FRay3d& Ray, FVector3d& HitOut, FVector3d& HitNormalOut) = 0;
};


} // end namespace UE::Geometry
} // end namespace UE
