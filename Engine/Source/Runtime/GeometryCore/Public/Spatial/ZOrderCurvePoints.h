// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/RandomStream.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Generate a "z-order curve" ordering for a point set by distributing the points to a quad- or octree and then reading the points indices back in tree traversal order
 * This is useful for giving the points spatial locality -- so points that are close together in the ordering tend to be close together spatially, also
 */
struct FZOrderCurvePoints
{
	/// Inputs

	// Maximum depth of quad/octree used to distribute points in space
	int32 MaxTreeDepth = 10;


	/// Outputs

	TArray<int32> Order;

public:

	FZOrderCurvePoints(int32 MaxTreeDepth = 10) : MaxTreeDepth(MaxTreeDepth)
	{
	}

	/**
	 * Compute an ordering for the input points that attempts to order points so adjacent points are close together spatially
	 * Supports both 2D and 3D points.
	 *
	 * @param Points				Points to re-order
	 * @param Bounds				Optional bounding box of the points; if an empty box is passed in, it will be computed for you
	 */
	GEOMETRYCORE_API void Compute(TArrayView<const FVector2d> Points, const FAxisAlignedBox2d& Bounds = FAxisAlignedBox2d::Empty());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector2f> Points, const FAxisAlignedBox2f& Bounds = FAxisAlignedBox2f::Empty());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector3d> Points, const FAxisAlignedBox3d& Bounds = FAxisAlignedBox3d::Empty());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector3f> Points, const FAxisAlignedBox3f& Bounds = FAxisAlignedBox3f::Empty());

};

/**
 * Generate a "Biased Randomized Insertion Order" (BRIO) ordering for a point set by randomly bucketing the points, then applying Z-Order-Curve sorting (above) to each bucket
 * This gives an ordering with point locality but enough randomization to typically avoid worst case behavior if using the ordering for Delaunay meshing
 */
struct FBRIOPoints
{
	/// Inputs

	// Maximum depth of quad/octree used to distribute points in space
	int32 MaxTreeDepth = 10;


	/// Outputs

	TArray<int32> Order;

public:

	FBRIOPoints(int32 MaxTreeDepth = 10) : MaxTreeDepth(MaxTreeDepth)
	{
	}

	/**
	 * Compute an ordering for the input points that attempts to order points so adjacent points are close together spatially, while still keeping the order sufficiently randomized to expect to avoid worst-case orderings for Delaunay triangulation
	 * Supports both 2D and 3D points.
	 *
	 * @param Points				Points to re-order
	 * @param Bounds				Optional bounding box of the points; if an empty box is passed in, it will be computed for you
	 */
	GEOMETRYCORE_API void Compute(TArrayView<const FVector2d> Points, const FAxisAlignedBox2d& Bounds = FAxisAlignedBox2d::Empty(), const FRandomStream& Random = FRandomStream());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector2f> Points, const FAxisAlignedBox2f& Bounds = FAxisAlignedBox2f::Empty(), const FRandomStream& Random = FRandomStream());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector3d> Points, const FAxisAlignedBox3d& Bounds = FAxisAlignedBox3d::Empty(), const FRandomStream& Random = FRandomStream());
	GEOMETRYCORE_API void Compute(TArrayView<const FVector3f> Points, const FAxisAlignedBox3f& Bounds = FAxisAlignedBox3f::Empty(), const FRandomStream& Random = FRandomStream());

};


} // end namespace UE::Geometry
} // end namespace UE
