// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMath.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/*
 * 3D Sphere stored as Center point and Radius
 */
template<typename T>
struct TSphere3
{
public:
	/** Center of the sphere */
	TVector<T> Center = TVector<T>::Zero();
	/** Radius of the sphere */
	T Radius = (T)0;

	TSphere3() = default;

	TSphere3(const TVector<T>& CenterIn, T RadiusIn)
		: Center(CenterIn), Radius(RadiusIn) {}

	/** @return Diameter of sphere */
	T Diameter() const
	{
		return (T)2 * Radius;
	}

	/** @return Circumference of sphere */
	T Circumference() const
	{
		return (T)2 * TMathUtil<T>::Pi * Radius;
	}

	/** @return Area of sphere */
	T Area() const
	{
		return Area(Radius);
	}

	/** @return Volume of sphere */
	T Volume() const
	{
		return Volume(Radius);
	}

	/** @return true if Sphere contains given Point */
	bool Contains(const TVector<T>& Point) const
	{
		T DistSqr = UE::Geometry::DistanceSquared(Center, Point);
		return DistSqr <= Radius * Radius;
	}

	/** @return true if Sphere contains given OtherSphere */
	bool Contains(const TSphere3<T>& OtherSphere) const
	{
		T CenterDist = Distance(Center, OtherSphere.Center);
		return (CenterDist + OtherSphere.Radius) <= Radius;
	}


	/**
	 * @return minimum squared distance from Point to Sphere surface for points outside sphere, 0 for points inside
	 */
	inline T DistanceSquared(const TVector<T>& Point) const
	{
		const T PosDistance = TMathUtil<T>::Max(SignedDistance(Point), (T)0);
		return PosDistance * PosDistance;
	}

	/**
	 * @return signed distance from Point to Sphere surface. Points inside sphere return negative distance.
	 */
	inline T SignedDistance(const TVector<T>& Point) const
	{
		return UE::Geometry::Distance(Center, Point) - Radius;
	}


	//
	// Sphere utility functions
	//

	/** @return Area of sphere with given Radius */
	static T Area(T Radius)
	{
		return (T)(4) * TMathUtil<T>::Pi * Radius * Radius;
	}

	/** @return Volume of sphere with given Radius */
	static T Volume(T Radius)
	{
		return (T)(4.0 / 3.0) * TMathUtil<T>::Pi * Radius * Radius * Radius;
	}
};


typedef TSphere3<float> FSphere3f;
typedef TSphere3<double> FSphere3d;

} // end namespace UE::Geometry
} // end namespace UE