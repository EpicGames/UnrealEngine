// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cmath>
#include <limits>
#include "Spline.h"
#include "InterpolationPolicies.h"
#include "SplineMath.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

struct UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") FKnot;
template <typename ValueType, int32 Degree> class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TBSpline;
	
enum class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") EParameterizationPolicy
{
	/** Equal spacing between knots regardless of control point positions */
	Uniform,

	/** Spacing based on actual distances between control points - good for uneven point distribution */
	ChordLength,

	/** Uses square root of distances - helps prevent loops and cusps in the curve */
	Centripetal,
};

struct FKnot
{
	float Value;
	uint32 Multiplicity;

	FKnot()
		: Value(0.0f)
		, Multiplicity(0)
	{
	}

	explicit FKnot(float InValue, uint32 InMultiplicity)
		: Value(InValue)
		, Multiplicity(InMultiplicity)
	{
	}

	/**
	 * Converts a pair-based knot vector to a flat knot vector
	 * @param PairKnots - The pair-based knot vector
	 * @return Flat knot vector with multiplicities expanded
	 */
	static TArray<float> ConvertPairToFlatKnots(const TArray<FKnot>& PairKnots)
	{
		TArray<float> FlatKnots;
		for (const FKnot& Knot : PairKnots)
		{
			for (uint32 i = 0; i < Knot.Multiplicity; ++i)
			{
				FlatKnots.Add(Knot.Value);
			}
		}
		return FlatKnots;
	}

	static bool IsValidKnotVector(const TArray<FKnot>& InKnots, int32 Degree)
	{
		TArray<float> FlatKnots = ConvertPairToFlatKnots(InKnots);
		// Need at least 2*(degree+1) knots
		if (FlatKnots.Num() < 2 * (Degree + 1))
		{
			return false;
		}

		// Check monotonicity
		for (int32 i = 1; i < FlatKnots.Num(); ++i)
		{
			if (FlatKnots[i] < FlatKnots[i - 1])
			{
				return false;
			}
		}

		return true;
	}

	bool operator==(const FKnot& Other) const
	{
		return FMath::IsNearlyEqual(Value, Other.Value) && Multiplicity == Other.Multiplicity;
	}

	friend FArchive& operator<<(FArchive& Ar, FKnot& Knot)
	{
		Ar << Knot.Value;
		Ar << Knot.Multiplicity;
		return Ar;
	}
};

// Generate compile-time type ID for this BSpline specialization
// Using degree in the implementation name for uniqueness
template <int Degree>
struct BSplineNameSelector
{
	static constexpr const TCHAR* Name = TEXT("BSpline");
};

template <>
struct BSplineNameSelector<2>
{
	static constexpr const TCHAR* Name = TEXT("BSpline2");
};

template <>
struct BSplineNameSelector<3>
{
	static constexpr const TCHAR* Name = TEXT("BSpline3");
};

template <typename InValueType, int32 InDegree>
class TBSpline
	: public TSplineInterface<InValueType>,
	  private TSelfRegisteringSpline<TBSpline<InValueType, InDegree>, InValueType>
{
public:
	using ValueType = InValueType;

	static constexpr int32 Degree = InDegree;
	static constexpr int32 WindowSize = Degree + 1;
	using FWindow = TStaticArray<const ValueType*, WindowSize>;
	using FWindowStore = TStaticArray<ValueType, WindowSize>;
	
	DECLARE_SPLINE_TYPE_ID(
		BSplineNameSelector<Degree>::Name,
		*TSplineValueTypeTraits<ValueType>::Name
	);

	TBSpline() = default;
	virtual ~TBSpline() override = default;

	template <typename T, typename = void>
	struct THasToString : std::false_type
	{
	};

	template <typename T>
	struct THasToString<T, std::void_t<decltype(std::declval<T>().ToString())>> : std::true_type
	{
	};

	void Dump() const
	{
		if constexpr (std::is_floating_point_v<ValueType>)
		{
			UE_LOGF(LogSpline, Verbose, "Values:");
			for (int32 Index = 0; Index < Values.Num(); ++Index)
			{
				UE_LOGF(LogSpline, Verbose, "  [%d] %f", Index, Values[Index]);
			}
		}
		else if constexpr (THasToString<ValueType>::value)
		{
			UE_LOGF(LogSpline, Verbose, "Values:");
			for (int32 Index = 0; Index < Values.Num(); ++Index)
			{
				UE_LOGF(LogSpline, Verbose, "  [%d] %ls", Index, *Values[Index].ToString());
			}
		}

		UE_LOGF(LogSpline, Verbose, "Knots:");
		PrintKnotVector();
	}

	// ISplineInterface Implementation
	virtual void Clear() override
	{
		Values.Empty();
		PairKnots.Empty();
	}

	virtual bool IsEqual(const ISplineInterface* OtherSpline) const override
	{
		if (OtherSpline->GetTypeId() == GetTypeId())
		{
			const TBSpline* Other = static_cast<const TBSpline*>(OtherSpline);
			return operator==(*Other);
		}

		return false;
	}

	virtual bool Serialize(FArchive& Ar) override
	{
		// Call immediate parent's Serialize
		if (!TSplineInterface<ValueType>::Serialize(Ar))
		{
			return false;
		}

		Ar << Values;
		Ar << PairKnots;
		Ar << bIsClosedLoop;
		Ar << bClampEnds;
		MarkFlatKnotsCacheDirty();
		return true;
	}

	bool operator==(const TBSpline& Other) const
	{
		return Values == Other.Values &&
			PairKnots == Other.PairKnots &&
			bIsClosedLoop == Other.bIsClosedLoop &&
			bClampEnds == Other.bClampEnds;
	}

	friend FArchive& operator<<(FArchive& Ar, TBSpline& BSpline)
	{
		BSpline.Serialize(Ar);
		return Ar;
	}

	virtual FInterval1f GetParameterSpace() const override
	{
		return GetKnotRange();
	}

	virtual void SetClosedLoop(bool bClosed) override
	{
		bIsClosedLoop = bClosed;
	}

	virtual bool IsClosedLoop() const override
	{
		return bIsClosedLoop;
	}

	virtual TUniquePtr<ISplineInterface> Clone() const override
	{
		TUniquePtr<TBSpline<ValueType, Degree>> Clone = MakeUnique<TBSpline<ValueType, Degree>>();

		// Copy all member variables
		Clone->Values = this->Values;
		Clone->PairKnots = this->PairKnots;
		Clone->bIsClosedLoop = this->bIsClosedLoop;
		Clone->bClampEnds = this->bClampEnds;

		// Copy infinity modes
		Clone->PreInfinityMode = this->PreInfinityMode;
		Clone->PostInfinityMode = this->PostInfinityMode;

		return Clone;
	}

	/** Returns the number of Bezier segments in the spline */
	virtual int32 GetNumberOfSegments() const override
	{
		// todo: implement
		ensureAlwaysMsgf(false, TEXT("TBSpline::GetNumberOfSegments is unimplemented!"));
		return INDEX_NONE;
	}

	/**
	 * Maps a segment index to its parameter range
	 * @param SegmentIndex - Index of the segment (0-based)
	 * @return True if the segment index is valid and mapping succeeded
	 */
	virtual FInterval1f GetSegmentParameterRange(int32 SegmentIndex) const override
	{
		// todo: implement
		ensureAlwaysMsgf(false, TEXT("TBSpline::GetSegmentParameterRange is unimplemented!"));
		return FInterval1f();
	}

	// TSplineInterface Implementation
	virtual ValueType EvaluateImpl(float Parameter) const override
	{
		int32 NumValues = NumKeys();
		if (NumValues == 0) return ValueType();
		if (NumValues == 1) return GetValue(0);

		FWindow Window;
		FWindowStore Scratch;
		FindWindow(Parameter, Window, Scratch);
		
		// If FindWindow fails, it will return an array of nullptr. InterpolateWindow assumes validity of elements.
		if (!Math::FSplineValidation::IsValidWindow<ValueType, WindowSize>(Window) ||
				!Math::FSplineValidation::IsValidParameter(Parameter))
		{
			return Math::FSplineValidation::GetDefaultValue<ValueType>();
		}

		return InterpolateWindow(Window, Parameter);
	}

	virtual float FindNearest(const ValueType& Point, float& OutSquaredDistance) const override
	{
		return 0;/* todo */
	}

	int32 NumKeys() const
	{
		return Values.Num();
	}

	const ValueType& GetValue(int32 Idx) const
	{
		return Values[Idx];
	}

	int32 AddValue(const ValueType& NewValue)
	{
		return Values.Add(NewValue);
	}

	bool SetValue(int32 Idx, const ValueType& NewValue)
	{
		if (!Values.IsValidIndex(Idx))
		{
			return false;
		}
		Values[Idx] = NewValue;
		return true;
	}

	int32 InsertValue(int32 Idx, const ValueType& NewValue)
	{
		if (Idx < 0 || Idx > Values.Num())
		{
			return Values.Add(NewValue);
		}
		return Values.Insert(NewValue, Idx);
	}

	virtual bool RemoveValue(int32 Index)
	{
		if (Values.IsValidIndex(Index))
		{
			Values.RemoveAt(Index);
			return true;
		}
		return false;
	}

	// Parameterization methods
	virtual int32 SetParameter(int32 Index, float NewParameter)
	{
		UE_LOGF(LogSpline, Warning, "SetParameter is disabled. Use Reparameterize() or a linear spline if you need manual control.");
		return INDEX_NONE;
	}

	// Returns parameter value for a control point index using Greville Abscissae
	virtual float GetParameter(int32 Index) const
	{
		UpdateFlatKnotsCache();
		if (Index < 0 || Index + Degree >= FlatKnots.Num() - 1)
			return 0.0f;

		float Sum = 0.0f;
		for (int j = 1; j <= Degree; ++j)
		{
			Sum += FlatKnots[Index + j];
		}
		return Sum / static_cast<float>(Degree);
	}

	// Find the control point index and local parameter for global parameter
	virtual int32 FindIndexForParameter(float Parameter, float& OutLocalParam) const
	{
		UpdateFlatKnotsCache();
		const TArray<float>& Knots = FlatKnots;

		if (Knots.IsEmpty())
			return INDEX_NONE;

		// Clamp parameter to valid range
		const float FirstValidKnot = Knots[Degree];
		const float LastValidKnot = Knots[Knots.Num() - Degree - 1];

		// Check for exact match with last knot (for insertion at the end)
		if (FMath::IsNearlyEqual(Parameter, LastValidKnot))
		{
			OutLocalParam = 0.0f; // No interpolation needed for exact match
			return NumKeys(); // Return index for appending
		}
		// Handle parameter near the end
		else if (Parameter >= LastValidKnot - UE_SMALL_NUMBER)
		{
			OutLocalParam = 1.0f;
			return NumKeys() - 2; // Last segment index
		}

		if (Parameter <= FirstValidKnot + UE_SMALL_NUMBER)
		{
			OutLocalParam = 0.0f;
			return 0;
		}


		// Find the appropriate span
		int32 Span = FindKnotSpan(Parameter);
		if (Span < Degree || Span >= Knots.Num() - 1)
			return INDEX_NONE;

		int32 Index = Span - Degree;

		// Calculate local parameter within span, with protection against near-zero denominators
		float SpanStart = Knots[Span];
		float SpanEnd = Knots[Span + 1];
		float SpanLength = SpanEnd - SpanStart;

		if (SpanLength < UE_SMALL_NUMBER)
		{
			// For nearly identical knots, use 0 or 1 based on which end is closer
			OutLocalParam = (Parameter - SpanStart < SpanEnd - Parameter)
				? 0.0f
				: 1.0f;
		}
		else
		{
			OutLocalParam = (Parameter - SpanStart) / SpanLength;
		}

		return Index;
	}

	void ResetKnotVector()
	{
		PairKnots.Reset();
		FlatKnots.Reset();
	}

	/**
	 * Returns how many times a given knot value appears.
	 * @param KnotIndex - Knot value to check
	 * @return Number of times the knot appears
	 */
	int32 GetKnotMultiplicity(int32 KnotIndex) const
	{
		if (PairKnots.IsValidIndex(KnotIndex))
		{
			return PairKnots[KnotIndex].Multiplicity;
		}

		return 0;
	}

	// Knot vector management
	virtual TArray<FKnot> GetKnotVector() const
	{
		return PairKnots;
	}

	bool ValidateKnotVector(const TArray<FKnot>& InKnots) const
	{
		const int32 RequiredKnots = GetExpectedNumKnots();
		int32 TotalExpandedKnots = 0;

		// Validate knot multiplicities and calculate total expanded knots  
		for (const FKnot& Knot : InKnots)
		{
			if (!ensureAlwaysMsgf(Knot.Multiplicity > 0,
								  TEXT("Invalid knot with zero multiplicity.")))
			{
				return false;
			}

			TotalExpandedKnots += Knot.Multiplicity;
		}

		if (!ensureAlwaysMsgf(TotalExpandedKnots >= RequiredKnots,
							  TEXT("Invalid knot vector size. Need %d knots, got %d."),
							  RequiredKnots,
							  TotalExpandedKnots))
		{
			return false;
		}

		// Verify knot values are non-decreasing (strict comparison, no epsilon)
		for (int32 i = 1; i < InKnots.Num(); ++i)
		{
			if (!ensureAlwaysMsgf(InKnots[i].Value >= InKnots[i - 1].Value,
								  TEXT("Invalid knot vector: knot values must be non-decreasing.")))
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Sets a custom knot vector using explicit knot/multiplicity pairs
	 * @param NewKnots - Array of knot values. Must be non-decreasing and have correct length (see GetExpectedNumKnots)
	 * @return True if the knot vector was valid and set successfully
	 */
	virtual bool SetKnotVector(const TArray<FKnot>& NewKnots)
	{
		if (!ValidateKnotVector(NewKnots))
		{
			return false;
		}

		// Directly assign the knots - no epsilon comparison needed
		PairKnots = NewKnots;

		// Mark the flat knots cache as dirty
		MarkFlatKnotsCacheDirty();

		UE_LOGF(LogSpline, Verbose, "Set knot vector (Direct) - Points: %d, Unique Knots: %d",
			NumKeys(), PairKnots.Num());

		PrintKnotVector();

		return true;
	}

	/**
	 * Updates the knot vector based on current control points and settings
	 * 
	 * @param Points - Array of control points and their Knots
	 * @param ParameterizationPolicy - Policy for generating knot vector
	 */
	virtual void Reparameterize(EParameterizationPolicy ParameterizationPolicy)
	{
		const TArray<ValueType>& Points = Values;
		int32 NumValues = Points.Num();
		if (NumValues < Degree + 1)
		{
			UE_LOGF(LogSpline, Warning, "Not enough points for B-spline: %d (need >= %d)",
				NumValues, Degree + 1);
			return;
		}

		const int32 KnotCount = NumValues + Degree + 1;
		PairKnots.Reset();

		switch (ParameterizationPolicy)
		{
		case EParameterizationPolicy::Uniform:
			GenerateUniformKnots(KnotCount);
			break;
		case EParameterizationPolicy::ChordLength:
			GenerateChordLengthKnots(KnotCount);
			break;
		case EParameterizationPolicy::Centripetal:
			GenerateCentripetalKnots(KnotCount);
			break;
		}

		// Mark the flat knots cache as dirty
		MarkFlatKnotsCacheDirty();

		UE_LOGF(LogSpline, Verbose, "Updated knot vector - Points: %d, Degree: %d, Knots: %d",
			Points.Num(), Degree, PairKnots.Num());
	}

	/** 
	 * Sets whether the spline should be clamped at endpoints
	 * When true, the curve will interpolate the first and last control points
	 */
	void SetClampedEnds(bool bInClampEnds)
	{
		bClampEnds = bInClampEnds;
	}

	bool IsClampedEnds() const
	{
		// Clamping only meaningful for non-closed curves
		return bClampEnds && !bIsClosedLoop;
	}

	// Get the valid parameter range in knot space
	FInterval1f GetKnotRange() const
	{
		UpdateFlatKnotsCache();
		if (FlatKnots.Num() - Degree - 1 < 0)
		{
			return FInterval1f::Empty();
		}
		return FInterval1f(PairKnots[0].Value, PairKnots.Last().Value);
	}

protected:
	/**
	 *  Sets the value of a knot
	 *  @param KnotIdx - Index of the knot to set
	 *  @param NewValue - Value to set
	**/
	void SetKnot(int32 KnotIdx, float NewValue)
	{
		if (KnotIdx < 0 || KnotIdx >= PairKnots.Num() || PairKnots[KnotIdx].Value == NewValue)
		{
			// Nothing to replace
			return;
		}

		PairKnots[KnotIdx].Value = NewValue;
		// Mark the flat knots cache as dirty
		MarkFlatKnotsCacheDirty();
	}

	/**
	 * Removes a knot at the specified index.
	 * 
	 * @param KnotIdx - Index of the knot to remove
	 * @return True if the knot was successfully removed, false if index was invalid
	 */
	bool RemoveKnot(int32 KnotIdx)
	{
		if (KnotIdx < 0 || KnotIdx >= PairKnots.Num())
		{
			return false;
		}
		PairKnots.RemoveAt(KnotIdx);
		// Mark the flat knots cache as dirty
		MarkFlatKnotsCacheDirty();
		return true;
	}

	/**
	 * Swaps two knots.
	 * 
	 * @param KnotIdxA - Index of first knot to swap.
	 * @param KnotIdxB - Index of second knot to swap.
	 */
	void SwapKnots(int32 KnotIdxA, int32 KnotIdxB)
	{
		if (KnotIdxA < 0 || KnotIdxB < 0 || KnotIdxA >= PairKnots.Num() || KnotIdxB >= PairKnots.Num())
		{
			return;
		}

		FKnot OldA = PairKnots[KnotIdxA];
		PairKnots[KnotIdxA] = PairKnots[KnotIdxB];
		PairKnots[KnotIdxB] = OldA;
		
		// Mark the flat knots cache as dirty
		MarkFlatKnotsCacheDirty();
	}

	/**
	 * Inserts a knot at the specified parameter value
	 * @param InKnot - Knot to insert
	 * @return index of new knot if knot insertion was successful, INDEX_NONE otherwise.
	 */
	int32 InsertKnot(FKnot InKnot)
	{
		// Binary search the knot vector:

		int32 Low = 0; // This will be our insert index
		int32 High = PairKnots.Num() - 1;

		while (Low <= High)
		{
			const int32 Mid = Low + (High - Low) / 2;

			if (PairKnots[Mid].Value == InKnot.Value)
			{
				UE_LOGF(LogSpline, Warning, "Knot insertion failed, knot already exists!");
				return INDEX_NONE;
			}
			else if (PairKnots[Mid].Value < InKnot.Value)
			{
				Low = Mid + 1;
			}
			else
			{
				High = Mid - 1;
			}
		}

		MarkFlatKnotsCacheDirty();
		return PairKnots.Insert(InKnot, Low);
	}

	struct FValidKnotSearchParams
	{
		FValidKnotSearchParams() = default;

		FValidKnotSearchParams(float InDesiredParameter)
			: DesiredParameter(InDesiredParameter)
		{
		}

		float DesiredParameter = 0.f;
		bool bSearchRight = true;
		bool bSearchLeft = true;
	};

	struct FKnotSearchCache
	{
		explicit FKnotSearchCache(const TArray<FKnot>& PairKnots)
		{
			// Build a set of existing knot values (normalize zeros to avoid -0/+0 duplicates)
			KnotValuesSet.Reserve(PairKnots.Num());
			for (const FKnot& Knot : PairKnots)
			{
				KnotValuesSet.Add(Param::NormalizeKey(Knot.Value));
			}
		}

		TSet<float> KnotValuesSet;
	};

	/**
	 * Finds the nearest available knot value that does not conflict with existing knots.
	 * @param InSearchParams - Describes value to search for and how to search.
	 * @return The nearest valid knot parameter that does not conflict with existing knots.
	 */
	float GetNearestAvailableKnotValue(const FValidKnotSearchParams& InSearchParams, const FKnotSearchCache& SearchCache) const
	{
		const float DesiredParameter = InSearchParams.DesiredParameter;

		if (!ensureAlwaysMsgf(!std::isnan(DesiredParameter), TEXT("DesiredParameter is NaN!")) ||
			!ensureAlwaysMsgf(InSearchParams.bSearchLeft || InSearchParams.bSearchRight, TEXT("Invalid search params!")) ||
			PairKnots.Num() == 0)
		{
			return DesiredParameter;
		}

		// Return the caller's exact value; NormalizeKey is only for set membership (folds -0.0f/+0.0f).
		if (!SearchCache.KnotValuesSet.Contains(Param::NormalizeKey(DesiredParameter)))
		{
			return DesiredParameter;
		}

		// Walk outward alternating sides
		float Left = DesiredParameter;
		float Right = DesiredParameter;

		// Copy so we can disable directions mid-search
		bool bSearchLeft = InSearchParams.bSearchLeft;
		bool bSearchRight = InSearchParams.bSearchRight;

		// Tiny dead-man budget to catch regressions; normal exits happen via guard rails.
		int32 StepBudget = 4096;
		for (;;)
		{
			bool AnyProgress = false;

			if (bSearchLeft)
			{
				const float NewLeft = Param::PrevDistinct(Left);
				if (NewLeft == Left)
				{
					bSearchLeft = false;
				} // Guard #1: no progress
				else
				{
					Left = NewLeft;
					AnyProgress = true;
					if (!SearchCache.KnotValuesSet.Contains(Param::NormalizeKey(Left)))
					{
						return Left;
					}
					if (Left <= -std::numeric_limits<float>::max()) // Guard #2: saturated
					{
						bSearchLeft = false;
					}
				}
			}

			if (bSearchRight)
			{
				const float NewRight = Param::NextDistinct(Right);
				if (NewRight == Right)
				{
					bSearchRight = false;
				} // Guard #1: no progress
				else
				{
					Right = NewRight;
					AnyProgress = true;
					if (!SearchCache.KnotValuesSet.Contains(Param::NormalizeKey(Right)))
					{
						return Right;
					}
					if (Right >= +std::numeric_limits<float>::max()) // Guard #2: saturated
					{
						bSearchRight = false;
					}
				}
			}

			if (!bSearchLeft && !bSearchRight) break; // Guard #3: nowhere left to search
			if (!AnyProgress) break; // Paranoid: should be covered by #1

			if (--StepBudget <= 0)
			{
				// Dead-man: catch unexpected spins
				ensureMsgf(false, TEXT("Step budget exhausted in GetNearestAvailableKnotValue"));
				break;
			}
		}

		// Fallback: never return the conflicting original. Pick a deterministic, distinct nudge.
		{
			// Canonicalize -0/+0, then use its *bits* to choose a side deterministically.
			const float Canon = Param::NormalizeKey(DesiredParameter);
			const uint32 Bits = Internal::FloatToBits(Canon);

			const bool bPreferRight =
				(InSearchParams.bSearchRight && (!InSearchParams.bSearchLeft || ((Bits & 1u) != 0)));

			// Step from the *original* DesiredParameter (not Canon) so we don’t alter the value unless needed.
			float Fallback = Param::Step(
				DesiredParameter, bPreferRight
				? Param::EDir::Right
				: Param::EDir::Left);

			float Probe = Param::NormalizeKey(Fallback);
			if (SearchCache.KnotValuesSet.Contains(Probe))
			{
				// Try the opposite direction once
				Fallback = Param::Step(
					DesiredParameter, bPreferRight
					? Param::EDir::Left
					: Param::EDir::Right);
				Probe = Param::NormalizeKey(Fallback);
			}

			ensureMsgf(!SearchCache.KnotValuesSet.Contains(Probe), TEXT("Fallback still conflicts; check knot packing logic."));
			return Fallback;
		}
	}

	virtual int32 GetExpectedNumKnots() const
	{
		const int32 NumValues = NumKeys();
		// For closed loops, we repeat the first Degree points at the end
		return this->IsClosedLoop()
			? (NumValues + (2 * Degree) + 1) // closed loop: n + 2p + 1
			: (NumValues + Degree + 1);
	}

	/** 
     * Generates a uniform knot vector with equal spacing
     * For clamped ends, multiplicity of degree+1 is used at endpoints
     */
	void GenerateUniformKnots(int32 KnotCount)
	{
		PairKnots.Reset();

		int32 NumPoints = NumKeys();
		if (this->IsClosedLoop())
		{
			// For a closed B-spline of degree d with n points:
			// - Need additional d points wrapping from start
			// - Need n+2d+1 knots total
			// - Knot spacing should be uniform to maintain periodicity
			const int32 n = NumPoints;
			const int32 d = Degree;
			const int32 NumKnots = n + 2 * d + 1;


			// For periodic B-splines, use uniform knot spacing
			for (int32 i = 0; i < NumKnots; ++i)
			{
				// Map to [0,n] range for n segments
				PairKnots.Add(FKnot(static_cast<float>(i), 1));
			}
		}
		else
		{
			for (int32 i = 0; i < KnotCount; ++i)
			{
				float Value;
				if (bClampEnds)
				{
					if (i <= Degree)
					{
						// Multiple knots at start for endpoint interpolation
						Value = 0.0f;
					}
					else if (i >= KnotCount - Degree - 1)
					{
						// Multiple knots at end for endpoint interpolation
						Value = static_cast<float>(KnotCount - 2 * Degree);
					}
					else
					{
						// Uniform spacing for internal knots
						Value = static_cast<float>(i - Degree);
					}
				}
				else
				{
					// Simple uniform spacing when not clamped
					Value = static_cast<float>(i);
				}
				PairKnots.Add(FKnot(Value, 1));
			}
		}
		MarkFlatKnotsCacheDirty();
	}

	/**
     * Generates a knot vector based on chord lengths between control points
     * This gives better parameterization when control points are unevenly spaced
     */
	void GenerateChordLengthKnots(int32 KnotCount)
	{
		const TArray<ValueType>& Points = Values;
		if (Points.Num() < 2)
		{
			GenerateUniformKnots(KnotCount);
			return;
		}

		const int32 n = Points.Num();
		const int32 d = Degree;
		const int32 NumKnots = this->IsClosedLoop()
			? (n + 2 * d + 1)
			: KnotCount;
		PairKnots.Reset();

		// Calculate chord lengths between consecutive points
		TArray<float> Chords;
		Chords.Reserve(Points.Num() - 1);
		float TotalLength = 0.0f;

		for (int32 i = 1; i < Points.Num(); ++i)
		{
			const float Length = static_cast<float>(Math::Distance(Points[i], Points[i - 1]));
			Chords.Add(Length);
			TotalLength += Length;
		}

		// For closed loop, add the closing segment length
		if (this->IsClosedLoop() && Points.Num() > 0)
		{
			const float ClosureLength = static_cast<float>(Math::Distance(Points[0], Points.Last()));
			Chords.Add(ClosureLength);
			TotalLength += ClosureLength;
		}

		if (this->IsClosedLoop())
		{
			// For closed loop, generate periodic knot vector based on chord lengths
			for (int32 i = 0; i < NumKnots; ++i)
			{
				PairKnots.Add(FKnot(static_cast<float>(i) * TotalLength / static_cast<float>(n + d), 1));
			}
		}
		else if (bClampEnds)
		{
			// For clamped ends, we need:
			// - First (Degree+1) knots equal to 0.0
			// - Last (Degree+1) knots equal to 1.0
			// - Middle knots based on chord lengths

			// First set starting knots to 0
			PairKnots.Add(FKnot(0.0f, Degree + 1));

			// Calculate internal knots based on accumulated chord lengths
			// We have (KnotCount - 2*(Degree+1)) internal knots to set
			const int32 NumInternalKnots = KnotCount - 2 * (Degree + 1);
			if (NumInternalKnots > 0)
			{
				// Compute normalized Knots at each control point
				TArray<float> Params;
				Params.SetNum(Points.Num());
				Params[0] = 0.0f;

				float AccumulatedLength = 0.0f;
				for (int32 i = 1; i < Points.Num(); ++i)
				{
					AccumulatedLength += Chords[i - 1];
					Params[i] = (TotalLength > UE_SMALL_NUMBER)
						? (AccumulatedLength / TotalLength)
						: (static_cast<float>(i) / static_cast<float>(Points.Num() - 1));
				}

				// Map these Knots to internal knots
				for (int32 i = 0; i < NumInternalKnots; ++i)
				{
					// Map i to appropriate parameter range
					const float t = static_cast<float>(i + 1) / static_cast<float>(NumInternalKnots + 1);
					// Map t to control point indices
					const int32 LowIndex = FMath::FloorToInt(t * static_cast<float>(Points.Num() - 1));
					const int32 HighIndex = FMath::Min(LowIndex + 1, Points.Num() - 1);
					const float Alpha = t * static_cast<float>(Points.Num() - 1 - LowIndex);

					// Interpolate Knots
					const float KnotValue = FMath::Lerp(Params[LowIndex], Params[HighIndex], Alpha);
					PairKnots.Add(FKnot(KnotValue, 1));
				}
			}
			// Set the last (Degree+1) knots to 1.0
			PairKnots.Add(FKnot(1.0f, Degree + 1));
		}
		else
		{
			// Unclamped knots - simple uniform distribution based on chord lengths
			for (int32 i = 0; i < KnotCount; ++i)
			{
				const float Param = static_cast<float>(i) / static_cast<float>(KnotCount - 1);
				PairKnots.Add(FKnot(Param * TotalLength, 1));
			}
		}
		MarkFlatKnotsCacheDirty();
	}

	/**
	 * Generates a knot vector using centripetal parameterization
	 * Uses square root of chord lengths which helps prevent cusps and unwanted loops
	 * Often provides the most visually pleasing results for interactive curve editing
	 */
	void GenerateCentripetalKnots(int32 KnotCount)
	{
		const TArray<ValueType>& Points = Values;
		if (Points.Num() < 2)
		{
			GenerateUniformKnots(KnotCount);
			return;
		}

		const int32 n = Points.Num();
		const int32 d = Degree;
		const int32 NumKnots = this->IsClosedLoop()
			? (n + 2 * d + 1)
			: KnotCount;
		PairKnots.Reset();

		TArray<float> Lengths;
		Lengths.Reserve(Points.Num() - 1);
		float TotalLength = 0.0f;

		for (int32 i = 1; i < Points.Num(); ++i)
		{
			// Square root of distance for centripetal parameterization
			const float Length = static_cast<float>(Math::CentripetalDistance(Points[i], Points[i - 1]));
			Lengths.Add(Length);
			TotalLength += Length;
		}

		// For closed loop, add the closing segment
		if (this->IsClosedLoop() && Points.Num() > 0)
		{
			const float ClosureLength = static_cast<float>(Math::CentripetalDistance(Points[0], Points.Last()));
			Lengths.Add(ClosureLength);
			TotalLength += ClosureLength;
		}

		if (this->IsClosedLoop())
		{
			// Generate periodic knot vector using centripetal parameterization
			for (int32 i = 0; i < NumKnots; ++i)
			{
				PairKnots.Add(FKnot(static_cast<float>(i) * TotalLength / static_cast<float>(n + d), 1));
			}
		}
		else if (bClampEnds)
		{
			// For clamped ends, same approach as chord length but with square root distances

			// First set starting knots to 0
			PairKnots.Add(FKnot(0.0f, Degree + 1));

			// Calculate internal knots based on accumulated centripetal Knots
			const int32 NumInternalKnots = KnotCount - 2 * (Degree + 1);
			if (NumInternalKnots > 0)
			{
				// Compute normalized Knots at each control point
				TArray<float> Params;
				Params.SetNum(Points.Num());
				Params[0] = 0.0f;

				float AccumulatedLength = 0.0f;
				for (int32 i = 1; i < Points.Num(); ++i)
				{
					AccumulatedLength += Lengths[i - 1];
					Params[i] = (TotalLength > UE_SMALL_NUMBER)
						? (AccumulatedLength / TotalLength)
						: (static_cast<float>(i) / static_cast<float>(Points.Num() - 1));
				}

				// Map these Knots to internal knots
				for (int32 i = 0; i < NumInternalKnots; ++i)
				{
					// Map i to appropriate parameter range
					const float t = static_cast<float>(i + 1) / static_cast<float>(NumInternalKnots + 1);
					// Map t to control point indices
					const int32 LowIndex = FMath::FloorToInt(t * static_cast<float>(Points.Num() - 1));
					const int32 HighIndex = FMath::Min(LowIndex + 1, Points.Num() - 1);
					const float Alpha = t * static_cast<float>(Points.Num() - 1 - LowIndex);

					// Interpolate Knots
					const float KnotValue = FMath::Lerp(Params[LowIndex], Params[HighIndex], Alpha);
					PairKnots.Add(FKnot(KnotValue, 1));
				}
			}
			// Set the last (Degree+1) knots to 1.0
			PairKnots.Add(FKnot(1.0f, Degree + 1));
		}
		else
		{
			// Unclamped knots
			for (int32 i = 0; i < KnotCount; ++i)
			{
				const float Param = static_cast<float>(i) / static_cast<float>(KnotCount - 1);
				PairKnots.Add(FKnot(Param * TotalLength, 1));
			}
		}
		MarkFlatKnotsCacheDirty();
	}

	/**
	 * Applies clamped multiplicity to the knot vector.
	 * This ensures that the first and last knots have Degree + 1 multiplicity if the ends are clamped.
	 * For unclamped splines (including closed splines), first and last knots will have a multiplicity of Degree.
	 */
	void ApplyClampedKnotsMultiplicity()
	{
		if (PairKnots.Num() == 0)
		{
			return;
		}

		if (IsClampedEnds())
		{
			// enforce degree + 1 multiplicity at the endpoints
			PairKnots[0].Multiplicity = Degree + 1;
			PairKnots.Last().Multiplicity = Degree + 1;
		}
		else
		{
			PairKnots[0].Multiplicity = Degree;
			PairKnots.Last().Multiplicity = Degree;
		}

		MarkFlatKnotsCacheDirty();
	}

	/**
	 * Updates the flat knot cache from the pair representation if the cache is dirty.
	 */
	void UpdateFlatKnotsCache() const
	{
		if (bFlatKnotsCacheDirty)
		{
			FlatKnots = FKnot::ConvertPairToFlatKnots(PairKnots);
			bFlatKnotsCacheDirty = false;
		}
	}

	/**
	 * Marks the flat knots cache as dirty.
	 */
	void MarkFlatKnotsCacheDirty() const
	{
		bFlatKnotsCacheDirty = true;
	}

	/**
	 * Prints the knot vector to the log for debugging.
	 */
	void PrintKnotVector() const
	{
		TArray<FKnot> KnotVector = GetKnotVector();

		for (int32 Index = 0; Index < KnotVector.Num(); ++Index)
		{
			const FKnot& Knot = KnotVector[Index];
			UE_LOGF(LogSpline, Verbose, "  [%d] Value: %f, Multiplicity: %u", Index, Knot.Value, Knot.Multiplicity);
		}

		for (int32 j = 0; j < this->FlatKnots.Num(); ++j)
		{
			UE_LOGF(LogSpline, Verbose, "  FlatKnot[%d] = %f", j, this->FlatKnots[j]);
		}
	}

private:
	/** Find knot span using binary search */
	int32 FindKnotSpan(float Parameter) const
	{
		UpdateFlatKnotsCache();
		const TArray<float>& Knots = FlatKnots;
		// First, check that the knot vector is populated.
		if (Knots.Num() == 0)
		{
			return 0; // Return a safe fallback.
		}

		const int32 NumValues = NumKeys();
		if (NumValues <= 0)
		{
			UE_LOGF(LogSpline, Error, "FindKnotSpan: Not enough knots to define a valid span.");
			return 0; // Safe fallback.
		}

		// Handle the last knot explicitly for clamped endpoints
		if (Parameter >= Knots.Last() - UE_SMALL_NUMBER)
		{
			return NumValues - 1; // Last span index
		}

		if (Parameter <= Knots[Degree] + UE_SMALL_NUMBER)
		{
			return Degree; // First valid span
		}

		// Binary search for the span
		int32 Low = Degree;
		int32 High = Knots.Num() - Degree - 1;

		while (Low <= High)
		{
			const int32 Mid = (Low + High) / 2;

			if (Parameter >= Knots[Mid] && Parameter < Knots[Mid + 1])
			{
				return Mid;
			}

			if (Parameter < Knots[Mid])
			{
				High = Mid - 1;
			}
			else
			{
				Low = Mid + 1;
			}
		}

		// Shouldn't reach here; return fallback
		UE_LOGF(LogSpline, Warning, "Fallback in FindKnotSpan for t=%f", Parameter);
		return Degree;
	}

	virtual void FindWindow(float Parameter, FWindow& OutWindow, FWindowStore& Scratch) const
	{
		OutWindow = {};
		if (NumKeys() < Degree + 1)
		{
			return;
		}

		int32 Span = FindKnotSpan(Parameter);
		// For closed loop, we need to ensure the span wraps properly
		if (this->IsClosedLoop())
		{
			// For closed loop:
			// - Span needs to wrap around number of control points
			// - Window indices wrap around array
			const int32 NumPoints = NumKeys();
			for (int32 i = 0; i <= Degree; ++i)
			{
				int32 Index = (Span - Degree + i) % NumPoints;
				if (Index < 0) Index += NumPoints;
				OutWindow[i] = &GetValue(Index);
			}
		}
		else
		{
			// Get control points for the window
			for (int32 i = 0; i <= Degree; ++i)
			{
				const int32 Index = Span - Degree + i;
				if (Index >= 0 && Index < NumKeys())
				{
					OutWindow[i] = &GetValue(Index);
				}
			}
		}
	}

	/**
	 * Evaluate the provided window using basis functions computed using the provided parameter. 
	 * @param Window The window to interpolate with the computed basis. All elements are assumed to be valid pointers.
	 * @param Parameter The parameter to use to compute the basis functions.
	 * @return The interpolated value.
	 */
	virtual ValueType InterpolateWindow(TArrayView<const ValueType* const> Window, float Parameter) const
	{
		TArray<float> Basis;

		if (PairKnots.IsEmpty())
		{
			return ValueType();
		}

		UpdateFlatKnotsCache();

		int32 Span = FindKnotSpan(Parameter);

		Math::ComputeBSplineBasisFunctions(FlatKnots, Span, Parameter, Degree, Basis);
		return TSplineInterpolationPolicy<ValueType>::InterpolateWithBasis(Window, Basis);
	}

protected:
	TArray<ValueType> Values;

	// Knot representation in pair form having multiplicity
	TArray<FKnot> PairKnots;

	// Cached flattened knot vector for compatible algorithms
	mutable TArray<float> FlatKnots;
	mutable bool bFlatKnotsCacheDirty;

	bool bIsClosedLoop = false;

	/** Whether to clamp the endpoints for interpolation of first/last control points */
	bool bClampEnds = true;
};

// Common type definitions
using FBSpline2f = TBSpline<FVector2f, 2>;
using FBSpline3f = TBSpline<FVector3f, 3>;
using FBSpline2d = TBSpline<FVector2d, 2>;
using FBSpline3d = TBSpline<FVector3d, 3>;
// Explicit degree type definitions if needed
using FQuadraticBSpline2f = TBSpline<FVector2f, 2>;
using FCubicBSpline3f = TBSpline<FVector3f, 3>;
using FQuarticBSpline3f = TBSpline<FVector3f, 4>;
using FQuinticBSpline2d = TBSpline<FVector2d, 5>;
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE	
