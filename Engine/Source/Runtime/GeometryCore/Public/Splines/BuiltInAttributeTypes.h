// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"
#include "ParameterizedTypes.h"
#include "SplineTypeId.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{


// Specialize TSplineValueTypeTraits for all built-in types
//-------------------------------------------------------

// Basic types
template<> struct TSplineValueTypeTraits<float> 
{ 
	static inline const FString Name = "Float";
};

template<> struct TSplineValueTypeTraits<double> 
{ 
	static inline const FString Name = "Double";
};

template<> struct TSplineValueTypeTraits<int32> 
{ 
	static inline const FString Name = "Int32";
};

template<> struct TSplineValueTypeTraits<bool> 
{ 
	static inline const FString Name = "Bool";
};

// Math types
template<> struct TSplineValueTypeTraits<FVector3d> 
{ 
	static inline const FString Name = "Vector";
};

template<> struct TSplineValueTypeTraits<FVector2D> 
{ 
	static inline const FString Name = "Vector2D";
};

template<> struct TSplineValueTypeTraits<FVector3f> 
{ 
	static inline const FString Name = "Vector3f";
};
	
template<> struct TSplineValueTypeTraits<FVector4> 
{ 
	static inline const FString Name = "Vector4";
};

template<> struct TSplineValueTypeTraits<FQuat> 
{ 
	static inline const FString Name = "Quat";
};

template<> struct TSplineValueTypeTraits<FRotator> 
{ 
	static inline const FString Name = "Rotator";
};

template<> struct TSplineValueTypeTraits<FTransform> 
{ 
	static inline const FString Name = "Transform";
};

// Color types
template<> struct TSplineValueTypeTraits<FLinearColor> 
{ 
	static inline const FString Name = "LinearColor";
};

template<> struct TSplineValueTypeTraits<FColor> 
{ 
	static inline const FString Name = "Color";
};

// String types
template<> struct TSplineValueTypeTraits<FName> 
{ 
	static inline const FString Name = "Name";
};

template<> struct TSplineValueTypeTraits<FString> 
{ 
	static inline const FString Name = "String";
};

template<> struct TSplineValueTypeTraits<FText> 
{ 
	static inline const FString Name = "Text";
};

} // namespace Spline
} // namespace Geometry
} // namesp