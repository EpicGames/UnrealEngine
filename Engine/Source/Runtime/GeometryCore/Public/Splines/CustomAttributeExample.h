// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Splines/InterpolationPolicies.h"
#include "Splines/BuiltInAttributeTypes.h"
#include "Splines/PolyBezierSpline.h"
#include "Splines/MultiSpline.h"
 
namespace UE
{
namespace Geometry
{
namespace Spline
{

/**
 * Simple custom attribute type for splines
 * Demonstrates the requirements for a custom attribute type
 * 
 * Custom types need to provide certain operations for use with splines:
 * 1. Basic arithmetic: operator-(const T&), operator+(const T&), operator*(float)
 * 2. Optional helper methods that improve performance: Size(), SizeSquared(), Dot(), etc.
 * 
 * If your type doesn't provide some of these methods, spline operations will still work
 * using fallback implementations, but may be less efficient.
 */
struct FFoo
{
    // A simple float value attribute
    float Value = 0.0f;
    
    // Constructor
    FFoo() = default;
    
    // Constructor with value
    explicit FFoo(float InValue) : Value(InValue) {}
    
    // Serialization support
    bool Serialize(FArchive& Ar)
    {
        Ar << Value;
        return true;
    }
        
    friend FArchive& operator<<(FArchive& Ar, FFoo& Foo)
    {
        Foo.Serialize(Ar);
        return Ar;
    }

	bool operator==(const FFoo& Other) const
    {
    	return Value == Other.Value;
    }

	// Add operator- for parameterization
	FFoo operator-(const FFoo& Other) const
    {
    	return FFoo(Value - Other.Value);
    }

	// Add operator+ for parameterization
	FFoo operator+(const FFoo& Other) const
    {
    	return FFoo(Value + Other.Value);
    }

	FFoo operator*(float Scale) const
    {
    	return FFoo(Value * Scale);
    }

	// Size method that returns the "magnitude" of this type
	// For a float-based type, we'll use absolute value
	float Size() const
    {
    	return FMath::Abs(Value);
    }

	// Square size (optimization to avoid square root)
	float SizeSquared() const
    {
    	return Value * Value;
    }

	// Dot product with another FFoo (required for some operations)
	float Dot(const FFoo& Other) const
    {
    	return Value * Other.Value;
    }

	// Get a normalized version (for when direction matters)
	FFoo GetSafeNormal() const
    {
    	if (FMath::IsNearlyZero(Value))
    		return FFoo(0.0f);
    	return FFoo(Value > 0.0f ? 1.0f : -1.0f);
    }
};

// Register the type with the attribute system by defining a specialization
// of TSplineValueTypeTraits with a Name field
template<> struct TSplineValueTypeTraits<FFoo> 
{ 
    static inline const FString Name = "Foo";
};

// Define a specialization of TSplineInterpolationPolicy to handle
// interpolation for this type
template<>
class TSplineInterpolationPolicy<FFoo>
{
public:
    // Basic parameter-based interpolation
    static FFoo Interpolate(TArrayView<const FFoo* const> Window, float Parameter)
    {
        if (Window.Num() < 2)
        {
            return Window.Num() == 1 ? *Window[0] : FFoo();
        }
        
        // Linear interpolation
        return FFoo(FMath::Lerp(Window[0]->Value, Window[1]->Value, Parameter));
    }
    
    // Window-based interpolation for spline implementations
    static FFoo InterpolateWithBasis(TArrayView<const FFoo* const> Window, TArrayView<const float> Basis)
    {
        if (Window.Num() == 0 || Basis.Num() == 0)
        {
            return FFoo();
        }
        
        // Weighted sum of values
        float Result = 0.0f;
        
        for (int32 i = 0; i < FMath::Min(Window.Num(), Basis.Num()); ++i)
        {
            Result += Window[i]->Value * Basis[i];
        }
        
        return FFoo(Result);
    }
    
    // Template-based n-th derivative calculation
    template<int32 Order>
    static FFoo EvaluateDerivative(TArrayView<const FFoo* const> Window, float Parameter)
    {
        if (Order == 0)
        {
            return Interpolate(Window, Parameter);
        }
        
        // For first derivative
        if (Order == 1 && Window.Num() >= 2)
        {
            // Basic first derivative for linear interpolation
            return FFoo((Window[1]->Value - Window[0]->Value));
        }
        
        // Higher order derivatives (could be more sophisticated)
        return FFoo(0.0f);
    }
};

/**
 * Example function demonstrating the FFoo type with the MultiSpline
 * 
 * FFoo implements all the necessary methods for use with splines:
 * - Basic arithmetic operators
 * - Size() and SizeSquared() for distance calculations
 * - Dot() for dot product calculations
 * - GetSafeNormal() for normalization
 * 
 * These methods are automatically detected by the UE::Geometry::Spline::Math
 * functions, allowing FFoo to work seamlessly with all spline operations.
 */
void CreatePolyBezierWithFooAttributes()
{
	// Create a PolyBezier spline with a single segment
    FPolyBezierSpline3d BaseSpline = FPolyBezierSpline3d::CreateLine(
		FVector(0, 0, 0),
		FVector(100, 100, 0)
	);

    // Create a MultiSpline that will host both the main spline and attribute channels
    TMultiSpline<FPolyBezierSpline3d> Spline;
    Spline.GetSpline() = BaseSpline; // Set the main spline
    
    // Get direct access to the BSpline implementation
    TBSpline<FFoo, 3>* FooSpline = Spline.CreateAttributeChannel<TBSpline<FFoo, 3>>("FooValues");
    if (FooSpline)
    {
        // Now we can directly add values using the BSpline API
        FooSpline->AddValue(FFoo(0.0f));   // Start point
        FooSpline->AddValue(FFoo(1.0f));   // End point
    }

    // Add another segment to the main spline

	Spline.GetSpline().AppendAnchor( FPolyBezierSpline3d::FAnchor(
		FVector(150, -50, 0),  // In Handle
		FVector(200, 0, 0),    // Anchor Point
		FVector(200, 100, 0))  // Out Handle
	);

    // Add another value directly to the BSpline
    if (FooSpline)
    {
        FooSpline->AddValue(FFoo(0.5f));   // Another point
        
        // We can use any BSpline-specific methods
        int32 NumFooValues = FooSpline->NumKeys();
        UE_LOGF(LogTemp, Display, "Attribute channel has %d values", NumFooValues);
        
        // Or access internal methods like reparameterization
        FooSpline->Reparameterize(EParameterizationPolicy::Uniform);
    }

	// Evaluate at various points
    for (float t = 0.0f; t <= 2.0f; t += 0.1f)
	{
        FVector Position = Spline.GetSpline().Evaluate(t);
		FFoo Foo = Spline.EvaluateAttribute<FFoo>("FooValues", t);
    
		UE_LOGF(LogTemp, Display, "At t=%.1f: Position=(%.1f, %.1f, %.1f), Foo=%.2f",
			t, Position.X, Position.Y, Position.Z, Foo.Value);
	}
}

} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE