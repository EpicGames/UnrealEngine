// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "SplineMath.h"
#include "ParameterizedTypes.h"

namespace UE
{
namespace Math
{
	template<typename T>
	struct TQuat;
}
}

namespace UE
{
namespace Geometry
{
namespace Spline
{
	
// Trait to detect if type T supports math operations needed for interpolation.
template<typename T, typename = void>
struct TIsMathInterpolable : std::false_type {};

template<typename T>
struct TIsMathInterpolable<T, std::void_t<
	decltype(FMath::Lerp(std::declval<T>(), std::declval<T>(), 0.5f)),
	decltype(std::declval<T>() + std::declval<T>()),
	decltype(std::declval<T>() * std::declval<float>())
	>> : std::true_type {};
/**
 * Base spline interpolation policy that works across all spline types
 * Extends the existing parameter-based interpolation with window-based methods
 */
template<typename T>
class TSplineInterpolationPolicy
{
public:
	// Keep existing parameter-based interpolation (used by attribute channels)
	static T Interpolate(TArrayView<const T* const> Window, float Parameter)
	{
		if (Window.Num() < 2)
		{
			return Window.Num() == 1 ? *Window[0] : T();
		}
		if constexpr (TIsMathInterpolable<T>::value)
		{
			return FMath::Lerp(*Window[0], *Window[1], Parameter);
		}
		else
		{
			// Fallback for non-math types:
			return Parameter < 0.5f ? *Window[0] : *Window[1];
		}
	}

	static T InterpolateCubic(TArrayView<const T* const> Window, float Parameter)
	{
		if (Window.Num() < 4)
		{
			// Fallback to linear interpolation if not enough points
			return Interpolate(Window, Parameter);
		}
		if constexpr (TIsMathInterpolable<T>::value)
		{
			// Calculate Bernstein basis functions
			const float t = Parameter;
			const float OneMinusT = 1.0f - t;
			const float t2 = t * t;
			const float OneMinusT2 = OneMinusT * OneMinusT;
		
			// Cubic Bernstein polynomials
			const float Basis[4] =
			{
				OneMinusT * OneMinusT2,     // (1-t)³
				3.0f * t * OneMinusT2,      // 3t(1-t)²
				3.0f * t2 * OneMinusT,      // 3t²(1-t)
				t * t2                      // t³
			};
		
			return TSplineInterpolationPolicy<T>::InterpolateWithBasis(Window, Basis);
		}
		else
		{
			// Fallback for non-math types:
			return Parameter < 0.5f ? *Window[1] : *Window[2];
		}
	}

	// Add window-based interpolation for spline implementations
	static T InterpolateWithBasis(TArrayView<const T* const> Window, TArrayView<const float> Basis)
	{
		if constexpr (TIsMathInterpolable<T>::value)
		{
			T Result = T();  // Assumes T() gives a neutral value (e.g. zero vector)
			int32 Count = FMath::Min(Window.Num(), Basis.Num());
			for (int32 i = 0; i < Count; ++i)
			{
				// Assumes T supports addition and multiplication by float.
				Result = Result + (*Window[i] * Basis[i]);
			}
			return Result;
		}
		else
		{
			// Fallback: simply return the first element if available.
			return Window.Num() > 0 ? *Window[0] : T();
		}
	}

	// Template-based n-th derivative calculation
	template<int32 Order>
	static T EvaluateDerivative(TArrayView<const T* const> Window, float Parameter)
	{
		if (Order == 0)
		{
			return Interpolate(Window, Parameter);
		}
		if constexpr (TIsMathInterpolable<T>::value)
		{
			// For math types, use the generic derivative helper.
			return Math::TGenericDerivativeHelper<T, Order>::Compute(Window, Parameter);
		}
		else
		{
			// For non-math types, derivatives are not meaningful.
			// Return a default constructed value.
			return T();
		}
	}
};

// Specialization only for int32
template<>
class TSplineInterpolationPolicy<int32>
{
public:
	static int32 Interpolate(TArrayView<const int32* const> Window, float Parameter)
    {
        if (Window.Num() < 2)
        {
            return Window.Num() == 1 ? *Window[0] : 0;
        }
        
        // Handle boundary cases
        if (Parameter <= 0.0f) return *Window[0];
        if (Parameter >= 1.0f) return *Window[1];
        
        // For intermediate values, explicitly convert to avoid implicit conversion warnings
        const float Start = static_cast<float>(*Window[0]);
        const float End = static_cast<float>(*Window[1]);
        const float Result = Start + Parameter * (End - Start);
        
        // Round to nearest integer
        return FMath::RoundToInt(Result);
    }
    
    static int32 InterpolateWithBasis(TArrayView<const int32* const> Window, TArrayView<const float> Basis)
    {
        float Result = 0.0f;
        for (int32 i = 0; i < FMath::Min(Window.Num(), Basis.Num()); ++i)
        {
            Result += static_cast<float>(*Window[i]) * Basis[i];
        }
        return FMath::RoundToInt(Result);
    }

	// Template-based n-th derivative calculation
	template<int32 Order>
	static int32 EvaluateDerivative(TArrayView<const int32* const> Window, float Parameter)
	{
		if (Order == 0)
		{
			return Interpolate(Window, Parameter);
		}
		return Math::TGenericDerivativeHelper<int32, Order>::Compute(Window, Parameter);
	}
};
	
// FLinearColor interpolation policy
template<>
class TSplineInterpolationPolicy<FLinearColor>
{
public:
	static FLinearColor Interpolate(TArrayView<const FLinearColor* const> Window, float Parameter)
	{
		if (Window.Num() < 2)
		{
			return Window.Num() == 1 ? *Window[0] : FLinearColor();
		}

		// Convert to linear color for better interpolation
		return FLinearColor::LerpUsingHSV(*Window[0],*Window[1], Parameter).ToFColor(true);
	}

	static FLinearColor InterpolateWithBasis(TArrayView<const FLinearColor* const> Window, TArrayView<const float> Basis)
	{
		// Convert all colors to linear space first
		TArray<FLinearColor> LinearColors;
		LinearColors.Reserve(Window.Num());
		for (const FLinearColor* Color : Window)
		{
			LinearColors.Add(*Color);
		}

		// Perform weighted sum in linear space
		FLinearColor Result = FLinearColor::Black;
		for (int32 i = 0; i < FMath::Min(Window.Num(), Basis.Num()); ++i)
		{
			Result += LinearColors[i] * Basis[i];
		}

		// Convert back to FColor
		return Result;
	}

	// Template-based n-th derivative calculation
	template<int32 Order>
	static FLinearColor EvaluateDerivative(TArrayView<const FLinearColor* const> Window, float Parameter)
	{
		if (Order == 0)
		{
			return Interpolate(Window, Parameter);
		}
		return Math::TGenericDerivativeHelper<FLinearColor, Order>::Compute(Window, Parameter);
	}
};

// Partial specialization for any quaternion type (TQuat)
template <typename RealType>
class TSplineInterpolationPolicy<UE::Math::TQuat<RealType>>
{
	using TQuat = UE::Math::TQuat<RealType>;
public:
    // Parameter-based interpolation using Slerp
	static TQuat Interpolate(TArrayView<const TQuat* const> Window, float Parameter)
	{
		if (Window.Num() < 2)
		{
			return Window.Num() == 1 ? *Window[0] : TQuat::Identity;
		}
        return TQuat::Slerp(*Window[0], *Window[1], static_cast<RealType>(Parameter));
	}

	/**
	 * Cubic interpolation using SQUAD (Spherical Quadrangle).
	 * Provides C1 continuous rotation interpolation matching FInterpCurve behavior.
	 * 
	 * Window layout matches cubic Bezier:
	 * - P0: Start rotation
	 * - C0: Out control (tangent at start)
	 * - C1: In control (tangent at end)
	 * - P1: End rotation
	 * - Alpha parameter is in [0,1]
	 */
	static TQuat InterpolateCubic(TArrayView<const TQuat* const> Window, float Alpha)
	{
		if (Window.Num() < 4)
		{
			// Fallback if not enough points
			return Interpolate(Window, Alpha);
		}
		
		return TQuat::Squad(
				*Window[0],  // P0 - start point
				*Window[1],  // C0 - out control
				*Window[3],  // P1 - end point
				*Window[2],  // C1 - in control
				Alpha);
	}
	
	/**
	 * Performs a weighted blend of multiple quaternions using arbitrary basis weights.
	 *
	 * This implements a *reference-centric* log–exp mean on S³:
	 *   1) Choose a reference quaternion Qref (here: the sample with max |weight|).
	 *   2) For each sample Qi, hemisphere-align to Qref, form the relative rotation
	 *      Qrel = Qref^{-1} * Qi, map to tangent via V = Log(Qrel).
	 *   3) Average the tangent vectors with the given weights and map back:
	 *      Result = Qref * Exp( average(V) ).
	 *
	 * Properties:
	 *  - Smooth, normalized, and works for arbitrary (even non-convex) weights.
	 *  - With two inputs and convex weights [1−t, t], this reduces exactly to SLERP.
	 *  - This is a *mean*, not a spherical Bezier/SQUAD; use de Casteljau/SQUAD for true cubic curves.
	 *  - If all weights are nonnegative, we normalize by the signed sum (convex case, allows cancellation).
	 *  - For mixed-sign weights we normalize by the sum of absolutes to avoid degeneracy.
	 */
	static TQuat InterpolateWithBasis(TArrayView<const TQuat* const> Window, TArrayView<const float> Basis)
	{
		const int32 N = FMath::Min(Window.Num(), Basis.Num());
		if (N <= 0)
		{
			return TQuat::Identity;
		}
		if (N == 1)
		{
			TQuat q = *Window[0];
			q.Normalize();
			return q;
		}
		// Optimization: detect linear case (2 points with [1-t, t] weights)
		if (N == 2)
		{
			const RealType W0 = static_cast<RealType>(Basis[0]);
			const RealType W1 = static_cast<RealType>(Basis[1]);
			const RealType Sum = W0 + W1;
			
			if (FMath::IsNearlyEqual(Sum, static_cast<RealType>(1), static_cast<RealType>(1e-4)) && 
				W0 >= static_cast<RealType>(0) && W1 >= static_cast<RealType>(0))
			{
				// Linear blend - use efficient Slerp
				TQuat Q0 = (*Window[0]); Q0.Normalize();
				TQuat Q1 = (*Window[1]); Q1.Normalize();
				
				if ((Q0 | Q1) < static_cast<RealType>(0))
				{
					Q1 = Q1 * static_cast<RealType>(-1);
				}
				
				return TQuat::Slerp(Q0, Q1, W1 / Sum);
			}
		}
		constexpr RealType DotAntipodalCut  = static_cast<RealType>(-0.95); // treat dot ≤ -0.95 as "nearly antipodal"
		constexpr RealType WeightBalanceTol = static_cast<RealType>(0.05);  // within 5% of each other -> "balanced"

		// ---------------------------------------------------------------------------
		// Antipodal pair detection: if two high-weight samples are nearly opposite
		// and their weights are nearly equal, we want cancellation about Identity.
		// ---------------------------------------------------------------------------
		bool bHasAntipodalBalancedPair = false;
		for (int32 i = 0; i < N && !bHasAntipodalBalancedPair; ++i)
		{
			for (int32 j = i + 1; j < N; ++j)
			{
				const RealType wi = static_cast<RealType>(Basis[i]);
				const RealType wj = static_cast<RealType>(Basis[j]);
				if (wi >= static_cast<RealType>(0) && wj >= static_cast<RealType>(0)) // convex-only check
				{
					const RealType sum = wi + wj;
					if (sum > static_cast<RealType>(UE_KINDA_SMALL_NUMBER))
					{
						const RealType balance = FMath::Abs(wi - wj) / sum;
						TQuat qi = *Window[i]; qi.Normalize();
						TQuat qj = *Window[j]; qj.Normalize();
						const RealType dot = (qi | qj);

						if (dot <= DotAntipodalCut && balance <= WeightBalanceTol)
						{
							bHasAntipodalBalancedPair = true;
							break;
						}
					}
				}
			}
		}
		
		// ---------------------------------------------------------------------------
		// Choose reference quaternion
		// - If antipodal balanced pair detected: use Identity so the pair cancels.
		// - Else: use the max-|weight| sample (your current strategy).
		// ---------------------------------------------------------------------------
		TQuat Qref;
		int32 RefIdx = 0;

		if (bHasAntipodalBalancedPair)
		{
			Qref = TQuat::Identity;
		}
		else
		{
			RealType MaxAbsW = static_cast<RealType>(0);
			for (int32 i = 0; i < N; ++i)
			{
				const RealType aw = FMath::Abs(static_cast<RealType>(Basis[i]));
				if (aw > MaxAbsW)
				{
					MaxAbsW = aw;
					RefIdx = i;
				}
			}
			Qref = *Window[RefIdx];
			Qref.Normalize();

			if (MaxAbsW <= static_cast<RealType>(UE_KINDA_SMALL_NUMBER))
			{
				return Qref; // nothing meaningful to average
			}
		}

		const TQuat QrefInv = Qref.Inverse();
		
		// Accumulate weighted logarithms in tangent space
		RealType SumX = 0, SumY = 0, SumZ = 0;
		RealType WeightSum = 0;          // signed sum for nonnegative bases
		RealType WeightAbsSum = 0;       // backup for mixed-sign cases
		bool bAllNonNegative = true;
		
		for (int32 i = 0; i < N; ++i)
		{
			const RealType Weight = static_cast<RealType>(Basis[i]);
			if (FMath::IsNearlyZero(Weight))
			{
				continue;
			}
			bAllNonNegative &= (Weight >= static_cast<RealType>(0));
			
			TQuat Qi = (*Window[i]);
			Qi.Normalize();

			// Hemisphere alignment relative to reference
			if ((Qref | Qi) < static_cast<RealType>(0))
			{
				Qi = Qi * static_cast<RealType>(-1);
			}

			// Relative rotation: Qrel = Qref^{-1} * Qi
			const TQuat Qrel = QrefInv * Qi;
			const TQuat V = Qrel.Log(); // tangent vector (pure imaginary)

			SumX += Weight * V.X;
			SumY += Weight * V.Y;
			SumZ += Weight * V.Z;
			WeightSum += Weight;
			WeightAbsSum += FMath::Abs(Weight);
		}

		// If all weights are nonnegative, use their (signed) sum so cancellations happen naturally.
		// If mixed signs, fall back to the absolute sum to avoid division by ~0 and keep continuity.
		const RealType Norm = bAllNonNegative ? WeightSum : WeightAbsSum;
		if (Norm <= static_cast<RealType>(UE_KINDA_SMALL_NUMBER))
		{
			return Qref;
		}

		// Map back to manifold: Qref * Exp(avg tangent)
		const TQuat D = TQuat(SumX / Norm, SumY / Norm, SumZ / Norm, static_cast<RealType>(0)).Exp();
		TQuat Result = Qref * D;
		Result.Normalize();
		return Result;
	}



	// Template-based n-th derivative calculation
	template<int32 Order>
	static TQuat EvaluateDerivative(TArrayView<const TQuat* const> Window, float Parameter)
	{
		if (Order == 0)
		{
			return Interpolate(Window, Parameter);
		}

		// First derivative - angular velocity
		if (Order == 1)
		{
            // Adaptive step size based on quaternion difference
            RealType base_h = static_cast<RealType>(0.01);
			TQuat Q0 = Interpolate(Window, Parameter);
            TQuat Q1base = Interpolate(Window, Parameter + static_cast<float>(base_h));
            RealType diff = Q0.AngularDistance(Q1base);
            RealType h = base_h * FMath::Clamp(
                diff, 
                static_cast<RealType>(0.001),
                static_cast<RealType>(0.1)
            );

            TQuat Q1 = Interpolate(Window, Parameter + static_cast<float>(h));
            return (Q1 * Q0.Inverse()).Log() * (static_cast<RealType>(1) / h);
		}

		// Second derivative - angular acceleration
		if (Order == 2)
		{
            RealType base_h = static_cast<RealType>(0.01);
            TQuat Q_m1 = Interpolate(Window, Parameter - static_cast<float>(base_h));
            TQuat Q_p1 = Interpolate(Window, Parameter + static_cast<float>(base_h));
            RealType diff = Q_m1.AngularDistance(Q_p1);
            RealType h = base_h * FMath::Clamp(
                diff / static_cast<RealType>(2),
                static_cast<RealType>(0.001),
                static_cast<RealType>(0.1)
            );

            TQuat V0 = EvaluateDerivative<1>(Window, Parameter - static_cast<float>(h));
            TQuat V1 = EvaluateDerivative<1>(Window, Parameter + static_cast<float>(h));
            return (V1 * V0.Inverse()).Log() * (static_cast<RealType>(1) / (static_cast<RealType>(2) * h));
		}

		// Higher orders generally not meaningful for rotations
		return TQuat::Identity;
	}
};
	
// FTransform interpolation policy
template<>
class TSplineInterpolationPolicy<FTransform>
{
public:
	static FTransform Interpolate(TArrayView<const FTransform* const> Window, float Parameter)
	{
		if (Window.Num() < 2)
		{
			return Window.Num() == 1 ? *Window[0] : FTransform::Identity;
		}

		const FTransform& A = *Window[0];
		const FTransform& B = *Window[1];

		FTransform Result;
		Result.SetLocation(FMath::Lerp(A.GetLocation(), B.GetLocation(), Parameter));
		Result.SetRotation(FQuat::Slerp(A.GetRotation(), B.GetRotation(), Parameter));
		Result.SetScale3D(FMath::Lerp(A.GetScale3D(), B.GetScale3D(), Parameter));

		return Result;
	}

	static FTransform InterpolateWithBasis(TArrayView<const FTransform* const> Window, TArrayView<const float> Basis)
	{
		// Handle empty case
		if (Window.Num() == 0 || Basis.Num() == 0)
		{
			return FTransform::Identity;
		}

		// Interpolate components separately
		FVector Location = FVector::ZeroVector;
		FQuat Rotation = FQuat::Identity;
		FVector Scale = FVector(1.0f);

		float TotalRotWeight = 0.0f;

		// Weighted sum of locations
		for (int32 i = 0; i < FMath::Min(Window.Num(), Basis.Num()); ++i)
		{
			const float Weight = Basis[i];
			if (Weight > UE_SMALL_NUMBER)
			{
				Location += Window[i]->GetLocation() * Weight;
				Scale += Window[i]->GetScale3D() * Weight;

				// Handle rotation separately with proper quaternion blending
				const FQuat& Q = Window[i]->GetRotation();
				const double Dot = Rotation.W * Q.W + Rotation.X * Q.X + Rotation.Y * Q.Y + Rotation.Z * Q.Z;
				Rotation += (Dot >= 0.0f ? Q : -Q) * Weight;
				TotalRotWeight += Weight;
			}
		}

		// Normalize the quaternion if we accumulated any weights
		if (TotalRotWeight > UE_SMALL_NUMBER)
		{
			Rotation.Normalize();
		}

		return FTransform(Rotation, Location, Scale);
	}

	template<int32 Order>
	static FTransform EvaluateDerivative(TArrayView<const FTransform* const> Window, float Parameter)
	{
		if (Order == 0)
		{
			return Interpolate(Window, Parameter);
		}

		// Split into components
		TArray<FVector> Translations;
		TArray<FQuat> Rotations;
		TArray<FVector> Scales;
        
		Translations.SetNum(Window.Num());
		Rotations.SetNum(Window.Num());
		Scales.SetNum(Window.Num());
        
		for (int32 i = 0; i < Window.Num(); ++i)
		{
			Translations[i] = Window[i]->GetTranslation();
			Rotations[i] = Window[i]->GetRotation();
			Scales[i] = Window[i]->GetScale3D();
		}

		// Calculate derivatives for each component
		TArrayView<const FVector> TransView(Translations.GetData(), Translations.Num());
		TArrayView<const FQuat> RotView(Rotations.GetData(), Rotations.Num());
		TArrayView<const FVector> ScaleView(Scales.GetData(), Scales.Num());

		FVector TransResult = TSplineInterpolationPolicy<FVector>::EvaluateDerivative<Order>(TransView, Parameter);
		FQuat RotResult = TSplineInterpolationPolicy<FQuat>::EvaluateDerivative<Order>(RotView, Parameter);
		FVector ScaleResult = TSplineInterpolationPolicy<FVector>::EvaluateDerivative<Order>(ScaleView, Parameter);

		return FTransform(RotResult, TransResult, ScaleResult);
	}
};


// Base derivative calculator
template <typename T, int32 Order>
struct TDerivativePolicy
{
	static T Compute(TArrayView<const T* const> Window, float Parameter)
	{
		return TSplineInterpolationPolicy<T>::template EvaluateDerivative<Order>(Window, Parameter);
	}
};
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE