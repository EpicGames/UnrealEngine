// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BSpline.h"
namespace UE
{
namespace Geometry
{
namespace Spline
{

template<typename T> class UE_EXPERIMENTAL(5.7, "") TLinearSpline;
	
template<typename T>
class TLinearSpline : public TBSpline<T, 1>
{
public:
	using Base = TBSpline<T, 1>;

	DECLARE_SPLINE_TYPE_ID(TEXT("BSpline1"), *TSplineValueTypeTraits<T>::Name);

	TLinearSpline()
	{
	}

	virtual float GetParameter(int32 Index) const override
	{
		return (Index >= 0 && Index + 1 < this->Knots.Num())
			? this->Knots[Index + 1]
			: 0.0f;
	}

	virtual bool SetParameter(int32 Index, float NewParam) override
	{
		if (Index < 0 || Index + 1 >= this->Knots.Num())
			return false;

		float Prev = (Index > 0) ? this->Knots[Index] : -FLT_MAX;
		float Next = (Index + 2 < this->Knots.Num()) ? this->Knots[Index + 2] : FLT_MAX;

		if (NewParam < Prev || NewParam > Next)
			return false;

		this->Knots[Index + 1] = NewParam;
		return true;
	}

};

using FLinearSplineFloat = TLinearSpline<float>; 
using FLinearSplineVector3d = TLinearSpline<FVector3d>;
 
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE