// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SplineInterfaces.h"
#include "BoxTypes.h"
#include "SplineTypeRegistry.h"
#include "Spline.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

template <typename SPLINETYPE> class UE_EXPERIMENTAL(5.7, "") TMultiSpline;
	
/**
 * Multi-spline implementation that can contain other splines
 * Provides attribute functionality while allowing direct access to implementation-specific methods
 */
template <typename SPLINETYPE>
class TMultiSpline :
	public TSplineWrapper<SPLINETYPE>,
	private TSelfRegisteringSpline<TMultiSpline<SPLINETYPE>, typename SPLINETYPE::ValueType>
{
public:
    enum class EMappingRangeSpace: uint8
    {
        Normalized,
        Parent
    };

private:
    /* Internal structure for child splines */
    struct FChildSpline
    {
		// Default constructor
    	FChildSpline() = default;
    
    	// Copy constructor - creates a deep copy of the spline
    	FChildSpline(const FChildSpline& Other)
			: MappingRangeSpace(Other.MappingRangeSpace)
			, MappingRange(Other.MappingRange)
    	{
    		// Clone the spline if it exists
    		if (Other.Spline)
    		{
    			Spline = Other.Spline->Clone();
    		}
    	}
    
    	// Move constructor
    	FChildSpline(FChildSpline&& Other) = default;
    
    	// Copy assignment - creates a deep copy
    	FChildSpline& operator=(const FChildSpline& Other)
    	{
    		if (this != &Other)
    		{
    			MappingRangeSpace = Other.MappingRangeSpace;
    			MappingRange = Other.MappingRange;
            
    			// Clone the spline if it exists
    			if (Other.Spline)
    			{
    				Spline = Other.Spline->Clone();
    			}
    			else
    			{
    				Spline.Reset();
    			}
    		}
    		return *this;
    	}
    
    	// Move assignment
    	FChildSpline& operator=(FChildSpline&& Other) = default;

    	friend FArchive& operator<<( FArchive& Ar, FChildSpline& Child )
    	{
    		Child.Serialize(Ar);
    		return Ar;
    	}
    	
		bool Serialize(FArchive& Ar)
		{
			Ar << MappingRangeSpace;
		    Ar << MappingRange.Min;
		    Ar << MappingRange.Max;

		    // We must create the proper spline before we can serialize it.
		    if (Ar.IsLoading())
		    {
    			FSplineTypeId::IdType TypeId;
    			
    			// We are about to read data from the archive that the spline itself will attempt to read,
    			// so we have to restore the archive position after reading it.
    			const int64 Pos = Ar.Tell();
    			{
    				int32 Version;
    				Ar << Version;
    				Ar << TypeId;
    			}
    			Ar.Seek(Pos);

    			Spline = FSplineTypeRegistry::CreateSpline(TypeId);

    			ensureAlways(Spline);	// if this failed, most likely the spline type has not been registered.
		    }
		    
		    Ar << *Spline;
		    
		    return true;
		}
    	
		float MapParameterToChildSpace(float ParentParameter, const FInterval1f& ParentSpace) const
		{
			if (!Spline)
			{
				return 0.f;
			}

			if (FInterval1f ChildSpace = Spline->GetParameterSpace(); !ChildSpace.IsEmpty())
			{
				FInterval1f MappedParentRange = GetParentSpaceRange(ParentSpace);
	
				float T = MappedParentRange.GetUnclampedT(ParentParameter);
				return ChildSpace.Interpolate(T);
			}
			else
			{
				return 0.f;
			}
		}
	
		float MapParameterFromChildSpace(float ChildParameter, const FInterval1f& ParentSpace) const
		{
			if (!Spline)
			{
				return 0.f;
			}

			if (FInterval1f ChildSpace = Spline->GetParameterSpace(); !ChildSpace.IsEmpty())
			{
				FInterval1f MappedParentRange = GetParentSpaceRange(ParentSpace);
	
				float T = ChildSpace.GetUnclampedT(ChildParameter);
				return MappedParentRange.Interpolate(T);
			}
			else
			{
				return 0.f;
			}
		}

        FInterval1f GetParentSpaceRange(const FInterval1f& ParentSpace) const
        {
            switch(MappingRangeSpace)
            {
                case EMappingRangeSpace::Normalized:
                    return FInterval1f(ParentSpace.Interpolate(MappingRange.Min), ParentSpace.Interpolate(MappingRange.Max));

                case EMappingRangeSpace::Parent:
                    return MappingRange;
                    
                default:
                    return FInterval1f::Empty();
            }
        }

        EMappingRangeSpace GetMappingRangeSpace() const
        { 
            return MappingRangeSpace;
        }

        void SetMappingRangeSpace(EMappingRangeSpace InMappingRangeSpace, const FInterval1f& ParentSpace)
        {
            if (MappingRangeSpace == InMappingRangeSpace)
            {
                return;
            }

            switch(InMappingRangeSpace)
            {
                case EMappingRangeSpace::Normalized:
                    MappingRange = FInterval1f(ParentSpace.GetT(MappingRange.Min), ParentSpace.GetT(MappingRange.Max));
                    break;

                case EMappingRangeSpace::Parent:
                    MappingRange = FInterval1f(ParentSpace.Interpolate(MappingRange.Min), ParentSpace.Interpolate(MappingRange.Max));
                    break;

                default:
                    break;
            }

            MappingRangeSpace = InMappingRangeSpace;
        }

        EMappingRangeSpace MappingRangeSpace = EMappingRangeSpace::Normalized;
        FInterval1f MappingRange = FInterval1f(0.f, 1.f);
        TUniquePtr<ISplineInterface> Spline;
    };

public:
	
    using SplineType = typename TSplineWrapper<SPLINETYPE>::SplineType;
	typedef typename TSplineWrapper<SPLINETYPE>::ValueType ValueType;
	
	static const FSplineTypeId::IdType& GetStaticTypeId()
	{
		static const FSplineTypeId::IdType CachedTypeId =
		FSplineTypeId::GenerateTypeId(*GetSplineTypeName(),
			*TSplineValueTypeTraits<ValueType>::Name);
		return CachedTypeId;
	}
	
	virtual FSplineTypeId::IdType GetTypeId() const override
	{
		return GetStaticTypeId();
	}
	
	static FString GetSplineTypeName()
	{
		return FString::Printf(TEXT("MultiSpline.%s"), *SplineType::GetSplineTypeName());
	}
	
	virtual FString GetImplementationName() const override
	{
		return GetSplineTypeName();
	}
	
    SplineType& GetSpline() { return TSplineWrapper<SPLINETYPE>::InternalSpline; }
    const SplineType& GetSpline() const { return TSplineWrapper<SPLINETYPE>::InternalSpline; }

	virtual bool operator==(const TMultiSpline<SplineType>& Other) const
    {
    	if (GetSpline() != Other.GetSpline())
    	{
    		return false;
    	}

    	TArray<FName> AttributeNames = GetAllAttributeChannelNames();
    	TArray<FName> OtherAttributeNames = Other.GetAllAttributeChannelNames();

    	if (AttributeNames.Num() != OtherAttributeNames.Num())
    	{
    		return false;
    	}
    	
    	for (FName& Name : AttributeNames)
    	{
    		if (!OtherAttributeNames.Contains(Name))
    		{
    			return false;
    		}

			const ISplineInterface* Child = Children[Name].Spline.Get();
    		const ISplineInterface* OtherChild = Other.Children[Name].Spline.Get();

    		if (!Child || !OtherChild || !Child->IsEqual(OtherChild))
    		{
    			return false;
    		}
    	}
    	
	    return true;
    }
	
	virtual void Clear() override
    {
		TSplineWrapper<SPLINETYPE>::Clear();

		for (const FName& Name : GetAllAttributeChannelNames())
		{
			ClearAttributeChannel(Name);
		}
    }

	virtual bool IsEqual(const ISplineInterface* OtherSpline) const override
    {
    	if (OtherSpline->GetTypeId() == GetTypeId())
    	{
    		const TMultiSpline* Other = static_cast<const TMultiSpline*>(OtherSpline);
    		return operator==(*Other);
    	}
    	
    	return false;
    }

	virtual bool Serialize(FArchive& Ar) override
	{
		if (!TSplineWrapper<SPLINETYPE>::Serialize(Ar))
		{
			return false;
		}

		Ar << Children;

		return true;
	}
	
	/**
	 * Gets a strongly-typed attribute channel with the specific implementation type
	 * 
	 * @tparam ImplType - The specific spline implementation type to get
	 * @param Name - Name of the attribute channel
	 * @return Pointer to the channel implementation, or nullptr if not found or can't be cast
	 */
    template <typename ImplType>
    ImplType* GetTypedAttributeChannel(FName Name) const
    {
        if (const FChildSpline* ExistingChannel = Children.Find(Name))
        {
            if (ExistingChannel->Spline.IsValid())
            {
            	// Get implementation names for comparison
            	const FString RequestedImplName = ImplType::GetSplineTypeName();
            	const FString ExistingImplName = ExistingChannel->Spline->GetImplementationName();
            
            	// Check implementation type compatibility
            	if (RequestedImplName == ExistingImplName)
            	{
            		// Safe to cast if implementation types match
            		return static_cast<ImplType*>(ExistingChannel->Spline.Get());
            	}

            	// Log warning about implementation type mismatch
            	UE_LOGF(LogSpline, Warning, "Failed to fetch attribute channel '%ls': Channel exists with implementation type '%ls', but requested type is '%ls'.",
					   *Name.ToString(), *ExistingImplName, *RequestedImplName);
            }
        }
        return nullptr;
    }

    /**
     * Gets an attribute channel interface of the specified type
     * @param Name Name of the attribute channel
     * @return Pointer to the channel interface, or nullptr if not found
     */
    template <typename AttrType>
    TSplineInterface<AttrType>* GetAttributeChannel(FName Name) const
    {
        if (const FChildSpline* ExistingChannel = Children.Find(Name))
        {
            if (ExistingChannel->Spline.IsValid())
		    {
			    const FString TypeName = TSplineValueTypeTraits<AttrType>::Name;
                const FString ExistingTypeName = ExistingChannel->Spline->GetValueTypeName();
			    
			    if (TypeName == ExistingTypeName)
			    {
                        return static_cast<TSplineInterface<AttrType>*>(ExistingChannel->Spline.Get());
			    }

			    // Channel exists but with a different type
			    UE_LOGF(LogSpline, Warning, "Failed to fetch attribute channel '%ls': Channel already exists with type '%ls', but requested type is '%ls'.",
					    *Name.ToString(), *ExistingTypeName, *TypeName);
		    }
        }

		return nullptr;
    }

	/**
	 * Gets all attribute channel names present in this spline
	 * @return Array of channel names
	 */
	TArray<FName> GetAllAttributeChannelNames() const
	{
	    TArray<FName> ChannelNames;
	    Children.GetKeys(ChannelNames);
	    return ChannelNames;
	}

	/**
	 * Gets names of attribute channels with a specific value type
	 * 
	 * @tparam AttrType - The value type to filter by (e.g., float, FVector)
	 * @return Array of channel names with the specified value type
	 */
	template <typename AttrType>
	TArray<FName> GetAttributeChannelNamesOfType() const
	{
	    TArray<FName> ChannelNames;
	    const FName TypeName = TSplineValueTypeTraits<AttrType>::Name;
	    
	    for (auto& Pair : Children)
	    {
	        if (Pair.Value.Spline.IsValid() && 
	            Pair.Value.Spline->GetValueTypeName() == TypeName)
	        {
	            ChannelNames.Add(Pair.Key);
	        }
	    }
	    
	    return ChannelNames;
	}

	/**
	 * Removes an attribute channel completely
	 * 
	 * @param Name - Name of the channel to remove
	 * @return True if the channel was found and removed
	 */
	bool RemoveAttributeChannel(const FName& Name)
	{
	    return Children.Remove(Name) > 0;
	}

	/**
	 * Clones an attribute channel to a new name
	 * 
	 * @tparam AttrType - The value type of the attribute channel
	 * @param SourceName - Name of the source channel to clone
	 * @param DestName - Name for the new cloned channel
	 * @return True if the channel was successfully cloned
	 */
	template <typename AttrType>
	bool CloneAttributeChannel(const FName& SourceName, const FName& DestName)
	{
	    // Check if destination already exists
	    if (Children.Contains(DestName))
	    {
	        UE_LOGF(LogSpline, Warning, "Cannot clone channel: Destination channel '%ls' already exists.", 
	               *DestName.ToString());
	        return false;
	    }
	    
	    // Get source channel
	    TSplineInterface<AttrType>* SourceChannel = GetAttributeChannel<AttrType>(SourceName);
	    if (!SourceChannel)
	    {
	        UE_LOGF(LogSpline, Warning, "Cannot clone channel: Source channel '%ls' not found or type mismatch.", 
	               *SourceName.ToString());
	        return false;
	    }
	    
	    // Get implementation information from source
	    const FString ImplName = SourceChannel->GetImplementationName();
	    
	    // Create new destination channel with same implementation
	    if (!CreateAttributeChannelInternal<AttrType>(DestName, ImplName))
	    {
	        UE_LOGF(LogSpline, Error, "Failed to create destination channel '%ls'.", 
	               *DestName.ToString());
	        return false;
	    }
	    
	    // Clone the source channel's data to the destination channel
	    TSplineInterface<AttrType>* DestChannel = GetAttributeChannel<AttrType>(DestName);
	    if (!DestChannel)
	    {
	        // This should not happen, but just in case
	        UE_LOGF(LogSpline, Error, "Internal error: Unable to retrieve just-created channel '%ls'.", 
	               *DestName.ToString());
	        return false;
	    }
	    
	    // Clone the source to the destination
	    // We need to use the ISplineInterface::Clone() approach and move the clone into our structure
	    TUniquePtr<ISplineInterface> ClonedSpline = SourceChannel->Clone();
	    
	    // Find and update the destination child's spline pointer
	    FChildSpline& DestChild = Children[DestName];
	    DestChild.Spline = MoveTemp(ClonedSpline);
	    
	    // Copy mapping settings from source
	    const FChildSpline& SourceChild = *Children.Find(SourceName);
	    DestChild.MappingRange = SourceChild.MappingRange;
	    DestChild.MappingRangeSpace = SourceChild.MappingRangeSpace;
	    
	    return true;
	}

    /**
     * Maps a parameter from parent space to child space
     * @param Name Name of the child spline
     * @param Parameter Parameter in parent space
     * @return Parameter mapped to child space
     */
    float MapParameterToChildSpace(FName Name, float Parameter) const
    {
        if (const FChildSpline* Child = Children.Find(Name))
        {
            return Child->MapParameterToChildSpace(Parameter, GetSpline().GetParameterSpace());
        }

        return 0.f;
    }
    
    /**
     * Maps a parameter from child space to parent space
     * @param Name Name of the child spline
     * @param Parameter Parameter in child space
     * @return Parameter mapped to parent space
     */
    float MapParameterFromChildSpace(FName Name, float Parameter) const
    {
        if (const FChildSpline* Child = Children.Find(Name))
        {
            return Child->MapParameterFromChildSpace(Parameter, GetSpline().GetParameterSpace());
        }
        
        return 0.f;
    }

    /**
     * Map a child spline's domain into the primary spline's domain.
     * @param Name Name of the child spline
     * @return The domain of the child spline in parent space.
     */
    FInterval1f GetMappedChildSpace(FName Name) const
    {
		if (const FChildSpline* Child = Children.Find(Name))
		{
			return Child->GetParentSpaceRange(GetSpline().GetParameterSpace());
		}
    
		return FInterval1f::Empty();
    }

	/**
     * Creates an attribute channel and returns the typed implementation
     * 
     * @tparam ImplType - The specific spline implementation type to create
     * @param Name - Name of the channel to create
     * @return Pointer to the created attribute channel, or nullptr if creation failed
     */
    template <typename ImplType>
    ImplType* CreateAttributeChannel(const FName& Name)
    {
        // Extract the value type from the implementation type
        using AttrType = typename ImplType::ValueType;

    	// Check if the type is already registered
    	uint32 TypeId = FSplineTypeRegistry::GetTypeId(ImplType::GetSplineTypeName(), 
													  TSplineValueTypeTraits<AttrType>::Name);
    	// If not registered, register it manually
    	if (TypeId == 0)
    	{
    		// Get type information for registration
    		TypeId = ImplType::GetStaticTypeId();
    		FString ImplName = ImplType::GetSplineTypeName();
    		FString ValueName = TSplineValueTypeTraits<AttrType>::Name;
        
    		// Register with a factory function that creates this specific type
    		bool bSuccess = FSplineTypeRegistry::RegisterType(
				TypeId,
				ImplName,
				ValueName,
				[]() { return MakeUnique<ImplType>(); }
			);
        
    		if (!bSuccess)
    		{
    			UE_LOGF(LogSpline, Error, "Failed to register spline type '%ls' with value type '%ls'",
					  *ImplName, *ValueName);
    			return nullptr;
    		}
    	}
    	
        // Create the channel using the existing name-based method
        CreateAttributeChannelInternal<AttrType>(Name, ImplType::GetSplineTypeName());
    
        // Return the typed implementation
        return GetTypedAttributeChannel<ImplType>(Name);
    }

	template <typename AttrSplineType>
	TArray<FName> GetAttributeChannelNamesBySplineType() const
    {
		TArray<FName> Names;

		for (const TPair<FName, FChildSpline>& Pair : Children)
		{
			if (Pair.Value.Spline->GetTypeId() == AttrSplineType::GetStaticTypeId())
			{
				Names.Add(Pair.Key);
			}
		}

		return Names;
    }

	template <typename AttrType>
    TArray<FName> GetAttributeChannelNamesByValueType() const
	{
		TArray<FName> Names;

		for (const TPair<FName, FChildSpline>& Pair : Children)
		{
			if (Pair.Value.Spline->GetValueTypeName() == TSplineValueTypeTraits<AttrType>::Name)
			{
				Names.Add(Pair.Key);
			}
		}

		return Names;
    }
	
    /**
     * Clears all values from a specific attribute channel
     * @param Name - Name of the attribute channel to clear
     * @return true if the channel was found and cleared, false otherwise
     */
    bool ClearAttributeChannel(const FName& Name)
    {
        if (const FChildSpline* Channel = Children.Find(Name))
        {
            Channel->Spline->Clear();
            return true;
        }
        return false;
    }

    /**
     * Checks for an attribute channel called Name
     * @param Name - Name of the attribute channel to find
     * @return true if an attribute channel called Name exists.
     */
    bool HasAttributeChannel(const FName& Name) const
    {
		return Children.Contains(Name);
    }
    
    /**
     * Gets a value from an attribute channel at the specified parameter
     * @param Name Name of the attribute channel
     * @param Parameter Parameter in parent space
     * @return Attribute value at the parameter
     */
    template <typename AttrType>
    AttrType EvaluateAttribute(const FName& Name, float Parameter) const
    {
        TSplineInterface<AttrType>* Channel = GetAttributeChannel<AttrType>(Name);
        if (!Channel)
        {
            return AttrType();
        }

        // Map parameter to child space
        float ChildParameter = MapParameterToChildSpace(Name, Parameter);
        
        // Evaluate in child space
        return Channel->Evaluate(ChildParameter);
    }

    /**
     * Sets the mapping range for an attribute channel
     * @param Name Name of the attribute channel
     * @param Range New mapping range
     * @param RangeSpace Space of the range (normalized or parent)
     * @return True if successful
     */
    bool SetAttributeChannelRange(FName Name, const FInterval1f& Range, EMappingRangeSpace RangeSpace)
    {
        FChildSpline* Child = Children.Find(Name);
        if (!Child)
        {
            return false;
        }
   
        Child->MappingRange = Range;
        Child->MappingRangeSpace = RangeSpace;
        return true;
    }

	/**
	 * Returns the mapping range for an attribute channel
	 * @param Name Name of the attribute channel
	 * @return The mapping range
	 */
	FInterval1f GetAttributeChannelRange(FName Name)
	{
		FChildSpline* Child = Children.Find(Name);
		if (!Child)
		{
			return FInterval1f::Empty();
		}

		return Child->MappingRange;
	}

	/**
	* Returns the mapping range space for an attribute channel
	* @param Name Name of the attribute channel
	* @return The mapping range space. Always returns normalized if the child spline is not found.
	*/
	EMappingRangeSpace GetAttributeChannelRangeSpace(FName Name)
	{
		FChildSpline* Child = Children.Find(Name);
		if (!Child)
		{
			return EMappingRangeSpace::Normalized;
		}

		return Child->MappingRangeSpace;
	}
    
private:

	/**
     * Helper method that creates an attribute channel with the specified type and implementation name
     * @param Name Name of the channel to create
     * @param ImplName Name of the implementation to use, defaults to "BSpline3"
     * @return True if channel was created successfully
     */
    template <typename AttrType>
    bool CreateAttributeChannelInternal(const FName& Name, const FString& ImplName)
    {
        // Check if channel already exists
        if (Children.Contains(Name))
        {
            UE_LOGF(LogSpline, Warning, "Failed to create attribute channel '%ls': Channel already exists.", *Name.ToString());
            return false;
        }

        // Look up the type ID
        uint32 TypeId = FSplineTypeRegistry::GetTypeId(ImplName, TSplineValueTypeTraits<AttrType>::Name);
        if (TypeId == 0)
        {
            UE_LOGF(LogSpline, Error, "Failed to find type ID for implementation '%ls' with value type '%ls'.",
                   *ImplName, *TSplineValueTypeTraits<AttrType>::Name);
            return false;
        }

        // Create child spline by type ID
        TUniquePtr<ISplineInterface> NewSpline = FSplineTypeRegistry::CreateSpline(TypeId);
        if (!NewSpline)
        {
                UE_LOGF(LogSpline, Error, "Failed to create spline of type '%ls'/'%ls' (ID: 0x%08X) for attribute channel '%ls'.",
               *ImplName, *TSplineValueTypeTraits<AttrType>::Name, TypeId, *Name.ToString());
            return false;
        }

        // Verify that the created spline is of the expected value type
        if (NewSpline->GetValueTypeName() != TSplineValueTypeTraits<AttrType>::Name)
        {
            UE_LOGF(LogSpline, Error, "Created spline has incorrect value type. Expected '%ls', got '%ls'.",
                   *TSplineValueTypeTraits<AttrType>::Name, *NewSpline->GetValueTypeName());
            return false;
        }

        // Add to children map
        FChildSpline& Child = Children.Add(Name);
        Child.Spline = MoveTemp(NewSpline);
        Child.MappingRangeSpace = EMappingRangeSpace::Normalized;
        Child.MappingRange = FInterval1f(0.0f, 1.0f);

        return true;
    }
	
    TMap<FName, FChildSpline> Children;
};

} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE