// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

/**
 * Vector type policies based on dimension and numeric type
 */
template<typename T, int32 D> 
struct TVectorPolicy
{
	// Default case - invalid
	using Type = void;
};

// Specializations for 2D vectors
template<> struct TVectorPolicy<float, 2> { using Type = FVector2f; };
template<> struct TVectorPolicy<double, 2> { using Type = FVector2D; };
template<> struct TVectorPolicy<int32, 2> { using Type = FIntPoint; };

// Specializations for 3D vectors  
template<> struct TVectorPolicy<float, 3> { using Type = FVector3f; };
template<> struct TVectorPolicy<double, 3> { using Type = FVector; };
template<> struct TVectorPolicy<int32, 3> { using Type = FIntVector; };

// Specializations for 4D vectors
template<> struct TVectorPolicy<float, 4> { using Type = FVector4f; };
template<> struct TVectorPolicy<double, 4> { using Type = FVector4d; };
template<> struct TVectorPolicy<int32, 4> { using Type = FIntVector4; };

/**
 * Traits to determine if a type is a valid numeric type for vectors
 */
template<typename T>
struct TIsValidVectorType
{
	static constexpr bool Value = 
		std::is_same<T, float>::value ||
		std::is_same<T, double>::value ||
		std::is_same<T, int32>::value;
};

template<typename T, int32 D>
struct TChooseVectorType
{
	static_assert(TIsValidVectorType<T>::Value, 
		 "Vector component type must be float, double, or int32");
	static_assert(D >= 2 && D <= 4,
		"Vector dimension must be 2, 3, or 4");

	using Type = typename TVectorPolicy<T, D>::Type;
	static_assert(!std::is_same<Type, void>::value,
		"Invalid vector type combination");
};

// Trait to detect if a type T has an Equals(const T&) method.
template<typename T, typename = void>
struct THasEqualsMethod : std::false_type {};

template<typename T>
struct THasEqualsMethod<T, std::void_t<decltype(std::declval<T>().Equals(std::declval<T>()))>>
    : std::true_type {};

// Trait to detect if type T has a static EqualTo(const T&, const T&) method.
template<typename T, typename = void>
struct THasEqualToMethod : std::false_type {};

template<typename T>
struct THasEqualToMethod<T, std::void_t<decltype(T::EqualTo(std::declval<T>(), std::declval<T>()))>>
	: std::true_type {};

/** Compile-time check for Serialize() method */
template<typename T, typename = void>
struct THasSerializeMethod : std::false_type {};

template<typename T>
struct THasSerializeMethod<T, 
	std::void_t<decltype(std::declval<T>().Serialize(std::declval<FArchive&>()))>
> : std::true_type {};

/** Compile-time check for FArchive << operator */
template<typename T, typename = void>
struct THasArchiveOperator : std::false_type {};

template<typename T>
struct THasArchiveOperator<T, 
	std::void_t<decltype(std::declval<FArchive&>() << std::declval<T&>())>
> : std::true_type {};

/**
 * Trait detectors for various value type operations
 * These traits are used to determine if a type supports certain operations
 */

// Detector for Size() method
template<typename T, typename = void>
struct THasSizeMethod : std::false_type {};

template<typename T>
struct THasSizeMethod<T, std::void_t<decltype(std::declval<T>().Size())>>
    : std::true_type {};

// Detector for SizeSquared() method
template<typename T, typename = void>
struct THasSizeSquaredMethod : std::false_type {};

template<typename T>
struct THasSizeSquaredMethod<T, std::void_t<decltype(std::declval<T>().SizeSquared())>>
    : std::true_type {};

// Detector for Dot() method
template<typename T, typename = void>
struct THasDotMethod : std::false_type {};

template<typename T>
struct THasDotMethod<T, std::void_t<decltype(std::declval<T>().Dot(std::declval<T>()))>>
    : std::true_type {};

// Detector for GetSafeNormal() method
template<typename T, typename = void>
struct THasGetSafeNormalMethod : std::false_type {};

template<typename T>
struct THasGetSafeNormalMethod<T, std::void_t<decltype(std::declval<T>().GetSafeNormal())>>
    : std::true_type {};

// Detector for operator- (subtraction)
template<typename T, typename = void>
struct THasSubtractionOperator : std::false_type {};

template<typename T>
struct THasSubtractionOperator<T, std::void_t<decltype(std::declval<T>() - std::declval<T>())>>
    : std::true_type {};

// Detector for operator+ (addition)
template<typename T, typename = void>
struct THasAdditionOperator : std::false_type {};

template<typename T>
struct THasAdditionOperator<T, std::void_t<decltype(std::declval<T>() + std::declval<T>())>>
    : std::true_type {};

// Detector for operator* (scalar multiplication)
template<typename T, typename = void>
struct THasScalarMultiplicationOperator : std::false_type {};

template<typename T>
struct THasScalarMultiplicationOperator<T, std::void_t<decltype(std::declval<T>() * std::declval<float>())>>
    : std::true_type {};

// Detector for ZeroVector static member (for proper Zero value)
template<typename T, typename = void>
struct THasZeroVectorMember : std::false_type {};

template<typename T>
struct THasZeroVectorMember<T, std::void_t<decltype(T::ZeroVector)>>
    : std::true_type {};
	

} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE