// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "BSpline.h"
#include "Misc/ScopeExit.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

namespace Private
{
	GEOMETRYCORE_API bool UseFindNearestUseSplineCulling();
}

template<typename ValueType> class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TPolyBezierSpline;
	
/* Augments B-Spline interface */
template<typename InValueType>
class TPolyBezierSpline :
	public TBSpline<InValueType, 3>,
	private TSelfRegisteringSpline<TPolyBezierSpline<InValueType>, InValueType>
{

public:
	
	using ValueType = InValueType;
	using Base = TBSpline<ValueType, 3>;
	using Base::Degree;

	using FWindow = typename Base::FWindow;
	using FWindowStore = typename Base::FWindowStore;
	using FFindWindowOverride = void(*)(FWindow& /*InOut*/,
								int32 /*SegmentIndex*/,
								const FInterval1f& /*SegmentRange*/,
								FWindowStore& /*Scratch*/);
	
private:

	using typename Base::FKnotSearchCache;
	using typename Base::FValidKnotSearchParams;
	using Base::ApplyClampedKnotsMultiplicity;

public:

	// Generate compile-time type ID for PolyBezier
	DECLARE_SPLINE_TYPE_ID(
		TEXT("PolyBezier"),
		*TSplineValueTypeTraits<ValueType>::Name
	);
	
    TPolyBezierSpline() = default;
    virtual ~TPolyBezierSpline() override = default;
	
	/**
	 * TPolyBezierSpline consists of an arbitrary number of anchor points.
	 * Adjacent anchor points implicitly define a cubic bezier segment.
	 * The cubic bezier segment defined by points A and B is <A_Anchor, A_OutHandle, B_InHandle, B_Anchor>.
	 */
	struct FAnchor
	{
		ValueType InHandle;
		ValueType Anchor;
		ValueType OutHandle;
	};

	TPolyBezierSpline(
		const FAnchor& Start,
		const FAnchor& End,
		EParameterizationPolicy Parameterization = EParameterizationPolicy::Uniform)
	: Base()
	{
		AppendAnchor(Start, Parameterization);
		AppendAnchor(End, Parameterization);
	}

	// Static factory methods for common shapes    
    // Create a straight line between two points
    static TPolyBezierSpline<ValueType> CreateLine(
        const ValueType& Start, 
        const ValueType& End)
    {
        // Calculate control points for a straight line
        // For a straight line, the control points should be evenly spaced
        ValueType P0 = Start;
        ValueType P3 = End;
        ValueType Direction = (End - Start);
        ValueType P1 = Start + Direction * 0.33f;
        ValueType P2 = Start + Direction * 0.66f;
        
		TPolyBezierSpline<ValueType> Result(
			FAnchor{P0, P0, P1},
			FAnchor{P2, P3, P3},
			EParameterizationPolicy::Uniform
		);

        return Result;
    }

    // Create a circular arc with given center, radius, and angle range
    static TPolyBezierSpline<ValueType> CreateCircleArc(
        const ValueType& Center,
        float Radius,
        float StartAngle,
        float EndAngle,
        int32 NumSegments = 4)
    {
		// Ensure angles are in the right order
		if (EndAngle < StartAngle)
		{
			float Temp = StartAngle;
			StartAngle = EndAngle;
			EndAngle = Temp;
		}
           
		// Ensure reasonable segment count
		NumSegments = FMath::Max(1, NumSegments);
           
		// Calculate angle per segment
		float AngleSpan = EndAngle - StartAngle;
		float AnglePerSegment = AngleSpan / static_cast<float>(NumSegments);
           
		// For large angles per segment, increase segment count for better approximation
		if (AnglePerSegment > UE_PI/4)
		{
			NumSegments = FMath::CeilToInt(AngleSpan / (UE_PI/4));
			AnglePerSegment = AngleSpan / static_cast<float>(NumSegments);
		}
           
		// Pre-compute all points along the arc for exact positioning
		TArray<ValueType> Points;
		Points.SetNum(NumSegments + 1);
           
		for (int32 i = 0; i <= NumSegments; i++)
		{
			float Angle = StartAngle + (static_cast<float>(i) * AnglePerSegment);
			Points[i] = Center + ValueType(
				Radius * FMath::Cos(Angle),
				Radius * FMath::Sin(Angle),
				0);
		}
           
		// Calculate first segment with precise tangents
		ValueType P0 = Points[0];
		ValueType P3 = Points[1];
           
		// Get normalized direction vectors from center to points
		ValueType Dir0 = Math::GetSafeNormal(P0 - Center);
		ValueType Dir3 = Math::GetSafeNormal(P3 - Center);
           
		// Calculate perpendicular vectors (normalized)
		ValueType Perp0 = ValueType(-Dir0.Y, Dir0.X, 0);
		ValueType Perp3 = ValueType(-Dir3.Y, Dir3.X, 0);
           
		// Calculate precise tangent scale
		float TangentScale = Radius * (4.0f / 3.0f) * 
									FMath::Tan(AnglePerSegment / 4.0f);
           
		// Create control points with exact perpendicular tangents
		ValueType P1 = P0 + Perp0 * TangentScale;
		ValueType P2 = P3 - Perp3 * TangentScale;

		TPolyBezierSpline<ValueType> Result(
			FAnchor{P0, P0, P1},
			FAnchor{P2, P3, P3},
			EParameterizationPolicy::Uniform
		);
           
		// Add remaining segments with carefully calculated control points
		for (int32 i = 1; i < NumSegments; i++)
		{
			P0 = Points[i];
			P3 = Points[i+1];
               
			// Calculate exact perpendicular vectors for this segment
			Dir0 = Math::GetSafeNormal(P0 - Center);
			Dir3 = Math::GetSafeNormal(P3 - Center);
			Perp0 = ValueType(-Dir0.Y, Dir0.X, 0);
			Perp3 = ValueType(-Dir3.Y, Dir3.X, 0);
               
			// Create control points with exact perpendicular tangents
			P1 = P0 + Perp0 * TangentScale;
			P2 = P3 - Perp3 * TangentScale;
               
			const int32 NewAnchorIdx = Result.AppendAnchor(FAnchor { P2, P3, P3 }, EParameterizationPolicy::Uniform);
			const int32 OldAnchorIdx = NewAnchorIdx - 1;
			Result.SetHandleOutOfAnchor(OldAnchorIdx, P1);
		}
           
		return Result;
    }
    
    // Create a complete circle
    static TPolyBezierSpline<ValueType> CreateCircle(
        const ValueType& Center,
        float Radius,
        int32 NumSegments = 4)
    {
		// Ensure reasonable segment count
		NumSegments = FMath::Max(4, NumSegments);
    
		// Create a full circle arc (0 to 2π)
		TPolyBezierSpline<ValueType> Result = 
			CreateCircleArc(Center, Radius, 0.0f, 2.0f * UE_PI, NumSegments);
    
		// Set as closed loop
		Result.SetClosedLoop(true);
		// Apply consistent parameterization
		Result.Reparameterize(EParameterizationPolicy::Centripetal);
		return Result;
    }
    
    // Create an ellipse with X and Y radii
    static TPolyBezierSpline<ValueType> CreateEllipse(
        const ValueType& Center,
        float RadiusX,
        float RadiusY,
        int32 NumSegments = 4)
    {
        // Ensure at least 4 segments for good quality
	    if (NumSegments < 4) NumSegments = 4;
	    
	    // Calculate angle per segment
	    const float AnglePerSegment = 2.0f * UE_PI / static_cast<float>(NumSegments);
	    
	    // Start with first point at angle 0
	    float CurrentAngle = 0.0f;
	    ValueType P0 = Center + ValueType(RadiusX * FMath::Cos(CurrentAngle), 
	                                      RadiusY * FMath::Sin(CurrentAngle), 0);
	    
	    // Calculate next point
	    CurrentAngle += AnglePerSegment;
	    ValueType P3 = Center + ValueType(RadiusX * FMath::Cos(CurrentAngle), 
	                                      RadiusY * FMath::Sin(CurrentAngle), 0);
	    
	    // Calculate correct tangent scale for this angle
	    const float TangentScale = (4.0f/3.0f) * FMath::Tan(AnglePerSegment / 4.0f);
	    
	    // Calculate tangent vectors (scaled appropriately for ellipse)
	    ValueType Tangent0 = ValueType(-RadiusX * FMath::Sin(0.0f), 
	                                  RadiusY * FMath::Cos(0.0f), 0) * TangentScale;
	    ValueType Tangent3 = ValueType(-RadiusX * FMath::Sin(CurrentAngle), 
	                                  RadiusY * FMath::Cos(CurrentAngle), 0) * TangentScale;
	    
	    ValueType P1 = P0 + Tangent0;
	    ValueType P2 = P3 - Tangent3;
	    
	    // Create the spline with the first segment
		TPolyBezierSpline<ValueType> Result(
			FAnchor{P0, P0, P1},
			FAnchor{P2, P3, P3},
			EParameterizationPolicy::Uniform
		);
	    
	    // Add remaining segments
	    for (int32 i = 1; i < NumSegments; ++i)
	    {
	        P0 = P3; // Start from previous endpoint
	        Tangent0 = Tangent3; // Reuse previous tangent
	        
	        // Calculate next endpoint
	        CurrentAngle += AnglePerSegment;
	        P3 = Center + ValueType(RadiusX * FMath::Cos(CurrentAngle), 
	                               RadiusY * FMath::Sin(CurrentAngle), 0);
	        
	        // Calculate tangent at next point
	        Tangent3 = ValueType(-RadiusX * FMath::Sin(CurrentAngle), 
	                            RadiusY * FMath::Cos(CurrentAngle), 0) * TangentScale;
	        
	        // Calculate control points
	        P1 = P0 + Tangent0;
	        P2 = P3 - Tangent3;
	        
	        // Add the segment
			const int32 NewAnchorIdx = Result.AppendAnchor(P2, P3, P3);
			const int32 OldAnchorIdx = NewAnchorIdx - 1;
			Result.SetHandleOutOfAnchor(OldAnchorIdx, P1);
	    }
	    
	    // Set as closed loop
	    Result.SetClosedLoop(true);
	            
        // Apply consistent parameterization
        Result.Reparameterize(EParameterizationPolicy::Centripetal);
        
        return Result;
    }
	
	virtual bool Serialize(FArchive& Ar) override
	{
		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		if (!Base::Serialize(Ar))
		{
			return false;
		}

		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PolyBezierSplineAnchorTripleLayout)
			{
				// At this point Values contains the legacy 4-point-per-seg layout.
				// Convert it in-place to the new anchor triple layout.
				MigrateLegacyBezierLayout();
			}
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SplineComponentReparameterizeOnLoad)
		{
			Ar << LoopKnotDelta;
		}

		// backup fix in case there are assets with the anchor layout missing pair knots
		if (GetNumAnchorPoints() > 0 && Base::PairKnots.IsEmpty())
		{
			Reparameterize();
		}

		return true;
	}

	// ISplineInterface Implementation
	virtual void Clear() override { Base::Clear(); }
	virtual TUniquePtr<ISplineInterface> Clone() const override
    {
        TUniquePtr<TPolyBezierSpline> Clone = MakeUnique<TPolyBezierSpline>();
        
        // Copy base class members
        Clone->Values = Base::Values;
        Clone->PairKnots = Base::PairKnots;
        Clone->bIsClosedLoop = Base::bIsClosedLoop;
        
        // Copy infinity modes
        Clone->PreInfinityMode = Base::PreInfinityMode;
        Clone->PostInfinityMode = Base::PostInfinityMode;
        
        return Clone;
    }
	
    float FindNearestOnSegment(const ValueType& Point, int32 SegmentIndex, float& OutSquaredDistance) const
    {
    	if (!IsValidSegmentIndex(SegmentIndex))
    	{
    		OutSquaredDistance = TNumericLimits<float>::Max();
    		return 0.0f;
    	}

		// Get Bezier control points for this segment
		TStaticArray<ValueType, 4> Coeffs = {};

		const ValueType* P0Ptr = nullptr;
		const ValueType* P1Ptr = nullptr;
		const ValueType* P2Ptr = nullptr;
		const ValueType* P3Ptr = nullptr;

		if (!IsClosingSegment(SegmentIndex))
		{
			P0Ptr = &Base::GetValue(P0Idx(SegmentIndex)); // A_s
			P1Ptr = &Base::GetValue(P1Idx(SegmentIndex)); // Out_s
			P2Ptr = &Base::GetValue(P2Idx(SegmentIndex)); // In_{s+1}
			P3Ptr = &Base::GetValue(P3Idx(SegmentIndex)); // A_{s+1}
		}
		else
		{
			const int32 NumAnchors = GetNumAnchorPoints();

			// Closing segment: A_last → A_0
			P0Ptr = &Base::GetValue(Close_P0Idx(NumAnchors)); // A_last
			P1Ptr = &Base::GetValue(Close_P1Idx(NumAnchors)); // Out_last
			P2Ptr = &Base::GetValue(Close_P2Idx());           // In_0
			P3Ptr = &Base::GetValue(Close_P3Idx());           // A_0
		}

		const ValueType& P0 = *P0Ptr;
		const ValueType& P1 = *P1Ptr;
		const ValueType& P2 = *P2Ptr;
		const ValueType& P3 = *P3Ptr;

    	// Setup coefficients in standard Bezier polynomial form relative to test point
    	Coeffs[0] = P0 - Point;                    // constant term
    	Coeffs[1] = (P1 - P0) * 3;                 // linear coefficient
    	Coeffs[2] = (P2 - P1*2 + P0) * 3;          // quadratic coefficient  
    	Coeffs[3] = P3 - P2*3 + P1*3 - P0;         // cubic coefficient

    	const float LocalT = Math::FindNearestPoint_Cubic(MakeArrayView(Coeffs), 0.0f, 1.0f, OutSquaredDistance);
    	return MapLocalSegmentParameterToGlobal(SegmentIndex, LocalT);
    }

	float PointDistanceSqToSegmentBBox(const ValueType &Point, int32 SegmentIndex) const
	{
		// Decide whether this is the synthetic closing segment

		// Get the control points
		const ValueType* P0Ptr = nullptr;
		const ValueType* P1Ptr = nullptr;
		const ValueType* P2Ptr = nullptr;
		const ValueType* P3Ptr = nullptr;

		if (!IsClosingSegment(SegmentIndex))
		{
			P0Ptr = &Base::GetValue(P0Idx(SegmentIndex));
			P1Ptr = &Base::GetValue(P1Idx(SegmentIndex));
			P2Ptr = &Base::GetValue(P2Idx(SegmentIndex));
			P3Ptr = &Base::GetValue(P3Idx(SegmentIndex));
		}
		else
		{
			const int32 NumAnchors = GetNumAnchorPoints();

			P0Ptr = &Base::GetValue(Close_P0Idx(NumAnchors)); // A_last
			P1Ptr = &Base::GetValue(Close_P1Idx(NumAnchors)); // Out_last
			P2Ptr = &Base::GetValue(Close_P2Idx());           // In_0
			P3Ptr = &Base::GetValue(Close_P3Idx());           // A_0
		}

		const ValueType& P0 = *P0Ptr;
		const ValueType& P1 = *P1Ptr;
		const ValueType& P2 = *P2Ptr;
		const ValueType& P3 = *P3Ptr;

		//	For cubics, convex hull is defined by control points, so use them to compute BBox. 
		ValueType BBoxMin = P0.ComponentMin(P1);
		ValueType BBoxMax = P0.ComponentMax(P1);

		BBoxMin = BBoxMin.ComponentMin(P2);
		BBoxMax = BBoxMax.ComponentMax(P2);

		BBoxMin = BBoxMin.ComponentMin(P3);
		BBoxMax = BBoxMax.ComponentMax(P3);

		// Get point on Bbox nearest to input point and compute distance sq
		ValueType PtOnBox = BBoxMin.ComponentMax(BBoxMax.ComponentMin(Point));
		return float(Math::SizeSquared(PtOnBox - Point));
	}

	virtual float FindNearest(const ValueType& Point, float& OutSquaredDistance) const override
	{
    	const int32 NumSegments = GetNumberOfSegments();
		float BestDistSq;
		float BestParam = FindNearestOnSegment(Point, 0, BestDistSq);
		
		float SegmentDistSq;
		if (Private::UseFindNearestUseSplineCulling())
		{
			for (int32 i = 1; i < NumSegments; ++i)
			{
				// For cubic spline: compute AABB and distance to point. 
				//	If distance > best result distance then reject segment. Saves a lot of computation (see FindNearestOnSegment)
				// 
				if (PointDistanceSqToSegmentBBox(Point, i) >= BestDistSq)
				{
					continue;
				}

				const float SplineParam = FindNearestOnSegment(Point, i, SegmentDistSq);
				if (SegmentDistSq < BestDistSq)
				{
					BestDistSq = SegmentDistSq;
					BestParam = SplineParam;
				}

				if (FMath::IsNearlyZero(BestDistSq))
				{
					break;
				}
			}
		}
		else
		{
			for (int32 i = 1; i < NumSegments; ++i)
			{
				const float SplineParam = FindNearestOnSegment(Point, i, SegmentDistSq);
				if (SegmentDistSq < BestDistSq)
				{
					BestDistSq = SegmentDistSq;
					BestParam = SplineParam;
				}
			}
		}
    	OutSquaredDistance = BestDistSq;
    	
    	return BestParam;
	}

    /** 
     * Evaluate nth derivative at parameter
     * @tparam Order - The derivative order (0 = position, 1 = first derivative, etc.)
     * @param Parameter - Parameter in spline space
     * @return nth derivative vector
     */
    template<int32 Order>
    ValueType EvaluateDerivative(float Parameter) const
    {
		FWindow Window;
		FWindowStore Scratch;
		FindWindow(Parameter, Window, Scratch);

		// If FindWindow fails, it will return an array of nullptr. InterpolateWindow assumes validity of elements.
		if (!Math::FSplineValidation::IsValidWindow<ValueType, Base::WindowSize>(Window) ||
			!Math::FSplineValidation::IsValidParameter(Parameter))
		{
			return Math::FSplineValidation::GetDefaultValue<ValueType>();
		}
		
        // Order 0 is just regular evaluation
	    if constexpr (Order == 0)
	    {
	        return InterpolateWindow(Window, Parameter);
	    }
    
    	// Convert global parameter to segment information
		float LocalT;
		int32 SegmentIndex = FindIndexForParameter(Parameter, LocalT);

		if (SegmentIndex == INDEX_NONE)
		{
			return Math::FSplineValidation::GetDefaultValue<ValueType>();
		}
    	
		const float SegmentScale = GetSegmentParameterRange(SegmentIndex).Length();
	    
		// Compute the derivative with proper scaling
		return Math::TBezierDerivativeCalculator<ValueType, Order>::Compute(
	        Window, LocalT, SegmentScale);
    }

	/**
	 * MANIPULATION METHODS
	 */
	
	/** Returns the number of Bezier segments in the spline */
	virtual int32 GetNumberOfSegments() const override
	{
		const int32 NumAnchors = GetNumAnchorPoints();
		return HasClosingSegment() ? NumAnchors : FMath::Max(0, NumAnchors - 1);
	}

	/**
	 * Maps a segment index to its parameter range
	 * @param SegmentIndex - Index of the segment (0-based)
	 * @return True if the segment index is valid and mapping succeeded
	 */
	virtual FInterval1f GetSegmentParameterRange(int32 SegmentIndex) const override
	{
		FInterval1f SegmentRange;

		const int32 NumSegments = GetNumberOfSegments();
		if (SegmentIndex < 0 || SegmentIndex >= NumSegments || Base::PairKnots.Num() == 0 || !Base::PairKnots.IsValidIndex(SegmentIndex))
		{
			return SegmentRange;
		}

		SegmentRange.Min = Base::PairKnots[SegmentIndex].Value;

		if (!HasClosingSegment())
		{
			// Open: knots are standard boundaries, so we still use i+1.
			if (SegmentIndex + 1 < Base::PairKnots.Num())
			{
				SegmentRange.Max = Base::PairKnots[SegmentIndex + 1].Value;
			}
			return SegmentRange;
		}

		// Closed: PairKnots holds starts of each segment; the final segment
		// closes the loop at Min + LoopKnotDelta.
		if (SegmentIndex < NumSegments - 1)
		{
			SegmentRange.Max = Base::PairKnots[SegmentIndex + 1].Value;
		}
		else
		{
			SegmentRange.Max = SegmentRange.Min + LoopKnotDelta;
		}

		return SegmentRange;
	}

	// Parameterization methods
	/**
	 * Reparameterizes the spline based on the provided points and mode
	 * @param Mode - New parameterization mode
	 * @param Points - Array of control points to use for reparameterization
	 * @return 
	 */
	virtual void Reparameterize(EParameterizationPolicy Mode = EParameterizationPolicy::Centripetal) override
	{
		const TArray<ValueType>& Points = Base::Values;
		const int32 NumValues = Points.Num();

		GenerateDistanceKnotsForBezier(Mode);
    	
		UE_LOGF(LogSpline, Verbose, "Set knot vector (Reparameterize) - Points: %d, Knots: %d, Mode: %d",
					   NumValues,  Base::PairKnots.Num(), static_cast<int32>(Mode));
		
		Base::PrintKnotVector();
	}
	
	virtual FInterval1f GetParameterSpace() const override
	{
		if (Base::PairKnots.Num() == 0)
		{
			return FInterval1f::Empty();
		}

		const float Min = Base::PairKnots[0].Value;
		const float LastKnot = Base::PairKnots.Last().Value;

		if (!HasClosingSegment())
		{
			return FInterval1f(Min, LastKnot);
		}

		// Closed: closing boundary is last real knot + LoopKnotDelta
		return FInterval1f(Min, LastKnot + LoopKnotDelta);
	}

	/**
	 * Gets the parameter value for a specific anchor point or the virtual closing point.
	 * The virtual closing point exists when there are at least 2 anchor points and the spline is closed. The index is GetNumAnchorPoints().
	 * @param Index - Index of the anchor point
	 * @return Parameter value at the specified point
	 */
	virtual float GetParameter(int32 Index) const override
	{
		const int32 NumAnchors = GetNumAnchorPoints();

		if (NumAnchors <= 0)
		{
			// todo: fail
			return 0.f;
		}

		// Getting the virtual closing knot.
		if (Index == NumAnchors && HasClosingSegment())
		{
			return Base::PairKnots.Last().Value + LoopKnotDelta;
		}

		if (Index < 0 || Index >= NumAnchors || !Base::PairKnots.IsValidIndex(Index))
		{
			// todo: fail
			return 0.f;
		}

		return Base::PairKnots[Index].Value;
	}

	/**
	 * Sets the parameter value for a specific anchor control point or the virtual closing point.
	 * This function will reorder the knot vector to ensure that it is always monotonically increasing.
	 * If the knot vector is reordered, anchor points are reordered such that they are always associated with the same knot.
	 * Setting the parameter of the virtual closing point will not cause any reordering. If reordering would occur, the function fails.
	 * @param Index - Index of the anchor point
	 * @param NewParameter - New parameter value to set
	 * @return The index of the point that was actually updated, or INDEX_NONE if failed
	 */ 
	virtual int32 SetParameter(int32 Index, float NewParameter) override
	{
		const int32 NumAnchors = GetNumAnchorPoints();
		const int32 NumSegments = GetNumberOfSegments();

		// Cannot set parameters for empty splines, fail.
		if (NumAnchors <= 0)
		{
			return INDEX_NONE;
		}

		// Trivial case, no possible reordering or numerical issues.
		if (NumAnchors == 1)
		{
			if (Index == 0)
			{
				Base::PairKnots[Index].Value = NewParameter;
				return 0;
			}
			else
			{
				return INDEX_NONE;
			}
		}

		// Setting the virtual closing knot.
		if (Index == NumAnchors && HasClosingSegment())
		{
			// No reordering when manipulating the closing knot, fail.
			if (NewParameter <= Base::PairKnots.Last().Value)
			{
				return INDEX_NONE;
			}

			LoopKnotDelta = NewParameter - Base::PairKnots.Last().Value;
			return Index;
		}

		// Because we special case the closing point above, we can trivially bounds check here.
		if (Index < 0 || Index >= NumAnchors)
		{
			return INDEX_NONE;
		}

		const int32 SegmentIndex = Index == 0 ? 0 : Index - 1;
		const int32 LocalPointIndex = Index == 0 ? 0 : 3;

		float OldParameter = Base::PairKnots[Index].Value;
    
		// Early exit if parameter isn't changing
		if (OldParameter == NewParameter)
		{
			return Index;
		}
		
		UE_LOGF(LogSpline, Verbose, "\t");
		UE_LOGF(LogSpline, Verbose, "Before SetParameter(%d, %f):", Index, NewParameter);
		Base::Dump();
		
		// Prevent reuse of an existing knot.
		const float DesiredNewParameter = NewParameter;
		FKnotSearchCache SearchCache{ Base::PairKnots };
		const float ActualNewParameter = Base::GetNearestAvailableKnotValue(DesiredNewParameter, SearchCache);

		const float DesiredToOldDistance = FMath::Abs(DesiredNewParameter - OldParameter);
		const float DesiredToActualNewDistance = FMath::Abs(DesiredNewParameter - ActualNewParameter);

		if (DesiredToActualNewDistance > DesiredToOldDistance)
		{
			// If our nearest valid knot is further away from the desired value than our current value is, we need to just keep our current value
			return Index;
		}
		
		int32 ReturnIndex = Index;
		Base::SetKnot(Index, NewParameter);

		// Fix the knot vector by flipping segments which have inverted parameter ranges until it is valid again.
		if (NewParameter < OldParameter)
		{
			int32 LeftSegmentIndex = Index - 1;

			// Repeated left flips
			while (LeftSegmentIndex >= 0 && GetSegmentParameterRange(LeftSegmentIndex).Min > GetSegmentParameterRange(LeftSegmentIndex).Max)
			{
				FlipSegment(LeftSegmentIndex);
				LeftSegmentIndex--;
				ReturnIndex--;
			}
		}
		else if (NewParameter > OldParameter)
		{
			int32 RightSegmentIndex = Index;

			while (RightSegmentIndex <= NumSegments - 1 && !IsClosingSegment(RightSegmentIndex) && GetSegmentParameterRange(RightSegmentIndex).Min > GetSegmentParameterRange(RightSegmentIndex).Max)
			{
				FlipSegment(RightSegmentIndex);
				RightSegmentIndex++;
				ReturnIndex++;
			}
		}
			
		UE_LOGF(LogSpline, Verbose, "\t");
		UE_LOGF(LogSpline, Verbose, "After SetParameter(%d, %f):", Index, NewParameter);
		Base::Dump();
		
		return ReturnIndex;
	}
	
	void FlipSegment(int32 Segment)
	{
		const int32 NumSegments = GetNumberOfSegments();
		const int32 NumAnchors  = GetNumAnchorPoints();
		if (Segment < 0 || Segment >= NumSegments || NumAnchors < 2)
		{
			return;
		}

		UE_LOGF(LogSpline, Verbose, "\t");
		UE_LOGF(LogSpline, Verbose, "Before FlipSegment(%d):", Segment);
		Base::Dump();
		
		// 1) Flip this segment's Bezier window: P0<->P3, P1<->P2
		{
			int32 Idx0, Idx1, Idx2, Idx3;
			GetBezierIndices(Segment, Idx0, Idx1, Idx2, Idx3);

			ValueType P0 = Base::GetValue(Idx0);
			ValueType P1 = Base::GetValue(Idx1);
			ValueType P2 = Base::GetValue(Idx2);
			ValueType P3 = Base::GetValue(Idx3);

			Base::SetValue(Idx0, P3);
			Base::SetValue(Idx1, P2);
			Base::SetValue(Idx2, P1);
			Base::SetValue(Idx3, P0);
		}

		// 2) Fix left neighbor (Segment-1): keep its out-tangent but move its end point to match
	    if (Segment > 0 || (HasClosingSegment() && Segment == 0))
	    {
	    	const int32 LeftSeg =
			 (Segment > 0) ? (Segment - 1) : (NumSegments - 1); // wrap in closed case

	    	int32 ThisIdx0, ThisIdx1, ThisIdx2, ThisIdx3;
	    	int32 LeftIdx0, LeftIdx1, LeftIdx2, LeftIdx3;

	    	GetBezierIndices(Segment, ThisIdx0, ThisIdx1, ThisIdx2, ThisIdx3);
	    	GetBezierIndices(LeftSeg, LeftIdx0, LeftIdx1, LeftIdx2, LeftIdx3);

	    	const ValueType ThisP0 = Base::GetValue(ThisIdx0);
	    	const ValueType LeftP2 = Base::GetValue(LeftIdx2);
	    	const ValueType LeftP3 = Base::GetValue(LeftIdx3);

	    	const ValueType Delta = LeftP3 - LeftP2; // original out-tangent
	    	const ValueType NewEnd = ThisP0;
	    	const ValueType NewIn  = NewEnd - Delta;

	    	Base::SetValue(LeftIdx3, NewEnd);
	    	Base::SetValue(LeftIdx2, NewIn);
	    }

	    // 3) Fix right neighbor (Segment+1): keep its in-tangent but move its start point to match
	    if (Segment < NumSegments - 1 || IsClosingSegment(Segment))
	    {
	    	const int32 RightSeg =
			 (Segment < NumSegments - 1) ? (Segment + 1) : 0; // wrap in closed case

	    	int32 ThisIdx0, ThisIdx1, ThisIdx2, ThisIdx3;
	    	int32 RightIdx0, RightIdx1, RightIdx2, RightIdx3;

	    	GetBezierIndices(Segment, ThisIdx0, ThisIdx1, ThisIdx2, ThisIdx3);
	    	GetBezierIndices(RightSeg, RightIdx0, RightIdx1, RightIdx2, RightIdx3);

	    	const ValueType ThisP3  = Base::GetValue(ThisIdx3);
	    	const ValueType RightP0 = Base::GetValue(RightIdx0);
	    	const ValueType RightP1 = Base::GetValue(RightIdx1);

	    	const ValueType Delta   = RightP0 - RightP1; // original in-tangent
	    	const ValueType NewStart = ThisP3;
	    	const ValueType NewOut   = NewStart - Delta;

	    	Base::SetValue(RightIdx0, NewStart);
	    	Base::SetValue(RightIdx1, NewOut);
	    }

	    // 4) Swap knot intervals for the flipped segment & its neighbor interval
	    Base::SwapKnots(Segment, Segment + 1);

	    UE_LOGF(LogSpline, Verbose, "\t");
	    UE_LOGF(LogSpline, Verbose, "After FlipSegment(%d):", Segment);
	    Base::Dump();
	}
	
	virtual int32 FindIndexForParameter(float Parameter, float& OutLocalParam) const override
	{
		const int32 NumSegments = GetNumberOfSegments();
		const UE::Geometry::FInterval1f ParameterSpace = GetParameterSpace();

		if (ParameterSpace.IsEmpty())
		{
			return INDEX_NONE;
		}

		// Left end: snap onto first segment start
		if (Parameter <= ParameterSpace.Min)
		{
			OutLocalParam = 0.0f;
			return 0;
		}

		// Right end: snap onto last segment end
		if (Parameter >= ParameterSpace.Max)
		{
			OutLocalParam = 1.0f;
			return NumSegments - 1;
		}

		const float LastKnot = Base::PairKnots.Last().Value;

		// We can now assume Parameter is contained by ParameterSpace, no need for clamping.

		// Closed loop: handle the synthetic closing segment [LastKnot, LastKnot + LoopKnotDelta]
		if (Parameter >= LastKnot && ensure(HasClosingSegment()))
		{
			const float t = (Parameter - LastKnot) / FMath::Max(ParameterSpace.Max - LastKnot, UE_SMALL_NUMBER);
			OutLocalParam = FMath::Clamp(t, 0.0f, 1.0f);
			return NumSegments - 1;
		}
        
		int32 SegmentIndex = INDEX_NONE;

		// todo: make CVar
		constexpr bool bEnableSegmentCaching = true;
		if (bEnableSegmentCaching && LastEvaluatedSegment != INDEX_NONE && LastEvaluatedSegment < Base::PairKnots.Num() - 1)
		{
			if (Parameter < Base::PairKnots[LastEvaluatedSegment + 1].Value && Parameter >= Base::PairKnots[LastEvaluatedSegment].Value)
			{
				// cache hit!
				SegmentIndex = LastEvaluatedSegment;
			}
		}
		
		if (SegmentIndex == INDEX_NONE)
		{
			// cache miss!
			int32 SegmentLow = 0;
			int32 SegmentHigh = Base::PairKnots.Num() - 2;
			SegmentIndex = SegmentLow + (SegmentHigh - SegmentLow) / 2;

			while (SegmentLow <= SegmentHigh)
			{
				int32 Mid = SegmentLow + (SegmentHigh - SegmentLow) / 2;
				const float& StartParam = Base::PairKnots[Mid].Value;
				const float& EndParam   = Base::PairKnots[Mid + 1].Value;

				if (Parameter >= StartParam && Parameter < EndParam)
				{
					SegmentIndex = Mid;
					break;
				}
				else if (Parameter < StartParam)
				{
					SegmentHigh = Mid - 1;
				}
				else
				{
					SegmentLow = Mid + 1;
				}
			}
		}
		
		// Update the cache.
		LastEvaluatedSegment = SegmentIndex;
        
		// Calculate local parameter
		float SegmentLength = Base::PairKnots[SegmentIndex + 1].Value - Base::PairKnots[SegmentIndex].Value;
		if (SegmentLength > UE_SMALL_NUMBER)
		{
			OutLocalParam = (Parameter - Base::PairKnots[SegmentIndex].Value) / SegmentLength;
		}
		else
		{
			OutLocalParam = 0.0f;
		}
        
		return SegmentIndex;
	}
	
	virtual void SetClosedLoop(bool bShouldClose) override
	{
		// Skip if state isn't changing
		if (bShouldClose == Base::IsClosedLoop())
		{
			return;
		}
        
		// Just flip the topology flag; do not add/remove any control points.
		Base::SetClosedLoop(bShouldClose);

		// Update knot multiplicities.
		ApplyClampedKnotsMultiplicity();
		ApplyInternalKnotsMultiplicity();
	}

	/**
	 * Maps a local segment parameter [0,1] to global parameter space
	 * @param SegmentIndex - Index of the segment
	 * @param LocalParam - Local parameter within segment [0,1]
	 * @return Parameter value in global space, or 0 if segment is invalid
	 */
	float MapLocalSegmentParameterToGlobal(int32 SegmentIndex, float LocalParam) const
	{
		FInterval1f SegmentRange = GetSegmentParameterRange(SegmentIndex);
		return SegmentRange.Interpolate(LocalParam);
	}

	void GetBezierIndices(int32 SegmentIndex, int32& OutP0Idx, int32& OutP1Idx, int32& OutP2Idx, int32& OutP3Idx) const
	{
		if (!IsClosingSegment(SegmentIndex))
		{
			OutP0Idx = P0Idx(SegmentIndex);
			OutP1Idx = P1Idx(SegmentIndex);
			OutP2Idx = P2Idx(SegmentIndex);
			OutP3Idx = P3Idx(SegmentIndex);
		}
		else
		{
			const int32 NumAnchors = GetNumAnchorPoints();

			OutP0Idx = Close_P0Idx(NumAnchors);
			OutP1Idx = Close_P1Idx(NumAnchors);
			OutP2Idx = Close_P2Idx();
			OutP3Idx = Close_P3Idx();
		}
	}
	
	/**
	 * Anchor points are unique segment endpoints.
	 * All segments have 2 corresponding anchor points.
	 * For a closed spline, the final segment's anchor points are the first and last points.
	 */
	int32 GetNumAnchorPoints() const
	{
		return Base::Values.Num() / 3;
	}

	void SetAnchorPoint(int32 AnchorIndex, const ValueType& NewPosition)
	{
		if (AnchorIndex < 0 || AnchorIndex >= GetNumAnchorPoints())
		{
			return;
		}
		const int32 AnchorValueIndex = AnchorBase(AnchorIndex) + 1;
		Base::SetValue(AnchorValueIndex, NewPosition);
	}

	ValueType GetAnchorPoint(int32 AnchorIndex) const
	{
		if (AnchorIndex < 0 || AnchorIndex >= GetNumAnchorPoints())
		{
			return ValueType();
		}
		const int32 AnchorValueIndex = AnchorBase(AnchorIndex) + 1;
		return Base::GetValue(AnchorValueIndex);
	}

	/** Per-anchor setters/getters */
	void SetHandleOutOfAnchor(int32 AnchorIdx, const ValueType& Out)
	{
		Base::SetValue(AnchorBase(AnchorIdx) + 2, Out);
	}

	const ValueType& GetHandleOutOfAnchor(int32 AnchorIdx) const
	{
		return Base::GetValue(AnchorBase(AnchorIdx) + 2);
	}
	
	void SetHandleInOfAnchor(int32 AnchorIdx, const ValueType& In)
	{
		Base::SetValue(AnchorBase(AnchorIdx) + 0, In);
	}

	const ValueType& GetHandleInOfAnchor(int32 AnchorIdx) const
	{
		return Base::GetValue(AnchorBase(AnchorIdx) + 0);
	}

	void SetAnchors(const TArray<FAnchor>& InAnchors, EParameterizationPolicy ParameterizationPolicy = EParameterizationPolicy::Centripetal)
	{
		Clear();

		AppendAnchors(InAnchors, ParameterizationPolicy);
	}

	void AppendAnchors(const TArray<FAnchor>& InAnchors, EParameterizationPolicy ParameterizationPolicy = EParameterizationPolicy::Centripetal)
	{
		const int32 ExpectedNumAnchors = GetNumAnchorPoints() + InAnchors.Num();

		Base::Values.Reserve(ExpectedNumAnchors * 3);
		Base::PairKnots.Reserve(ExpectedNumAnchors * 3);

		for (const FAnchor& Anchor : InAnchors)
		{
			AppendAnchor(Anchor, ParameterizationPolicy);
		}
	}

	int32 PrependAnchor(const FAnchor& InAnchor, float Parameter)
	{
		// Cannot prepend if Parameter is not less than all of our knots.
		if (Base::PairKnots.Num() > 0 && Parameter >= Base::PairKnots[0].Value)
		{
			return INDEX_NONE;
		}

		Base::InsertValue(0, InAnchor.InHandle);
		Base::InsertValue(1, InAnchor.Anchor);
		Base::InsertValue(2, InAnchor.OutHandle);

		// Degree of 1 is arbitrary, it will be set correctly based on loop and clamping flags by ApplyClampedKnotsMultiplicity.
		const int32 KnotIndex = Base::InsertKnot(FKnot(Parameter, 1));

		ensureAlwaysMsgf(KnotIndex == 0, TEXT("Unexpected knot insertion index when prepending an anchor!"));

		ApplyClampedKnotsMultiplicity();
		ApplyInternalKnotsMultiplicity();

		return 0;
	}
	
	int32 PrependAnchor(const FAnchor& InAnchor, EParameterizationPolicy ParameterizationPolicy = EParameterizationPolicy::Centripetal)
	{
		float NewParameter;

		if (GetNumAnchorPoints() <= 0)
		{
			NewParameter = 0.f;
		}
		else
		{
			const ValueType OldAnchor = GetAnchorPoint(0);
			const float OldParameter = GetParameter(0);

			const float SegmentLength = ComputeSegmentLength(InAnchor.Anchor, OldAnchor, ParameterizationPolicy);
			const float NewParamByLength = OldParameter - SegmentLength;
			const float NewParamByStep = Param::PrevDistinct(OldParameter);

			NewParameter = FMath::Min(NewParamByLength, NewParamByStep);
		}

		return PrependAnchor(InAnchor, NewParameter);
	}

	int32 AppendAnchor(const FAnchor& InAnchor, float Parameter)
	{
		// Cannot append if Parameter is not greater than all of our knots.
		if (Base::PairKnots.Num() > 0 && Parameter <= Base::PairKnots.Last().Value)
		{
			return INDEX_NONE;
		}

		const int32 InsertIndex = GetNumAnchorPoints();

		Base::AddValue(InAnchor.InHandle);
		Base::AddValue(InAnchor.Anchor);
		Base::AddValue(InAnchor.OutHandle);

		// Degree of 1 is arbitrary, it will be set correctly based on loop and clamping flags by ApplyClampedKnotsMultiplicity.
		const int32 KnotIndex = Base::InsertKnot(FKnot(Parameter, 1));

		ensureAlwaysMsgf(KnotIndex == InsertIndex, TEXT("Unexpected knot insertion index when appending an anchor!"));

		ApplyClampedKnotsMultiplicity();
		ApplyInternalKnotsMultiplicity();

		return InsertIndex;
	}
	
	int32 AppendAnchor(const FAnchor& InAnchor, EParameterizationPolicy ParameterizationPolicy = EParameterizationPolicy::Centripetal)
	{
		const int32 NumAnchors = GetNumAnchorPoints();

		float NewParameter;

		if (NumAnchors <= 0)
		{
			NewParameter = 0.f;
		}
		else
		{
			const ValueType OldAnchor = GetAnchorPoint(NumAnchors - 1);
			const float OldParameter = GetParameter(NumAnchors - 1);

			const float SegmentLength = ComputeSegmentLength(InAnchor.Anchor, OldAnchor, ParameterizationPolicy);
			const float NewParamByLength = OldParameter + SegmentLength;
			const float NewParamByStep = Param::NextDistinct(OldParameter);

			NewParameter = FMath::Max(NewParamByLength, NewParamByStep);
		}

		return AppendAnchor(InAnchor, NewParameter);
	}

	/** Per-anchor structural inserts */
	int32 InsertAnchor(const FAnchor& InAnchor, float Parameter)
	{
		if (GetNumAnchorPoints() <= 0)
		{
			return PrependAnchor(InAnchor, Parameter);
		}

		const FInterval1f ParameterSpace = GetParameterSpace();

		if (!ensureAlwaysMsgf(!ParameterSpace.IsEmpty(), TEXT("Parameter space is empty but we do not have an empty spline!")))
		{
			return INDEX_NONE;
		}
		else if (Parameter < ParameterSpace.Min)
		{
			return PrependAnchor(InAnchor, Parameter);
		}
		else if (Parameter > ParameterSpace.Max)
		{
			return AppendAnchor(InAnchor, Parameter);
		}

		float LocalT;
		const int32 SegmentIndex = FindIndexForParameter(Parameter, LocalT);

		if (SegmentIndex < 0 || SegmentIndex >= GetNumberOfSegments())
		{
			// If Parameter is contained by ParameterSpace (confirmed by Prepend/Append special casing above)
			// but the parameter maps to an invalid segment, something is wrong that we can't handle nicely.
			return INDEX_NONE;
		}

		// Segment S is bounded by anchors S and S+1, except for the closing segment which is bounded by anchors S and 0.
		// We will special case insertion on the closing segment as this is simply an append operation.
		if (IsClosingSegment(SegmentIndex))
		{
			return AppendAnchor(InAnchor, Parameter);
		}

		// We may now assume anchors SegmentIndex and SegmentIndex + 1 are valid without bounds checking.
		// In order to insert onto segment S, we must insert at the index S + 1 so that the new anchor is the new endpoint of segment S.
		
		FInterval1f SegmentRange = GetSegmentParameterRange(SegmentIndex);

		if (Parameter == SegmentRange.Min)
		{
			Parameter = Param::NextDistinct(Parameter);
		}
		else if (Parameter == SegmentRange.Max)
		{
			Parameter = Param::PrevDistinct(Parameter);
		}
		
		// Bail out if we lost usable range, this should not happen.
		if (Parameter <= SegmentRange.Min || Parameter >= SegmentRange.Max)
		{
			ensureAlways(false);
			return INDEX_NONE;
		}

		const int32 InsertIndex = SegmentIndex + 1;
		Base::InsertValue(AnchorBase(InsertIndex) + 0, InAnchor.InHandle);
		Base::InsertValue(AnchorBase(InsertIndex) + 1, InAnchor.Anchor);
		Base::InsertValue(AnchorBase(InsertIndex) + 2, InAnchor.OutHandle);

		// We always use Degree as the multiplicity because this is always an interior knot.
		const int32 KnotIndex = Base::InsertKnot(FKnot(Parameter, Degree));
		ensureAlways(InsertIndex == KnotIndex);

		return InsertIndex;
	}

	/**
	 * Removes a point from the spline.
	 * @param PointIndex - Index of the point to remove
	 * @return true if successfully removed
	 */
	bool RemoveAnchor(int32 PointIndex)
	{
		if (PointIndex < 0 || PointIndex >= GetNumAnchorPoints())
		{
			return false;
		}

		const int32 BaseIdx = AnchorBase(PointIndex);
		Base::RemoveValue(BaseIdx + 2);
		Base::RemoveValue(BaseIdx + 1);
		Base::RemoveValue(BaseIdx + 0);

		Base::RemoveKnot(PointIndex);

		ApplyClampedKnotsMultiplicity();
		ApplyInternalKnotsMultiplicity();

		return true;
	}

	// Split a segment using De Casteljau
	bool SplitSegment(int32 SegmentIndex, float SegmentT)
	{
		if (SegmentIndex < 0 || SegmentIndex >= GetNumberOfSegments())
		{
			return false;
		}

		FInterval1f SegmentRange = GetSegmentParameterRange(SegmentIndex);
		float Parameter = SegmentRange.Interpolate(SegmentT);

		if (Parameter == SegmentRange.Min)
		{
			Parameter = Param::NextDistinct(Parameter);
		}
		else if (Parameter == SegmentRange.Max)
		{
			Parameter = Param::PrevDistinct(Parameter);
		}
		
		// Bail out if we lost usable range
		if (Parameter <= SegmentRange.Min || Parameter >= SegmentRange.Max)
		{
			return false;
		}

		ValueType Position = Base::Evaluate(Parameter);

		FWindow Window;
		if (!BuildRawWindowForSegment(SegmentIndex, Window))
		{
			// If we cannot build a window for segment index, something is wrong and we cannot handle it nicely.
			return false;
		}

		ValueType P0 = *Window[0];
		ValueType P1 = *Window[1];
		ValueType P2 = *Window[2];
		ValueType P3 = *Window[3];
		
		// Use De Casteljau algorithm to split the curve
		const float t = SegmentT;
		const float mt = 1.0f - t;
		
		// Calculate split control points
		ValueType Q0 = P0;
		ValueType Q1 = P0 * mt + P1 * t;
		ValueType Q2 = (P0 * mt + P1 * t) * mt + (P1 * mt + P2 * t) * t;
		ValueType Q3 = Position;  // Use provided position for the split point
		
		ValueType R0 = Position;  // Use provided position for the split point
		ValueType R1 = (P1 * mt + P2 * t) * mt + (P2 * mt + P3 * t) * t;
		ValueType R2 = P2 * mt + P3 * t;
		ValueType R3 = P3;

		// Fixup endpoints of segment being split.
		TPair<int32, int32> SegmentAnchors = GetSegmentAnchors(SegmentIndex);
		SetHandleOutOfAnchor(SegmentAnchors.Key, Q1);
		SetHandleInOfAnchor(SegmentAnchors.Value, R2);

		// Insert anchor, splitting the segment.
		InsertAnchor(FAnchor { Q2, Q3, R1 }, Parameter);

		return true;
	}

	virtual int32 GetExpectedNumKnots() const override
	{
		const int32 NumAnchors = GetNumAnchorPoints();
		const int32 NumSegments = GetNumberOfSegments();

		if (NumAnchors == 0)
		{
			return 0;
		}
		else if (NumAnchors == 1)
		{
			if (Base::IsClosedLoop())
			{
				return 3;
			}
			else
			{
				return 4;
			}
		}

		if (!HasClosingSegment())
		{
			// Open cubic poly-Bézier:
			// start multiplicity = 4, end multiplicity = 4, each interior boundary = 3
			// Total = 4 + (NumSegments - 1) * 3 + 4
			return 4 + (NumSegments - 1) * 3 + 4;
		}
		else
		{
			// Closed cubic poly-Bézier:
			// all boundaries behave like "interior" → multiplicity = 3
			// There are NumSegments+1 boundaries including the wrap
			// Total = 3 * (NumSegments + 1)
			return 3 * (NumSegments + 1);
		}
	}

	virtual TArray<FKnot> GetKnotVector() const override
	{
		TArray<FKnot> KnotVector = Base::GetKnotVector();

		if (HasClosingSegment() && ensureAlwaysMsgf(KnotVector.Num() > 0, TEXT("Knot vector is empty, cannot compute the closing knot.")))
		{
			KnotVector.Add(FKnot(KnotVector.Last().Value + LoopKnotDelta, Degree));
		}

		return KnotVector;
	}

	virtual bool SetKnotVector(const TArray<FKnot>& NewKnots) override
	{
		if (!Base::ValidateKnotVector(NewKnots))
		{
			return false;
		}

		Base::PairKnots = NewKnots;

		if (HasClosingSegment())
		{
			const int32 LastIdx     = Base::PairKnots.Num() - 1;
			const float ClosingKnot = Base::PairKnots[LastIdx].Value;
			const float PrevKnot    = Base::PairKnots[LastIdx - 1].Value;

			// Store the closing interval length and drop the closing knot.
			LoopKnotDelta = ClosingKnot - PrevKnot;
			ensure(LoopKnotDelta > 0.f);

			Base::PairKnots.RemoveAt(LastIdx, 1, EAllowShrinking::Yes);
		}
		
		Base::MarkFlatKnotsCacheDirty();

		UE_LOGF(LogSpline, Verbose, "Set knot vector (Direct) - Unique Knots: %d",
			   Base::PairKnots.Num());

		Base::PrintKnotVector();

		return true;
	}

	void SetFindWindowOverride(FFindWindowOverride Fn) { FindWindowOverrideFn = Fn; }
	
private:

	// Segment S is bounded by anchors S and S + 1. Closing segment is bounded by S and 0.
	TPair<int32, int32> GetSegmentAnchors(int32 SegmentIndex)
	{
		if (SegmentIndex < 0 || SegmentIndex >= GetNumberOfSegments())
		{
			return { INDEX_NONE, INDEX_NONE };
		}

		if (IsClosingSegment(SegmentIndex))
		{
			return { SegmentIndex, 0 };
		}

		return { SegmentIndex, SegmentIndex + 1 };
	}

	static constexpr int32 AnchorBase(int32 i){ return 3*i; }

	static constexpr int32 P0Idx(int32 s) { return AnchorBase(s) + 1; } // A_s
	static constexpr int32 P1Idx(int32 s) { return AnchorBase(s) + 2; } // Out_s
	static constexpr int32 P2Idx(int32 s) { return AnchorBase(s + 1); } // In_{s+1}
	static constexpr int32 P3Idx(int32 s) { return AnchorBase(s + 1) + 1; } // A_{s+1}
	// closing
	static constexpr int32 Close_P0Idx(int32 NumAnchors) { return AnchorBase(NumAnchors - 1) + 1; }
	static constexpr int32 Close_P1Idx(int32 NumAnchors) { return AnchorBase(NumAnchors - 1) + 2; }
	static constexpr int32 Close_P2Idx() { return AnchorBase(0); }
	static constexpr int32 Close_P3Idx() { return AnchorBase(0) + 1; }

	
	/**
	 * Builds the raw Bezier window for a segment index
	 * Returns false if segment is invalid.
	 */ 
	bool BuildRawWindowForSegment(int32 SegmentIndex, FWindow& OutWindow) const
	{
		OutWindow = {};

		const int32 NumAnchors = GetNumAnchorPoints();
		if (NumAnchors < 2)
		{
			return false;
		}

		const int32 NumSegments = GetNumberOfSegments();
		if (SegmentIndex < 0 || SegmentIndex >= NumSegments)
		{
			return false;
		}

		auto I = [this](int32 idx){ return &Base::Values[idx]; };

		if (!IsClosingSegment(SegmentIndex))
		{
			OutWindow[0] = I(P0Idx(SegmentIndex)); // A_s
			OutWindow[1] = I(P1Idx(SegmentIndex)); // Out_s
			OutWindow[2] = I(P2Idx(SegmentIndex)); // In_{s+1}
			OutWindow[3] = I(P3Idx(SegmentIndex)); // A_{s+1}
		}
		else
		{
			OutWindow[0] = I(Close_P0Idx(NumAnchors)); // A_last
			OutWindow[1] = I(Close_P1Idx(NumAnchors)); // Out_last
			OutWindow[2] = I(Close_P2Idx()); // In_0
			OutWindow[3] = I(Close_P3Idx()); // A_0
		}

		return Math::FSplineValidation::IsValidWindow<ValueType, Base::WindowSize>(OutWindow);
	}

	void MigrateLegacyBezierLayout()
	{
	    const int32 NumValues = Base::Values.Num();

		// Empty spline, no migration necessary.
		if (NumValues == 0)
		{
			return;
		}

		// Handle legacy case where we do not have a complete bezier segment to migrate:
		switch (NumValues)
		{
			// Idx 0 is position, other values are garbage.
			case 1: [[fallthrough]];
			case 2: [[fallthrough]];
			case 3:
			{
				ValueType In(ForceInitToZero);
				ValueType A = Base::Values[0];
				ValueType Out(ForceInitToZero);

				Base::Values = { In, A, Out };
				Base::PairKnots = { FKnot( 0, GetExpectedNumKnots())};
								
				Base::MarkFlatKnotsCacheDirty();

				return;
			}

			default:
				break;
		}

	    // Legacy layout (actual):
	    // [ P0, C0_0, C1_0, P1,
	    //   P1_Duplicate, C0_1, C1_1, P2,
	    //   P2_Duplicate, C0_2, C1_2, P3,
	    //   ...
	    //   P_{n-1}_Duplicate, C0_{n-1}, C1_{n-1}, Pn ]

		// We handle cases 0-3 above
	    checkf(NumValues >= 4,
	        TEXT("Legacy PolyBezier must have at least 1 segment (4 values minimum)"));

	    checkf(NumValues % 4 == 0,
	        TEXT("Legacy PolyBezier layout mismatch (NumValues = %d)"), NumValues);

		const bool bClosed = Base::IsClosedLoop();

	    const int32 NumSegments = NumValues / 4;

	    const TArray<ValueType>& Legacy = Base::Values;
		// For open: anchors = NumSegments + 1
		// For closed: anchors = NumSegments (closing segment is virtual)
		const int32 NumAnchors = bClosed ? NumSegments : (NumSegments + 1);
		
	    TArray<ValueType> NewValues;
	    NewValues.SetNumUninitialized(NumAnchors * 3);

	    // Anchor positions P_j
	    auto IndexP = [](int32 j) -> int32
	    {
	        // P0 is at index 0.
	        // For j > 0, P_j is the end of the (j-1)-th segment.
	        return (j == 0) ? 0 : (3 + 4 * (j - 1));
	    };

	    // Outgoing handle C0_j for segment j
	    auto IndexC0 = [](int32 segIdx) -> int32
	    {
	        // Segment segIdx starts at 4*segIdx:
	        // [ P_j_or_dup, C0_j, C1_j, P_{j+1} ]
	        return 1 + 4 * segIdx;
	    };

	    // Incoming handle C1_j for segment j
	    auto IndexC1 = [](int32 segIdx) -> int32
	    {
	        return 2 + 4 * segIdx;
	    };

	    for (int32 AnchorIdx = 0; AnchorIdx < NumAnchors; ++AnchorIdx)
	    {
	        ValueType In(ForceInitToZero);
	        ValueType A(ForceInitToZero);
	        ValueType Out(ForceInitToZero);

	        // A_j = P_j
	        const int32 PIndex = IndexP(AnchorIdx);
	        check(Legacy.IsValidIndex(PIndex));
	        A = Legacy[PIndex];

	    	// In_j
	    	if (!bClosed)
	    	{
	    		if (AnchorIdx == 0)
	    		{
	    			// No incoming segment, start flat.
	    			In = A;
	    		}
	    		else
	    		{
	    			// From previous segment's C1
	    			const int32 PrevSeg = AnchorIdx - 1;
	    			const int32 C1Index = IndexC1(PrevSeg);
	    			check(Legacy.IsValidIndex(C1Index));
	    			In = Legacy[C1Index];
	    		}
	    	}
	    	else
	    	{
	    		// Closed: every anchor has an incoming segment.
	    		const int32 PrevSeg = (AnchorIdx == 0) ? (NumSegments - 1) : (AnchorIdx - 1);
	    		const int32 C1Index = IndexC1(PrevSeg);
	    		check(Legacy.IsValidIndex(C1Index));
	    		In = Legacy[C1Index];
	    	}

	    	// Out_j
	    	if (!bClosed)
	    	{
	    		if (AnchorIdx < NumSegments)
	    		{
	    			const int32 C0Index = IndexC0(AnchorIdx);
	    			check(Legacy.IsValidIndex(C0Index));
	    			Out = Legacy[C0Index];
	    		}
	    		else
	    		{
	    			// Last anchor has no outgoing segment in open curves.
	    			Out = A;
	    		}
	    	}
	    	else
	    	{
	    		// Closed: every anchor has outgoing segment j
	    		const int32 Seg = AnchorIdx;
	    		const int32 C0Index = IndexC0(Seg);
	    		check(Legacy.IsValidIndex(C0Index));
	    		Out = Legacy[C0Index];
	    	}

	        const int32 NewBase = AnchorIdx * 3;
	        NewValues[NewBase + 0] = In;
	        NewValues[NewBase + 1] = A;
	        NewValues[NewBase + 2] = Out;
	    }

	    Base::Values = MoveTemp(NewValues);

		// Fix up knot representation for closed loops.
		// Legacy data uses an explicit closing knot in PairKnots; the new
		// representation uses LoopKnotDelta instead.
		if (bClosed && Base::PairKnots.Num() > 2)
		{
			const int32 LastIdx     = Base::PairKnots.Num() - 1;
			const float ClosingKnot = Base::PairKnots[LastIdx].Value;
			const float PrevKnot    = Base::PairKnots[LastIdx - 1].Value;

			// Store the closing interval length and drop the closing knot.
			LoopKnotDelta = ClosingKnot - PrevKnot;

			Base::PairKnots.RemoveAt(LastIdx, 1, EAllowShrinking::Yes);
		}
		
		// Values changed (and potentially PairKnots layout), so flat-knot cache is dirty.
	    Base::MarkFlatKnotsCacheDirty();
	}

	/**
	 * Computes chord or centripetal distances between segment endpoints
	 * Uses those distances to build a proportional knot vector with the proper multiplicities for each Bézier segment.
	 * @param Mode - Knot generation mode to use
	 */
	void GenerateDistanceKnotsForBezier(EParameterizationPolicy Mode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GenerateDistanceKnotsForBezier);

		// 3-tuple layout:
		// Values[3*i + 0] = In_i
		// Values[3*i + 1] = A_i
		// Values[3*i + 2] = Out_i

		const int32 NumAnchors  = GetNumAnchorPoints();

		if (NumAnchors <= 0)
		{
			Base::PairKnots.Reset();
			Base::MarkFlatKnotsCacheDirty();
			return;
		}
		else if (NumAnchors == 1)
		{
			// If we have a single anchor, our number of expected knots is itself the multiplicity of a single distinct knot.
			Base::PairKnots = { FKnot(0.f, GetExpectedNumKnots()) };
			Base::MarkFlatKnotsCacheDirty();
			return;
		}

		// If we have > 1 anchors (guaranteed by the above branches), we ought to have at least 1 segment.
		const int32 NumSegments = GetNumberOfSegments();
		ensureAlways(NumSegments > 0);
			
		TArray<float> SegmentLengths;
		SegmentLengths.Reserve(NumSegments);
	        
		// For a Bezier spline, the control points don't all lie on the curve
		// So we need to compute chord lengths between segment endpoints only.
		// Segment i goes from A_i to A_{i+1} (open) or A_{(i+1)%N} (closed).
		const bool bClosed = HasClosingSegment();

		for (int32 i = 0; i < NumSegments; ++i)
		{
			const int32 StartAnchorIdx = i;
			const int32 EndAnchorIdx   = bClosed
				? (i + 1) % NumAnchors
				: (i + 1); // for open, NumSegments == NumAnchors - 1, so this is always valid
	            
			const ValueType& P0 = Base::Values[AnchorBase(StartAnchorIdx) + 1]; // A_i
			const ValueType& P3 = Base::Values[AnchorBase(EndAnchorIdx)   + 1]; // A_{i+1} or A_0 (closed)

			SegmentLengths.Add(ComputeSegmentLength(P0, P3, Mode));
		}
        
		// Now create a knot vector that respects:
		// 1. Bezier segment structure (knot multiplicity)
		// 2. Proportional chord/centripetal lengths
        
		// This will only return Val if Offset is exactly 0.f. Otherwise, it is guaranteed to change in the direction of Offset.
		// We sacrifice a bit of accuracy for the assumption that Val will actually change.
		auto SafeAdd = [](float Val, float Offset) -> float
		{
			if (Offset == 0.f) return Val;
				
			const float Result = Val + Offset;
			if (Result != Val) return Result;

			// Guaranteed one-step progress even under FTZ/DAZ:
			return UE::Geometry::Spline::Param::Step(
				Val, Offset > 0.f ? UE::Geometry::Spline::Param::EDir::Right
								  : UE::Geometry::Spline::Param::EDir::Left);
		};
		
		// Reset pair knots
		Base::PairKnots.Reset();
        
		FValidKnotSearchParams SearchParams;
		SearchParams.bSearchLeft = false;	// we are appending repeatedly, we want insertion order to be predictable (always go after conflicting knots).

		FKnotSearchCache SearchCache{ Base::PairKnots };

		const auto InsertKnotAndUpdateCache = [this, &SearchCache, &SearchParams](uint32 InMultiplicity)
		{
			const float KnotValue = Base::GetNearestAvailableKnotValue(SearchParams, SearchCache);
			SearchCache.KnotValuesSet.Add(Param::NormalizeKey(KnotValue));
			Base::InsertKnot(FKnot(KnotValue, InMultiplicity));
		};

		constexpr int32 EndMult = Degree + 1;
		constexpr int32 InteriorMult = Degree;
        if (!bClosed)
        {
            // OPEN:  start mult=4, interior mult=3, end mult=4
            // Total knots = 4 + (NumSegments - 1) * 3 + 4
    
            // Add start knot with multiplicity EndMult = 4
            SearchParams.DesiredParameter = 0.f;
            InsertKnotAndUpdateCache(EndMult);
            
            // Calculate accumulated distances for internal knots
            float AccumLength = 0.0f;
            for (int32 i = 0; i < NumSegments - 1; ++i) 
            {
                if (i < SegmentLengths.Num())
                {
                    AccumLength = SafeAdd(AccumLength, SegmentLengths[i]);
                    
                    // Use accumulated length as the desired knot value
                    SearchParams.DesiredParameter = AccumLength;
                    
                    // Interior knots at segment boundaries with multiplicity 3
                    InsertKnotAndUpdateCache(InteriorMult);
                }
            }
    
            // Final end knot with multiplicity 4
            SearchParams.DesiredParameter = SafeAdd(
                AccumLength,
                SegmentLengths.Num() > 0 ? SegmentLengths.Last() : 0.0f);
    
            InsertKnotAndUpdateCache(EndMult);
        }
        else
        {
            // CLOSED:
            // All boundaries behave like interior ones, multiplicity = InteriorMult = 3.
		    // We create one boundary per segment (at its start) and store the
		    // total loop length in LoopKnotDelta instead of inserting a
		    // separate closing knot.
    
            float AccumLength = 0.0f;
    
            // Boundaries 0 .. NumSegments-1 at starts of each segment
            for (int32 BoundaryIdx = 0; BoundaryIdx < NumSegments; ++BoundaryIdx)
            {
                SearchParams.DesiredParameter = AccumLength;
                InsertKnotAndUpdateCache(InteriorMult);
    
                if (BoundaryIdx < SegmentLengths.Num())
                {
                	// If this is the closing segment (last one), remember its param length
                	if (BoundaryIdx == NumSegments - 1)
                	{
                		LoopKnotDelta = SegmentLengths[BoundaryIdx];
                	}

                	AccumLength = SafeAdd(AccumLength, SegmentLengths[BoundaryIdx]);
                }
            }
        	if (NumSegments == 0 || SegmentLengths.Num() == 0)
        	{
        		LoopKnotDelta = 0.0f;
        	}
        }
        // --------------------------------------------------------------------

		// Mark flat knots cache as dirty
		Base::MarkFlatKnotsCacheDirty();
	}

    virtual void FindWindow(float Parameter, FWindow& OutWindow, FWindowStore& Scratch) const override
    {
		OutWindow = {};
		const int32 NumAnchors = GetNumAnchorPoints();
		if (NumAnchors < 2)
		{
			return;
		}

		float LocalT;
		int32 SegmentIndex = FindIndexForParameter(Parameter, LocalT);

		if (SegmentIndex == INDEX_NONE)
		{
			return;
		}

		if (!BuildRawWindowForSegment(SegmentIndex, OutWindow))
		{
			return;
		}

		if (FindWindowOverrideFn)
		{
			const FInterval1f SegmentRange = GetSegmentParameterRange(SegmentIndex);
			FindWindowOverrideFn(OutWindow, SegmentIndex, SegmentRange, Scratch);
		}
    }

    virtual ValueType InterpolateWindow(TArrayView<const ValueType* const> Window, float Parameter) const override 
    {		
        if (Window.Num() < 4)
        {
			return Math::FSplineValidation::GetDefaultValue<ValueType>();
        }

		float LocalT;
		int32 SegmentIndex = FindIndexForParameter(Parameter, LocalT);

		if (SegmentIndex == INDEX_NONE)
		{
			return Math::FSplineValidation::GetDefaultValue<ValueType>();
		}

        return TSplineInterpolationPolicy<ValueType>::InterpolateCubic(Window, LocalT);
    }

	/**
	 * Finds the local parameter on a segment closest to the given position
	 * 
	 * @param SegmentIndex Index of the segment
	 * @param Position Position to find closest point to
	 * @param OutSquaredDistance Output parameter for squared distance
	 * @return Local parameter [0,1] within the segment
	 */
	float FindLocalParameterNearestToPosition(
		int32 SegmentIndex,
		const ValueType& Position, 
		float& OutSquaredDistance) const
    {
    	int32 NumSegments = GetNumberOfSegments();
    	// Validate segment index
    	if (SegmentIndex < 0 || SegmentIndex >= NumSegments)
    	{
    		OutSquaredDistance = TNumericLimits<float>::Max();
    		return 0.0f;
    	}
    
    	// Get global parameter nearest to position
    	float GlobalParam = FindNearestOnSegment(Position, SegmentIndex, OutSquaredDistance);
		
    	// Convert to local parameter
    	FInterval1f SegmentRange = GetSegmentParameterRange(SegmentIndex);
    	float SegmentLength = SegmentRange.Max - SegmentRange.Min;
    
    	if (SegmentLength == 0.f)
    		return 0.0f;
        
    	return (GlobalParam - SegmentRange.Min) / SegmentLength;
    }
	
	/**
	 * Checks if the spline is actually closed, which is semantically different from IsClosedLoop for 0 and 1 point splines.
	 */
	bool HasClosingSegment() const
	{
		return Base::IsClosedLoop() && GetNumAnchorPoints() > 1;
	}

	/**
	 * Checks if a segment index identifies the closing segment
	 * @param SegmentIndex - The index to check
	 * @return true if the segment is the closing segment
	 */
	bool IsClosingSegment(int32 SegmentIndex) const
	{
		return HasClosingSegment() && (SegmentIndex == GetNumAnchorPoints() - 1);
	}

	/**
	 * Checks if a segment index is valid
	 * @param Index - Index to check
	 * @return true if the index is valid
	 */
	bool IsValidSegmentIndex(int32 Index) const
    {
    	int32 NumSegments = GetNumberOfSegments();
    	return Index >= 0 && Index < NumSegments;
    }
	
	/**
	 * Helper to compute segment length based on parameterization policy
	 */
	float ComputeSegmentLength(
		const ValueType& P0,
		const ValueType& P3,
		EParameterizationPolicy ParameterizationPolicy)
    {
    	switch (ParameterizationPolicy)
    	{
    	case EParameterizationPolicy::ChordLength:
    		return static_cast<float>(Math::Distance(P0, P3));
    	case EParameterizationPolicy::Centripetal:
    		return static_cast<float>(Math::CentripetalDistance(P0, P3));
    	case EParameterizationPolicy::Uniform:
    	default:
			return 1.0f;
    	}
    }

	/**
	 * Applies multiplicity to the internal knots based on the spline's degree
	 */
	void ApplyInternalKnotsMultiplicity()
	{
		// fix internal knots to be clamped to Degree
		for (int32 i = 1; i < Base::PairKnots.Num() - 1; ++i)
		{
			Base::PairKnots[i].Multiplicity = Degree;
		}
	}

private:

	/** This is the last segment fetched by parameter. Used to accelerate predictable sampling patterns. */
	mutable int32 LastEvaluatedSegment = INDEX_NONE;
	
	/** Optional override for FindWindow*/
	FFindWindowOverride FindWindowOverrideFn = nullptr;

	/** Length of the closing segment’s parameter interval. Only meaningful when HasClosingSegment() == true. */
	float LoopKnotDelta = 1.f;
};


template<> 
inline float TPolyBezierSpline<FQuat>::PointDistanceSqToSegmentBBox(const ValueType& Point, int32 SegmentIndex) const
{
	return 0;
}

template<> 
inline float TPolyBezierSpline<float>::PointDistanceSqToSegmentBBox(const ValueType& Point, int32 SegmentIndex) const
{
	return 0;
}

// Common type definitions
using FPolyBezierSpline2f = TPolyBezierSpline<FVector2f>;
using FPolyBezierSpline3f = TPolyBezierSpline<FVector3f>;
using FPolyBezierSpline2d = TPolyBezierSpline<FVector2d>;
using FPolyBezierSpline3d = TPolyBezierSpline<FVector3d>;
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE
