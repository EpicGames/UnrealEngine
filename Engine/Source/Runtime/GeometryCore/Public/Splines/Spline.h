// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SplineTypeId.h"
#include "BuiltInAttributeTypes.h"
#include "SplineTypeRegistry.h"
#include "SplineInterfaces.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

template <typename SPLINETYPE> class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TSplineWrapper;
	
/* 
 * A spline which wraps another fully implemented spline (SplineType must derive from TSplineInterface) 
 */
template <typename SPLINETYPE>
class TSplineWrapper : public TSplineInterface<typename SPLINETYPE::ValueType>
{
public:
    typedef SPLINETYPE SplineType;
	typedef typename SplineType::ValueType ValueType;  // Explicitly define ValueType

    virtual ~TSplineWrapper() override = default;

    // ISplineInterface Forwarding
    virtual void Clear() override { InternalSpline.Clear(); }
	virtual bool IsEqual(const ISplineInterface* OtherSpline) const override { return InternalSpline.IsEqual(OtherSpline); }
	virtual int32 GetNumberOfSegments() const override { return InternalSpline.GetNumberOfSegments(); }
	virtual FInterval1f GetSegmentParameterRange(int32 SegmentIndex) const override { return InternalSpline.GetSegmentParameterRange(SegmentIndex); }
    virtual bool Serialize(FArchive& Ar) override
    {
    	// Call parent's Serialize
    	if (!TSplineInterface<ValueType>::Serialize(Ar))
    	{
    		return false;
    	}
       	// Serialize wrapped spline
	    return InternalSpline.Serialize(Ar);
    }
    virtual FString GetValueTypeName() const override { return InternalSpline.GetValueTypeName(); }
    virtual FString GetImplementationName() const override { return InternalSpline.GetImplementationName(); }
    virtual FInterval1f GetParameterSpace() const override { return InternalSpline.GetParameterSpace(); }
    virtual void SetClosedLoop(bool bClosed) override { InternalSpline.SetClosedLoop(bClosed); }
    virtual bool IsClosedLoop() const override { return InternalSpline.IsClosedLoop(); }

    // TSplineInterface Forwarding

	// TSplineInterface<ValueType>::PreInfinityMode is the one that actually matters, but we propagate to InternalSpline for clarity
	virtual void SetPreInfinityMode(EOutOfBoundsHandlingMode InMode) override
    {
    	InternalSpline.SetPreInfinityMode(InMode);
	    TSplineInterface<ValueType>::PreInfinityMode = InMode;
    }

	// TSplineInterface<ValueType>::PostInfinityMode is the one that actually matters, but we propagate to InternalSpline for clarity
	virtual void SetPostInfinityMode(EOutOfBoundsHandlingMode InMode) override
    {
    	InternalSpline.SetPostInfinityMode(InMode);
	    TSplineInterface<ValueType>::PostInfinityMode = InMode;
    }
	
	virtual EOutOfBoundsHandlingMode GetPreInfinityMode() const override { return InternalSpline.GetPreInfinityMode(); }
	virtual EOutOfBoundsHandlingMode GetPostInfinityMode() const override { return InternalSpline.GetPostInfinityMode(); }
	
    virtual ValueType EvaluateImpl(float Parameter) const override { return InternalSpline.EvaluateImpl(Parameter); }
    virtual float FindNearest(const ValueType& Point, float& OutSquaredDistance) const override
        { return InternalSpline.FindNearest(Point, OutSquaredDistance); }
    virtual TUniquePtr<ISplineInterface> Clone() const override
    {
    	TUniquePtr<TSplineWrapper<SplineType>> ClonedWrapper = MakeUnique<TSplineWrapper<SplineType>>();
    
    	// Clone the internal spline (assuming SplineType has proper copy semantics)
    	ClonedWrapper->InternalSpline = InternalSpline;
    
    	// Copy infinity modes and any other wrapper-specific properties
    	ClonedWrapper->PreInfinityMode = this->PreInfinityMode;
    	ClonedWrapper->PostInfinityMode = this->PostInfinityMode;
    
    	return ClonedWrapper;
    }
	friend FArchive& operator<<(FArchive& Ar, TSplineWrapper& SplineWrapper)
    {
    	SplineWrapper.Serialize(Ar);
    	return Ar;	
    }
protected:
    SplineType InternalSpline;
};

/**
 * Creates a spline from an archive
 */
inline TUniquePtr<ISplineInterface> CreateSplineFromArchive(FArchive& Ar)
{
	// Store original archive position
	int64 OriginalPos = Ar.Tell();
    
	// Read version
	int32 Version;
	Ar << Version;
    
	if (Version < 1)
	{
		UE_LOGF(LogSpline, Error, "Unsupported spline serialization version: %d", Version);
		return nullptr;
	}
    
	// Read type ID
	FSplineTypeRegistry::TypeId TypeId;
	Ar << TypeId;
    
	// Create instance
	TUniquePtr<ISplineInterface> Result = FSplineTypeRegistry::CreateSpline(TypeId);
    
	if (!Result)
	{
		UE_LOGF(LogSpline, Error, "Failed to create spline type 0x%08X", TypeId);
		return nullptr;
	}
    
	// Reset archive position so ISplineInterface::Serialize can read version and TypeId again
	Ar.Seek(OriginalPos);
    
	// Now deserialize
	Result->Serialize(Ar);
    
	return Result;
}
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE