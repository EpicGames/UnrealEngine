// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoxTypes.h"
#include "SplineTypeId.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

// Forward declarations
class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") ISplineInterface;
template <typename ValueType> class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TSplineInterface;

/**
 * Enumeration for out-of-bounds handling modes
 */
enum class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") EOutOfBoundsHandlingMode
{
    Zero,       // Null out the values outside of the bounds
    Constant,   // Choose the nearest of the first/last value
    Cycle       // Repeat the curve indefinitely (mod of the parameter space)
};

/**
 * Interface for all spline types, where spline is defined as a piecewise continuous curve.
 * Provides the base functionality independent of value type.
 */
class ISplineInterface
{
public:
    virtual ~ISplineInterface() = default;

    virtual void Clear() = 0;
    virtual TUniquePtr<ISplineInterface> Clone() const = 0;
	virtual bool IsEqual(const ISplineInterface* OtherSpline) const = 0;
    
    virtual FString GetValueTypeName() const = 0;
    virtual FString GetImplementationName() const = 0;
    virtual FInterval1f GetParameterSpace() const = 0;
    virtual void SetClosedLoop(bool bClosed) = 0;
    virtual bool IsClosedLoop() const = 0;
    /**
     * Gets the type ID for this spline instance
     * @return Type ID that uniquely identifies this spline class/value type combination
     */
    virtual FSplineTypeId::IdType GetTypeId() const = 0;
    
	virtual int32 GetNumberOfSegments() const = 0;
	virtual FInterval1f GetSegmentParameterRange(int32 SegmentIndex) const = 0;
	
    virtual bool Serialize(FArchive& Ar)
    {
        // Simple version marker for future compatibility
        int32 Version = 1;
        Ar << Version;
    
        if (Ar.IsSaving())
        {
            // Just write type ID (4 bytes)
            FSplineTypeId::IdType TypeId = GetTypeId();
            Ar << TypeId;
        }
        else // Loading
        {
            // Just read (and discard) the type ID - object already created
            FSplineTypeId::IdType TypeId;
            Ar << TypeId;
        }
    
        return true;
    }
    
    friend FArchive& operator<<(FArchive& Ar, ISplineInterface& Spline)
    {
        Spline.Serialize(Ar);
        return Ar;    
    }
};

/**
 * Typed interface for splines with a specific value type
 * Provides evaluation and other type-specific functionality
 */
template <typename InValueType>
class TSplineInterface : public ISplineInterface
{
    static FString TypeName;

public:
    using ValueType = InValueType;

    /**
     * Default constructor - ensures value type name is set
     */
    TSplineInterface()
    {
        // Ensure the value type name is initialized if not already
        if (TypeName.IsEmpty())
        {
            TypeName = TSplineValueTypeTraits<ValueType>::Name;
        }
    }
    
    virtual ~TSplineInterface() override = default;

    /**
     * Evaluates the spline at the given parameter value
     * @param Parameter The parameter value to evaluate at
     * @return The spline value at the parameter
     */
    ValueType Evaluate(float Parameter) const
    {
    	if (GetParameterSpace().IsEmpty())
    	{
    		return ValueType();
    	}
    	
        return EvaluateImpl(HandleOutOfBounds(Parameter));
    }
    
    /**
     * Finds the nearest parameter value to a point
     * @param Point The point to find nearest to
     * @param OutSquaredDistance The squared distance to the nearest point
     * @return The parameter value at the nearest point
     */
    virtual float FindNearest(const ValueType& Point, float& OutSquaredDistance) const = 0;
    
    /**
     * Serializes the spline
     */
    virtual bool Serialize(FArchive& Ar) override
    {
        // Call base class Serialize first to handle version and type ID
        if (!ISplineInterface::Serialize(Ar))
        {
            return false;
        }
        // Serialize infinity modes
        uint8 PreMode = static_cast<uint8>(PreInfinityMode);
        uint8 PostMode = static_cast<uint8>(PostInfinityMode);
        Ar << PreMode;
        Ar << PostMode;
        
        if (Ar.IsLoading())
        {
            PreInfinityMode = static_cast<EOutOfBoundsHandlingMode>(PreMode);
            PostInfinityMode = static_cast<EOutOfBoundsHandlingMode>(PostMode);
        }
        return true;
    }
    
    friend FArchive& operator<<(FArchive& Ar, TSplineInterface& Spline)
    {
        Spline.Serialize(Ar);
        return Ar;    
    }
    
    /**
     * Default implementation of type ID (derived classes should override)
     */
    virtual FSplineTypeId::IdType GetTypeId() const override { return 0; }
    
    /**
     * Set and get the type name for the value type
     */
    static void SetTypeName(const FString& InTypeName) { TypeName = InTypeName; } 
    virtual FString GetValueTypeName() const override { return TypeName; }

    /** Set Pre-Infinity handling mode when out of bounds */ 
    virtual void SetPreInfinityMode(EOutOfBoundsHandlingMode InMode){ PreInfinityMode = InMode; }

    /** Set Post-Infinity handling mode when out of bounds */
    virtual void SetPostInfinityMode(EOutOfBoundsHandlingMode InMode){ PostInfinityMode = InMode; }

    /** Get the current Pre-Infinity mode */
    virtual EOutOfBoundsHandlingMode GetPreInfinityMode() const{ return PreInfinityMode; }

    /** Get the current Post-Infinity mode */
    virtual EOutOfBoundsHandlingMode GetPostInfinityMode() const{ return PostInfinityMode; }

protected:
    /** How to handle parameters outside the mapped range */
    EOutOfBoundsHandlingMode PreInfinityMode = EOutOfBoundsHandlingMode::Zero;
    EOutOfBoundsHandlingMode PostInfinityMode = EOutOfBoundsHandlingMode::Zero;

    /** Apply out-of-bounds handling to parameter */
    float HandleOutOfBounds(float Parameter) const
    {
        FInterval1f ParentSpaceRange = GetParameterSpace();
        if (Parameter < ParentSpaceRange.Min)
        {
            switch (PreInfinityMode)
            {
            case EOutOfBoundsHandlingMode::Zero:
                return 0.0f;
            case EOutOfBoundsHandlingMode::Constant:
                return ParentSpaceRange.Min;
            case EOutOfBoundsHandlingMode::Cycle:
                {
                    const float Range = ParentSpaceRange.Length();
                    return FMath::Fmod(Parameter - ParentSpaceRange.Min, Range) + ParentSpaceRange.Min;
                }
            }
        }
        else if (Parameter > ParentSpaceRange.Max)
        {
            switch (PostInfinityMode)
            {
            case EOutOfBoundsHandlingMode::Zero:
                return 0.0f;
            case EOutOfBoundsHandlingMode::Constant:
                return ParentSpaceRange.Max;
            case EOutOfBoundsHandlingMode::Cycle:
                {
                    const float Range = ParentSpaceRange.Length();
                    return FMath::Fmod(Parameter - ParentSpaceRange.Min, Range) + ParentSpaceRange.Min;
                }
            }
        }
        return Parameter;
    }

    /**
     * Implementation of evaluate with parameter handling
     */
    virtual ValueType EvaluateImpl(float Parameter) const = 0;
};

// Initialize static member
template <typename ValueType>
inline FString TSplineInterface<ValueType>::TypeName;

} // namespace Spline
} // namespace Geometry
} // namespace UE