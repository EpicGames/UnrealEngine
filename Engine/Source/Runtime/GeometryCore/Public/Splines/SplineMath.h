// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "CoreMinimal.h"
#include "SplineInterfaces.h"
#include "ParameterizedTypes.h"
#include <cstdint>
#include <limits>

namespace UE
{
namespace Geometry
{
namespace Spline
{
struct FKnot;

// -------- Internal: numeric primitives (not public API) -----------------
namespace Internal
{
	inline uint32 FloatToBits(float f)
	{
		uint32 u;
		FMemory::Memcpy(&u, &f, sizeof(float));
		return u;
	}

	inline float BitsToFloat(uint32 u)
	{
		float f;
		FMemory::Memcpy(&f, &u, sizeof(float));
		return f;
	}

	// Map IEEE754 float to a monotonic integer ordering and back.
	inline uint32 ToOrdered(float f)
	{
		const uint32 u = FloatToBits(f);
		const bool neg = (u & 0x80000000u) != 0;
		return neg
			? ~u
			: (u ^ 0x80000000u);
	}

	inline float FromOrdered(uint32 ordered)
	{
		const uint32 u = (ordered & 0x80000000u)
			? (ordered ^ 0x80000000u)
			: ~ordered;
		return BitsToFloat(u);
	}

	/** Adjacent representable *normal-or-better* float (FTZ-proof).
	 *  Skips subnormals (snaps to ±FLT_MIN) and saturates at ±FLT_MAX.
	 */
	inline float NextafterNoSubnormal(float from, bool bStepRight)
	{
		if (FMath::IsNaN(from) || !FMath::IsFinite(from))
		{
			return from; // keep NaN/Inf unchanged
		}
		if (from == 0.0f)
		{
			const float minNorm = std::numeric_limits<float>::min();
			return bStepRight
				? +minNorm
				: -minNorm;
		}

		uint32 ordered = ToOrdered(from);
		if (bStepRight)
		{
			++ordered;
		}
		else
		{
			--ordered;
		}
		float y = FromOrdered(ordered);

		const float minNorm = std::numeric_limits<float>::min();
		const float ay = FMath::Abs(y);
		if (ay > 0.0f && ay < minNorm)
		{
			// Preserve the computed result's sign to maintain monotonicity
			const float snapped = std::copysign(minNorm, y);

			if (snapped == from)
			{
				// We're at ±FLT_MIN stepping toward zero; make monotone progress to signed zero
				y = std::copysign(0.0f, y);
			}
			else
			{
				y = snapped;
			}
		}
		if (!FMath::IsFinite(y))
		{
			y = bStepRight
				? +std::numeric_limits<float>::max()
				: -std::numeric_limits<float>::max();
		}
		return y;
	}

	/** Normalize -0.0f to +0.0f for use as a key (avoids phantom duplicates). */
	inline float NormalizeZeroForKey(float v)
	{
		return (v == 0.0f)
			? 0.0f
			: v;
	}
} // namespace Internal

// -------- Public: parameter helpers (domain-centric names) ---------------
namespace Param
{
	enum class EDir : uint8 { Left, Right };

	/** Step to a distinct adjacent parameter (float), FTZ-proof. */
	inline float NextDistinct(float t)
	{
		return Internal::NextafterNoSubnormal(t, /*right=*/true);
	}

	inline float PrevDistinct(float t)
	{
		return Internal::NextafterNoSubnormal(t, /*right=*/false);
	}

	/** Step one distinct param in given direction. */
	inline float Step(float t, EDir d)
	{
		return Internal::NextafterNoSubnormal(t, d == EDir::Right);
	}

	/** Step and clamp to an interval (when you want to stay inside bounds). */
	inline float StepInside(float t, EDir d, const FInterval1f& bounds)
	{
		return FMath::Clamp(Internal::NextafterNoSubnormal(t, d == EDir::Right), bounds.Min, bounds.Max);
	}

	/** Key sanitizer for sets/maps that use exact float equality. */
	inline float NormalizeKey(float t)
	{
		return Internal::NormalizeZeroForKey(t);
	}
} // namespace Param

namespace Math
{
	/**
	 * Size/magnitude of a value
	 */
	template <typename T>
	double Size(const T& Value)
	{
		if constexpr (THasSizeMethod<T>::value)
			return static_cast<double>(Value.Size());
		else if constexpr (std::is_arithmetic_v<T>)
			return FMath::Abs(static_cast<double>(Value));
		else
			return 1.0; // Default for other types
	}

	/**
	 * Squared size (optimization to avoid sqrt)
	 */
	template <typename T>
	double SizeSquared(const T& Value)
	{
		if constexpr (THasSizeSquaredMethod<T>::value)
			return static_cast<double>(Value.SizeSquared());
		else if constexpr (std::is_arithmetic_v<T>)
			return static_cast<double>(Value * Value);
		else
			return Size(Value) * Size(Value);
	}

	/**
	 * Distance between values
	 */
	template <typename T>
	double Distance(const T& A, const T& B)
	{
		if constexpr (THasSubtractionOperator<T>::value)
			return Size(B - A);
		else
			return 1.0; // Default fallback
	}

	/**
	 * Centripetal distance calculation (for parameterization)
	 */
	template <typename T>
	double CentripetalDistance(const T& A, const T& B)
	{
		return FMath::Sqrt(Distance(A, B));
	}

	/**
	 * Dot product between values
	 */
	template <typename T>
	double Dot(const T& A, const T& B)
	{
		if constexpr (THasDotMethod<T>::value)
			return static_cast<double>(A.Dot(B));
		else if constexpr (std::is_arithmetic_v<T>)
			return static_cast<double>(A * B);
		else
			return 0.0; // Fallback
	}

	/**
	 * Get normalized version of a value
	 */
	template <typename T>
	T GetSafeNormal(const T& Value)
	{
		if constexpr (THasGetSafeNormalMethod<T>::value)
			return Value.GetSafeNormal();
		else if constexpr (std::is_arithmetic_v<T>)
		{
			const double AbsValue = FMath::Abs(static_cast<double>(Value));
			return AbsValue < UE_KINDA_SMALL_NUMBER
				? T(0)
				: (Value > 0
					? T(1)
					: T(-1));
		}
		else
			return Value; // Fallback
	}

	/**
	 * Check if two values are equal within tolerance
	 */
	template <typename T>
	bool Equals(const T& A, const T& B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		if constexpr (THasEqualsMethod<T>::value)
			return A.Equals(B, Tolerance);
		else if constexpr (std::is_arithmetic_v<T>)
			return FMath::Abs(static_cast<float>(A - B)) <= Tolerance;
		else if constexpr (THasSubtractionOperator<T>::value)
			return Distance(A, B) <= Tolerance;
		else
			return false; // Fallback
	}


	/** Helper for validating types and inputs */
	struct FSplineValidation
	{
		/** Get the default/zero value for a type */
		template <typename T>
		static T GetDefaultValue()
		{
			if constexpr (TIsConstructible<T, EForceInit>::Value)
			{
				return T(ForceInit);
			}
			else
			{
				return T();
			}
		}

		template <typename T, int32 N>
		static bool IsValidWindow(const TArrayView<const T* const>& Window)
		{
			for (int32 i = 0; i < N; ++i)
			{
				if (!Window.IsValidIndex(i) || Window[i] == nullptr)
				{
					return false;
				}
			}
			return true;
		}

		static bool IsValidParameter(float Parameter)
		{
			return !FPlatformMath::IsNaN(Parameter) &&
				FPlatformMath::IsFinite(Parameter);
		}

		template <typename T>
		static bool IsValidGeometry(TArrayView<const T* const> Window, float Tolerance = UE_KINDA_SMALL_NUMBER)
		{
			if (Window.Num() < 2) return false;

			// Check for degenerate/repeated points
			for (int32 i = 1; i < Window.Num(); ++i)
			{
				if (DistanceSquared(*Window[i], *Window[i - 1]) < Tolerance)
				{
					return false;
				}
			}

			return true;
		}
	};

	// Cached factorial table access
	template <int32 N>
	const TArray<double>& GetFactorialTable()
	{
		static TArray<double> Table;
		if (Table.Num() == 0)
		{
			Table.SetNum(N + 1);
			Table[0] = 1.0;
			for (int32 i = 1; i <= N; ++i)
			{
				Table[i] = Table[i - 1] * i;
			}
		}
		return Table;
	}

	// Cached binomial coefficient table
	template <int32 N>
	const TArray<TArray<double>>& GetBinomialTable()
	{
		static TArray<TArray<double>> Table;
		if (Table.Num() == 0)
		{
			Table.SetNum(N + 1);
			for (int32 i = 0; i <= N; ++i)
			{
				Table[i].SetNum(i + 1);
				Table[i][0] = Table[i][i] = 1.0;
				for (int32 j = 1; j < i; ++j)
				{
					Table[i][j] = Table[i - 1][j - 1] + Table[i - 1][j];
				}
			}
		}
		return Table;
	}

	// Bezier basis computation
	template <int32 Order>
	static void ComputeBezierBasis(float t, TArray<float>& OutBasis)
	{
		OutBasis.SetNumZeroed(4); // Cubic Bezier

		const float mt = 1.0f - t;

		if constexpr (Order == 0)
		{
			// Position basis
			OutBasis[0] = mt * mt * mt;
			OutBasis[1] = 3.0f * mt * mt * t;
			OutBasis[2] = 3.0f * mt * t * t;
			OutBasis[3] = t * t * t;
		}
		else if constexpr (Order == 1)
		{
			// First derivative basis
			OutBasis[0] = -3.0f * mt * mt;
			OutBasis[1] = 3.0f * mt * (1.0f - 3.0f * t);
			OutBasis[2] = 3.0f * t * (2.0f - 3.0f * t);
			OutBasis[3] = 3.0f * t * t;
		}
		else if constexpr (Order == 2)
		{
			// Second derivative basis
			OutBasis[0] = 6.0f * mt;
			OutBasis[1] = -12.0f + 18.0f * t;
			OutBasis[2] = 6.0f - 18.0f * t;
			OutBasis[3] = 6.0f * t;
		}
		else if constexpr (Order == 3)
		{
			// Third derivative basis (constant)
			OutBasis[0] = -6.0f;
			OutBasis[1] = 18.0f;
			OutBasis[2] = -18.0f;
			OutBasis[3] = 6.0f;
		}
	}

	// compute basis functions for a given degree
	static void ComputeBSplineBasisFunctions(
		const TArray<float>& KnotVector,
		int32 Span,
		float Parameter,
		int32 Degree,
		TArray<float>& OutBasis)
	{
		OutBasis.SetNum(Degree + 1);
		TArray<float> Left;
		TArray<float> Right;
		Left.SetNum(Degree + 1);
		Right.SetNum(Degree + 1);

		OutBasis[0] = 1.0f;

		for (int32 j = 1; j <= Degree; ++j)
		{
			const int32 LeftIndex = Span + 1 - j;
			const int32 RightIndex = Span + j;

			if (LeftIndex < 0 || LeftIndex >= KnotVector.Num() ||
				RightIndex < 0 || RightIndex >= KnotVector.Num())
			{
				UE_LOGF(LogTemp, Warning, "Invalid knot indices in ComputeBasisFunctions: LeftIndex=%d, RightIndex=%d",
					LeftIndex, RightIndex);
				// Instead of continuing, we should zero out the basis and return.
				OutBasis.SetNumZeroed(Degree + 1);
				return;
			}

			Left[j] = Parameter - KnotVector[LeftIndex];
			Right[j] = KnotVector[RightIndex] - Parameter;

			float Saved = 0.0f;

			for (int32 r = 0; r < j; ++r)
			{
				float Temp;
				const float Denominator = Right[r + 1] + Left[j - r];
				// Correctly handle zero denominator
				if (FMath::IsNearlyZero(Denominator))
				{
					Temp = 0.0f;
				}
				else
				{
					Temp = OutBasis[r] / Denominator;
				}

				OutBasis[r] = Saved + Right[r + 1] * Temp;
				Saved = Left[j - r] * Temp;
			}

			OutBasis[j] = Saved;
		}
	}

	/** Template-based derivative calculators for spline types */

	// Helper struct for shared derivative implementation
	template <typename T, int32 Order>
	struct TGenericDerivativeHelper
	{
		static T Compute(TArrayView<const T* const> Window, float Parameter)
		{
			// Basic validation
			if (!FSplineValidation::IsValidWindow<T, Order+1>(Window))
			{
				return FSplineValidation::GetDefaultValue<T>();
			}

			if (!FSplineValidation::IsValidParameter(Parameter))
			{
#if ENABLE_NAN_DIAGNOSTIC
				logOrEnsureNanError(TEXT("Invalid parameter in derivative calculation"));
#endif
				return FSplineValidation::GetDefaultValue<T>();
			}

			// Get pre-computed binomial coefficients
			static const TArray<TArray<double>>& BinomialTable = GetBinomialTable<20>();
			static const TArray<double>& FactorialTable = GetFactorialTable<20>();

			const int32 WindowSize = Window.Num();
			if (Order >= WindowSize || Order >= BinomialTable.Num())
			{
				return FSplineValidation::GetDefaultValue<T>();
			}

			// Optional geometry validation for higher derivatives
			if (Order > 1 && !FSplineValidation::IsValidGeometry(Window))
			{
				return FSplineValidation::GetDefaultValue<T>();
			}

			T Result = FSplineValidation::GetDefaultValue<T>();
			const float t = Parameter;
			const float mt = 1.0f - t;

			// Compute derivative using binomial expansion
			for (int32 i = 0; i <= WindowSize - Order - 1; ++i)
			{
				double Coeff = BinomialTable[WindowSize - 1][i];

				// Protect against overflow
				if (Coeff > DBL_MAX / (WindowSize - 1))
				{
					return FSplineValidation::GetDefaultValue<T>();
				}

				// Apply derivative scaling
				Coeff *= FactorialTable[WindowSize - 1] / FactorialTable[WindowSize - Order - 1];

				// Apply parameter terms with underflow protection
				if (mt > 0.0f)
				{
					const float PowMT = FMath::Pow(mt, WindowSize - Order - 1 - i);
					if (FPlatformMath::IsFinite(PowMT))
					{
						Coeff *= PowMT;
					}
				}
				if (t > 0.0f)
				{
					const float PowT = FMath::Pow(t, i);
					if (FPlatformMath::IsFinite(PowT))
					{
						Coeff *= PowT;
					}
				}

				Result += *Window[i] * Coeff;
			}
			return Result;
		}
	};

	// Computes the nth derivative (where n == DerivOrder) of a BSpline curve.
	// Note: This implementation assumes that DerivOrder <= Degree.
	template <typename T, int32 DerivOrder>
	struct TBSplineDerivativeCalculator
	{
		/**
		 * Computes the DerivOrder-th derivative of a BSpline curve at a given parameter.
		 *
		 * @param Window     A view of pointers to control points for the current BSpline segment.
		 *                   For a BSpline of degree p, this view should contain p+1 control points.
		 * @param Knots      The global knot vector.
		 * @param Span       The knot span index such that Parameter ∈ [Knots[Span], Knots[Span+1]).
		 * @param Parameter  The parameter at which to evaluate.
		 * @param Degree     The degree of the current BSpline (should be ≥ DerivOrder).
		 *
		 * @return The DerivOrder-th derivative (of type T) evaluated at Parameter.
		 */
		static T Compute(
			TArrayView<const T* const> Window,
			const TArray<float>& Knots,
			int32 Span,
			float Parameter,
			int32 Degree)
		{
			// Basic validation
			if (Window.Num() < Degree + 1 || DerivOrder > Degree)
			{
				return T();
			}

			TArray<T> DerivControlPoints;
			DerivControlPoints.SetNum(Degree);
			const int32 KnotStart = Span - Degree;

			// Determine if knots are uniformly spaced
			bool bIsUniform = true;
			const float h = Knots[1] - Knots[0];
			for (int32 i = 2; i < Knots.Num(); ++i)
			{
				if (!FMath::IsNearlyEqual(Knots[i] - Knots[i - 1], h, UE_KINDA_SMALL_NUMBER))
				{
					bIsUniform = false;
					break;
				}
			}

			// Compute derivative control points
			for (int32 i = 0; i < Degree; ++i)
			{
				const int32 KnotDenom = KnotStart + i + Degree - DerivOrder + 2; //Knot index for the denominator
				const int32 KnotNum = KnotStart + i + 1; //Knot index in the numerator

				if (KnotDenom >= Knots.Num() || KnotNum >= Knots.Num() ||
					KnotDenom < 0 || KnotNum < 0)
				{
					return T();
				}

				const float Denom = Knots[KnotDenom] - Knots[KnotNum];

				// Handle uniform knots differently
				if (bIsUniform)
				{
					// For uniform knots, use the fixed interval h instead of Denom
					DerivControlPoints[i] = ((*Window[i + 1] - *Window[i])) * (static_cast<float>(Degree) / h);
					//Correct scaling for uniform knots
				}
				else
				{
					DerivControlPoints[i] = ((*Window[i + 1] - *Window[i])) * (static_cast<float>(Degree) / Denom);
					//Correct scaling for non-uniform knots
				}
			}

			// Adjust scale factor for uniform knots
			//Removing the scale factor
			float ScaleFactor = 1.0;

			// Recurse for higher derivatives with adjusted knots and parameters
			if constexpr (DerivOrder > 1)
			{
				//No change here
				TArray<const T*> DerivWindow;
				DerivWindow.SetNum(DerivControlPoints.Num());
				for (int32 i = 0; i < DerivControlPoints.Num(); ++i)
				{
					DerivWindow[i] = &DerivControlPoints[i];
				}

				// Trim the knot vector for uniform case
				TArray<float> TrimmedKnots;
				if (bIsUniform)
				{
					TrimmedKnots.Reserve(Knots.Num() - 2);
					for (int32 i = 1; i < Knots.Num() - 1; ++i)
					{
						TrimmedKnots.Add(Knots[i]);
					}
				}
				else
				{
					TrimmedKnots = Knots;
				}

				return TBSplineDerivativeCalculator<T, DerivOrder - 1>::Compute(
					DerivWindow, //Reduced degree control points
					TrimmedKnots, //Remove the first and the last knot as the degree is now p-1
					Span, // Keep Span the same for uniform knots
					Parameter, //Same parameter
					Degree - 1 //Reduce the degree
				) * ScaleFactor; //Multiply with scale factor
			}
			else
			{
				// Base case: First derivative
				TArray<float> Basis;
				Math::ComputeBSplineBasisFunctions(Knots, Span, Parameter, Degree - 1, Basis);

				T Result = T();
				for (int32 i = 0; i < Degree; ++i)
				{
					Result += DerivControlPoints[i] * Basis[i];
				}
				return Result * ScaleFactor;
			}
		}
	};

	// Base case: 0th derivative is just the original BSpline evaluation.
	template <typename T>
	struct TBSplineDerivativeCalculator<T, 0>
	{
		/**
		 * Evaluates the BSpline curve at Parameter using de Boor's algorithm.
		 *
		 * @param Window     A view of pointers to the BSpline's control points (should be Degree+1 points).
		 * @param Knots      The (possibly trimmed) knot vector.
		 * @param Span       The knot span index.
		 * @param Parameter  The parameter at which to evaluate.
		 * @param Degree     The degree of the BSpline.
		 *
		 * @return The BSpline value (of type T) at Parameter.
		 */
		static T Compute(
			TArrayView<const T* const> Window,
			const TArray<float>& Knots,
			int32 Span,
			float Parameter,
			int32 Degree)
		{
			// Regular evaluation using current basis functions
			TArray<float> Basis;
			ComputeBSplineBasisFunctions(Knots, Span, Parameter, Degree, Basis);

			T Result = T();
			for (int32 i = 0; i <= Degree; ++i)
			{
				Result += *Window[i] * Basis[i];
			}
			return Result;
		}
	};


	/**
	 * Bezier derivative calculator.
	 * Computes derivatives and applies segment scaling.
	 * 
	 * @tparam T Value type (FVector, FQuat, FTransform, etc.)
	 * @tparam Order Derivative order (0=position, 1=velocity, 2=acceleration, etc.)
	 */
	template <typename T, int32 Order>
	struct TBezierDerivativeCalculator
	{
		static T Compute(TArrayView<const T* const> Window, float Parameter, float SegmentScale)
		{
			if constexpr (Order > 3)
			{
				return FSplineValidation::GetDefaultValue<T>();
			}

			if (!FSplineValidation::IsValidWindow<T, 4>(Window) || // Cubic Bezier needs 4 points
				!FSplineValidation::IsValidParameter(Parameter))
			{
				return FSplineValidation::GetDefaultValue<T>();
			}

			// Use existing basis computation
			TArray<float> Basis;
			ComputeBezierBasis<Order>(Parameter, Basis);

			// Apply basis to control points
			T Result = FSplineValidation::GetDefaultValue<T>();
			for (int32 i = 0; i < Window.Num(); ++i)
			{
				Result += *Window[i] * Basis[i];
			}

			// Apply segment scaling based on derivative order
			// For derivatives, we need to divide by SegmentScale^Order because 
			// derivatives scale inversely with parameter range
			if constexpr (Order > 0)
			{
				// If L == 0, the segment is degenerate in parameter space
				// return the zero derivative (well-defined limit)
				if (SegmentScale == 0.0f)
				{
					return FSplineValidation::GetDefaultValue<T>();
				}
				
				// Calculate scaling factor: 1 / SegmentScale^Order
				float ScalingFactor = FMath::Pow(1.0f / SegmentScale, static_cast<float>(Order));
				return Result * ScalingFactor;
			}
			else
			{
				return Result; // No scaling needed for position (Order = 0)
			}
		}
	};


	/** 
	 * Find nearest point on a cubic Bezier curve segment to a target point
	 * @param Coeffs - Array of 4 coefficients in Bezier form relative to target point
	 * @param StartParam - Start parameter of segment
	 * @param EndParam - End parameter of segment  
	 * @param OutSquaredDistance - Output squared distance to nearest point
	 * @return Parameter at nearest point between StartParam and EndParam
	 */

	template <typename ValueType>
	float FindNearestPoint_Cubic(
		const TArrayView<ValueType> Coeffs,
		float StartParam,
		float EndParam,
		float& OutSquaredDistance)
	{
		// Test endpoints first
		const ValueType& P0 = Coeffs[0]; // Already relative to target
		const ValueType At1 = P0 + Coeffs[1] + Coeffs[2] + Coeffs[3]; // Curve at t=1

		float BestDistSq = static_cast<float>(Math::SizeSquared(P0));
		float BestParam = StartParam;

		const float EndDistSq = static_cast<float>(Math::SizeSquared(At1));
		if (EndDistSq < BestDistSq)
		{
			BestDistSq = EndDistSq;
			BestParam = EndParam;
		}

		// Find roots of (C(t) - P).Dot(C'(t))
		const ValueType& P1 = Coeffs[1];
		const ValueType& P2 = Coeffs[2];
		const ValueType& P3 = Coeffs[3];

		TStaticArray<double, 6> RootCoeffs;
		RootCoeffs[5] = 3.0 * Dot(P3, P3);
		RootCoeffs[4] = 5.0 * Dot(P3, P2);
		RootCoeffs[3] = 4.0 * Dot(P3, P1) + 2.0 * Dot(P2, P2);
		RootCoeffs[2] = 3.0 * Dot(P2, P1) + 3.0 * Dot(P3, P0);
		RootCoeffs[1] = Dot(P1, P1) + 2.0 * Dot(P2, P0);
		RootCoeffs[0] = Dot(P1, P0);

		// Find roots in [0,1]  
		UE::Math::TPolynomialRootSolver<double, 5> RootSolver(RootCoeffs, 0, 1);

		// Check each root
		for (int32 RootIdx = 0; RootIdx < RootSolver.Roots.Num(); ++RootIdx)
		{
			const float t = static_cast<float>(RootSolver.Roots[RootIdx]);
			const float Param = FMath::Lerp(StartParam, EndParam, t);

			const ValueType Point = P0 +
				P1 * t +
				P2 * (t * t) +
				P3 * (t * t * t);

			const float DistSq = static_cast<float>(Math::SizeSquared(Point));
			if (DistSq < BestDistSq)
			{
				BestDistSq = DistSq;
				BestParam = Param;
			}
		}

		OutSquaredDistance = BestDistSq;
		return BestParam;
	}

	template <typename ValueType>
	ValueType ComputeFirstDerivativeCentralDifference(const TSplineInterface<ValueType>& Spline, double Param,
													  double Step = UE_KINDA_SMALL_NUMBER)
	{
		FInterval1f ParameterSpace = Spline.GetParameterSpace();
		Step = FMath::Max(Step, UE_KINDA_SMALL_NUMBER);

		const double ParamRight = FMath::Clamp(Param + Step, ParameterSpace.Min, ParameterSpace.Max);
		const double ParamLeft = FMath::Clamp(Param - Step, ParameterSpace.Min, ParameterSpace.Max);

		const ValueType ValueRight = Spline.Evaluate(ParamRight);
		const ValueType ValueLeft = Spline.Evaluate(ParamLeft);

		return (ValueRight - ValueLeft) / (2.0 * Step);
	}

	template <typename ValueType>
	ValueType ComputeSecondDerivativeCentralDifference(const TSplineInterface<ValueType>& Spline, double Param, double Step = 0.1)
	{
		FInterval1f ParameterSpace = Spline.GetParameterSpace();
		Step = FMath::Max(Step, UE_KINDA_SMALL_NUMBER);

		const double ParamRight = FMath::Clamp(Param + Step, ParameterSpace.Min, ParameterSpace.Max);
		Param = FMath::Clamp(Param, ParameterSpace.Min, ParameterSpace.Max);
		const double ParamLeft = FMath::Clamp(Param - Step, ParameterSpace.Min, ParameterSpace.Max);

		const ValueType ValueRight = Spline.Evaluate(ParamRight);
		const ValueType Value = Spline.Evaluate(Param);
		const ValueType ValueLeft = Spline.Evaluate(ParamLeft);

		return (ValueRight - 2.0 * Value + ValueLeft) / (Step * Step);
	}
} // end namespace UE::Geometry::Spline::Math
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE
