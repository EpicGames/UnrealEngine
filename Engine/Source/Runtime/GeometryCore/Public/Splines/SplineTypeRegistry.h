// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SplineTypeId.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "SplineInterfaces.h" 
#include "Templates/Function.h"

namespace UE
{
namespace Geometry
{
namespace Spline
{

/**
 * Registry for spline types
 * Handles registration, lookup, and creation of spline implementations
 */
class FSplineTypeRegistry
{
public:
    // Type ID definition
    using TypeId = FSplineTypeId::IdType;
    
    /**
     * Factory function type for creating spline instances
     */
    using FactoryFunction = TFunction<TUniquePtr<ISplineInterface>()>;
    
    /**
     * Registers a type ID with a factory function
     * 
     * @param TypeId The type ID to register
     * @param ImplName Implementation name
     * @param ValueTypeName Value type name
     * @param Factory Function that creates instances of this type
     * @return True if registration succeeded
     */
    static inline bool RegisterType(
        TypeId TypeId,
        const FString& ImplName, 
        const FString& ValueTypeName,
        FactoryFunction Factory)
    {
        // Check if this type is already registered
        TPair<FString, FString> NamePair(ImplName, ValueTypeName);
		UE::TScopeLock Lock(GetRegistryLock());
        
        // Check for existing registration with same ID but different names (hash collision)
        if (const TPair<FString, FString>* ExistingNames = GetIdToNames().Find(TypeId))
        {
            if (FSplineTypeId::CheckTypeIdConflict(TypeId, ImplName, ValueTypeName, 
                                                  ExistingNames->Key, ExistingNames->Value))
            {
                // Hash collision detected - registration failed
                return false;
            }
            
            // Type already registered with same names - just check that factory exists
            if (!GetFactories().Contains(TypeId))
            {
                // Factory missing but ID registered - add missing factory
                GetFactories().Add(TypeId, MoveTemp(Factory));
            }
            
            // Already registered correctly
            return true;
        }
        
        // New registration - add to all maps
        GetIdToNames().Add(TypeId, NamePair);
        GetNamesToId().Add(NamePair, TypeId);
        GetFactories().Add(TypeId, MoveTemp(Factory));
        
        UE_LOGF(LogSpline, Log, "Registered spline type: TypeId=0x%08X, Impl=%ls, Value=%ls",
            TypeId, *ImplName, *ValueTypeName);
        
        return true;
    }
    
    /**
     * Creates a spline instance from a type ID
     * 
     * @param TypeId The type ID to create
     * @return A new spline instance or nullptr if not found
     */
    static inline TUniquePtr<ISplineInterface> CreateSpline(TypeId TypeId)
    {
		UE::TScopeLock Lock(GetRegistryLock());
        
        const FactoryFunction* Factory = GetFactories().Find(TypeId);
        if (Factory)
        {
            return (*Factory)();
        }
        
        // Log helpful error message
        const TPair<FString, FString>* Names = GetIdToNames().Find(TypeId);
        if (Names)
        {
            UE_LOGF(LogSpline, Warning, "No factory for type ID 0x%08X (Impl=%ls, Value=%ls)",
                TypeId, *Names->Key, *Names->Value);
        }
        else
        {
            UE_LOGF(LogSpline, Warning, "Unknown type ID 0x%08X", TypeId);
        }
        
        return nullptr;
    }
    
    /**
     * Creates a spline instance by name pair
     * 
     * @param ImplName Implementation name
     * @param ValueTypeName Value type name
     * @return A new spline instance or nullptr if not found
     */
    static inline TUniquePtr<ISplineInterface> CreateSpline(
        const FString& ImplName,
        const FString& ValueTypeName)
    {
        TPair<FString, FString> NamePair(ImplName, ValueTypeName);
        
		UE::TScopeLock Lock(GetRegistryLock());
        
        const TypeId* TypeId = GetNamesToId().Find(NamePair);
        if (TypeId && *TypeId != 0)
        {
            return CreateSpline(*TypeId);
        }
        
        UE_LOGF(LogSpline, Warning, "No registered type for Impl=%ls, Value=%ls",
            *ImplName, *ValueTypeName);
        return nullptr;
    }
    
    /**
     * Gets the names associated with a type ID
     * 
     * @param TypeId The type ID to look up
     * @param OutImplName [out] Implementation name
     * @param OutValueTypeName [out] Value type name
     * @return True if the type ID was found
     */
    static inline bool GetTypeNames(
        TypeId TypeId,
        FString& OutImplName,
        FString& OutValueTypeName)
    {
        UE::TScopeLock Lock(GetRegistryLock());
        
        const TPair<FString, FString>* Names = GetIdToNames().Find(TypeId);
        if (Names)
        {
            OutImplName = Names->Key;
            OutValueTypeName = Names->Value;
            return true;
        }
        return false;
    }
    
    /**
     * Gets the type ID for a name pair
     * 
     * @param ImplName Implementation name
     * @param ValueTypeName Value type name
     * @return The type ID or 0 if not found
     */
    static inline TypeId GetTypeId(
        const FString& ImplName,
        const FString& ValueTypeName)
    {
        TPair<FString, FString> NamePair(ImplName, ValueTypeName);
        
		UE::TScopeLock Lock(GetRegistryLock());
        
        const TypeId* Id = GetNamesToId().Find(NamePair);
        if (Id)
        {
            return *Id;
        }
        return 0;
    }
    
private:
    /**
     * Gets the registry lock (thread-safe singleton access)
     */
    static inline FTransactionallySafeCriticalSection& GetRegistryLock()
    {
        static FTransactionallySafeCriticalSection RegistryLock;
        return RegistryLock;
    }
    
    /**
     * Gets the map from type ID to name pair (thread-safe singleton access)
     */
    static inline TMap<TypeId, TPair<FString, FString>>& GetIdToNames()
    {
        static TMap<TypeId, TPair<FString, FString>> IdToNames;
        return IdToNames;
    }
    
    /**
     * Gets the map from name pair to type ID (thread-safe singleton access)
     */
    static inline TMap<TPair<FString, FString>, TypeId>& GetNamesToId()
    {
        static TMap<TPair<FString, FString>, TypeId> NamesToId;
        return NamesToId;
    }
    
    /**
     * Gets the map from type ID to factory function (thread-safe singleton access)
     */
    static inline TMap<TypeId, FactoryFunction>& GetFactories()
    {
        static TMap<TypeId, FactoryFunction> Factories;
        return Factories;
    }
};

/**
 * Self-registering mixin template for spline classes
 * Uses CRTP to enable automatic registration on first instantiation
 * 
 * @tparam DerivedType The spline class being registered
 * @tparam ValueType The value type of the spline
 */
template<typename DerivedType, typename ValueType>
class TSelfRegisteringSpline
{
protected:
    // Constructor ensures type is registered
    TSelfRegisteringSpline()
    {
        static bool bRegistered = false;
        if (!bRegistered)
        {
            bRegistered = true;
            RegisterType();
        }
    }
    
    // Virtual destructor for safe inheritance
    virtual ~TSelfRegisteringSpline() = default;
    
private:
    // Register this type with the type registry
    static inline void RegisterType()
    {
        // Get type information from the derived class
        FSplineTypeRegistry::TypeId TypeId = DerivedType::GetStaticTypeId();
        FString ImplName = DerivedType::GetSplineTypeName();
        FString ValueName = TSplineValueTypeTraits<ValueType>::Name;
        
        // Register factory function
        FSplineTypeRegistry::RegisterType(
            TypeId,
            ImplName,
            ValueName,
            []() { return MakeUnique<DerivedType>(); }
        );
    }
};

} // namespace Spline
} // namespace Geometry
} // namespace UE