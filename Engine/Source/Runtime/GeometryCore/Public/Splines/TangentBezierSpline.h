// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TangentsPolicy.h"
#include "Splines/PolyBezierSpline.h"
namespace UE
{
namespace Geometry
{
namespace Spline
{
	
template<typename ValueType> struct UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TTangentBezierControlPoint;
template<typename ValueType> class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") TTangentBezierSpline;
	
/**
 * Enum defining how tangents are computed for a spline control point.
 */
enum class UE_EXPERIMENTAL(5.7, "New spline APIs are experimental.") ETangentMode : uint8
{
	/** FInterpCurve: Copies prev tangents if prev is not curve key */
	LegacyAuto,
	
	/** FInterpCurve: Clamped version with copying behavior */
	LegacyAutoClamped,
	
	/** User-specified tangents */
	User,
	
	/** Broken tangents - in and out can be different */
	Broken,
	
	/** FInterpCurve behavior (continuity-preserving) */
	LegacyLinear,
		
	/** Constant - no interpolation between points */
	Constant,
	
	/** Automatically compute unclamped tangents based on surrounding points */
	Auto,
    
	/** Automatically compute clamped tangents based on surrounding points */
	AutoClamped,
    
	/** Linear tangents aligned with adjacent points */
	Linear,
    
	/** Unknown or invalid tangent mode */
	Unknown
};

/**
 * Structure representing a control point on a tangent-based spline curve
 */
template<typename ValueType>
struct TTangentBezierControlPoint 
{
	/** Position of the control point */
	ValueType Position;

	/** Incoming tangent vector */
	ValueType TangentIn;

	/** Outgoing tangent vector */
	ValueType TangentOut;

	/** Tangent computation mode */
	ETangentMode TangentMode;

	/** Default constructor creates a point at origin with zero tangents */
	TTangentBezierControlPoint ()
		: Position(ValueType())
		, TangentIn(ValueType())
		, TangentOut(ValueType())
		, TangentMode(ETangentMode::Auto)
	{
	}

	/** Constructor with position only - defaults to auto tangents */
	explicit TTangentBezierControlPoint (const ValueType& InPosition)
		: Position(InPosition)
		, TangentIn(ValueType())
		, TangentOut(ValueType())
		, TangentMode(ETangentMode::Auto)
	{
	}

	/** Full constructor */
	TTangentBezierControlPoint (
		const ValueType& InPosition,
		const ValueType& InTangentIn,
		const ValueType& InTangentOut,
		ETangentMode InTangentMode)
		: Position(InPosition)
		, TangentIn(InTangentIn)
		, TangentOut(InTangentOut)
		, TangentMode(InTangentMode)
	{
	}
	
	void Serialize(FArchive& Ar)
	{
		Ar << Position;
		Ar << TangentIn;
		Ar << TangentOut;
		// Serialize as uint8 for compatibility
		uint8 TangentModeValue = static_cast<uint8>(TangentMode);
		Ar << TangentModeValue;
        
		if (Ar.IsLoading())
		{
			TangentMode = static_cast<ETangentMode>(TangentModeValue);
		}
		
	}
    
	friend FArchive& operator<<(FArchive& Ar, TTangentBezierControlPoint& Point)
	{
		Point.Serialize(Ar);
		return Ar;
	}

	bool operator==(const TTangentBezierControlPoint& Other) const
	{
		return Position == Other.Position
			&& TangentIn == Other.TangentIn
			&& TangentOut == Other.TangentOut
			&& TangentMode == Other.TangentMode;
	}
};
	
/**
 * A spline that provides tangent-based control over curve shape while using 
 * piecewise Bezier curves internally for evaluation. Supports both manual
 * tangent control and automatic tangent computation.
 */
template<typename InValueType>
class TTangentBezierSpline :
	public TSplineWrapper<TPolyBezierSpline<InValueType>>,
	private TSelfRegisteringSpline<TTangentBezierSpline<InValueType>, InValueType>
{
public:
	
	using ValueType 					= InValueType;
	using Base 							= TPolyBezierSpline<ValueType>;
	using FTangentBezierControlPoint 	= TTangentBezierControlPoint<ValueType>;
	using FWindow      					= typename Base::FWindow;
	using FWindowStore 					= typename Base::FWindowStore;
	using TangentOps 					= Math::TTangentOps<ValueType>;
	
	// Generate compile-time type ID for TangentBezier
	DECLARE_SPLINE_TYPE_ID(
		TEXT("TangentBezier"),
		*TSplineValueTypeTraits<ValueType>::Name
	);
	
	TTangentBezierSpline()
	{
		InstallFindWindowOverride();
	}
	
	virtual ~TTangentBezierSpline() override = default;
	
	/** Default constructor with at least one segment */
	TTangentBezierSpline(const ValueType& StartPoint, const ValueType& EndPoint)
		: TTangentBezierSpline()
 
	{
		Tension = 0.f;
		TangentModes = {ETangentMode::Auto, ETangentMode::Auto};
		AssignInternalSpline(Base::CreateLine(StartPoint, EndPoint));
	}
   
    /**  Copy constructor */
    TTangentBezierSpline(const TTangentBezierSpline& Other)
	: TTangentBezierSpline()
	{
		Tension = Other.Tension;
		TangentModes = Other.TangentModes;
		AssignInternalSpline(Other.InternalSpline);
	}

    /**  Copy assignment */
    TTangentBezierSpline& operator=(const TTangentBezierSpline& Other)
    {
        if (this != &Other)
        {
            Tension = Other.Tension;
            TangentModes = Other.TangentModes;
			bStationaryEndpoint = Other.bStationaryEndpoint;
        	AssignInternalSpline(Other.InternalSpline);
        }
        return *this;
    }

	/** Move constructor */
	TTangentBezierSpline(TTangentBezierSpline&& Other) noexcept
		: TTangentBezierSpline() 
	{
		Tension = Other.Tension;
		TangentModes = MoveTemp(Other.TangentModes);
		bStationaryEndpoint = Other.bStationaryEndpoint;
		AssignInternalSpline(MoveTemp(Other.InternalSpline));
	}
	/** Move assignment */
	TTangentBezierSpline& operator=(TTangentBezierSpline&& Other) noexcept
	{
		if (this != &Other)
		{
			Tension = Other.Tension;
			TangentModes = MoveTemp(Other.TangentModes);
			bStationaryEndpoint = Other.bStationaryEndpoint;
			AssignInternalSpline(MoveTemp(Other.InternalSpline));
		}
		return *this;
	}

	virtual bool IsEqual(const ISplineInterface* OtherSpline) const override
	{
		if (OtherSpline->GetTypeId() == GetTypeId())
		{
			const TTangentBezierSpline* Other = static_cast<const TTangentBezierSpline*>(OtherSpline);
			return operator==(*Other);
		}
		
		return false;
	}

    virtual bool Serialize(FArchive& Ar) override
    {
		// Call immediate parent's Serialize (TSplineWrapper)
		if (!TSplineWrapper<TPolyBezierSpline<ValueType>>::Serialize(Ar))
		{
			return false;
		}
        
        Ar << Tension;
		// Serialize as uint8 for compatibility
		int32 NumTangentModes = TangentModes.Num();
		Ar << NumTangentModes;
        
		if (Ar.IsLoading())
		{
			TangentModes.SetNum(NumTangentModes);

			InstallFindWindowOverride();
		}
        
		for (int32 i = 0; i < NumTangentModes; ++i)
		{
			uint8 TangentModeValue = static_cast<uint8>(TangentModes[i]);
			Ar << TangentModeValue;
            
			if (Ar.IsLoading())
			{
				TangentModes[i] = static_cast<ETangentMode>(TangentModeValue);
			}
		}

		Ar << bStationaryEndpoint;
		
		if (Ar.IsLoading())
		{
			const int32 OldNumTangentModes = TangentModes.Num();
			const int32 NumPoints = GetNumPoints();

			if (OldNumTangentModes != NumPoints)
			{
				UE_LOGF(LogSpline, Display, "Incorrect number of tangent modes loaded (loaded %d, expected %d), fixing up.", OldNumTangentModes, NumPoints);

				TangentModes.SetNum(NumPoints);
				const int32 NewNumTangentModes = TangentModes.Num();

				for (int32 TangentModeIdx = OldNumTangentModes; TangentModeIdx < NewNumTangentModes; ++TangentModeIdx)
				{
					TangentModes[TangentModeIdx] = ETangentMode::Auto;
				}
			}
		}

        return true;
    }

    friend FArchive& operator<<(FArchive& Ar, TTangentBezierSpline& Spline)
    {
        Spline.Serialize(Ar);
        return Ar;
    }

	bool operator==(const TTangentBezierSpline<ValueType>& Other) const
	{
		return	InternalSpline == Other.InternalSpline &&
				Tension == Other.Tension &&
				TangentModes == Other.TangentModes &&
				bStationaryEndpoint == Other.bStationaryEndpoint;
	}

	static Base::FAnchor AnchorFromControlPoint(const FTangentBezierControlPoint& InPoint)
	{
		return {
			TangentOps::TangentInToP2(InPoint.Position, InPoint.TangentIn),
			InPoint.Position,
			TangentOps::TangentOutToP1(InPoint.Position, InPoint.TangentOut)
		};
	}

	// Static shape generators

    /**
     * Creates a straight line between two points
     * @param StartPoint - Start point of the line
     * @param EndPoint - End point of the line
     * @return New TangentBezierSpline initialized as a line
     */
    static TTangentBezierSpline CreateLine(
        const ValueType& StartPoint, 
        const ValueType& EndPoint)
    {
		// Create empty spline - we'll manually build it
		TTangentBezierSpline Result;
        
		// Clear the default initialization
		Result.AssignInternalSpline(Base::CreateLine(StartPoint, EndPoint));
		
        Result.Reparameterize(EParameterizationPolicy::Uniform);
        return Result;
    }

    /**
     * Creates a circular arc with specified parameters
     * @param Center - Center point of the circle
     * @param Radius - Radius of the circle
     * @param StartAngle - Start angle in radians
     * @param EndAngle - End angle in radians
     * @param NumSegments - Number of segments to use (more segments = smoother curve)
     * @return New TangentBezierSpline initialized as an arc
     */
    static TTangentBezierSpline CreateCircleArc(
        const ValueType& Center,
        float Radius,
        float StartAngle,
        float EndAngle,
        int32 NumSegments = 4)
    {
        // Create empty spline - we'll manually build it
        TTangentBezierSpline Result;
        
        // Clear the default initialization
        Result.AssignInternalSpline(Base::CreateCircleArc(
            Center, Radius, StartAngle, EndAngle, NumSegments));
        
		// Set up tangent status for all points
		Result.TangentModes.Init(ETangentMode::Auto, Result.GetNumPoints());
        
        // Apply consistent parameterization - even though the internal PolyBezier already calls
        // Reparameterize, we need to do it again here to ensure consistency
        Result.Reparameterize(EParameterizationPolicy::Uniform);
        
        return Result;
    }

    /**
     * Creates a complete circle
     * @param Center - Center point of the circle
     * @param Radius - Radius of the circle
     * @param NumSegments - Number of segments to use (more segments = smoother curve)
     * @return New TangentBezierSpline initialized as a circle
     */
    static TTangentBezierSpline CreateCircle(
        const ValueType& Center,
        float Radius,
        int32 NumSegments = 4)
    {
        // Create the circle
        TTangentBezierSpline Result;
        Result.AssignInternalSpline(Base::CreateCircle(
            ValueType(Center), Radius, NumSegments));
        
        // Set up tangent status for all points
		Result.TangentModes.Init(ETangentMode::Auto, Result.GetNumPoints());
		
		// Apply consistent parameterization
		Result.Reparameterize(EParameterizationPolicy::Uniform);
        
        return Result;
    }

    /**
     * Creates an ellipse with specified parameters
     * @param Center - Center point of the ellipse
     * @param RadiusX - Radius along X axis
     * @param RadiusY - Radius along Y axis
     * @param NumSegments - Number of segments to use (more segments = smoother curve)
     * @return New TangentBezierSpline initialized as an ellipse
     */
    static TTangentBezierSpline CreateEllipse(
        const ValueType& Center,
        float RadiusX,
        float RadiusY,
        int32 NumSegments = 4)
    {
        // Create the ellipse
        TTangentBezierSpline Result;
        Result.AssignInternalSpline(Base::CreateEllipse(
            Center, RadiusX, RadiusY, NumSegments));
        
		// Set up tangent status for all points
		Result.TangentModes.Init(ETangentMode::Auto, Result.GetNumPoints());
        
        // Apply consistent parameterization
        Result.Reparameterize(EParameterizationPolicy::Uniform);
        
        return Result;
    }

	virtual TUniquePtr<ISplineInterface> Clone() const override
	{
		TUniquePtr<TTangentBezierSpline<ValueType>> Clone = MakeUnique<TTangentBezierSpline<ValueType>>();
    
		// Copy internal spline
		Clone->AssignInternalSpline(this->InternalSpline);
    
		// Copy tangent settings
		Clone->Tension = this->Tension;
		Clone->TangentModes = this->TangentModes;
		Clone->bStationaryEndpoint = this->bStationaryEndpoint;
    
		// Copy infinity modes
		Clone->PreInfinityMode = this->PreInfinityMode;
		Clone->PostInfinityMode = this->PostInfinityMode;
    
		return Clone;
	}

	/**
	 * Replaces all points in the spline with the provided points.
	 * @param Points - Array of control points to set
	 * @param Policy - The policy to use for knot generation as the points are added
	 * @return True if the points were successfully set
	 */
	bool SetPoints(const TArray<FTangentBezierControlPoint>& Points, EParameterizationPolicy Policy)
	{		
		Clear();
		TangentModes.Reset(Points.Num());

		for (const FTangentBezierControlPoint& Point : Points)
		{
			AppendPoint(Point, Policy);
		}
	    	    
	    return true;
	}

	/**
	 * Appends multiple points to the spline.
	 * @param Points - Array of control points to add
	 * @param Policy - The policy to use for knot generation as the points are added
	 * @return True if the points were successfully added
	 */
	bool AppendPoints(const TArray<FTangentBezierControlPoint>& Points, EParameterizationPolicy Policy)
     {
        const int32 NumPoints = Points.Num();
        
        // Need at least 2 points to add a valid segment
        if (NumPoints < 2)
        {
            UE_LOGF(LogSpline, Warning, "AppendPoints requires at least 2 points to add a valid segment. Got %d points.", NumPoints);
            return false;
        }
        
        // If the spline is currently empty (though it shouldn't be based on our design),
        // this is equivalent to SetPoints
        if (GetNumPoints() == 0)
        {
            return SetPoints(Points, Policy);
        }

		// For each point, use AddPoint 
		for (int32 i = 0; i < NumPoints; ++i)
		{
			// Skip the first point if it's the same as our last existing point 
			// (to avoid duplicates at connection point)
			if (i == 0)
			{
				const int32 ExistingNumPoints = GetNumPoints();
				const ValueType LastExistingPoint = GetValue(ExistingNumPoints - 1);
                    
				// Check if connection points are reasonably close
				constexpr double ConnectionTolerance = 1e-4;
				bool bSameEndpoint;

				if constexpr (Math::TIsQuaternionType<ValueType>::value)
				{
					// angular distance
					const double dot = Math::Dot(LastExistingPoint, Points[0].Position);
					const double ang = FMath::Acos(FMath::Clamp(dot, -1.0, 1.0)); // radians
					bSameEndpoint = (ang <= ConnectionTolerance);
				}
				else
				{
					const double DistanceSquared = Math::SizeSquared(LastExistingPoint - Points[0].Position);
					bSameEndpoint = (DistanceSquared <= ConnectionTolerance);
				}
                    
				if (bSameEndpoint)
				{
					// Points are close - the first existing point is the same as our first new point
					// So update the last point's tangent mode if needed
					if (Points[0].TangentMode != ETangentMode::User && Points[0].TangentMode != ETangentMode::Broken)
					{
						SetPointTangentMode(ExistingNumPoints - 1, Points[0].TangentMode);
					}
					continue; // Skip adding this point
				}
			}
                
			// Add each point normally
			AppendPoint(Points[i], Policy);
		}
        
        return true;
    }
    
	/**
	 * Prepends a control point to the beginning of the spline.
	 * @param Point - Control point to prepend
	 * @param Parameter - The parameter of the control point being prepended, must be less than all existing knots.
	 * @return Index of the new point, or INDEX_NONE if the operation failed.
	 */
	int32 PrependPoint(const FTangentBezierControlPoint& Point, float Parameter)
	{
		// Add the anchor
		int32 NewIndex = InternalSpline.PrependAnchor(AnchorFromControlPoint(Point), Parameter);

		if (NewIndex != INDEX_NONE)
		{
			// Set tangent mode
			TangentModes.InsertDefaulted(NewIndex);
			SetPointTangentMode(NewIndex, Point.TangentMode);
		}

		return NewIndex;
	}

    /**
	 * Prepends a control point to the beginning of the spline.
	 * @param Point - Control point to prepend.
	 * @param Policy - The policy to use for generating the knot for the point being prepended.
	 * @return Index of the new point, or INDEX_NONE if the operation failed.
	 */
	int32 PrependPoint(const FTangentBezierControlPoint& Point, EParameterizationPolicy Policy)
	{
		// Add the anchor
		int32 NewIndex = InternalSpline.PrependAnchor(AnchorFromControlPoint(Point), Policy);
		
		if (NewIndex != INDEX_NONE)
		{
			// Set tangent mode
			TangentModes.InsertDefaulted(NewIndex);
			SetPointTangentMode(NewIndex, Point.TangentMode);
		}

		return NewIndex;
	}
	
    /** 
	 * Appends a control point to the end of the spline.
	 * @param Point - Control point to append.
	 * @param Parameter - The parameter of the control point being appended, must be greater than all existing knots (but may be less than the virtual closing knot).
	 * @return Index of the new point, or INDEX_NONE if the operation failed.
	 */
    int32 AppendPoint(const FTangentBezierControlPoint& Point, float Parameter)
    {
		// Add the anchor
		int32 NewIndex = InternalSpline.AppendAnchor(AnchorFromControlPoint(Point), Parameter);

		if (NewIndex != INDEX_NONE)
		{
			// Set tangent mode
			TangentModes.InsertDefaulted(NewIndex);
			SetPointTangentMode(NewIndex, Point.TangentMode);
		}

		return NewIndex;
    }

    /** 
	 * Appends a control point to the end of the spline.
	 * @param Point - Control point to append.
	 * @param Policy - The policy to use for generating the knot for the point being appended.
	 * @return Index of the new point, or INDEX_NONE if the operation failed.
	 */
    int32 AppendPoint(const FTangentBezierControlPoint& Point, EParameterizationPolicy Policy)
    {
		// Add the anchor
		int32 NewIndex = InternalSpline.AppendAnchor(AnchorFromControlPoint(Point), Policy);
		if (NewIndex != INDEX_NONE)
		{
			// Set tangent mode
			TangentModes.InsertDefaulted(NewIndex);
			SetPointTangentMode(NewIndex, Point.TangentMode);
		}

		return NewIndex;
    }

	/**
	 * Inserts a new point at the specified parameter along the spline.
	 * @param Point - Control point to insert.
	 * @param Parameter - The parameter of the control point being inserted.
	 * @return Index of the new point, or INDEX_NONE if the operation failed.
	 */
    int32 InsertPoint(const FTangentBezierControlPoint& Point, float Parameter)
    {
		// Add the anchor
		int32 NewIndex = InternalSpline.InsertAnchor(AnchorFromControlPoint(Point), Parameter);

		if (NewIndex != INDEX_NONE)
		{
			// Set tangent mode
			TangentModes.InsertDefaulted(NewIndex);
			SetPointTangentMode(NewIndex, Point.TangentMode);
		}

		return NewIndex;
    }
	
	/**
	 * Gets the control point at the specified index.
	 * @param Index - Index of the point to retrieve.
	 * @return Control point at the specified index.
	 */
	FTangentBezierControlPoint GetPoint(int32 Index) const
	{
		if (Index < 0 || Index >= GetNumPoints())
		{
			return FTangentBezierControlPoint();
		}
    
		FTangentBezierControlPoint Result;
		Result.Position = GetValue(Index);
		Result.TangentIn = GetTangentIn(Index);
		Result.TangentOut = GetTangentOut(Index);
    
		// Get tangent mode
		if (TangentModes.IsValidIndex(Index))
		{
			Result.TangentMode = TangentModes[Index];
		}
		else
		{
			Result.TangentMode = ETangentMode::Auto;
		}
    
		return Result;
	}
	
    /**
     * Removes a point from the spline
     * @param Index - Index of the point to remove
     */
    void RemovePoint(int32 Index)
	{
		if (Index < 0 || Index >= GetNumPoints())
		{
			return;
		}

		// Remove the corresponding tangent mode
		if (ensureAlwaysMsgf(TangentModes.IsValidIndex(Index), TEXT("Incorrect number of tangent modes!")))
		{
			TangentModes.RemoveAt(Index);
		}

		// Remove the point
		InternalSpline.RemoveAnchor(Index);
    
		// Ensure tangent modes array size matches number of points after removal
		if (TangentModes.Num() > GetNumPoints())
		{
			TangentModes.SetNum(GetNumPoints());
		}
    }

	/** 
	 * Modifies an existing point
	 * @param Index - Index of the point to modify
	 * @param Point - New control point data
	 */
	void ModifyPoint(int32 Index, const FTangentBezierControlPoint& Point)
	{
        if (Index < 0 || Index >= GetNumPoints())
        {
            return;
        }

        SetPointTangentMode(Index, Point.TangentMode);

        // Update position
		InternalSpline.SetAnchorPoint(Index, Point.Position);
        
        // If not auto tangents, apply the specified tangents
        if (Point.TangentMode != ETangentMode::Auto && 
            Point.TangentMode != ETangentMode::AutoClamped &&
            Point.TangentMode != ETangentMode::LegacyAuto && 
            Point.TangentMode != ETangentMode::LegacyAutoClamped && 
            Point.TangentMode != ETangentMode::Linear && 
            Point.TangentMode != ETangentMode::LegacyLinear && 
            Point.TangentMode != ETangentMode::Constant)
        {
            SetTangentIn(Index, Point.TangentIn);
            SetTangentOut(Index, Point.TangentOut);
        }
    }

	void SetValue(int32 Index, const ValueType& NewValue)
	{
		FTangentBezierControlPoint NewPoint = GetPoint(Index);
		NewPoint.Position = NewValue;
		ModifyPoint(Index, NewPoint);
	}
	
    /** Gets position at specified parent space parameter */
    ValueType GetValue(float Parameter) const
    {
        return InternalSpline.Evaluate(Parameter);
    }
    
    /** Gets the value of the specified point */
    ValueType GetValue(int32 Index) const
    {
    	const int32 NumPoints = GetNumPoints();

		if (NumPoints == 0 || Index < 0 || Index > GetNumPoints())
		{
			return ValueType();
		}

		return InternalSpline.GetAnchorPoint(Index);
    }

    /** Gets tangent at parameter in spline space */
    ValueType GetTangent(float Parameter) const
    {
        return InternalSpline.template EvaluateDerivative<1>(Parameter);
    }
    
    /** Gets incoming tangent for the specified point */
    ValueType GetTangentIn(int32 Index) const
    {
		const int32 NumPoints = GetNumPoints();

		if (Index < 0 || Index >= NumPoints)
		{
			return ValueType();
		}
		
		const ValueType P3 = GetValue(Index);
		const ValueType P2 = InternalSpline.GetHandleInOfAnchor(Index);
		return TangentOps::P2ToTangentIn(P3, P2);
    }
    
	/** Gets outgoing tangent for the specified point */
	ValueType GetTangentOut(int32 Index) const
	{
		const int32 NumPoints = GetNumPoints();
		if (Index < 0 || Index >= NumPoints)
		{
			return ValueType();
		}
		const ValueType P0 = GetValue(Index);
		const ValueType P1 = InternalSpline.GetHandleOutOfAnchor(Index);

		return TangentOps::P1ToTangentOut(P0, P1);
	}

    /** Sets tangent in for the specified point */
    void SetTangentIn(int32 Index, const ValueType& NewTangent)
    {
		if (Index < 0 || Index >= GetNumPoints())
		{
			return;
		}

		ValueType P3 = GetValue(Index);		
        ValueType P2 = TangentOps::TangentInToP2(P3, NewTangent);

        InternalSpline.SetHandleInOfAnchor(Index, P2);   
    }
    
    /** Sets tangent out for the specified point */
    void SetTangentOut(int32 Index, const ValueType& NewTangent)
    {
		if (Index < 0 || Index >= GetNumPoints())
		{
		    return;
		}

		ValueType P0 = GetValue(Index);		
		ValueType P1 = TangentOps::TangentOutToP1(P0, NewTangent);
      
     	InternalSpline.SetHandleOutOfAnchor(Index, P1);
    }

    /** Gets the number of points in the spline */
    int32 GetNumPoints() const
    {
		return InternalSpline.GetNumAnchorPoints();
    }

	virtual int32 GetNumberOfSegments() const override
	{
		return InternalSpline.GetNumberOfSegments();
	}

	virtual FInterval1f GetSegmentParameterRange(int32 SegmentIndex) const override
	{
		return InternalSpline.GetSegmentParameterRange(SegmentIndex);
	}

    /** Checks if a point's tangents are automatically computed */
    bool IsAutoTangent(int32 Index) const
    {
		return TangentModes.IsValidIndex(Index) && (TangentModes[Index] == ETangentMode::Auto ||
													TangentModes[Index] == ETangentMode::AutoClamped ||
													TangentModes[Index] == ETangentMode::LegacyAuto ||
													TangentModes[Index] == ETangentMode::LegacyAutoClamped);
    }

	/**
	  * Sets the tangent mode for a specific point
	  * @param Index - Index of the point
	  * @param Mode - New tangent mode to set
	  */
	void SetPointTangentMode(int32 Index, ETangentMode Mode)
	{
		const int32 NumPoints = GetNumPoints();

		if (Index < 0 || Index >= NumPoints)
		{
			return;
		}

		if (!TangentModes.IsValidIndex(Index))
		{
			// If we get here, TangentModes was in a bad state. We will correct the number of TangentModes, defaulting new points to Auto, before proceeding with the operation.

			UE_LOGF(LogSpline, Warning, "Index %d is an invalid index into TangentModes for %ls spline with %d points.", Index, this->IsClosedLoop() ? TEXT("a closed") : TEXT("an open"), NumPoints);

			const int32 OldTangentModesNum = TangentModes.Num();
			TangentModes.SetNum(NumPoints);
			const int32 NewTangentModesNum = TangentModes.Num();

			for (int32 TangentModesIdx = OldTangentModesNum; TangentModesIdx < NewTangentModesNum; ++TangentModesIdx)
			{
				TangentModes[TangentModesIdx] = ETangentMode::Auto;
			}
		}
        
		TangentModes[Index] = Mode;
	}
	
	virtual void Clear() override
	{
		InternalSpline.Clear();
		TangentModes.Empty();
	}

	/** Gets the parameter value for the specified anchor point index */
	float GetParameter(int32 Index) const
	{
		return InternalSpline.GetParameter(Index);
	}

	/** Sets the parameter value for the specified anchor point index */
	int32 SetParameter(int32 Index, float NewParameter)
    {
		return InternalSpline.SetParameter(Index, NewParameter);
    }

	virtual FInterval1f GetParameterSpace() const override
	{
		return InternalSpline.GetParameterSpace();
	}

	int32 FindSegmentIndex(float Parameter, float &OutLocalParam) const
	{
		return InternalSpline.FindIndexForParameter(Parameter, OutLocalParam);
	}

    /**
	 * Updates tangents for all control points based on their tangent modes
	 */
	void UpdateTangents()
	{
	    const int32 NumPoints = GetNumPoints();

	    if (NumPoints < 2)
	    {
	        return;
	    }
		check(TangentModes.Num() == NumPoints);
		
		// Update tangents for each point according to its mode
		for (int32 i = 0; i < NumPoints; ++i)
		{
			UpdateTangent(i);
		}
	}

	/**
	 * Updates tangents for a specific point based on its tangent mode
	 * @param Index - Index of the point to update
	 */
	void UpdateTangent(int32 Index)
	{
		const int32 NumPoints = GetNumPoints();
		const int32 NumSegments = GetNumberOfSegments();

		if (Index < 0 || Index >= NumPoints || !ensureAlwaysMsgf(TangentModes.IsValidIndex(Index), TEXT("No tangent mode for point!")))
		{
			return;
		}

		const ETangentMode Mode = TangentModes[Index];

		if (Mode == ETangentMode::User || Mode == ETangentMode::Broken)
		{
			return;
		}

		if constexpr (Math::TIsQuaternionType<ValueType>::value)
		{
			if (Mode == ETangentMode::Linear || Mode == ETangentMode::LegacyLinear)
			{
				const ValueType Q = GetValue(Index).GetNormalized();
				SetTangentIn(Index,  Q);
				SetTangentOut(Index, Q);
				return; 
			}
		}

		// Get the position of the current point
		const ValueType& Current = GetValue(Index);

		// Calculate indices handling closed loops
		const int32 PrevIndex = (Index > 0) ? (Index - 1) : (this->IsClosedLoop() ? NumPoints - 1 : Index);
		const int32 NextIndex = (Index < NumPoints - 1) ? (Index + 1) : (this->IsClosedLoop() ? 0 : Index);
		const ValueType& Prev = GetValue(PrevIndex);
		const ValueType& Next = GetValue(NextIndex);

		// Get the actual parameter values for each point
		float PrevParam = GetParameter(PrevIndex);
		float CurrentParam = GetParameter(Index);
		float NextParam = GetParameter(NextIndex);
		
		// If we are closed and updating the first or last point, we need to do some special work 
		// to get virtual PrevParam or NextParam. The 'virtual' parameter is not the actual parameter 
		// of the previous or next point, but the current parameter +- the knot delta (closing segment length).
		if (this->IsClosedLoop())
		{
			if (Index == 0 && PrevIndex == NumPoints - 1)
			{
				// First point, prev wraps from end - subtract the closing segment length
				// The closing segment is the last segment in a closed loop
				PrevParam = CurrentParam - GetSegmentParameterRange(NumSegments - 1).Length();
			}
			else if (Index == NumPoints - 1 && NextIndex == 0)
			{
				// Last point, next wraps to start - add the closing segment length
				NextParam = CurrentParam + GetSegmentParameterRange(NumSegments - 1).Length();
			}
		}

		const float dTL = FMath::Max(UE_KINDA_SMALL_NUMBER, CurrentParam  - PrevParam); // span to the *left*
		const float dTR = FMath::Max(UE_KINDA_SMALL_NUMBER, NextParam - CurrentParam ); // span to the *right*
		
		// Calculate tangents based on mode
		ValueType InTangent = GetTangentIn(Index); // Keep existing if not changing
		ValueType OutTangent = GetTangentOut(Index);
		
		switch (Mode)
		{
		case ETangentMode::Auto:
		case ETangentMode::AutoClamped:
			{
				const bool bIsStationaryEndpoint = bStationaryEndpoint && 
										  (Index == 0 || Index == NumPoints - 1) && 
										  !this->IsClosedLoop();
        
				if (bIsStationaryEndpoint)
				{
					if constexpr (Math::TIsQuaternionType<ValueType>::value)
					{
						InTangent  = Current; // control = key rotation
						OutTangent = Current;
					}
					else
					{
						// Stationary endpoints get zero tangent
						InTangent  = ValueType(0);
						OutTangent = ValueType(0);
					}
				}
				else
				{
					const bool bWantClamping = (Mode == ETangentMode::AutoClamped);
					TangentOps::ComputeAutoTangent(
						PrevParam, Prev, CurrentParam, Current, NextParam, Next,
						Tension, bWantClamping, InTangent);
					OutTangent = InTangent;
				}
			}
			break;

		case ETangentMode::Linear:
			{
				InTangent  = Current - Prev;
				OutTangent = Next - Current;
			}
			break;
			
		case ETangentMode::LegacyLinear:
		{
			// FInterpCurve-compatible behavior
			OutTangent = Next - Current;
			InTangent = IsCurveKey(TangentModes[PrevIndex]) ? InTangent = OutTangent : (Current - Prev);
		}
			break;
		case ETangentMode::LegacyAuto:
		case ETangentMode::LegacyAutoClamped:
			{
				if (bStationaryEndpoint && (Index == 0 || Index == NumPoints - 1) && !this->IsClosedLoop())
				{
					// Stationary endpoints get zero tangent
					InTangent  = ValueType(0);
					OutTangent = ValueType(0);
				}
				else if (IsCurveKey(TangentModes[PrevIndex]))
				{
					// Use TTangentOps which handles clamping correctly
					const bool bWantClamping = (Mode == ETangentMode::LegacyAutoClamped);
					TangentOps::ComputeAutoTangent(
						PrevParam, Prev,
						CurrentParam, Current,
						NextParam, Next,
						Tension,
						bWantClamping,
						InTangent);
					OutTangent = InTangent;
				}
				else
				{
					// Previous is Linear or Constant - copy its tangents (legacy behavior)
					InTangent = GetTangentIn(PrevIndex);
					OutTangent = GetTangentOut(PrevIndex);
				}
			}
			break;

		case ETangentMode::Constant:
			{
				// Zero tangents for constant interpolation
				InTangent = ValueType();
				OutTangent = ValueType();
			}
			break;

		case ETangentMode::User:
			break;
			
		case ETangentMode::Unknown:
		default:
			// Default to Auto mode if unknown
			{
				ValueType Tangent;
				TangentOps::ComputeAutoTangent(
						PrevParam, Prev,
						CurrentParam, Current,
						NextParam, Next,
						Tension,
						false,
						Tangent);
				InTangent = Tangent;
				OutTangent = Tangent;
			}
			break;
		}

		// Apply the calculated tangents
		SetTangentIn(Index, InTangent);
		SetTangentOut(Index, OutTangent);
	}

	virtual ValueType EvaluateImpl(float Parameter) const override
	{
		if (GetNumPoints() == 1)
		{
			return GetValue(0);
		}
		
		return InternalSpline.EvaluateImpl(Parameter);
	}
	
	virtual float FindNearest(const ValueType& Point, float& OutSquaredDistance) const override
	{
        // Delegate to internal spline
        return InternalSpline.FindNearest(Point, OutSquaredDistance);
    }

	void Reparameterize(EParameterizationPolicy Policy = EParameterizationPolicy::Uniform)
    {
	    InternalSpline.Reparameterize(Policy);
    }

	void SetKnotVector(const TArray<FKnot>& InKnots)
	{
		InternalSpline.SetKnotVector(InKnots);
	}
	
	TArray<FKnot> GetKnotVector() const
	{
		return InternalSpline.GetKnotVector();
	}
    
    /** Getter for internal spline */
    const Base& GetInternalSpline() const
    {
        return InternalSpline;
    }
	
	float GetTension() const
	{
		return Tension;
	}

	void SetTension(const float InTension)
	{
		this->Tension = InTension;
	}

	ETangentMode GetTangentMode(int32 Index) const
	{
		if (Index < 0 || Index >= GetNumPoints())
		{
			return ETangentMode::Unknown;
		}
		return TangentModes[Index];
	}

	void SetTangentModes(const TArray<ETangentMode>& InTangentModes)
	{
		TangentModes = InTangentModes;
	}
	
	const TArray<ETangentMode>& GetTangentModes() const
	{
		return TangentModes;
	}

	void SetStationaryEndpoints(bool bInStationaryEndpoints)
	{
		bStationaryEndpoint = bInStationaryEndpoints;
	}

	bool IsStationaryEndpoints() const
	{
		return bStationaryEndpoint;
	}
	
	float FindNearestOnSegment(const ValueType& Point, int32 SegmentIndex, float& OutSquaredDistance) const
		{ return InternalSpline.FindNearestOnSegment(Point, SegmentIndex, OutSquaredDistance); }

private:

	void InstallFindWindowOverride()
	{
		InternalSpline.SetFindWindowOverride(&RescaleNormalizedHandlesForSegment);
	}
	
	/** ONLY adjusts P1/P2 in caller-provided scratch; generic over WindowSize */
	static void RescaleNormalizedHandlesForSegment(FWindow& Window, int32 SegmentIndex,  const FInterval1f& SegRange, FWindowStore& Scratch)
	{
		// bail if window not complete
		if (!Math::FSplineValidation::IsValidWindow<ValueType, 4>(Window))
		{
			return;
		}
		const float L = SegRange.Length();
		const ValueType& P0 = *Window[0];
		const ValueType& P1 = *Window[1];
		const ValueType& P2 = *Window[2];
		const ValueType& P3 = *Window[3];
		
		ValueType TOut = TangentOps::P1ToTangentOut(P0, P1);
		ValueType TIn  = TangentOps::P2ToTangentIn(P3, P2);
		
		// Only write adjusted P1/P2 into scratch; keep P0,P3 pointers intact
		Scratch[1] = TangentOps::TangentOutToP1(P0, TOut * L);
		Scratch[2] = TangentOps::TangentInToP2(P3, TIn * L);
		Window[1] = &Scratch[1];
		Window[2] = &Scratch[2];
	}

	static bool IsCurveKey(ETangentMode Mode)
	{
		return Mode == ETangentMode::Auto || 
			   Mode == ETangentMode::AutoClamped ||
			   Mode == ETangentMode::LegacyAuto ||
			   Mode == ETangentMode::LegacyAutoClamped ||
			   Mode == ETangentMode::User ||
			   Mode == ETangentMode::Broken;
	}
	
	void AssignInternalSpline(const Base& NewSpline)
	{
		InternalSpline = NewSpline;
		InstallFindWindowOverride();
	}

	void AssignInternalSpline(Base&& NewSpline)
	{
		InternalSpline = MoveTemp(NewSpline);
		InstallFindWindowOverride();
	}

protected:

	/* Tension parameter for auto-computed tangents [0,1] */
	float Tension = 0.0f;

	/* How tangents should be computed for each point */
	TArray<ETangentMode> TangentModes;

	bool bStationaryEndpoint = false;
	
	TStaticArray<ValueType, TPolyBezierSpline<ValueType>::Base::WindowSize> IntermediateWindow;

private:

	using TSplineWrapper<TPolyBezierSpline<ValueType>>::InternalSpline;
};

using FTangentBezierSpline3f = TTangentBezierSpline<FVector3f>; 
using FTangentBezierSpline3d = TTangentBezierSpline<FVector3d>; 
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE