// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SplineInterfaces.h"
#include "ParameterizedTypes.h"
#include <cstdint>
#include <limits>

namespace UE
{
namespace Geometry
{
namespace Spline
{
namespace Math
{
	/** Computes Tangent for a curve segment */
	template< class T, class U > 
	inline void AutoCalcTangent( const T& PrevP, const T& P, const T& NextP, const U& Tension, T& OutTan )
	{
		OutTan = (1.f - Tension) * ( (P - PrevP) + (NextP - P) );
	}


	/**
	 * This actually returns the control point not a tangent. This is expected by the CubicInterp function for Quaternions
	 */
	template< class U > 
	inline void AutoCalcTangent( const FQuat& PrevP, const FQuat& P, const FQuat& NextP, const U& Tension, FQuat& OutTan  )
	{
		FQuat::CalcTangents(PrevP, P, NextP, Tension, OutTan);
	}
	
	/**
	 * Opt-in trait for types that are contiguous floats
	 * Only types with this trait can use component-wise clamping
	 */
	template<typename T>
	struct TIsContiguousFloat : std::false_type {};

	template<> struct TIsContiguousFloat<float>       : std::true_type {};
	template<> struct TIsContiguousFloat<double>      : std::true_type {};
	template<> struct TIsContiguousFloat<FVector2D>   : std::true_type {};
	template<> struct TIsContiguousFloat<FVector>     : std::true_type {};
	template<> struct TIsContiguousFloat<FVector4>    : std::true_type {};
	template<> struct TIsContiguousFloat<FTwoVectors> : std::true_type {};
	// Add more as needed: FVector4f, FLinearColor, FPlane, etc.

	/**
	 * Opt-in trait for quaternion types
	 */
	template<typename T>
	struct TIsQuaternionType : std::false_type {};

	template<> struct TIsQuaternionType<TQuat<float>>  : std::true_type {};
	template<> struct TIsQuaternionType<TQuat<double>> : std::true_type {};

	/**
	 * Per-component clamp path (matches InterpCurvePoint's ComputeClampableFloatVectorCurveTangent)
	 * NOTE: clamped branch does NOT divide by (NextTime - PrevTime); unclamped does.
	 */
	template<typename T>
	static std::enable_if_t<TIsContiguousFloat<T>::value, void>
	ComputeClampableTangent(
		float PrevTime, const T& PrevPoint,
		float CurTime,  const T& CurPoint,
		float NextTime, const T& NextPoint,
		float Tension,
		bool  bWantClamping,
		T&    OutTangent)
		{
			if (bWantClamping)
			{
				static_assert(std::is_trivially_copyable_v<T>, "Type must be trivially copyable.");
				static_assert(sizeof(T) % sizeof(float) == 0, "Type size must be a multiple of float.");

				const float* Prev = reinterpret_cast<const float*>(&PrevPoint);
				const float* Cur  = reinterpret_cast<const float*>(&CurPoint);
				const float* Next = reinterpret_cast<const float*>(&NextPoint);
				float*       Out  = reinterpret_cast<float*>(&OutTangent);

				constexpr int32 N = int32(sizeof(T) / sizeof(float));
				for (int32 i = 0; i < N; ++i)
				{
					const float Clamped = ClampFloatTangent(
						Prev[i], PrevTime,
						Cur[i],  CurTime,
						Next[i], NextTime);

					Out[i] = (1.0f - Tension) * Clamped;
				}
			}
			else
			{
				AutoCalcTangent(PrevPoint, CurPoint, NextPoint, Tension, OutTangent);
				const float PrevToNext = FMath::Max(UE_KINDA_SMALL_NUMBER, NextTime - PrevTime);
				OutTangent /= PrevToNext;
			}
		}
	template<typename T> struct TTangentOps;

	/**
	 * Policy for turning tangents into Bezier P1/P2 and computing auto tangents
	 */
	template<typename T>
	struct TTangentOps
	{
	    static constexpr bool bContig = TIsContiguousFloat<T>::value;
	    static constexpr bool bQuat   = TIsQuaternionType<T>::value;

	    /** Compute auto/auto-clamped tangent */
	    static void ComputeAutoTangent(float PrevTime, const T& Prev,
	                                   float CurTime,  const T& Cur,
	                                   float NextTime, const T& Next,
	                                   float Tension, bool bClamp, T& Out)
	    {
	        static_assert(bContig && !bQuat, "Vector path only");
	        ComputeClampableTangent(PrevTime, Prev, CurTime, Cur, NextTime, Next, Tension, bClamp, Out);
	    }

	    /** Turn out tangents into the cubic’s P1 control (Bezier) */
	    static T TangentOutToP1(const T& P0, const T& Tout)
	    {
	        static_assert(bContig && !bQuat, "Vector path only");
	        return P0 + (Tout / 3.0f);
	    }

		/** Turn in tangents into the cubic’s P2 control (Bezier) */
		static T TangentInToP2(const T& P3, const T& Tin)
	    {
	    	static_assert(bContig && !bQuat, "Vector path only");
	    	return P3 - (Tin  / 3.0f);
	    }

		/** P1 to tangent out */
		static T P1ToTangentOut(const T& P0, const T& P1)
	    {
	    	static_assert(bContig && !bQuat, "Vector path only");
	    	return (P1 - P0) * 3.0f;
	    }
		
		/** P2 to tangent in */
		static T P2ToTangentIn(const T& P3, const T& P2)
	    {
	    	static_assert(bContig && !bQuat, "Vector path only");
	    	return (P3 - P2) * 3.0f;
	    }
	};

	/**
	 * Quaternion specialization: "tangents" are control quats (SQUAD controls)
	 */
	template<typename Scalar>
	struct TTangentOps<UE::Math::TQuat<Scalar>>
	{
		using Quat = UE::Math::TQuat<Scalar>;
	    static void ComputeAutoTangent(float PrevTime, const Quat& Prev,
	                                   float CurTime,  const Quat& Cur,
	                                   float NextTime, const Quat& Next,
	                                   float Tension, bool /*bClamp*/, Quat& OutControl)
	    {
	        Quat P = Prev, C = Cur, N = Next;
	    	
	    	AutoCalcTangent(P, C, N, Tension, OutControl);
	    	const float PrevToNextTimeDiff = FMath::Max< float >( UE_KINDA_SMALL_NUMBER, NextTime - PrevTime );
		    
	    	OutControl /= PrevToNextTimeDiff;
	    }

		/** Turn out tangents into the cubic’s P1 control (Bezier) */
		static Quat TangentOutToP1(const Quat& P0, const Quat& Tout)
	    {
	    	// For quats, the "tangents" are already the control quats used by cubic interp
	    	return Tout;
	    }

		/** Turn in tangents into the cubic’s P2 control (Bezier) */
		static Quat TangentInToP2(const Quat& P3, const Quat& Tin)
	    {
	    	// For quats, the "tangents" are already the control quats used by cubic interp
	    	return Tin;
	    }

		/** P1 to tangent out */
		static Quat P1ToTangentOut(const Quat& P0, const Quat& P1)
	    {
	    	return P1;
	    }

		/** P2 to tangent in */
		static Quat P2ToTangentIn(const Quat& P3, const Quat& P2)
	    {
	    	return P2;
	    }
	};
	
	
} // end namespace UE::Geometry::Spline::Math
} // end namespace UE::Geometry::Spline
} // end namespace UE::Geometry
} // end namespace UE
