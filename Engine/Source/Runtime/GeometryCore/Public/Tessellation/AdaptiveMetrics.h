// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tessellation/Affine.h"
#include "Math/Vector.h"
#include "Math/UnrealMathUtility.h"
#include "Image/DisplacementMap.h"

namespace UE {
namespace Geometry {
namespace TessellationUtil {

/**
 * Helper methods for use in adaptive tessellation callbacks in displacement policy for 
 * TAdaptiveTesselator and TParallelRefinement.
 */

/**
 * Get bounds of UVs in a UV subtriangle
 * 
 * @param Barycentrics  Barycentric coordinates of three vertices of a subtriangle
 * @param VertexUVs     UVs of a triangle
 * @param UVBounds      component-wise bounds of UV coordinates of subtriangle
 */
inline void CalculateUVBounds(
	const FVector3f Barycentrics[3],
	const FVector2f VertexUVs[3],
	FVector2f OutUVBounds[2])
{
	OutUVBounds[0] = FVector2f( std::numeric_limits<float>::max(),  std::numeric_limits<float>::max());
	OutUVBounds[1] = FVector2f(-std::numeric_limits<float>::max(), -std::numeric_limits<float>::max());
							 
	for (int k = 0; k < 3; k++)
	{
		FVector2f UV;
		UV =  VertexUVs[0] * Barycentrics[k].X;
		UV += VertexUVs[1] * Barycentrics[k].Y;
		UV += VertexUVs[2] * Barycentrics[k].Z;

		OutUVBounds[0] = FVector2f::Min(OutUVBounds[0], UV);
		OutUVBounds[1] = FVector2f::Max(OutUVBounds[1], UV);
	}
}

/**
 * Calculate conservative bounds of the squared error of the displaced position.
 *  
 * VertexNormals and Displacements are given on the corners of a base-triangle.
 * The subtriangle to evaluate on is defined by the Barycentrics.
 * UVBounds are given with respect to subtriangle.
 * 
 * DisplacementBoundsRefinements controls the accuracy of the bounds when retrieving
 * bounds from the displacement map (quality speed/tradeoff). See 
 * FDisplacementMap::SampleHierarchical.
 * 
 * Magnitude, Center are the usual affine maps on FDisplacementMap values. Set to
 * 1 and 0 if they are already set on the displacement map. They are only provided here,
 * to allow modification without changing the displacement map.
 */
template <typename T>
inline UE::Math::TVector2<T> CalculateErrorBounds(
	const UE::Math::TVector<T> Barycentrics[3],
	const UE::Math::TVector<T> VertexNormals[3],
	const FVector2f UVBounds[2], 
	const UE::Math::TVector<T> Displacements[3],
	const FDisplacementMap& DisplacementMap,
	const float Center,
	const FVector2f MagnitudeBounds, // bounds 
	const int32 DisplacementBoundsRefinements)
{
	T MinBarycentric0 = FMath::Min3(Barycentrics[0].X, Barycentrics[1].X, Barycentrics[2].X);
	T MaxBarycentric0 = FMath::Max3(Barycentrics[0].X, Barycentrics[1].X, Barycentrics[2].X);

	T MinBarycentric1 = FMath::Min3(Barycentrics[0].Y, Barycentrics[1].Y, Barycentrics[2].Y);
	T MaxBarycentric1 = FMath::Max3(Barycentrics[0].Y, Barycentrics[1].Y, Barycentrics[2].Y);

	TAffine<T, 2> Barycentric0(MinBarycentric0, MaxBarycentric0, 0);
	TAffine<T, 2> Barycentric1(MinBarycentric1, MaxBarycentric1, 1);
	TAffine<T, 2> Barycentric2 = TAffine<T, 2 >(1.0) - Barycentric0 - Barycentric1;

	TAffine<UE::Math::TVector<T>, 2> LerpedDisplacement;
	LerpedDisplacement  = TAffine<UE::Math::TVector<T>, 2>(Displacements[0]) * Barycentric0;
	LerpedDisplacement += TAffine<UE::Math::TVector<T>, 2>(Displacements[1]) * Barycentric1;
	LerpedDisplacement += TAffine<UE::Math::TVector<T>, 2>(Displacements[2]) * Barycentric2;

	TAffine<UE::Math::TVector<T>, 2> Normal;
	Normal  = TAffine<UE::Math::TVector<T>, 2>(VertexNormals[0]) * Barycentric0;
	Normal += TAffine<UE::Math::TVector<T>, 2>(VertexNormals[1]) * Barycentric1;
	Normal += TAffine<UE::Math::TVector<T>, 2>(VertexNormals[2]) * Barycentric2;
	Normal = Normalize(Normal);

	const FVector2f DisplacementBounds = DisplacementMap.SampleHierarchical(UVBounds[0], UVBounds[1], DisplacementBoundsRefinements) - FVector2f(Center);

	// Displacement can be negative, but Magnitude is always positive.
	const FVector2f ScaledDisplacementBounds( FMath::Min(MagnitudeBounds[0]*DisplacementBounds[0], MagnitudeBounds[1]*DisplacementBounds[0]),
		                                      FMath::Max(MagnitudeBounds[0]*DisplacementBounds[1], MagnitudeBounds[1]*DisplacementBounds[1]));

	TAffine<T, 2> Displacement( ScaledDisplacementBounds[0], ScaledDisplacementBounds[1] );
	TAffine<T, 2> Error = (Normal * Displacement - LerpedDisplacement).SizeSquared();

	return UE::Math::TVector2<T>(Error.GetMin(), Error.GetMax());
}


/**
 * TAdaptiveTessellation samples the triangle at the finest level to determine
 * the optimal split position in terms of maximizing the error reduction.
 *  
 * This method can be used to compute the number of samples for this sub-sampling step.
 * 
 * @param Barycentrics  Barycentric coordinates of three vertices of a subtriangle
 * @param VertexUVs     UVs of a triangle
 * @param MapRes        Resolution of FDisplacementMap used
 */
template <typename T>
inline int32 GetNumSamples(
	const UE::Math::TVector<T> Barycentrics[3],
	const FVector2f VertexUVs[3],
	const FVector2f MapRes) 
{
	FVector2f UVs[3];
	for (int k = 0; k < 3; k++)
	{
		UVs[k] =  VertexUVs[0] * static_cast<float>(Barycentrics[k].X);
		UVs[k] += VertexUVs[1] * static_cast<float>(Barycentrics[k].Y);
		UVs[k] += VertexUVs[2] * static_cast<float>(Barycentrics[k].Z);

		UVs[k].X *= MapRes[0];
		UVs[k].Y *= MapRes[1];
	}

	FVector2f Edge01 = UVs[1] - UVs[0];
	FVector2f Edge12 = UVs[2] - UVs[1];
	FVector2f Edge20 = UVs[0] - UVs[2];

	T MaxEdgeLength = FMath::Sqrt(FMath::Max3(
		Edge01.SizeSquared(),
		Edge12.SizeSquared(),
		Edge20.SizeSquared()));

	T AreaInTexels = FMath::Abs(0.5 * (Edge01 ^ Edge12));

	return static_cast<int32>(FMath::CeilToInt(FMath::Max(MaxEdgeLength, AreaInTexels)));
}

} // namespace TessellationUtil
} // namespace Geometry
} // namespace UE