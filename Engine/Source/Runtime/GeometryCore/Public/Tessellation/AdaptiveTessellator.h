// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MathUtil.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

#include "Async/ParallelFor.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "IndexTypes.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include "Containers/HashTable.h"

// GeometryCore/VectorUtil.h clashes with NaniteUtilities/VectorUtil.h. Workaround: TriangleTypes.h includes VectorUtil.h
#include "TriangleTypes.h" 

#include <limits>
#include <atomic>

namespace UE {
namespace Geometry {

// Info struct returned by edge split operation
struct FEdgeSplitInfo
{
	using FIndex2i = UE::Geometry::FIndex2i;
	FIndex2i OldTriIndex;  //< triangle indices before split
	FIndex2i NewTriIndex;  //< newly added triangles indices NewTriIndex[i] will be on the same side as OldTriIndex[i] for i=0,1, or IndexConstants::InvalidID on failure
	FIndex2i NewVertIndex; //< indices of newly introduced vertices (NewVertIndex[i] is the shared vertex of OldTriIndex[i] and NewTriIndex[i])
};

struct FPokeInfo
{
	using FIndex3i = UE::Geometry::FIndex3i;
	FIndex3i NewTriIndex;  //< indices of the newly created triangles, or IndexConstants::InvalidID on failure
	FIndex3i EdgeIndex;    //< for each of the new triangles the edge index corresponding to the original triangle edge
	int32    NewVertIndex; //< index of newly created vertex
};

struct FFlipEdgeInfo
{
	using FIndex2i = UE::Geometry::FIndex2i;
	FIndex2i Triangles;    //< triangle indices corresponding to the newly flipped triangles, or IndexConstants::InvalidID on failure
	FIndex2i SharedEdge;   //< local edge index [0..2] of the new shared edge, with respect to triangle indices above
};

enum class EAdaptiveTessellationRefinementMethod
{
	// builtin Nanite refinement strategy, with optimized/sampled split location
	HierarchicalSplit,
	
	// custom callback for refinement and refinement location
	Custom
};

struct FAdaptiveTessellatorOptions
{
	// target error to achieve (with respect to error defined by displacement policy)
	double TargetError; 

	// one-dimensional rate (edge-length) at which features are sampled. triangles with edge-length smaller than
	// this value will not be refined
	double SampleRate; 

	// controls cosine-check to permit Delaunay edge-flips 
	double CosineThreshold { 0.999 };

	// vertices with same positions are considered to be identical and stay at the same position after displacement
	bool bCrackFree { true };

	// apply displacement to the tessellated mesh
	bool bFinalDisplace { true };

	// upper bound on the number of triangles of the tessellated mesh. Note that reaching this limit, the algorithm
	// will not guarantee an optimal use of the triangle budget. It is purely for safe-guard purposes.
	//
	uint32 MaxTriangles { 10'000'000u };

	EAdaptiveTessellationRefinementMethod RefinementMethod { EAdaptiveTessellationRefinementMethod::HierarchicalSplit };

	bool bSingleThreaded { false };
};

// Configured by MeshT and DisplacementPolicyT, which provide mesh topology functionality and
// evaluation/bounds of the displacement function.
// 
// See MinimalMeshTopology for an example topology implementation.
// 
template <typename MeshT, typename DisplacementPolicyT>
class TAdaptiveTessellator
{
private:

	inline static uint32 HashPosition(const FVector3f& Position)
	{
		union { float f; uint32 i; } x;
		union { float f; uint32 i; } y;
		union { float f; uint32 i; } z;

		x.f = Position.X;
		y.f = Position.Y;
		z.f = Position.Z;

		return Murmur32({
			Position.X == 0.0f ? 0u : x.i,
			Position.Y == 0.0f ? 0u : y.i,
			Position.Z == 0.0f ? 0u : z.i
			});
	}

	inline static uint32 HashPosition(const FVector3d& Position)
	{
		union { double f; uint64 i; } x;
		union { double f; uint64 i; } y;
		union { double f; uint64 i; } z;

		x.f = Position.X;
		y.f = Position.Y;
		z.f = Position.Z;

		return static_cast<uint32>(Murmur64({
			Position.X == 0.0 ? 0u : x.i,
			Position.Y == 0.0 ? 0u : y.i,
			Position.Z == 0.0 ? 0u : z.i
			}));
	}


	template <typename T>
	inline void SubtriangleBarycentrics(uint32 TriX, uint32 TriY, uint32 FlipTri, uint32 NumSubdivisions, UE::Math::TVector<T> Barycentrics[3])
	{
		/*
			Vert order:
			1    0__1
			|\   \  |
			| \   \ |  <= flip triangle
			|__\   \|
			0   2   2
		*/

		uint32 VertXY[3][2] =
		{
			{ TriX,		TriY	},
			{ TriX,		TriY + 1},
			{ TriX + 1,	TriY	},
		};
		VertXY[0][1] += FlipTri;
		VertXY[1][0] += FlipTri;

		for (int Corner = 0; Corner < 3; Corner++)
		{
			Barycentrics[Corner][0] = static_cast<T>(VertXY[Corner][0]);
			Barycentrics[Corner][1] = static_cast<T>(VertXY[Corner][1]);
			Barycentrics[Corner][2] = static_cast<T>(NumSubdivisions - VertXY[Corner][0] - VertXY[Corner][1]);
			Barycentrics[Corner] /= static_cast<T>(NumSubdivisions);
		}
	}

	template <typename T>
	inline static constexpr T EquilateralArea( T EdgeLength )
	{
		constexpr T sqrt3_4 = T(0.4330127018922193);
		return sqrt3_4 * FMath::Square( EdgeLength * EdgeLength );
	}

	template <typename T>
	inline static T SubtriangleArea( const UE::Math::TVector<T>& Barycentrics0, const UE::Math::TVector<T>& Barycentrics1, const UE::Math::TVector<T>& Barycentrics2, T TriangleArea )
	{
		// Area * Determinant using triple product
		return TriangleArea * FMath::Abs( Barycentrics0 | ( Barycentrics1 ^ Barycentrics2 ) );
	}

	template <typename T>
	// https://math.stackexchange.com/questions/3748903/closest-point-to-triangle-edge-with-barycentric-coordinates
	inline static constexpr T DistanceToEdge( T Barycentric, T EdgeLength, T TriangleArea )
	{
		return T(2.) * Barycentric * TriangleArea / EdgeLength;
	}

	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
	
  	inline static constexpr int32 NextEdge(int32 e) { return (e+1)%3; }
	inline static constexpr int32 PrevEdge(int32 e) { return (e+2)%3; }

  public:

	using RealType = typename MeshT::RealType; // the field related to the vector-space
	using VecType  = typename MeshT::VecType;  // for positions and normals

    // Initialize and run adaptive tessellation
	TAdaptiveTessellator(
		MeshT& InMesh,                                //< mesh interface
		DisplacementPolicyT& InDisplacementPolicy,    //< displacement and error interface
		const FAdaptiveTessellatorOptions& InOptions)                   
		: Mesh(InMesh)
		, DisplacementPolicy(InDisplacementPolicy)
		, Options(InOptions)
	{
		const EParallelForFlags ParallelForFlags = Options.bSingleThreaded ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;

		TriSplitRecords.AddUninitialized( Mesh.MaxTriID() );
		Displacements.Init( VecType( std::numeric_limits<RealType>::signaling_NaN(), 
		                             std::numeric_limits<RealType>::signaling_NaN(), 
									 std::numeric_limits<RealType>::signaling_NaN() ), Mesh.MaxVertexID() );

		for( int32 TriIndex = 0; TriIndex < Mesh.MaxTriID(); ++TriIndex )
		{
			if (!Mesh.IsValidTri(TriIndex))
			{
				continue;
			}
			for( int k = 0; k < 3; k++ )
			{
				const uint32 VID = Mesh.GetVertexIndex(TriIndex, k);
				check(Mesh.IsValidVertex(VID));
				if (!VectorUtil::IsFinite(Displacements[VID]))
				{
					Displacements[VID] = DisplacementPolicy.GetVertexDisplacement(VID, TriIndex);
					if (!ensure(UE::Geometry::VectorUtil::IsFinite(Displacements[VID])))
					{
						Displacements[VID] = VecType(0., 0., 0.);
					}
				}
			}
		}

		SplitRequests.SetNum( TriSplitRecords.Num() );
		NumSplits = 0;
		
		NumNewTriangles.store(0);

		if ( static_cast<uint32>(Mesh.MaxTriID()) >= Options.MaxTriangles )
		{
			return;
		}
		
		ParallelFor( TEXT("TAdaptiveTessellator.FindSplitBVH.PF"), TriSplitRecords.Num(), 32,
		[&]( uint32 TriIndex )
		{
			CheckSplitAndEnqueue( TriIndex, 0 );
		}, ParallelForFlags );
	
		int32 Iter = 0;
		while( NumSplits )
		{
			// Size to atomic count and sort for deterministic order
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TAdaptiveTessellator.SortSplits.SF");
				SplitRequests.SetNum( NumSplits, EAllowShrinking::No );
				SplitRequests.Sort();
			}

			for( int32 i = 0; i < SplitRequests.Num(); i++ )
			{
				TriSplitRecords[ SplitRequests[i] ].RequestIndex = i;
			}
	
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TAdaptiveTessellator.SplitTriangles.SF");
				while( SplitRequests.Num() ) 
				{
					SplitTriangle( SplitRequests.Pop() );
				}
			}

			if ( static_cast<uint32>(Mesh.MaxTriID()) >= Options.MaxTriangles)
			{
				break;
			}

			SplitRequests.SetNum( TriSplitRecords.Num() );
			NumSplits = 0;

			NumNewTriangles.store(0);

			ParallelFor( TEXT("TAdaptiveTessellator.FindSplitBVH.PF"), FindRequests.Num(), 32,
				[&]( uint32 i )
				{
					CheckSplitAndEnqueue( FindRequests[i], Iter );
				}, ParallelForFlags);
				
			FindRequests.Reset();

			Iter++;
		}

		if( Options.bCrackFree && Mesh.IsTriangleSoup())
		{
			TArray<VecType> OldDisplacements;
			OldDisplacements.AddUninitialized( Displacements.Num() );
			Swap( Displacements, OldDisplacements );

			FHashTable HashTable( 1 << FMath::FloorLog2( Mesh.MaxVertexID() ), Mesh.MaxVertexID() );
			ParallelFor( TEXT("TAdaptiveTessellator.HashVerts.PF"), Mesh.MaxVertexID(), 4096,
				[&]( int32 i )
				{
					if (Mesh.IsValidVertex(i))
					{
						// HashTable expects 32bit
						HashTable.Add_Concurrent( static_cast<uint32>(HashPosition(Mesh.GetVertexPosition(i))), i );
					}
				}, ParallelForFlags );

			ParallelFor( TEXT("TAdaptiveTessellator.HashVerts.PF"), Mesh.MaxVertexID(), 4096,
				[&]( int32 VID )
				{
					if (!Mesh.IsValidVertex(VID))
					{
						return;
					}

					VecType	Average( 0.0 );
					int32	Count = 0;

					uint32 Hash = static_cast<uint32>(HashPosition( Mesh.GetVertexPosition(VID) ));
					for( uint32 OtherIndex = HashTable.First( Hash ); HashTable.IsValid( OtherIndex ); OtherIndex = HashTable.Next( OtherIndex ) )
					{
						if( Mesh.GetVertexPosition(VID) == Mesh.GetVertexPosition(OtherIndex) )
						{
							Average += OldDisplacements[ OtherIndex ];
							Count++;
						}
					}

					Displacements[VID] = Average / Count;
				}, ParallelForFlags );
		}

		if (Options.bFinalDisplace)
		{
			ParallelFor(TEXT("TAdaptiveTessellator.Displace.PF"), Mesh.MaxVertexID(), 4096,
				[&](int32 VID)
				{
					if (Mesh.IsValidVertex(VID))
					{
						if (ensure(VectorUtil::IsFinite(Displacements[VID])))
						{
							Mesh.SetVertexPosition(VID, Mesh.GetVertexPosition(VID) + Displacements[VID]);
						}
					}
				}, ParallelForFlags);
		}

		// Free memory
		Displacements.Empty();
		check(Displacements.Max() == 0);

		TriSplitRecords.Empty();
		FindRequests.Empty();
		SplitRequests.Empty();
	}

private:
	inline bool CouldFlipEdge( int32 TriIndex, int32 TriEdgeIndex ) const 
	{
		// Don't flip boundary edges
		const int32 AdjTriIndex = Mesh.GetAdjTriangle(TriIndex, TriEdgeIndex);
		if (AdjTriIndex < 0)
		{
			return false;
		}

		if (!Mesh.EdgeManifoldCheck(TriIndex, TriEdgeIndex)) 
		{
			return false;
		}

		if (!Mesh.AllowEdgeFlip(TriIndex, TriEdgeIndex, AdjTriIndex))
		{
			return false;
		}
		
		const VecType TriNormal = Mesh.GetTriangleNormal( TriIndex );
		const VecType AdjNormal = Mesh.GetTriangleNormal( AdjTriIndex );

		return ( TriNormal | AdjNormal ) > RealType(Options.CosineThreshold);
	}

	void FindSplitBVH( uint32 TriIndex );

	void SplitTriangle(uint32 TriIndex);

	inline void SetDisplacementAt(int32 VertexIndex, VecType Displacement)
	{
		if (VertexIndex >= Displacements.Num())
		{
			// this should be progressive growth behavior
			Displacements.AddUninitialized(1 + VertexIndex - Displacements.Num());
		}
		Displacements[VertexIndex] = Displacement;
	}

	inline void GrowTriRecords(int32 Num)
	{
		if (Num > TriSplitRecords.Num())
		{
			TriSplitRecords.AddDefaulted(Num - TriSplitRecords.Num());
		}
	}
	
	struct FTriSplit
	{
		VecType	SplitBarycentrics;
		int32	RequestIndex = -1;
	};

	void AddFindRequest(int32 TriIndex);

	void CheckSplitAndEnqueue( int32 TriIndex, int32 Level ); // thread-safe

	void EnqueueSplitRequest( int32 TriIndex, VecType Barycentrics ); // thread-safe

	void RemoveSplitRequest(int32 TriIndex);

	void TryDelaunayFlip( int32 TriIndex, int TriEdgeIndex );

	MeshT&                             Mesh;
	DisplacementPolicyT&               DisplacementPolicy;
	const FAdaptiveTessellatorOptions& Options;

	TArray<VecType>      Displacements;
	TArray<FTriSplit>	 TriSplitRecords;
	TArray<uint32>		 FindRequests;
	TArray<uint32>		 SplitRequests; // triangle indices

	std::atomic<uint32>	 NumSplits;
	std::atomic<uint32>  NumNewTriangles;
};

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::FindSplitBVH( uint32 TriIndex )
{
	const VecType P0 = Mesh.GetVertexPosition(TriIndex, 0);
	const VecType P1 = Mesh.GetVertexPosition(TriIndex, 1);
	const VecType P2 = Mesh.GetVertexPosition(TriIndex, 2);
	
	const VecType Edge01 = P1 - P0;
	const VecType Edge12 = P2 - P1;
	const VecType Edge20 = P0 - P2;

	const VecType EdgeLengths( Edge01.Length(), Edge12.Length(), Edge20.Length() );

	if( EdgeLengths[0] < Options.SampleRate &&
		EdgeLengths[1] < Options.SampleRate &&
		EdgeLengths[2] < Options.SampleRate )
	{
		return;
	}

	const RealType TriArea = RealType(0.5) * ( Edge01 ^ Edge20 ).Size();

	// area of equilateral triangle with edgelength = SampleRate. We want to have 1 sample per triangle of this size
	const RealType SampleArea = static_cast<RealType>(EquilateralArea( Options.SampleRate ));
	
	const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

	const VecType& Displacement0 = Displacements[ Triangle.A ];
	const VecType& Displacement1 = Displacements[ Triangle.B ];
	const VecType& Displacement2 = Displacements[ Triangle.C ];

	bool bCouldFlipEdge[3];
	for( uint32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++ ) 
		bCouldFlipEdge[ EdgeIndex ] = CouldFlipEdge( TriIndex, EdgeIndex );

	VecType	 BestSplit = VecType::ZeroVector;
	RealType BestError = -1.0f;

	struct FNode
	{
		RealType ErrorMin;
		RealType ErrorMax;
		uint32	 TriX	: 13;
		uint32	 TriY	: 13;
		uint32	 FlipTri	: 1;
		uint32	 Level	: 4;
	};

	TArray<FNode, TInlineAllocator<256>> Candidates; // heap, element with largest maxerror at the root

	FNode Node;
	Node.ErrorMin	= 0.0f;
	Node.ErrorMax	= MAX_flt;
	Node.TriX		= 0;
	Node.TriY		= 0;
	Node.FlipTri	= 0;
	Node.Level		= 0;

	{
		VecType Barycentrics[3] =
		{
			{ 1.0f, 0.0f, 0.0f },
			{ 0.0f, 1.0f, 0.0f },
			{ 0.0f, 0.0f, 1.0f },
		};

		const UE::Math::TVector2<RealType> ErrorBounds = DisplacementPolicy.GetErrorBounds(
			Barycentrics,
			Displacement0,
			Displacement1,
			Displacement2,
			TriIndex );
		
		Node.ErrorMin = ErrorBounds.X;
		Node.ErrorMax = ErrorBounds.Y;
	}

	RealType ErrorMinimum = Options.TargetError;

	while(	Node.ErrorMax >= ErrorMinimum &&
			Node.ErrorMax > BestError * 1.25f + Options.TargetError )
	{
		for( uint32 ChildIndex = 0; ChildIndex < 4; ChildIndex++ )
		{
			FNode ChildNode;
			ChildNode.Level = Node.Level + 1;
			ChildNode.FlipTri = Node.FlipTri;

			/*
				|\		\|\|
				|\|\	  \|
			*/

			ChildNode.TriX = Node.TriX * 2 + ( ChildIndex & 1 );
			ChildNode.TriY = Node.TriY * 2 + ( ChildIndex >> 1 );

			if( Node.FlipTri )
			{
				ChildNode.TriX ^= 1;
				ChildNode.TriY ^= 1;
			}

			if( ChildIndex == 3 )
			{
				ChildNode.TriX		^= 1;
				ChildNode.TriY		^= 1;
				ChildNode.FlipTri	^= 1;
			}

			VecType Barycentrics[3];
			SubtriangleBarycentrics<RealType>( ChildNode.TriX, ChildNode.TriY, ChildNode.FlipTri, 1 << ChildNode.Level, Barycentrics );

			// Get the error bounds of the child triangle
			{
				UE::Math::TVector2<RealType> ErrorBounds = DisplacementPolicy.GetErrorBounds(
					Barycentrics,
					Displacement0,
					Displacement1,
					Displacement2,
					TriIndex );

				ChildNode.ErrorMin = ErrorBounds.X;
				ChildNode.ErrorMax = ErrorBounds.Y;

				// Clamp bounds just in case float precision results in them not perfectly nesting
				ChildNode.ErrorMin = FMath::Max( ChildNode.ErrorMin, Node.ErrorMin );
				ChildNode.ErrorMax = FMath::Min( ChildNode.ErrorMax, Node.ErrorMax );
			}

			// ErrorMinimum is the minimum of TargetError and the smallest error achieved so far by any created children
			if( ErrorMinimum > ChildNode.ErrorMax )
				continue;

			if( ErrorMinimum < ChildNode.ErrorMin )
				ErrorMinimum = ChildNode.ErrorMin;

			if( ChildNode.ErrorMax > BestError * 1.25f + Options.TargetError )
			{
				int32 NumSamples = 64;

				bool bStopTraversal = ChildNode.Level == 12 || ChildNode.ErrorMax - ChildNode.ErrorMin < Options.TargetError;
				if( !bStopTraversal )
				{
					RealType ChildArea = SubtriangleArea<RealType>( Barycentrics[0], Barycentrics[1], Barycentrics[2], TriArea );
					
					const bool bSmallEnough = ChildArea < SampleArea * RealType(16.);
					NumSamples = FMath::Min( NumSamples, static_cast<int32>(FMath::CeilToInt( ChildArea / SampleArea )) );

					bStopTraversal = bSmallEnough;
				}

				if( !bStopTraversal )
				{
					NumSamples = FMath::Min( NumSamples, DisplacementPolicy.GetNumSamples(Barycentrics, Triangle, TriIndex));
					bStopTraversal = NumSamples <= 16;
				}

				if( bStopTraversal )
				{
					for( int32 i = 0; i < NumSamples; i++ )
					{
						// Hammersley sequence
						uint32 Reverse = ReverseBits< uint32 >( i + 1 );
						Reverse = Reverse ^ (Reverse >> 1);

						UE::Math::TVector2<RealType> Random;

						// todo see UniformSampleTrianglePoint
						Random.X = RealType( i + 1 ) / RealType( NumSamples + 1 );
						Random.Y = RealType( Reverse >> 8 ) * 0x1p-24f;

						// Square to triangle
						if( Random.X < Random.Y )
						{
							Random.X *= 0.5f;
							Random.Y -= Random.X;
						}
						else
						{
							Random.Y *= 0.5f;
							Random.X -= Random.Y;
						}

						VecType Split;
						Split.X = Random.X;
						Split.Y = Random.Y;
						Split.Z = 1.0f - Split.X - Split.Y;

						RealType DistToEdge[3];
						DistToEdge[0] = DistanceToEdge<RealType>( Split[2], EdgeLengths[0], TriArea );
						DistToEdge[1] = DistanceToEdge<RealType>( Split[0], EdgeLengths[1], TriArea );
						DistToEdge[2] = DistanceToEdge<RealType>( Split[1], EdgeLengths[2], TriArea );

						uint32 e0 = FMath::Min3Index( DistToEdge[0], DistToEdge[1], DistToEdge[2] ); // index of barycentric coordinate that is closest to any edge
						uint32 e1 = (1 << e0) & 3;
						uint32 e2 = (1 << e1) & 3;

						CA_ASSUME(e1 <= 2);
						CA_ASSUME(e2 <= 2);

						bool bTooCloseToEdge = DistToEdge[ e0 ] < 0.5f * Options.SampleRate;
						if( bTooCloseToEdge && !bCouldFlipEdge[ e0 ] )
						{
							Split[ e0 ] = Split[ e0 ] / ( Split[ e0 ] + Split[ e1 ] );
							Split[ e1 ] = 1.0f - Split[ e0 ];
							Split[ e2 ] = 0.0f;

							bool bTooCloseToEdge1 = !bCouldFlipEdge[ e1 ] && DistanceToEdge<RealType>( Split[ e0 ], EdgeLengths[ e1 ], TriArea ) < 0.5f * Options.SampleRate;
							bool bTooCloseToEdge2 = !bCouldFlipEdge[ e2 ] && DistanceToEdge<RealType>( Split[ e1 ], EdgeLengths[ e2 ], TriArea ) < 0.5f * Options.SampleRate;
							bTooCloseToEdge = bTooCloseToEdge1 || bTooCloseToEdge2;
						}

						if( !bTooCloseToEdge )
						{
							const VecType NewDisplacement = DisplacementPolicy.GetDisplacement(Split, TriIndex);

							VecType LerpedDisplacement;
							LerpedDisplacement  = Displacement0 * Split.X;
							LerpedDisplacement += Displacement1 * Split.Y;
							LerpedDisplacement += Displacement2 * Split.Z;

							const RealType Error = ( NewDisplacement - LerpedDisplacement ).SizeSquared();

							if( BestError < Error )
							{
								BestSplit = Split;
								BestError = Error;
							}
						}
					}
				}
				else
				{
					// bStopTraversal == false, push the node on the traversal heap (with the largest error at the root)
					Candidates.HeapPush( ChildNode,
						[&]( const FNode& Node0, const FNode& Node1 )
						{
							return Node0.ErrorMax > Node1.ErrorMax;
						} );
				}
			}
		}

		if( Candidates.IsEmpty() )
			break;

		Candidates.HeapPop( Node,
			[&]( const FNode& Node0, const FNode& Node1 )
			{
				return Node0.ErrorMax > Node1.ErrorMax;
			}, EAllowShrinking::No );
	}

	if( BestError > Options.TargetError )
	{
		EnqueueSplitRequest( TriIndex, BestSplit );
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::SplitTriangle( uint32 TriIndex )
{
	const VecType Barycentrics = TriSplitRecords[ TriIndex ].SplitBarycentrics;
	TriSplitRecords[ TriIndex ].RequestIndex = -1;

	// first check whether we are splitting along an edge
	for( uint32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++ )
	{
		if( Barycentrics[ ( EdgeIndex + 2 ) % 3 ] == RealType(0) )
		{
			if (!Mesh.AllowEdgeSplit(TriIndex, EdgeIndex))
			{
				return;
			}

			const FEdgeSplitInfo SplitInfo = Mesh.SplitEdge( TriIndex, EdgeIndex, Barycentrics[EdgeIndex] );

			if (SplitInfo.NewTriIndex[0] == IndexConstants::InvalidID)
			{
				return;
			}

			int NumNewTris = 0;
			// evaluate the displacement at the newly created vertices, possibly material seam
			// requires to evaluate displacement twice
			for( int32 j = 0; j < 2; j++ )
			{
				if (SplitInfo.OldTriIndex[j] >= 0)
				{
					const VecType NewDisplacement = DisplacementPolicy.GetVertexDisplacement(SplitInfo.NewVertIndex[j], SplitInfo.OldTriIndex[j]);

					// handle cases where split edge introduces 1 or 2 vertices at the edge. if 1, average the results
					if (j == 1 && SplitInfo.NewVertIndex[1] == SplitInfo.NewVertIndex[0])
					{
						VecType& D = Displacements[SplitInfo.NewVertIndex[1]];
						D = 0.5f * (D + NewDisplacement);
					}
					else
					{
						SetDisplacementAt(SplitInfo.NewVertIndex[j], NewDisplacement);
					}
					
					++NumNewTris;
				}
			}

			GrowTriRecords(FMath::Max(SplitInfo.NewTriIndex[0], SplitInfo.NewTriIndex[1])+1);
			check(TriSplitRecords.Num() <= Mesh.MaxTriID());
						
			for( int32 j = 0; j < 2; j++ )
			{
				if (SplitInfo.OldTriIndex[j] >= 0)
				{
					RemoveSplitRequest( SplitInfo.OldTriIndex[j] );

					AddFindRequest( SplitInfo.OldTriIndex[j] );
					AddFindRequest( SplitInfo.NewTriIndex[j] );

					TryDelaunayFlip( SplitInfo.OldTriIndex[j], 1 );
					TryDelaunayFlip( SplitInfo.NewTriIndex[j], 2 );
				}
			}
			
			// we have performed an edge-split and are done here
			return;
		}
	}

	// regular interior point split (poke)
	const VecType NewDisplacement = DisplacementPolicy.GetDisplacement(Barycentrics, TriIndex);
	const FPokeInfo pokeInfo = Mesh.PokeTriangle(TriIndex, Barycentrics);

	if (pokeInfo.NewTriIndex[0] != IndexConstants::InvalidID)
	{
		check(pokeInfo.NewTriIndex[0] == TriIndex);

		GrowTriRecords(FMath::Max(pokeInfo.NewTriIndex[1], pokeInfo.NewTriIndex[2]) + 1);
		check(pokeInfo.NewTriIndex[1] < Mesh.MaxTriID());
		check(pokeInfo.NewTriIndex[2] < Mesh.MaxTriID());

		SetDisplacementAt(pokeInfo.NewVertIndex, NewDisplacement);

		for (uint32 LocalTriIndex = 0; LocalTriIndex < 3; ++LocalTriIndex)
		{
			AddFindRequest(pokeInfo.NewTriIndex[LocalTriIndex]);
			TryDelaunayFlip(pokeInfo.NewTriIndex[LocalTriIndex], pokeInfo.EdgeIndex[LocalTriIndex]);
		}
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::AddFindRequest( int32 TriIndex )
{
	check(TriIndex < TriSplitRecords.Num());
	int32& RequestIndex = TriSplitRecords[ TriIndex ].RequestIndex;
	if( RequestIndex != -2 )
	{
		check( RequestIndex == -1 );
		FindRequests.Add( TriIndex );
		RequestIndex = -2;
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::CheckSplitAndEnqueue( const int32 TriIndex, const int32 Level )
{
	TriSplitRecords[ TriIndex ].RequestIndex = -1;

	if ( Options.RefinementMethod == EAdaptiveTessellationRefinementMethod::HierarchicalSplit )
	{
		FindSplitBVH( TriIndex );
	}
	else
	{
		VecType SplitVertexBary;
		if (DisplacementPolicy.ShouldRefine( TriIndex, Displacements, SplitVertexBary, Level ))
		{
			EnqueueSplitRequest( TriIndex, SplitVertexBary );
		}
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::EnqueueSplitRequest( int32 TriIndex, VecType Barycentrics )
{
	uint32 AddTrianglesPerRefine = 3;
	if (Barycentrics[0] == RealType(0) || Barycentrics[1] == RealType(0) || Barycentrics[2] == RealType(0) )
	{
		AddTrianglesPerRefine = 2;
	}

	if ( NumNewTriangles.fetch_add(AddTrianglesPerRefine) < Options.MaxTriangles - Mesh.MaxTriID() )
	{
		TriSplitRecords[ TriIndex ].SplitBarycentrics = Barycentrics;
		TriSplitRecords[ TriIndex ].RequestIndex = NumSplits++;
		SplitRequests[ TriSplitRecords[ TriIndex ].RequestIndex ] = TriIndex;
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::RemoveSplitRequest( int32 TriIndex )
{
	int32& RequestIndex = TriSplitRecords[ TriIndex ].RequestIndex;
	if( RequestIndex >= 0 )
	{
		TriSplitRecords[ SplitRequests.Last() ].RequestIndex = RequestIndex;
		SplitRequests.RemoveAtSwap( RequestIndex, EAllowShrinking::No );
		RequestIndex = -1;
	}
}

template <typename MeshT, typename DisplacementPolicyT>
inline void TAdaptiveTessellator<MeshT, DisplacementPolicyT>::TryDelaunayFlip( int32 TriIndex, int32 TriEdgeIndex )
{
	if( !CouldFlipEdge( TriIndex, TriEdgeIndex ) ) 
		return;

	auto ComputeCotangent = [this]( uint32 TriIndex, int TriEdgeIndex ) -> RealType
	{
		const uint32 e0 = TriEdgeIndex;
		const uint32 e1 = NextEdge(TriEdgeIndex);
		const uint32 e2 = PrevEdge(TriEdgeIndex);

		const VecType P0 = Mesh.GetVertexPosition(TriIndex, e0);
		const VecType P1 = Mesh.GetVertexPosition(TriIndex, e1);
		const VecType P2 = Mesh.GetVertexPosition(TriIndex, e2);
				
		const VecType Edge01 = P1 - P0;
		const VecType Edge12 = P2 - P1;
		const VecType Edge20 = P0 - P2;

		VecType EdgeLengthsSqr(
			Edge01.SizeSquared(),
			Edge12.SizeSquared(),
			Edge20.SizeSquared() );

		const RealType TriArea = RealType(0.5) * ( Edge01 ^ Edge20 ).Size();
		return RealType(0.25) * ( -EdgeLengthsSqr.X + EdgeLengthsSqr.Y + EdgeLengthsSqr.Z ) / TriArea;
	};

	const TPair<int32, int32> OppositeEdge = Mesh.GetAdjEdge(TriIndex, TriEdgeIndex);

	const RealType LaplacianWeight = RealType(0.5) * (ComputeCotangent( TriIndex, TriEdgeIndex ) + 
		                                              ComputeCotangent( OppositeEdge.Get<0>(), OppositeEdge.Get<1>() ));

	const bool bFlipEdge = LaplacianWeight < RealType(-1e-6);
	if( bFlipEdge )
	{
		const FFlipEdgeInfo FlipEdgeInfo = Mesh.FlipEdge(TriIndex, TriEdgeIndex);

		if (FlipEdgeInfo.Triangles[0] != IndexConstants::InvalidID)
		{
			check(FlipEdgeInfo.Triangles[1] != IndexConstants::InvalidID);

			const int32 AdjTriIndex = OppositeEdge.Get<0>();
			const int32 AdjTriEdgeIndex = OppositeEdge.Get<1>();

			RemoveSplitRequest(TriIndex);
			RemoveSplitRequest(AdjTriIndex);

			AddFindRequest(TriIndex);
			AddFindRequest(AdjTriIndex);

			TryDelaunayFlip(FlipEdgeInfo.Triangles[0], NextEdge(FlipEdgeInfo.SharedEdge[0]));
			TryDelaunayFlip(FlipEdgeInfo.Triangles[0], PrevEdge(FlipEdgeInfo.SharedEdge[0]));

			TryDelaunayFlip(FlipEdgeInfo.Triangles[1], PrevEdge(FlipEdgeInfo.SharedEdge[1]));
			TryDelaunayFlip(FlipEdgeInfo.Triangles[1], NextEdge(FlipEdgeInfo.SharedEdge[1]));
		}
	}
}

} // namespace Geometry
} // namespace UE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Templates/IsMemberPointer.h"
#endif
