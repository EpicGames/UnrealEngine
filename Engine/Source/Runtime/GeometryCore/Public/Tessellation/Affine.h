// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Concepts/FloatingPoint.h"
#include "Math/Vector.h"

namespace UE {
namespace Geometry {

namespace Affine
{

template <typename T> using TVector2 = UE::Math::TVector2<T>;
template <typename T> using TVector  = UE::Math::TVector<T>;
template <typename T> using TVector4 = UE::Math::TVector4<T>;

template< typename T > FORCEINLINE T Abs(const T A) { return FMath::Abs(A); }
template< typename T > FORCEINLINE TVector2<T> Abs(const TVector2<T>& V) { return TVector2<T>(FMath::Abs(V.X), FMath::Abs(V.Y)); }
template< typename T > FORCEINLINE TVector <T> Abs(const TVector <T>& V) { return TVector <T>(FMath::Abs(V.X), FMath::Abs(V.Y), FMath::Abs(V.Z)); }
template< typename T > FORCEINLINE TVector4<T> Abs(const TVector4<T>& V) { return TVector4<T>(FMath::Abs(V.X), FMath::Abs(V.Y), FMath::Abs(V.Z), FMath::Abs(V.W)); }

} // namespace Affine


// Affine arithmetic
// 
// [ Thonat et al. 2021, "Tessellation-Free Displacement Mapping for Ray Tracing" ]
// [ de Figueiredo and Stolf 2004, "Affine Arithmetic: Concepts and Applications" ]
// [ Rump and Kashiwagi 2015, "Implementation and improvements of affine arithmetic" ]

template< typename T, uint32 Num >
struct TAffine
{
	T c;
	T K;
	T e[ Num ];

	TAffine() {}
	FORCEINLINE TAffine( T Constant )
		: c( Constant )
		, K( 0.0f )
	{
		for( uint32 i = 0; i < Num; i++ )
			e[i] = T( 0.0f );
	}

	FORCEINLINE TAffine( T Min, T Max )
		: c( 0.5f * ( Min + Max ) )
		, K( 0.5f * ( Max - Min ) )
	{
		for( uint32 i = 0; i < Num; i++ )
			e[i] = T( 0.0f );
	}

	FORCEINLINE TAffine( T Min, T Max, uint32 Index )
		: c( 0.5f * ( Min + Max ) )
		, K( 0.0f )
	{
		for( uint32 i = 0; i < Num; i++ )
			e[i] = T( 0.0f );
		e[ Index ] = 0.5f * ( Max - Min );
	}

	FORCEINLINE TAffine< T, Num >& operator+=( const TAffine< T, Num >& Other )
	{
		c += Other.c;
		K += Other.K;
		for( uint32 i = 0; i < Num; i++ )
			e[i] += Other.e[i];
		
		return *this;
	}

	FORCEINLINE TAffine< T, Num > operator+( const TAffine< T, Num >& Other ) const
	{
		return TAffine< T, Num >(*this) += Other;
	}

	FORCEINLINE TAffine< T, Num >& operator-=( const TAffine< T, Num >& Other )
	{
		c -= Other.c;
		K += Other.K;
		for( uint32 i = 0; i < Num; i++ )
			e[i] -= Other.e[i];
		
		return *this;
	}

	FORCEINLINE TAffine< T, Num > operator-( const TAffine< T, Num >& Other ) const
	{
		return TAffine< T, Num >(*this) -= Other;
	}

	FORCEINLINE T GetMin() const
	{
		T Result = c - Affine::Abs(K);
		for( uint32 i = 0; i < Num; i++ )
			Result -= Affine::Abs( e[i] );

		return Result;
	}

	FORCEINLINE T GetMax() const
	{
		T Result = c + Affine::Abs(K);
		for( uint32 i = 0; i < Num; i++ )
			Result += Affine::Abs( e[i] );

		return Result;
	}

	// Smaller than (v|v)
	// require that T has SizeSquared and declares floating point type FReal 
	template <typename Q = T>
		requires (UE::CFloatingPoint<typename Q::FReal> && std::is_member_function_pointer_v<decltype(&Q::SizeSquared)>)
	FORCEINLINE TAffine<typename Q::FReal, Num> SizeSquared() const
	{
		using FReal = typename Q::FReal;

		TAffine< FReal, Num > Result;
		Result.c = c.SizeSquared();
		Result.K = FReal(2.0) * Affine::Abs( c | K );

		T Extent = K;
		for( uint32 i = 0; i < Num; i++ )
		{
			Extent += Affine::Abs( e[i] );
			Result.e[i] = FReal(2.0) * ( c | e[i] );
		}

		Result.c += FReal(0.5) * Extent.SizeSquared();
		Result.K += FReal(0.5) * Extent.SizeSquared();

		return Result;
	}
};

template< typename T, typename U, uint32 Num >
FORCEINLINE TAffine< T, Num > operator*( const TAffine< T, Num >& A, const TAffine< U, Num >& B )
{
	TAffine< T, Num > Result;
	Result.c = A.c * B.c;
	Result.K =
		Affine::Abs( A.K * B.c ) +
		Affine::Abs( A.c * B.K );

	T AK = A.K;
	U BK = B.K;
	for( uint32 i = 0; i < Num; i++ )
	{
		Result.e[i] = A.e[i] * B.c + A.c * B.e[i];
		AK += Affine::Abs( A.e[i] );
		BK += Affine::Abs( B.e[i] );
	}
	Result.K += AK * BK;

	return Result;
}

template< typename T, uint32 Num >
FORCEINLINE TAffine< float, Num > operator|( const TAffine< T, Num >& A, const TAffine< T, Num >& B )
{
	TAffine< float, Num > Result;
	Result.c = A.c | B.c;
	Result.K =
		Affine::Abs( A.K | B.c ) +
		Affine::Abs( A.c | B.K );

	T AK = A.K;
	T BK = B.K;
	for( uint32 i = 0; i < Num; i++ )
	{
		Result.e[i] = ( A.e[i] | B.c ) + ( A.c | B.e[i] );
		AK += Affine::Abs( A.e[i] );
		BK += Affine::Abs( B.e[i] );
	}
	Result.K += AK | BK;

	return Result;
}

template <UE::CFloatingPoint T, uint32 Num>
FORCEINLINE TAffine<T, Num> Clamp( const TAffine< T, Num >& x, T Min, T Max )
{
	// Using Chebyshev approximation
	T xMin = x.GetMin();
	T xMax = x.GetMax();
	T FuncMin = FMath::Clamp( xMin, Min, Max );
	T FuncMax = FMath::Clamp( xMax, Min, Max );

	if( Min <= xMin && xMax <= Max )
		return x;
	if( xMax <= Min )
		return TAffine< T, Num >( Min );
	if( xMin >= Max )
		return TAffine< T, Num >( Max );

	T Alpha = ( FuncMax - FuncMin ) / ( xMax - xMin );
	T Gamma = T(0.5) * ( T(1.) - Alpha ) * ( FuncMax + FuncMin );
	T Delta = ( T(1.) - Alpha ) * FuncMax - Gamma;

	TAffine<T, Num > Result;
	Result.c = Alpha * x.c + Gamma;
	Result.K = FMath::Abs( Alpha * x.K ) + Delta;
	for( uint32 i = 0; i < Num; i++ )
		Result.e[i] = Alpha * x.e[i];
	
	return Result;
}

template <UE::CFloatingPoint T, uint32 Num>
FORCEINLINE TAffine<T, Num> InvSqrt( const TAffine< T, Num >& x )
{
	// Using min range approximation
	T xMin = FMath::Max( T(1.e-4), x.GetMin() );
	T xMax = FMath::Max( T(1.e-4), x.GetMax() );
	T FuncMin = FMath::InvSqrt( xMin );
	T FuncMax = FMath::InvSqrt( xMax );

	T Alpha = -T(0.5) * FuncMax * FuncMax * FuncMax;
	T Gamma = T(0.5) * ( FuncMin + FuncMax - Alpha * ( xMin + xMax ) );
	T Delta = FMath::Abs( T(0.5) * ( FuncMin - FuncMax - Alpha * ( xMin - xMax ) ) );

	TAffine< T, Num > Result;
	Result.c = Alpha * x.c + Gamma;
	Result.K = FMath::Abs( Alpha * x.K ) + Delta;

	for( uint32 i = 0; i < Num; i++ ) 
		Result.e[i] = Alpha * x.e[i];

	return Result;
}

template< typename T, uint32 Num >
FORCEINLINE TAffine< T, Num > Normalize( const TAffine< T, Num >& x )
{
	using FReal = typename T::FReal;
	return x * InvSqrt( Clamp<FReal, Num>( x.SizeSquared(), FReal(1.e-4), FReal(1.0) ) );
}

} // namespace Geometry
} // namespace UE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Templates/EnableIf.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsMemberPointer.h"
#endif
