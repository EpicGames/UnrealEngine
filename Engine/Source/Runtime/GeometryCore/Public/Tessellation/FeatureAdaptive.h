// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/UnrealMathUtility.h"

namespace UE {
namespace Geometry {

// [ Baensch et al. "A Finite Element Method for Surface Diffusion", 2004] Sec 5.4
// Feature adaptive refinement criterion
template <class T>
class TFeatureSensitivityRefinementCheck
{
public:
	
	// @param InFeatureSensitivity   sensitivity of feature check, between 0 and 1
	// @param InFeatureSize          the grid size parameter (h_T in the paper)
	// @param V0, V1, V2             triangle vertices
	TFeatureSensitivityRefinementCheck(const T InFeatureSensitivity, const T InFeatureSize,
		const Math::TVector<T>& V0, 
		const Math::TVector<T>& V1,
		const Math::TVector<T>& V2)
		: Threshold(FMath::Max(T(1.) / (InFeatureSensitivity*T(0.5) + T(0.5)) - T(0.95), T(0.05)))
		, TriVertices{ V0, V1, V2 }
		, TriNormal( (V2-V0) ^ (V1-V0) )
		, LSqN(TriNormal.SquaredLength())
		, IncircleRadius(CalculateInRadius(V0, V1, V2))
		, FeatureSize(InFeatureSize)
	{
	}

	[[nodiscard]] FORCEINLINE bool NeedsCheck() const
	{
		return (Threshold * FeatureSize < IncircleRadius * UE_PI); // Early-out (also if IncircleRadius is NaN)
	}

	// check whether the edge should be refined refined
	// @param LocalEdge local edge index with respect to the triangle vertices ordering
	// @param FarVertex vertex on the other side of that edge
	//  
	[[nodiscard]] FORCEINLINE bool CheckCriterion(const int32 LocalEdge, const UE::Math::TVector<T>& FarVertex ) const 
	{
		const Math::TVector<T> NT = (TriVertices[(LocalEdge + 1) % 3] - TriVertices[LocalEdge]) ^ (FarVertex - TriVertices[LocalEdge]);
		const T LSqNT = NT.SquaredLength();

		return ((TriNormal | NT) < FMath::Sqrt(LSqN * LSqNT) * FMath::Cos(Threshold * FeatureSize / IncircleRadius));
	}

private:

	static T CalculateInRadius( const UE::Math::TVector<T>& V0, const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2 )
	{
		const T EdgeLengths_D[] = { (V1 - V0).Size(), (V2 - V1).Size(), (V0 - V2).Size() };
		const T SemiPerimeter = T(0.5) * (EdgeLengths_D[0] + EdgeLengths_D[1] + EdgeLengths_D[2]);
		const T TriArea = FMath::Sqrt(FMath::Max(T(0.), SemiPerimeter * (SemiPerimeter - EdgeLengths_D[0]) * (SemiPerimeter - EdgeLengths_D[1]) * (SemiPerimeter - EdgeLengths_D[2]))); // Heron
		return TriArea / SemiPerimeter;
	}

    const T                Threshold;
	const Math::TVector<T> TriVertices[3];
	const Math::TVector<T> TriNormal;
	const T                LSqN;            // SquaredLength of Normal vector
	const T                IncircleRadius;
	const T                FeatureSize;
};

} // namespace Geometry
} // namespace UE