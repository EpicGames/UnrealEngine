// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "IndexTypes.h"
#include "VectorUtil.h"
#include "Async/ParallelFor.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include <atomic>

namespace UE {
namespace Geometry {

template <typename MeshType, typename RefinementPolicyType>
class TParallelAdaptiveRefinement : public FNoncopyable
{
private:

	struct FTriSplitRecord
	{
		uint16 SplitMask                 { 0 }; //< bitmask: per-edge refinement tagging [0..3)
		std::atomic<uint16> GreenAdjMask { 0 }; //< bitmask: encode which of the neighbors [0..3) are green refinement 
		std::atomic_flag bClaimed = ATOMIC_FLAG_INIT;

		inline void Reset()
		{
			SplitMask = 0;
			GreenAdjMask.store(0);
			bClaimed.clear();
		}

		// split one edge (green refinement)
		bool IsSplit1() const
		{
			return SplitMask == 1 || SplitMask == 2 || SplitMask == 4;
		}

		// split two edges
		bool IsSplit2() const
		{
			return SplitMask == 3 || SplitMask == 5 || SplitMask == 6;
		}

		// split all three edges (red refinement)
		bool IsSplit3() const
		{
			return SplitMask == 7;
		}
	};

	struct FEdgeSplitRecord
	{
		void Reset()
		{
			SplitVertex = IndexConstants::InvalidID;
			HalfEdges[0] = FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID);
			HalfEdges[1] = FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID);
			NewEdge = IndexConstants::InvalidID;

			bTagged.clear();
			EntryCountPreWrite.store(0);
			EntryCountPostWrite.store(0);
		}

		int32 SplitVertex { IndexConstants::InvalidID };

		// each entry pair corresponds to the half-edge indices of one side of the split
		FIndex2i HalfEdges[2] { FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID),
								FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID) };

		int32 NewEdge { IndexConstants::InvalidID };

		std::atomic_flag   bTagged = ATOMIC_FLAG_INIT; // tagged for refinement
		std::atomic<int32> EntryCountPreWrite  { 0 };  // current number of entries, before write
		std::atomic<int32> EntryCountPostWrite { 0 };  // number of half-edge pairs written
	};

	struct FConcurrencyState
	{
		TArray<FEdgeSplitRecord> EdgeSplitRecords;
		TArray<FTriSplitRecord>  TriSplitRecords;
		std::atomic<int32>       HalfEdgeCount;   // current number of halfedges, for parallel append
		std::atomic<int32>       EdgeCount;       // current number of edges, for parallel append

		void Resize(const int32 NumTris, const int32 NumEdges)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TParallelAdaptiveRefinement.FConcurrencyState.Resize");

			for (FTriSplitRecord& Record : TriSplitRecords)
			{
				Record.Reset();
			}
			if (NumTris > TriSplitRecords.Num())
			{
				TriSplitRecords.AddDefaulted(NumTris - TriSplitRecords.Num());
			}
			check(TriSplitRecords.Num() == NumTris);

			for (FEdgeSplitRecord &Record : EdgeSplitRecords)
			{
				Record.Reset();
			}

			if (NumEdges > EdgeSplitRecords.Num())
			{
				EdgeSplitRecords.AddDefaulted(NumEdges - EdgeSplitRecords.Num());
			}
		}
	};

public:
	using RealType = typename MeshType::RealType; // the field related to the vector-space
	using VecType  = typename MeshType::VecType;  // for positions and normals

	struct FOptions
	{
		int32 ParallelForBatchSize  { 1024 };

		//< pre-reserve enough space for growing up to this many vertices
		int32 ReserveVertices = 2'000'000;

		bool bApplyNeighborhoodRefinement { true };
	};

	// Initialize and run adaptive refinement
	TParallelAdaptiveRefinement(
		MeshType& InMesh,                               //< mesh interface
		RefinementPolicyType& InRefinementPolicy,       //< displacement and error interface
		const FOptions& InOptions)  
		: Mesh(InMesh)
		, RefinementPolicy(InRefinementPolicy)
		, Options(InOptions)
	{
		const int ParallelForBatchSize = Options.ParallelForBatchSize;
		const EParallelForFlags ParallelForFlags = ParallelForBatchSize > 0 ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

		const int32 ReserveTriangles = 2 * Options.ReserveVertices;
		const int32 ReserveEdges     = 3 * Options.ReserveVertices;

		FConcurrencyState ConcurrencyState;
		TArray<int32> RefinementCand;      //< list of triangles to inspect for refinement
		TArray<int32> TriSplitQueue;       //< list of triangle indices scheduled for refinement
		TArray<int32> EdgeSplitRequests;   //< list of edge indices scheduled for refinement

		RefinementCand.Reserve(ReserveTriangles);
		TriSplitQueue.Reserve(ReserveTriangles);
		EdgeSplitRequests.Reserve(ReserveEdges);

		ConcurrencyState.TriSplitRecords.Reserve(ReserveTriangles);
		ConcurrencyState.EdgeSplitRecords.Reserve(ReserveEdges);

		Mesh.ReserveVertices(Options.ReserveVertices);
		Mesh.ReserveTriangles(ReserveTriangles);

		RefinementCand.Init(0, Mesh.MaxTriID());
		for (int32 TriangleIdx(0); TriangleIdx<Mesh.MaxTriID(); ++TriangleIdx)
		{
			RefinementCand[TriangleIdx] = TriangleIdx;
		}

		std::atomic<int32> NumRefinementCand(RefinementCand.Num());

		for (int32 Level(0); Level < 32; ++Level)
		{
			if (NumRefinementCand.load() == 0) {
				break;
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TParallelAdaptiveRefinement.ResizeRequests");
				// reserve sufficient space for tris and edges to be split
				TriSplitQueue.SetNum(Mesh.MaxTriID());
				EdgeSplitRequests.SetNum(Mesh.EdgeCount());
				ConcurrencyState.Resize(Mesh.MaxTriID(), Mesh.EdgeCount());
			}

			check(ConcurrencyState.TriSplitRecords.Num() == Mesh.MaxTriID());

			std::atomic<int32> TriSplitQueueSize(0);
			std::atomic<int32> NumEdgesToSplit(0);

			auto TagForRefine = [&](const int32 Idx)
			{
				const int32 TriIndex = RefinementCand[Idx];

				if (!Mesh.IsValidTri(TriIndex))
				{
					return;
				}

				if (ShouldRefine(TriIndex, Level))
				{
					if (!ConcurrencyState.TriSplitRecords[TriIndex].bClaimed.test_and_set())
					{
						TriSplitQueue[TriSplitQueueSize.fetch_add(1, std::memory_order_relaxed)] = TriIndex;
						ConcurrencyState.TriSplitRecords[TriIndex].SplitMask = 7;

						for (int32 LocalEdgeIdx=0; LocalEdgeIdx<3; ++LocalEdgeIdx)
						{
							const int32 EdgeIdx = Mesh.GetEdgeIndex(TriIndex, LocalEdgeIdx);
							FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIdx];

							// check whether this is the first time visiting this edge
							if (!Edge.bTagged.test_and_set(std::memory_order_relaxed))
							{
								EdgeSplitRequests[ NumEdgesToSplit.fetch_add(1, std::memory_order_relaxed) ] = EdgeIdx;
							}
						}
					}

					if (Options.bApplyNeighborhoodRefinement)
					{
						// for more gradual mesh regularity, tag neighbors for refinement.
						for (int NbIdx=0; NbIdx<3; ++NbIdx)
						{
							const int32 AdjTriIndex = Mesh.GetAdjTriangle(TriIndex, NbIdx);
							if (AdjTriIndex != IndexConstants::InvalidID && RefinementPolicy.AllowRefine(AdjTriIndex, Level) && 
								!ConcurrencyState.TriSplitRecords[AdjTriIndex].bClaimed.test_and_set())
							{
								check(Mesh.IsValidTri(AdjTriIndex));

								TriSplitQueue[TriSplitQueueSize.fetch_add(1, std::memory_order_relaxed)] = AdjTriIndex;
								ConcurrencyState.TriSplitRecords[AdjTriIndex].SplitMask = 7;

								for (int32 LocalEdgeIdx=0; LocalEdgeIdx<3; ++LocalEdgeIdx)
								{
									const int32 EdgeIdx = Mesh.GetEdgeIndex(AdjTriIndex, LocalEdgeIdx);
									FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIdx];

									// check whether this is the first time visiting this edge
									if (!Edge.bTagged.test_and_set(std::memory_order_relaxed))
									{
										EdgeSplitRequests[ NumEdgesToSplit.fetch_add(1, std::memory_order_relaxed) ] = EdgeIdx;
									}
								}
							}
						}
					}
				}
			};

			// tag triangles for refinement;
			ParallelForTemplate( TEXT("TParallelAdaptiveRefinement.TagForRefine.PF"), NumRefinementCand.load(), ParallelForBatchSize, TagForRefine, ParallelForFlags );

			NumRefinementCand.store(0);
			if (TriSplitQueueSize.load() == 0)
			{
				break;
			}

			std::atomic<int32> NumNewInteriorEdges(0);
			std::atomic<int32> NumNewTriangles(0);

			// identify neighboring triangles closure split masks (T-junctions)
			ParallelFor( TEXT("TParallelAdaptiveRefinement.ResolveNeighbors.PF"), Mesh.MaxTriID(), ParallelForBatchSize, [&](const int32 TriIndex)
			{
				if (!Mesh.IsValidTri(TriIndex))
				{
					return;
				}

				int32 SplitMask = ConcurrencyState.TriSplitRecords[TriIndex].SplitMask;

				if (SplitMask != 0)
				{
					check(SplitMask == 7);
				}
				else
				{
					for (int LocalEdgeIdx=0; LocalEdgeIdx<3; ++LocalEdgeIdx)
					{
						const int32 AdjTriIndex = Mesh.GetAdjTriangle(TriIndex, LocalEdgeIdx);
						if (AdjTriIndex != IndexConstants::InvalidID && ConcurrencyState.TriSplitRecords[AdjTriIndex].IsSplit3())
						{
							SplitMask |= 1 << LocalEdgeIdx;
						}
					}
				}

				if (ConcurrencyState.TriSplitRecords[TriIndex].SplitMask == 0 && SplitMask != 0)
				{
					// record that this triangle needs to be split as a consequence as well
					TriSplitQueue[TriSplitQueueSize.fetch_add(1, std::memory_order_relaxed)] = TriIndex;
				}

				ConcurrencyState.TriSplitRecords[TriIndex].SplitMask = SplitMask;
				ConcurrencyState.TriSplitRecords[TriIndex].GreenAdjMask.store(uint16(0));

				if (SplitMask == 7)
				{
					// full split
					NumNewInteriorEdges.fetch_add(3, std::memory_order_relaxed);
					NumNewTriangles.fetch_add(3, std::memory_order_relaxed);
				}
				else if (SplitMask == 1 || SplitMask == 2 || SplitMask == 4)
				{
					// green refinement
					NumNewInteriorEdges.fetch_add(1, std::memory_order_relaxed);
					NumNewTriangles.fetch_add(1, std::memory_order_relaxed);
				}
				else if (SplitMask != 0)
				{
					NumNewInteriorEdges.fetch_add(2, std::memory_order_relaxed);
					NumNewTriangles.fetch_add(2, std::memory_order_relaxed);
				}
			}, ParallelForFlags );

			// for each green refined triangles, store the edge mask for each neighboring triangle that is marked for refinement
			ParallelFor( TEXT("TParallelAdaptiveRefinement.IdentifyGreenNeighbors.PF"), TriSplitQueueSize.load(), ParallelForBatchSize, [&](int32 Idx)
			{
				const int32 TriIndex = TriSplitQueue[Idx];
				check(ConcurrencyState.TriSplitRecords[TriIndex].SplitMask != 0);

				if (ConcurrencyState.TriSplitRecords[TriIndex].IsSplit1())
				{
					// mark neighbors
					for (int32 NbIdx=0; NbIdx<3; ++NbIdx)
					{
						const TPair<int32,int32> TriEdge = Mesh.GetAdjEdge(TriIndex, NbIdx);
						if (TriEdge.Get<0>() != IndexConstants::InvalidID && ConcurrencyState.TriSplitRecords[TriEdge.Get<0>()].SplitMask != 0)
						{
							ConcurrencyState.TriSplitRecords[TriEdge.Get<0>()].GreenAdjMask.fetch_or(static_cast<uint16>(1u << TriEdge.Get<1>()));
						}
					}
				}
			}, ParallelForFlags);

			// grow the arrays for edges and vertices
			int32 PrevEdgeCount, PrevVtxCount;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TParallelAdaptiveRefinement.GrowVerts");

				const int32 NewVertexCount = NumEdgesToSplit.load();
				const int32 NewEdgeCount = NewVertexCount + NumNewInteriorEdges.load();

				ConcurrencyState.EdgeCount.store(Mesh.EdgeCount() + NewVertexCount);

				PrevEdgeCount = Mesh.AddEdges(NewEdgeCount);
				PrevVtxCount = Mesh.AddVertices(NewVertexCount);

				ConcurrencyState.EdgeSplitRecords.AddDefaulted( NewEdgeCount );
				check(ConcurrencyState.EdgeSplitRecords.Num() == Mesh.EdgeCount());

				RefinementPolicy.AllocateVertices(NewVertexCount);
			}

			// assign new edge-midpoints, assign displacement
			ParallelForTemplate(TEXT("TParallelAdaptiveRefinement.MakeEdgeMidpoints.PF"), NumEdgesToSplit.load(), ParallelForBatchSize, [&](const int32 RequestIdx)
			{
				const int32 EdgeIndex = EdgeSplitRequests[RequestIdx];
				FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIndex];

				Edge.NewEdge = PrevEdgeCount + RequestIdx;
				Edge.SplitVertex = PrevVtxCount + RequestIdx;

				FIndex2i VertexIndices, Tris;

				Mesh.GetEdge(EdgeIndex, VertexIndices, Tris);
				Mesh.InterpolateVertex(0.5f, Edge.SplitVertex, EdgeIndex);
				RefinementPolicy.VertexAdded(Edge.SplitVertex, Tris);

			}, ParallelForFlags  );

			ConcurrencyState.HalfEdgeCount.store(Mesh.HalfEdgeCount());
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("TParallelAdaptiveRefinement.GrowHalfEdges");

				Mesh.AddTriangles(NumNewTriangles.load());
				RefinementCand.AddDefaulted(NumNewTriangles.load());
			}

			// walk through refinement queue, apply refinement, enqueue triangles to check for refinement
			ParallelFor( TEXT("TParallelAdaptiveRefinement.RefineTri.PF"), TriSplitQueueSize.load(), ParallelForBatchSize, [&](int32 Idx)
			{
				const int32 TriIndex = TriSplitQueue[Idx];
				check(ConcurrencyState.TriSplitRecords[TriIndex].SplitMask != 0);

				if (ConcurrencyState.TriSplitRecords[TriIndex].IsSplit3())
				{
					const int32 FirstNewTri = Split3(TriIndex, ConcurrencyState);

					// enqueue all 4 children as refinement candidates
					const int32 QueueIdx = NumRefinementCand.fetch_add(4, std::memory_order_relaxed);
					RefinementCand[QueueIdx+0] = FirstNewTri+0;
					RefinementCand[QueueIdx+1] = FirstNewTri+1;
					RefinementCand[QueueIdx+2] = FirstNewTri+2;
					RefinementCand[QueueIdx+3] = TriIndex;
				}
				else if (ConcurrencyState.TriSplitRecords[TriIndex].IsSplit2())
				{
					Split2(TriIndex, ConcurrencyState);
				}
				else if (ConcurrencyState.TriSplitRecords[TriIndex].IsSplit1())
				{
					Split1(TriIndex, ConcurrencyState);
				}

				// All triangle split records should be in clean state after this loop
				ConcurrencyState.TriSplitRecords[TriIndex].Reset();
			}, ParallelForFlags );

			check(ConcurrencyState.HalfEdgeCount.load() == Mesh.HalfEdgeCount());

			// Mesh.CheckConsistency();
			
			TriSplitQueueSize.store(0);
		}
	}

	// split 1 edge, keep two edges (1 new triangle)
	inline void Split1(int32 TriIndex, FConcurrencyState& ConcurrencyState)
	{
		const int32 SplitMask = ConcurrencyState.TriSplitRecords[TriIndex].SplitMask;
		check(ConcurrencyState.TriSplitRecords[TriIndex].IsSplit1());

		// determine the edge that is split
		int32 LocSplitEdge = 0;
		if (SplitMask == 2) LocSplitEdge = 1;
		else if (SplitMask == 4) LocSplitEdge = 2;

		const int32 NextLocEdge = (LocSplitEdge+1)%3;
		const int32 PrevLocEdge = (LocSplitEdge+2)%3;

		const int32 EdgeIndices[3] = { Mesh.GetEdgeIndex(TriIndex, 0),
									   Mesh.GetEdgeIndex(TriIndex, 1),
									   Mesh.GetEdgeIndex(TriIndex, 2) };

		const FIndex3i BaseTri = Mesh.GetTriangle(TriIndex);

		const int32 SplitEdgeIdx = Mesh.GetEdgeIndex(TriIndex, LocSplitEdge);
		const int32 SplitVertexIdx = ConcurrencyState.EdgeSplitRecords[SplitEdgeIdx].SplitVertex;
		check(SplitVertexIdx != IndexConstants::InvalidID);

		const int32 AdjHalfEdges[3] = { Mesh.GetAdjHalfEdge(TriIndex, 0),
										Mesh.GetAdjHalfEdge(TriIndex, 1),
										Mesh.GetAdjHalfEdge(TriIndex, 2) };

		check(AdjHalfEdges[LocSplitEdge] != IndexConstants::InvalidID);

		// the adjacency for the outer edge doesn't change
		FIndex3i NewTri = BaseTri;
		NewTri[NextLocEdge] = SplitVertexIdx;

		Mesh.SetTriangle(TriIndex, NewTri, TriIndex);

		// allocate 3 new half-edges and indices. this is the first index to write to
		const int32 HalfEdge = ConcurrencyState.HalfEdgeCount.fetch_add(3, std::memory_order_relaxed);
		check(HalfEdge + 2 < Mesh.MaxTriID() * 3);

		// first new triangle

		const int32 NewTriIndex = HalfEdge / 3;
		Mesh.SetTriangle(NewTriIndex, FIndex3i(SplitVertexIdx, BaseTri[NextLocEdge], BaseTri[PrevLocEdge]), TriIndex);

		// for the interior edge
		const int32 NewEdgeIndex = ConcurrencyState.EdgeCount.fetch_add(1, std::memory_order_relaxed);
		check(NewEdgeIndex < Mesh.EdgeCount());

		// interior edge
		Mesh.LinkEdge(HalfEdge + 2, TriIndex * 3 + NextLocEdge, NewEdgeIndex, IndexConstants::InvalidID);

		struct FLinkTask
		{
			int32 HalfEdge;     //< halfedge to relink
			int32 LocalEdge;    //< local edge index
			int32 AdjHalfEdge;  //< adjacent half-edge before the update
			int32 Edge;         //<
		};

		const FLinkTask LinkTasks[2] = { {TriIndex * 3 + PrevLocEdge, PrevLocEdge, AdjHalfEdges[PrevLocEdge], EdgeIndices[PrevLocEdge] },
										 {HalfEdge + 1              , NextLocEdge, AdjHalfEdges[NextLocEdge], EdgeIndices[NextLocEdge] } };


		// update adjacency for the unsplit edges
		for (const FLinkTask& LinkTask : LinkTasks )
		{
			// only if the neighbor is of type split-1, we need a safe concurrent update. split-2 is constructed such that the unsplit
			// edge doesn't require an adjacency update
			if (LinkTask.AdjHalfEdge == IndexConstants::InvalidID || !(ConcurrencyState.TriSplitRecords[TriIndex].GreenAdjMask.load() & static_cast<uint16>(1 << LinkTask.LocalEdge)))
			{
				Mesh.LinkEdge(LinkTask.HalfEdge, LinkTask.AdjHalfEdge, LinkTask.Edge, LinkTask.Edge);
			}
			else
			{
				FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[LinkTask.Edge];
				const int32 EntryCountPreWrite = Edge.EntryCountPreWrite.fetch_add(1, std::memory_order_relaxed);
				check(EntryCountPreWrite < 2);

				Edge.HalfEdges[EntryCountPreWrite] = FIndex2i( LinkTask.HalfEdge, IndexConstants::InvalidID );

				if (1 == Edge.EntryCountPostWrite.fetch_add(1, std::memory_order_acq_rel))
				{
					check(Edge.HalfEdges[0][1] == IndexConstants::InvalidID);
					check(Edge.HalfEdges[1][1] == IndexConstants::InvalidID);

					// second entry is being written, so we can now link the edges
					Mesh.LinkEdge(Edge.HalfEdges[0][0], Edge.HalfEdges[1][0], LinkTask.Edge, LinkTask.Edge);
				}
			}
		}

		// update adjacency for the split edge
		FIndex2i BoundaryHalfEdges = { 3 * TriIndex + LocSplitEdge, HalfEdge + 0 };
		{
			const int32 EdgeIdx = EdgeIndices[LocSplitEdge];
			FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIdx];

			const int32 EntryCountPreWrite = Edge.EntryCountPreWrite.fetch_add(1, std::memory_order_relaxed);
			check(EntryCountPreWrite < 2);

			Edge.HalfEdges[EntryCountPreWrite] = BoundaryHalfEdges;

			// second atomic necessary to make sure the memory written to in other thread in slot 0 is available
			// todo: might be faster to use std::memory_order_relaxed and use a atomic_thread_fence
			if (1 == Edge.EntryCountPostWrite.fetch_add(1, std::memory_order_acq_rel))
			{
				check(Edge.HalfEdges[0][0] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[0][1] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[1][0] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[1][1] != IndexConstants::InvalidID);

				// second entry is being written, so we can now link the edges
				Mesh.LinkEdge(Edge.HalfEdges[0][0], Edge.HalfEdges[1][1], EdgeIdx, EdgeIdx);
				Mesh.LinkEdge(Edge.HalfEdges[0][1], Edge.HalfEdges[1][0], Edge.NewEdge, EdgeIdx);
			}
		}
	}

	// split 2 edges, keep one edge (2 new triangles)
	inline void Split2(int32 TriIndex, FConcurrencyState& ConcurrencyState)
	{
		const FIndex3i BaseTri = Mesh.GetTriangle(TriIndex);

		const int32 SplitMask = ConcurrencyState.TriSplitRecords[TriIndex].SplitMask;

		int32 LocPreserveEdge = 0; // the one edge that is NOT split
		if (SplitMask == 3) {
			LocPreserveEdge = 2;
		} else if (SplitMask == 5) {
			LocPreserveEdge = 1;
		} else {
			check(SplitMask == 6);
			LocPreserveEdge = 0;
		}

		// edges to reconnect
		const int32 EdgeIndices[2] = { Mesh.GetEdgeIndex(TriIndex, (LocPreserveEdge+1)%3),
									   Mesh.GetEdgeIndex(TriIndex, (LocPreserveEdge+2)%3) };

		const int32 SplitVertexIdx0 = ConcurrencyState.EdgeSplitRecords[EdgeIndices[0]].SplitVertex;
		const int32 SplitVertexIdx1 = ConcurrencyState.EdgeSplitRecords[EdgeIndices[1]].SplitVertex;

		const int32 h0 = TriIndex * 3 + (LocPreserveEdge+0);
		const int32 h1 = TriIndex * 3 + (LocPreserveEdge+1)%3;
		const int32 h2 = TriIndex * 3 + (LocPreserveEdge+2)%3;

		// the tri connected to the unsplit edge
		FIndex3i NewTri = BaseTri;
		NewTri[(LocPreserveEdge + 2) % 3] = SplitVertexIdx1;
		Mesh.SetTriangle(TriIndex, NewTri, TriIndex);

		// allocate 6 new half-edges and indices. this is the first index to write to
		const int32 HalfEdge = ConcurrencyState.HalfEdgeCount.fetch_add(6, std::memory_order_relaxed);
		check(HalfEdge + 5 < Mesh.MaxTriID() * 3);

		// two new interior edges
		const int32 NewEdgeIndex = ConcurrencyState.EdgeCount.fetch_add(2, std::memory_order_relaxed);


		const int32 NewTriIndex = HalfEdge / 3;

		// T0
		Mesh.SetTriangle(NewTriIndex + 0, FIndex3i(BaseTri[(LocPreserveEdge + 2) % 3], SplitVertexIdx1, SplitVertexIdx0), TriIndex);

		// T1
		Mesh.SetTriangle(NewTriIndex + 1, FIndex3i(SplitVertexIdx0, SplitVertexIdx1, BaseTri[(LocPreserveEdge + 1) % 3]), TriIndex);

		// interior edges
		Mesh.LinkEdge(HalfEdge + 1, HalfEdge + 3, NewEdgeIndex  , IndexConstants::InvalidID);
		Mesh.LinkEdge(HalfEdge + 4, h1          , NewEdgeIndex+1, IndexConstants::InvalidID);

		const FIndex2i BoundaryHalfEdges[2] = { { HalfEdge + 5, HalfEdge + 2 },
												{ HalfEdge + 0, h2           } };

		for (int32 BoundaryIdx=0; BoundaryIdx<2; ++BoundaryIdx)
		{
			const int32 EdgeIdx = EdgeIndices[BoundaryIdx];
			FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIdx];

			const int32 EntryCountPreWrite = Edge.EntryCountPreWrite.fetch_add(1, std::memory_order_relaxed);
			Edge.HalfEdges[EntryCountPreWrite] = BoundaryHalfEdges[BoundaryIdx];

			// second atomic necessary to make sure the memory written to in other thread in slot 0 is available
			// todo: might be faster to use std::memory_order_relaxed and use a atomic_thread_fence (not x86-64)
			if (1 == Edge.EntryCountPostWrite.fetch_add(1, std::memory_order_acq_rel))
			{
				check( Edge.HalfEdges[0][0] != IndexConstants::InvalidID );
				check( Edge.HalfEdges[0][1] != IndexConstants::InvalidID );
				check( Edge.HalfEdges[1][0] != IndexConstants::InvalidID );
				check( Edge.HalfEdges[1][1] != IndexConstants::InvalidID );

				// second entry is being written, so we can now link the edges
				Mesh.LinkEdge(Edge.HalfEdges[0][0], Edge.HalfEdges[1][1], EdgeIdx, EdgeIdx);
				Mesh.LinkEdge(Edge.HalfEdges[0][1], Edge.HalfEdges[1][0], Edge.NewEdge, EdgeIdx);
			}
		}
	}

	// split 3 edges, triangle into 4 triangles (3 new triangles)
	// return first index of newly added triangles
	inline int32 Split3(int32 TriIndex, FConcurrencyState& ConcurrencyState)
	{
		const FIndex3i BaseTri = Mesh.GetTriangle(TriIndex);

		const int32 BaseEdgeIndices[3] = { Mesh.GetEdgeIndex(TriIndex, 0),
										   Mesh.GetEdgeIndex(TriIndex, 1),
										   Mesh.GetEdgeIndex(TriIndex, 2) };

		const int32 BaseAdj[3] = { Mesh.GetAdjTriangle(TriIndex, 0),
								   Mesh.GetAdjTriangle(TriIndex, 1),
								   Mesh.GetAdjTriangle(TriIndex, 2) };

		const int32 SplitVertexIdx0 = ConcurrencyState.EdgeSplitRecords[BaseEdgeIndices[0]].SplitVertex;
		const int32 SplitVertexIdx1 = ConcurrencyState.EdgeSplitRecords[BaseEdgeIndices[1]].SplitVertex;
		const int32 SplitVertexIdx2 = ConcurrencyState.EdgeSplitRecords[BaseEdgeIndices[2]].SplitVertex;

		// replace the base triangle with the one in the middle
		Mesh.SetTriangle(TriIndex, FIndex3i(SplitVertexIdx0, SplitVertexIdx1, SplitVertexIdx2), TriIndex);

		// allocate 9 new half-edges and indices. this is the first index to write to
		const int32 HalfEdge = ConcurrencyState.HalfEdgeCount.fetch_add(9, std::memory_order_relaxed);
		const int32 NewTriIndex = HalfEdge / 3;

		// T0
		Mesh.SetTriangle(NewTriIndex + 0, FIndex3i(BaseTri[0], SplitVertexIdx0, SplitVertexIdx2), TriIndex);

		// T1
		Mesh.SetTriangle(NewTriIndex + 1, FIndex3i(BaseTri[1], SplitVertexIdx1, SplitVertexIdx0), TriIndex);

		// T2
		Mesh.SetTriangle(NewTriIndex + 2, FIndex3i(BaseTri[2], SplitVertexIdx2, SplitVertexIdx1), TriIndex);

		const int32 NewEdgeIndex = ConcurrencyState.EdgeCount.fetch_add(3, std::memory_order_relaxed);
		check(NewEdgeIndex < Mesh.EdgeCount());

		// interior edges
		Mesh.LinkEdge( HalfEdge + 1, TriIndex * 3 + 2, NewEdgeIndex + 0, IndexConstants::InvalidID );
		Mesh.LinkEdge( HalfEdge + 4, TriIndex * 3 + 0, NewEdgeIndex + 1, IndexConstants::InvalidID );
		Mesh.LinkEdge( HalfEdge + 7, TriIndex * 3 + 1, NewEdgeIndex + 2, IndexConstants::InvalidID );

		// pairs of halfedges corresponding to the split edge as seen from this triangle
		const FIndex2i BoundaryHalfEdges[3] = { { HalfEdge + 0, HalfEdge + 5 },
												{ HalfEdge + 3, HalfEdge + 8 },
												{ HalfEdge + 6, HalfEdge + 2 } };

		for (int32 LocalEdgeIdx=0; LocalEdgeIdx<3; ++LocalEdgeIdx)
		{
			const int32 EdgeIdx = BaseEdgeIndices[LocalEdgeIdx];
			FEdgeSplitRecord& Edge = ConcurrencyState.EdgeSplitRecords[EdgeIdx];

			if (BaseAdj[LocalEdgeIdx] == IndexConstants::InvalidID)
			{
				// boundary case: we can just update locally without needing data from the other side
				Mesh.LinkEdge(BoundaryHalfEdges[LocalEdgeIdx][0], IndexConstants::InvalidID, EdgeIdx, EdgeIdx);
				Mesh.LinkEdge(BoundaryHalfEdges[LocalEdgeIdx][1], IndexConstants::InvalidID, Edge.NewEdge, EdgeIdx);
				continue;
			}

			const int32 EntryCountPreWrite = Edge.EntryCountPreWrite.fetch_add(1, std::memory_order_relaxed);
			Edge.HalfEdges[EntryCountPreWrite] = BoundaryHalfEdges[LocalEdgeIdx];

			// second atomic necessary to make sure the memory written to in other thread in slot 0 is available
			// todo: might be faster to use std::memory_order_relaxed and use a atomic_thread_fence
			if (1 == Edge.EntryCountPostWrite.fetch_add(1, std::memory_order_acq_rel))
			{
				check(Edge.HalfEdges[0][0] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[0][1] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[1][0] != IndexConstants::InvalidID);
				check(Edge.HalfEdges[1][1] != IndexConstants::InvalidID);

				// second entry is being written, so we can now link the edges
				Mesh.LinkEdge(Edge.HalfEdges[0][0], Edge.HalfEdges[1][1], EdgeIdx, EdgeIdx);
				Mesh.LinkEdge(Edge.HalfEdges[0][1], Edge.HalfEdges[1][0], Edge.NewEdge, EdgeIdx);
			}
		}
		check(HalfEdge % 3 == 0);
		return HalfEdge / 3;
	}

	[[nodiscard]] bool ShouldRefine(int32 TriIndex, int32 Iter) const
	{
		FVector3f SplitVertexBary;
		return RefinementPolicy.ShouldRefine(TriIndex, SplitVertexBary, Iter);
	}

private:

	MeshType&               Mesh;
	RefinementPolicyType&   RefinementPolicy;
	const FOptions&         Options;
};

} // namespace Geometry
} // namespace UE