// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Templates/Function.h"
#include "Misc/Optional.h"
#include "Containers/BitArray.h"
#include "Containers/Array.h"

namespace UE {
namespace Geometry {

class FDynamicMesh3;
	
using FDisplaceFunc = TFunction<FVector3d (const int32 VertexIndex)>;

/** 
 * Apply one iteration of mesh smoothing. If bPreserveVolume is set the new vertex positions will be adjusted to preserve
 * the enclosing mesh volume, favoring only tangential shifts.
 * 
 * If ActiveVertices is provided, only vertices for which the bit is set will be processed.
 */
GEOMETRYCORE_API void 
RegularizeVolumePreserving(FDynamicMesh3& Mesh, const bool bPreserveVolume, const TOptional<TBitArray<>>& ActiveVertices = {});

/**
 * Apply edge flips on non-Delaunay edges in order of importance.
 * Intrinsic edge-flips would split the flipped edge and produce edges that are on the same surface.
 * This function relaxes this and applies honest flips when the surface is planar enough (when the
 * change in the oriented dihedral angle is smaller than FlipAngleThreshold)
 * 
 * See [Fisher, Springborn, Bobenko, Schroeder, "An Algorithm for the Construction of Intrinsic Delaunay Triangulations
 * with Applications to Digital Geometry Processing", 2006]
 *
 *
 * @param Mesh mesh to be modified
 * @param Displacements array of per-vertex displacements (optional)
 * @param DisplacementFunctor if Displacements are passed in, this functor needs to be provided to construct new displacements at split vertices
 * @param FlipAngleThreshold controls honest vs intrinsic flips 
 */
GEOMETRYCORE_API void 
IntrinsicDelaunayFlipEdge(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double FlipAngleThreshold = 0.1 * UE_PI);

inline void IntrinsicDelaunayFlipEdge(FDynamicMesh3& Mesh, const double FlipAngleThreshold = 0.1 * UE_PI)
{
	TArray<FVector3d> NullDisplacements;
	IntrinsicDelaunayFlipEdge(Mesh, NullDisplacements, FDisplaceFunc(), FlipAngleThreshold);
}

/**
 * Walk through edges in priority order, splitting edges according to threshold criterion
 *
 * @param Mesh mesh to be modified
 * @param Displacements array of per-vertex displacements (optional)
 * @param DisplacementFunctor if Displacements are passed in, this functor needs to be provided to construct new displacements at split vertices
 * @param SplitThreshold trigger split if ratio of longest edge/(sum of length of other edges) exceeds this threshold
 */
GEOMETRYCORE_API void 
SplitLongEdges(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double SplitThreshold = 0.9);

inline void SplitLongEdges(FDynamicMesh3& Mesh, const double SplitThreshold = 0.9)
{
	TArray<FVector3d> NullDisplacements;
	SplitLongEdges(Mesh, NullDisplacements, FDisplaceFunc(), SplitThreshold);
}
	
} // namespace Geometry
} // namespace UE
