// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "TriangleTypes.h"
#include "CompGeom/ExactPredicates.h"

namespace UE::Geometry::TetUtil
{
	// Fill an array with the oriented triangle faces of the tet
	template<bool bReverseOrientation = false>
	void GetTetFaces(const FIndex4i& Tet, FIndex3i OutFaces[4])
	{
		const FIndex3i* FaceOrdering = GetTetFaceOrdering<bReverseOrientation>();
		OutFaces[0] = FIndex3i(Tet[FaceOrdering[0].A], Tet[FaceOrdering[0].B], Tet[FaceOrdering[0].C]);
		OutFaces[1] = FIndex3i(Tet[FaceOrdering[1].A], Tet[FaceOrdering[1].B], Tet[FaceOrdering[1].C]);
		OutFaces[2] = FIndex3i(Tet[FaceOrdering[2].A], Tet[FaceOrdering[2].B], Tet[FaceOrdering[2].C]);
		OutFaces[3] = FIndex3i(Tet[FaceOrdering[3].A], Tet[FaceOrdering[3].B], Tet[FaceOrdering[3].C]);
	}

	// Test whether point Pt is inside or on a (non-degenerate) tetrahedron Tet.
	// Note: Will always report 'outside' for a fully degenerate (flat) tetrahedron
	// @return 0 if Pt is outside Tet (or Tet is degenerate), 1 if Pt is inside a positively-oriented Tet, -1 if Pt is inside and Tet is inverted
	template<typename RealType>
	int32 IsInsideExact(const TTetrahedron3<RealType>& Tet, TVector<RealType>& Pt)
	{
		int32 SeenSide = 0;
		for (int32 TetFace = 0; TetFace < 4; ++TetFace)
		{
			FIndex3i FaceInds = Tet.GetFaceIndices(TetFace);
			int32 Side = TMathUtil<RealType>::SignAsInt(ExactPredicates::Orient3<RealType>(Tet.V[FaceInds.A], Tet.V[FaceInds.B], Tet.V[FaceInds.C], Pt));
			if (Side != 0)
			{
				if (Side * SeenSide < 0)
				{
					return 0;
				}
				SeenSide = Side;
			}
		}
		
		return SeenSide;
	}
 
} // end namespace UE::Geometry::TetUtil