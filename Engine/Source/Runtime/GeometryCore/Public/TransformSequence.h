// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TransformTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TTransformSequence3 represents a sequence of 3D transforms. 
 */
template<typename RealType>
class TTransformSequence3
{
protected:
	TArray<TTransformSRT3<RealType>, TInlineAllocator<2>> Transforms;

public:

	/**
	 * Add Transform to the end of the sequence, ie Seq(P) becomes NewTransform * Seq(P)
	 */
	void Append(const TTransformSRT3<RealType>& Transform)
	{
		Transforms.Add(Transform);
	}

	/**
	 * Add Transform to the end of the sequence, ie Seq(P) becomes NewTransform * Seq(P)
	 */
	void Append(const FTransform& Transform)
	{
		Transforms.Add( TTransformSRT3<RealType>(Transform) );
	}

	/**
	 * Add all transforms in given sequence to the end of this sequence, ie Seq(P) becomes SequenceToAppend * Seq(P)
	 */
	void Append(const TTransformSequence3<RealType>& SequenceToAppend)
	{
		for (const TTransformSRT3<RealType>& Transform : SequenceToAppend.Transforms)
		{
			Append(Transform);
		}
	}

	/**
	 * @return number of transforms in the sequence
	 */
	int32 Num() const { return Transforms.Num(); }

	/**
	 * @return transforms in the sequence
	 */
	const TArray<TTransformSRT3<RealType>,TInlineAllocator<2>>& GetTransforms() const { return Transforms; }


	/**
	 * @return true if any transform in the sequence has nonuniform scaling
	 */
	bool HasNonUniformScale(RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			if (Transform.HasNonUniformScale(Tolerance))
			{
				return true;
			}
		}
		return false;
	}


	void AppendInverse(const FTransform& Transform, const RealType Tolerance = TMathUtil<RealType>::ZeroTolerance)
	{
		AppendInverse(TTransformSRT3<RealType>(Transform), Tolerance);
	}

	void AppendInverse(const TTransformSRT3<RealType>& Transform, const RealType Tolerance = TMathUtil<RealType>::ZeroTolerance)
	{
		if (Transform.CanRepresentInverse(Tolerance))
		{
			Append(Transform.InverseUnsafe(Tolerance));
		}
		else
		{
			TTransformSRT3<RealType> InvS = TTransformSRT3<RealType>::Identity();
			InvS.SetScale(TTransformSRT3<RealType>::GetSafeScaleReciprocal(Transform.GetScale3D(), Tolerance));

			TTransformSRT3<RealType> InvRT = Transform;
			InvRT.SetScale(TVector<RealType>::One());
			InvRT = InvRT.InverseUnsafe();
			Append(InvRT);
			Append(InvS);
		}
	}

	void AppendInverse(const TTransformSequence3<RealType>& SequenceToAppend, const RealType Tolerance = TMathUtil<RealType>::ZeroTolerance)
	{
		for (int32 Idx = Transforms.Num() - 1; Idx >= 0; --Idx)
		{
			AppendInverse(Transforms[Idx], Tolerance);
		}
	}

	// Create the inverse of a transform sequence. Note: Transforms with both non-uniform scale and rotation may be split into two transforms. Zeros in scale will be replaced with unit scales.
	TTransformSequence3<RealType> GetInverse(const RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		TTransformSequence3<RealType> InverseTransformSeq;
		for (int32 Idx = Transforms.Num() - 1; Idx >= 0; --Idx)
		{
			InverseTransformSeq.AppendInverse(Transforms[Idx]);
		}
		return InverseTransformSeq;
	}

	/**
	 * @return Cumulative scale across Transforms.
	 */
	TVector<RealType> GetAccumulatedScale() const
	{
		TVector<RealType> FinalScale = TVector<RealType>::One();
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			FinalScale = FinalScale * Transform.GetScale();
		}
		return FinalScale;
	}

	/**
	 * @return Whether the sequence will invert a shape (by negative scaling) when applied
	 */
	bool WillInvert() const
	{
		RealType Det = 1;
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			Det *= Transform.GetDeterminant();
		}
		return Det < 0;
	}

	/**
	 * Set scales of all transforms to (1,1,1)
	 */
	void ClearScales()
	{
		for (TTransformSRT3<RealType>& Transform : Transforms)
		{
			Transform.SetScale(TVector<RealType>::One());
		}
	}

	/**
	 * @return point P with transform sequence applied
	 */
	TVector<RealType> TransformPosition(UE::Math::TVector<RealType> P) const
	{
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			P = Transform.TransformPosition(P);
		}
		return P;
	}

	/**
	 * @return point P with inverse transform sequence applied
	 */
	TVector<RealType> InverseTransformPosition(UE::Math::TVector<RealType> P) const
	{
		int32 N = Transforms.Num();
		for (int32 k = N - 1; k >= 0; k--)
		{
			P = Transforms[k].InverseTransformPosition(P);
		}
		return P;
	}

	/**
	 * @return Vector V with transform sequence applied
	 */
	TVector<RealType> TransformVector(UE::Math::TVector<RealType> V) const
	{
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			V = Transform.TransformVector(V);
		}
		return V;
	}


	/**
	 * @return Normal with transform sequence applied
	 */
	TVector<RealType> TransformNormal(UE::Math::TVector<RealType> Normal) const
	{
		for (const TTransformSRT3<RealType>& Transform : Transforms)
		{
			Normal = Transform.TransformNormal(Normal);
		}
		return Normal;
	}

	/**
	 * @return true if each Transform in this sequence is equivalent to each transform of another sequence under the given test
	 */
	template<typename TransformsEquivalentFunc>
	bool IsEquivalent(const TTransformSequence3<RealType>& OtherSeq, TransformsEquivalentFunc TransformsTest) const
	{
		int32 N = Transforms.Num();
		if (N == OtherSeq.Transforms.Num())
		{
			bool bAllTransformsEqual = true;
			for (int32 k = 0; k < N && bAllTransformsEqual; ++k)
			{
				bAllTransformsEqual = bAllTransformsEqual && TransformsTest(Transforms[k], OtherSeq.Transforms[k]);
			}
			return bAllTransformsEqual;
		}
		return false;
	}
};

typedef TTransformSequence3<float> FTransformSequence3f;
typedef TTransformSequence3<double> FTransformSequence3d;


} // end namespace UE::Geometry
} // end namespace UE