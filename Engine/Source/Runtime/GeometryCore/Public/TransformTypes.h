// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "Quaternion.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * TTransformSRT3 is a variant of the standard UE FTransform/TTransform that uses the 
 * UE::Geometry::TQuaternion instead of the UE::Math::TQuat. 
 * 
 * Transform order is Scale, then Rotate, then Translate.
 * So mathematically (T * R * S) * v , assuming traditional matrix-vector multiplication order
 * (note that the UE::Math library uses the opposite vector*Matrix ordering for TMatrix!)
 * 
 * Implicit constructors/casts are defined to/from FTransform3d and FTransform3f.
 * This will likely be revised in future so that conversions that lose precision are explicit.
 * 
 */
template<typename RealType>
class TTransformSRT3
{
protected:
	TQuaternion<RealType> Rotation;
	TVector<RealType> Translation;
	TVector<RealType> Scale3D;

public:

	TTransformSRT3()
	{
		Rotation = TQuaternion<RealType>::Identity();
		Translation = TVector<RealType>::Zero();
		Scale3D = TVector<RealType>::One();
	}

	TTransformSRT3(const TQuaternion<RealType>& RotationIn, const UE::Math::TVector<RealType>& TranslationIn, const UE::Math::TVector<RealType>& ScaleIn)
	{
		Rotation = RotationIn;
		Translation = TranslationIn;
		Scale3D = ScaleIn;
	}

	explicit TTransformSRT3(const TQuaternion<RealType>& RotationIn, const UE::Math::TVector<RealType>& TranslationIn)
	{
		Rotation = RotationIn;
		Translation = TranslationIn;
		Scale3D = TVector<RealType>::One();
	}

	explicit TTransformSRT3(const UE::Math::TVector<RealType>& TranslationIn)
	{
		Rotation = TQuaternion<RealType>::Identity();
		Translation = TranslationIn;
		Scale3D = TVector<RealType>::One();
	}

	TTransformSRT3(const FTransform3f& Transform)
	{
		Rotation = TQuaternion<RealType>(Transform.GetRotation());
		Translation = TVector<RealType>(Transform.GetTranslation());
		Scale3D = TVector<RealType>(Transform.GetScale3D());
	}

	TTransformSRT3(const FTransform3d& Transform)
	{
		Rotation = TQuaternion<RealType>(Transform.GetRotation());
		Translation = TVector<RealType>(Transform.GetTranslation());
		Scale3D = TVector<RealType>(Transform.GetScale3D());
	}

	/**
	 * @return identity transform, IE no rotation, zero origin, unit scale
	 */
	static TTransformSRT3<RealType> Identity()
	{
		return TTransformSRT3<RealType>(TQuaternion<RealType>::Identity(), TVector<RealType>::Zero(), TVector<RealType>::One());
	}

	/**
	 * @return this transform converted to FTransform 
	 */
	operator FTransform3f() const
	{
		return FTransform3f((FQuat4f)Rotation, (FVector3f)Translation, (FVector3f)Scale3D);
	}

	/**
	* @return this transform converted to FTransform3d
	*/
	operator FTransform3d() const
	{
		return FTransform3d((FQuat4d)Rotation, (FVector3d)Translation, (FVector3d)Scale3D);
	}

	/**
	 * @return Rotation portion of Transform, as Quaternion
	 */
	const TQuaternion<RealType>& GetRotation() const 
	{ 
		return Rotation; 
	}

	/**
	* @return Rotation portion of Transform, as FRotator
	*/
	FRotator GetRotator() const 
	{ 
		return (FRotator)Rotation; 
	}

	/** 
	 * Set Rotation portion of Transform 
	 */
	void SetRotation(const TQuaternion<RealType>& RotationIn)
	{
		Rotation = RotationIn;
	}

	/**
	 * @return Translation portion of transform
	 */
	const TVector<RealType>& GetTranslation() const
	{
		return Translation;
	}

	/**
	 * set Translation portion of transform
	 */
	void SetTranslation(const UE::Math::TVector<RealType>& TranslationIn)
	{
		Translation = TranslationIn;
	}

	/**
	 * @return Scale portion of transform
	 */
	const TVector<RealType>& GetScale() const
	{
		return Scale3D;
	}

	/**
	 * @return Scale portion of transform
	 */
	const TVector<RealType>& GetScale3D() const
	{
		return Scale3D;
	}

	/**
	 * set Scale portion of transform
	 */
	void SetScale(const UE::Math::TVector<RealType>& ScaleIn)
	{
		Scale3D = ScaleIn;
	}

	/**
	 * @return determinant of scale matrix
	 */
	RealType GetDeterminant() const
	{
		return Scale3D.X * Scale3D.Y * Scale3D.Z;
	}

	/**
	 * @return true if scale is nonuniform, within tolerance
	 */
	bool HasNonUniformScale(RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		return (TMathUtil<RealType>::Abs(Scale3D.X - Scale3D.Y) > Tolerance)
			|| (TMathUtil<RealType>::Abs(Scale3D.X - Scale3D.Z) > Tolerance)
			|| (TMathUtil<RealType>::Abs(Scale3D.Y - Scale3D.Z) > Tolerance);
	}

	/**
	 * Attempts to return an inverse, but will give an incorrect result if the transform has both non-uniform scaling and rotation,
	 * because TTransformSRT3<RealType> cannot represent the true inverse in that case.
	 */
	TTransformSRT3<RealType> InverseUnsafe(RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		TQuaternion<RealType> InvRotation = Rotation.Inverse();
		TVector<RealType> InvScale3D = GetSafeScaleReciprocal(Scale3D, Tolerance);
		TVector<RealType> InvTranslation = InvRotation * (InvScale3D * -Translation);
		return TTransformSRT3<RealType>(InvRotation, InvTranslation, InvScale3D);
	}

	/** Reports wheth the inverse is representable with a single TTransformSRT3. (Ignores zeros in scale.) */
	bool CanRepresentInverse(RealType Tolerance = TMathUtil<RealType>::ZeroTolerance) const
	{
		// Note: Could also return true if there is a non-uniform scale aligned with the rotation axis ...
		return Scale3D.AllComponentsEqual(Tolerance) || Rotation.IsIdentity(Tolerance);
	}


	// The following templates perform vector-type-conversion variants.
	// This allows applying a float transform to double vector and vice-versa.
	// Whether this should be allowed is debatable. However in practice it is extremely rare to convert an
	// entire float transform to a double transform in order to apply to a double vector, which is the only
	// case where this conversion is an issue


	/**
	 * @return input point with QST transformation applied, ie QST(P) = Rotate(Scale*P) + Translate
	 */
	template<typename RealType2>
	TVector<RealType2> TransformPosition(const TVector<RealType2>& P) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			//Transform using QST is following
			//QST(P) = Q.Rotate(S*P) + T where Q = quaternion, S = scale, T = translation
			return Rotation * (Scale3D*P) + Translation;
		}
		else
		{
			return TVector<RealType2>(TransformPosition(TVector<RealType>(P)));
		}
	}

	/**
	 * @return input vector with QS transformation applied, ie QS(V) = Rotate(Scale*V)
	 */
	template<typename RealType2>
	TVector<RealType2> TransformVector(const UE::Math::TVector<RealType2>& V) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return Rotation * (Scale3D*V);
		}
		else
		{
			return TVector<RealType2>(TransformVector(TVector<RealType>(V)));
		}
	}

	/**
	 * @return input vector with Q transformation applied, ie Q(V) = Rotate(V)
	 */
	template<typename RealType2>
	TVector<RealType2> TransformVectorNoScale(const UE::Math::TVector<RealType2>& V) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return Rotation * V;
		}
		else
		{
			return TVector<RealType2>(TransformVectorNoScale(TVector<RealType>(V)));
		}
	}

	/**
	 * Surface Normals are special, their transform is Rotate( Normalize( (1/Scale) * Normal) ) ).
	 * However 1/Scale requires special handling in case any component is near-zero.
	 * @return input surface normal with transform applied.
	 */
	template<typename RealType2>
	TVector<RealType2> TransformNormal(const UE::Math::TVector<RealType2>& Normal) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			// transform normal by a safe inverse scale + normalize, and a standard rotation
			const TVector<RealType>& S = Scale3D;
			RealType DetSign = FMathd::SignNonZero(S.X * S.Y * S.Z); // we only need to multiply by the sign of the determinant, rather than divide by it, since we normalize later anyway
			TVector<RealType> SafeInvS(S.Y*S.Z*DetSign, S.X*S.Z*DetSign, S.X*S.Y*DetSign);
			return TransformVectorNoScale( Normalized(SafeInvS*Normal) );
		}
		else
		{
			return TVector<RealType2>(TransformNormal(TVector<RealType>(Normal)));
		}
	}


	/**
	 * @return input vector with inverse-QST transformation applied, ie QSTinv(P) = InverseScale(InverseRotate(P - Translate))
	 */
	template<typename RealType2>
	TVector<RealType2> InverseTransformPosition(const UE::Math::TVector<RealType2>& P) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return GetSafeScaleReciprocal(Scale3D) * Rotation.InverseMultiply(P - Translation);
		}
		else
		{
			return TVector<RealType2>(InverseTransformPosition(TVector<RealType>(P)));
		}
	}

	/**
	 * @return input vector with inverse-QS transformation applied, ie QSinv(V) = InverseScale(InverseRotate(V))
	 */
	template<typename RealType2>
	TVector<RealType2> InverseTransformVector(const UE::Math::TVector<RealType2>& V) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return GetSafeScaleReciprocal(Scale3D) * Rotation.InverseMultiply(V);
		}
		else
		{
			return TVector<RealType2>(InverseTransformVector(TVector<RealType>(V)));
		}
	}


	/**
	 * @return input vector with inverse-Q transformation applied, ie Qinv(V) = InverseRotate(V)
	 */
	template<typename RealType2>
	TVector<RealType2> InverseTransformVectorNoScale(const UE::Math::TVector<RealType2>& V) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return Rotation.InverseMultiply(V);
		}
		else
		{
			return TVector<RealType2>(InverseTransformVectorNoScale(TVector<RealType>(V)));
		}
	}


	/**
	 * Surface Normals are special, their inverse transform is InverseRotate( Normalize(Scale * Normal) ) )
	 * @return input surface normal with inverse transform applied.
	 */
	template<typename RealType2>
	TVector<RealType2> InverseTransformNormal(const UE::Math::TVector<RealType2>& Normal) const
	{
		if constexpr (std::is_same_v<RealType, RealType2>)
		{
			return Normalized( Scale3D * InverseTransformVectorNoScale(Normal) );
		}
		else
		{
			return TVector<RealType2>(InverseTransformNormal(TVector<RealType>(Normal)));
		}
	}


	UE::Math::TRay<RealType> TransformRay(const UE::Math::TRay<RealType> &Ray) const
	{
		TVector<RealType> Origin = TransformPosition(Ray.Origin);
		TVector<RealType> Direction = Normalized(TransformVector(Ray.Direction));
		return UE::Math::TRay<RealType>(Origin, Direction);
	}


	UE::Math::TRay<RealType> InverseTransformRay(const UE::Math::TRay<RealType> &Ray) const
	{
		TVector<RealType> InvOrigin = InverseTransformPosition(Ray.Origin);
		TVector<RealType> InvDirection = Normalized(InverseTransformVector(Ray.Direction));
		return UE::Math::TRay<RealType>(InvOrigin, InvDirection);
	}

	/**
	 * Clamp all scale components to a minimum value. Sign of scale components is preserved.
	 * This is used to remove uninvertible zero/near-zero scaling.
	 */
	void ClampMinimumScale(RealType MinimumScale = TMathUtil<RealType>::ZeroTolerance)
	{
		for (int j = 0; j < 3; ++j)
		{
			RealType Value = Scale3D[j];
			if (TMathUtil<RealType>::Abs(Value) < MinimumScale)
			{
				Value = MinimumScale * TMathUtil<RealType>::SignNonZero(Value);
				Scale3D[j] = Value;
			}
		}
	}

	

	static TVector<RealType> GetSafeScaleReciprocal(const TVector<RealType>& InScale, RealType Tolerance = TMathUtil<RealType>::ZeroTolerance)
	{
		TVector<RealType> SafeReciprocalScale;
		if (TMathUtil<RealType>::Abs(InScale.X) <= Tolerance)
		{
			SafeReciprocalScale.X = (RealType)0;
		}
		else
		{
			SafeReciprocalScale.X = (RealType)1 / InScale.X;
		}

		if (TMathUtil<RealType>::Abs(InScale.Y) <= Tolerance)
		{
			SafeReciprocalScale.Y = (RealType)0;
		}
		else
		{
			SafeReciprocalScale.Y = (RealType)1 / InScale.Y;
		}

		if (TMathUtil<RealType>::Abs(InScale.Z) <= Tolerance)
		{
			SafeReciprocalScale.Z = (RealType)0;
		}
		else
		{
			SafeReciprocalScale.Z = (RealType)1 / InScale.Z;
		}

		return SafeReciprocalScale;
	}
};
typedef TTransformSRT3<float> FTransformSRT3f;
typedef TTransformSRT3<double> FTransformSRT3d;


} // end namespace UE::Geometry
} // end namespace UE