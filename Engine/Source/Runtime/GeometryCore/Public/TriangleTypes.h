// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "VectorUtil.h"
#include "IndexTypes.h"
#include "BoxTypes.h"
#include "SegmentTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Triangle utility functions
 */
namespace TriangleUtil
{
	/**
	 * @return the edge length of an equilateral/regular triangle with the given area
	 */
	template<typename RealType>
	RealType EquilateralEdgeLengthForArea(RealType TriArea)
	{
		return TMathUtil<RealType>::Sqrt(((RealType)4 * TriArea) / TMathUtil<RealType>::Sqrt3);
	}

};




template<typename RealType>
struct TTriangle2
{
	TVector2<RealType> V[3];

	TTriangle2() {}

	TTriangle2(const TVector2<RealType>& V0, const TVector2<RealType>& V1, const TVector2<RealType>& V2)
	{
		V[0] = V0;
		V[1] = V1;
		V[2] = V2;
	}

	TTriangle2(const TVector2<RealType> VIn[3])
	{
		V[0] = VIn[0];
		V[1] = VIn[1];
		V[2] = VIn[2];
	}

	TVector2<RealType> BarycentricPoint(RealType Bary0, RealType Bary1, RealType Bary2) const
	{
		return Bary0 * V[0] + Bary1 * V[1] + Bary2 * V[2];
	}

	TVector2<RealType> BarycentricPoint(const TVector<RealType>& BaryCoords) const
	{
		return BaryCoords[0] * V[0] + BaryCoords[1] * V[1] + BaryCoords[2] * V[2];
	}

	TVector<RealType> GetBarycentricCoords(const TVector2<RealType>& Point) const
	{
		return VectorUtil::BarycentricCoords(Point, V[0], V[1], V[2]);
	}

	/**
	 * @param A first vertex of triangle
	 * @param B second vertex of triangle
	 * @param C third vertex of triangle
	 * @return signed area of triangle
	 */
	static RealType SignedArea(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C)
	{
		return ((RealType)0.5) * ((A.X*B.Y - A.Y*B.X) + (B.X*C.Y - B.Y*C.X) + (C.X*A.Y - C.Y*A.X));
	}

	/** @return signed area of triangle */
	RealType SignedArea() const
	{
		return SignedArea(V[0], V[1], V[2]);
	}
	
	/** @return unsigned area of the triangle */
	RealType Area() const
	{
		return TMathUtil<RealType>::Abs(SignedArea());
	}


	/**
	 * @param A first vertex of triangle
	 * @param B second vertex of triangle
	 * @param C third vertex of triangle
	 * @param QueryPoint test point
	 * @return true if QueryPoint is inside triangle
	 */
	static bool IsInside(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C, const TVector2<RealType>& QueryPoint)
	{
		RealType Sign1 = Orient(A, B, QueryPoint);
		RealType Sign2 = Orient(B, C, QueryPoint);
		RealType Sign3 = Orient(C, A, QueryPoint);
		return (Sign1*Sign2 > 0) && (Sign2*Sign3 > 0) && (Sign3*Sign1 > 0);
	}

	/** @return true if QueryPoint is inside triangle */
	bool IsInside(const TVector2<RealType>& QueryPoint) const
	{
		return IsInside(V[0], V[1], V[2], QueryPoint);
	}


	/** 
	 * @return true if QueryPoint is inside triangle or on edge. Note that this is slower 
	 *  than IsInside because of the need to handle degeneracy and tolerance. 
	 */
	static bool IsInsideOrOn(const TVector2<RealType>& A, const TVector2<RealType>& B, 
		const TVector2<RealType>& C, const TVector2<RealType>& QueryPoint, RealType Tolerance = (RealType)0)
	{
		RealType Sign1 = TSegment2<RealType>::GetSide(A, B, QueryPoint, Tolerance);
		RealType Sign2 = TSegment2<RealType>::GetSide(B, C, QueryPoint, Tolerance);
		RealType Sign3 = TSegment2<RealType>::GetSide(C, A, QueryPoint, Tolerance);

		// If any of the signs are opposite, then definitely outside
		if (Sign1 * Sign2 < 0 || Sign2 * Sign3 < 0 || Sign3 * Sign1 < 0)
		{
			return false;
		}

		// If some signs were zero, then things are more complicated because we are either colinear
		//  with that edge, or the edge is degenerate enough to get a 0 value on the DotPerp.
		int NumZero = static_cast<int>(Sign1 == 0) + static_cast<int>(Sign2 == 0) + static_cast<int>(Sign3 == 0);
		switch (NumZero)
		{
		case 0:
			// All were nonzero and none disagreed, so inside.
			return true;
		case 1:
			// If only one sign was zero, seems more than likely that we're on that edge.
			//  Hard to imagine some underflow case where that isn't actually the case, but we'll
			//  do the segment check just in case.
			return Sign1 == 0 ? TSegment2<RealType>::IsOnSegment(A, B, QueryPoint, Tolerance)
				: Sign2 == 0 ? TSegment2<RealType>::IsOnSegment(B, C, QueryPoint, Tolerance)
				: TSegment2<RealType>::IsOnSegment(C, A, QueryPoint, Tolerance);
		case 2:
			// Two signs were zero, so we expect to be on the vertex between those edges
			return Sign1 != 0 ? TVector2<RealType>::DistSquared(QueryPoint, C) <= Tolerance * Tolerance
				: Sign2 != 0 ? TVector2<RealType>::DistSquared(QueryPoint, A) <= Tolerance * Tolerance
				: TVector2<RealType>::DistSquared(QueryPoint, B) <= Tolerance * Tolerance;
		case 3:
			// All three signs were zero. We're dealing with a denerate triangle of some
			//  sort. It should be sufficient to check any two segments
			return TSegment2<RealType>::IsOnSegment(A, B, QueryPoint, Tolerance)
				|| TSegment2<RealType>::IsOnSegment(B, C, QueryPoint, Tolerance);
		default:
			return ensure(false);
		}
	}

	/**
	 * @return true if QueryPoint is inside triangle or on edge. Note that this is slower
	 *  than IsInside because of the need to handle degeneracy and tolerance.
	 */
	bool IsInsideOrOn(const TVector2<RealType>& QueryPoint, RealType Tolerance = 0) const
	{
		return IsInsideOrOn(V[0], V[1], V[2], QueryPoint, Tolerance);
	}


	/**
	 * More robust (because it doesn't multiply orientation test results) inside-triangle test for oriented triangles only
	 *  (the code early-outs at the first 'outside' edge, which only works if the triangle is oriented as expected)
	 * @return 1 if outside, -1 if inside, 0 if on boundary
	 */
	int IsInsideOrOn_Oriented(const TVector2<RealType>& QueryPoint, RealType Tolerance = (RealType)0) const
	{
		return IsInsideOrOn_Oriented(V[0], V[1], V[2], QueryPoint, Tolerance);
	}

	/**
	 * More robust (because it doesn't multiply orientation test results) inside-triangle test for oriented triangles only
	 *  (the code early-outs at the first 'outside' edge, which only works if the triangle is oriented as expected)
	 * @return 1 if outside, -1 if inside, 0 if on boundary
	 */
	static int IsInsideOrOn_Oriented(const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C, 
		const TVector2<RealType>& QueryPoint, RealType Tolerance = (RealType)0)
	{
		checkSlow(Orient(A, B, C) <= 0); // TODO: remove this checkSlow; it's just to make sure the orientation is as expected

		RealType Sign1 = TSegment2<RealType>::GetSide(A, B, QueryPoint, Tolerance);
		if (Sign1 > 0)
		{
			return 1;
		}
		
		RealType Sign2 = TSegment2<RealType>::GetSide(B, C, QueryPoint, Tolerance);
		if (Sign2 > 0)
		{
			return 1;
		}
		
		RealType Sign3 = TSegment2<RealType>::GetSide(A, C, QueryPoint, Tolerance);
		if (Sign3 < 0) // note this edge is queried backwards so the sign test is also backwards
		{
			return 1;
		}

		// If some signs were zero, then things are more complicated because we are either colinear
		//  with that edge, or the edge is degenerate enough to get a 0 value on the DotPerp.
		int NumZero = static_cast<int>(Sign1 == 0) + static_cast<int>(Sign2 == 0) + static_cast<int>(Sign3 == 0);
		bool bIsOnEdge = false;
		
		switch (NumZero)
		{
		case 0:
			// All signs were nonzero and in the correct direction, so must be inside triangle.
			return -1;
		case 1:
			// If only one sign was zero, seems more than likely that we're on the opposite edge.
			//  Hard to imagine some underflow case where that isn't actually the case, but we'll
			//  do the segment check just in case.
			bIsOnEdge = Sign1 == 0 ? TSegment2<RealType>::IsOnSegment(A, B, QueryPoint, Tolerance)
				: Sign2 == 0 ? TSegment2<RealType>::IsOnSegment(B, C, QueryPoint, Tolerance)
				: TSegment2<RealType>::IsOnSegment(C, A, QueryPoint, Tolerance);
			break;
		case 2:
			// Two signs were zero, so it better be on the vertex between those edges
			bIsOnEdge = Sign1 != 0 ? TVector2<RealType>::DistSquared(QueryPoint, C) <= Tolerance * Tolerance
				: Sign2 != 0 ? TVector2<RealType>::DistSquared(QueryPoint, A) <= Tolerance * Tolerance
				: TVector2<RealType>::DistSquared(QueryPoint, B) <= Tolerance * Tolerance;
			break;
		case 3:
			// All three signs were zero. We're dealing with a degenerate triangle of some
			//  sort. It should be sufficient to check any two segments.
			bIsOnEdge = TSegment2<RealType>::IsOnSegment(A, B, QueryPoint, Tolerance)
				|| TSegment2<RealType>::IsOnSegment(B, C, QueryPoint, Tolerance);
			break;
		default:
			ensure(false);
		}
		return bIsOnEdge ? 0 : 1;
	}


};

typedef TTriangle2<float> FTriangle2f;
typedef TTriangle2<double> FTriangle2d;






template<typename RealType>
struct TTriangle3
{
	TVector<RealType> V[3];

	TTriangle3() {}

	TTriangle3(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		V[0] = V0;
		V[1] = V1;
		V[2] = V2;
	}

	TTriangle3(const TVector<RealType> VIn[3])
	{
		V[0] = VIn[0];
		V[1] = VIn[1];
		V[2] = VIn[2];
	}

	TVector<RealType> BarycentricPoint(RealType Bary0, RealType Bary1, RealType Bary2) const
	{
		return Bary0*V[0] + Bary1*V[1] + Bary2*V[2];
	}

	TVector<RealType> BarycentricPoint(const TVector<RealType> & BaryCoords) const
	{
		return BaryCoords[0]*V[0] + BaryCoords[1]*V[1] + BaryCoords[2]*V[2];
	}

	TVector<RealType> GetBarycentricCoords(const TVector<RealType>& Point) const
	{
		return VectorUtil::BarycentricCoords(Point, V[0], V[1], V[2]);
	}

	/** @return vector that is perpendicular to the plane of this triangle */
	TVector<RealType> Normal() const
	{
		return VectorUtil::Normal(V[0], V[1], V[2]);
	}

	/** @return centroid of this triangle */
	TVector<RealType> Centroid() const
	{
		constexpr RealType f = 1.0 / 3.0;
		return TVector<RealType>(
			(V[0].X + V[1].X + V[2].X) * f,
			(V[0].Y + V[1].Y + V[2].Y) * f,
			(V[0].Z + V[1].Z + V[2].Z) * f
		);
	}

	/** grow the triangle around the centroid */
	void Expand(RealType Delta)
	{
		TVector<RealType> Centroid(Centroid());
		V[0] += Delta * ((V[0] - Centroid).Normalized());
		V[1] += Delta * ((V[1] - Centroid).Normalized());
		V[2] += Delta * ((V[2] - Centroid).Normalized());
	}
};

typedef TTriangle3<float> FTriangle3f;
typedef TTriangle3<double> FTriangle3d;
typedef TTriangle3<int> FTriangle3i;


// Note: More TetUtil functions are in TetUtil.h; this subset is here for use by the TTetrahedron3 struct below
namespace TetUtil
{
	// @return a C array of vertex orderings for each of the 4 triangle faces of a tetrahedron
	template<bool bReverseOrientation = false>
	inline const FIndex3i* GetTetFaceOrdering()
	{
		const static FIndex3i FaceMap[4]{ FIndex3i(0,1,2), FIndex3i(0,3,1), FIndex3i(0,2,3), FIndex3i(1,3,2) };
		const static FIndex3i FaceMapRev[4]{ FIndex3i(1,0,2), FIndex3i(3,0,1), FIndex3i(2,0,3), FIndex3i(3,1,2) };
		if constexpr (bReverseOrientation)
		{
			return FaceMapRev;
		}
		else
		{
			return FaceMap;
		}
	}
}

template<typename RealType>
struct TTetrahedron3
{
	TVector<RealType> V[4];

	TTetrahedron3() {}

	TTetrahedron3(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2, const TVector<RealType>& V3)
	{
		V[0] = V0;
		V[1] = V1;
		V[2] = V2;
		V[3] = V3;
	}

	TTetrahedron3(const TVector<RealType> VIn[4])
	{
		V[0] = VIn[0];
		V[1] = VIn[1];
		V[2] = VIn[2];
		V[3] = VIn[3];
	}

	TVector<RealType> BarycentricPoint(RealType Bary0, RealType Bary1, RealType Bary2, RealType Bary3) const
	{
		return Bary0 * V[0] + Bary1 * V[1] + Bary2 * V[2] + Bary3 * V[3];
	}

	TVector<RealType> BarycentricPoint(const TVector4<RealType>& BaryCoords) const
	{
		return BaryCoords[0] * V[0] + BaryCoords[1] * V[1] + BaryCoords[2] * V[2] + BaryCoords[3] * V[3];
	}

	TAxisAlignedBox3<RealType> Bounds() const
	{
		FAxisAlignedBox3d RetBounds;
		RetBounds.Contain(V[0]);
		RetBounds.Contain(V[1]);
		RetBounds.Contain(V[2]);
		RetBounds.Contain(V[3]);
		return RetBounds;
	}

	// Get the ith triangular face of the tetrahedron, as indices into the four tetrahedron vertices
	template<bool bReverseOrientation = false>
	static FIndex3i GetFaceIndices(int32 Idx)
	{
		return TetUtil::GetTetFaceOrdering<bReverseOrientation>()[Idx];
	}

	template<bool bReverseOrientation = false>
	TTriangle3<RealType> GetFace(int32 Idx) const
	{
		FIndex3i Face = GetFaceIndices<bReverseOrientation>(Idx);
		return TTriangle3<RealType>(V[Face.A], V[Face.B], V[Face.C]);
	}

	/** @return centroid of this tetrahedron */
	TVector<RealType> Centroid() const
	{
		constexpr RealType f = 1.0 / 4.0;
		return TVector<RealType>(
			(V[0].X + V[1].X + V[2].X + V[3].X) * f,
			(V[0].Y + V[1].Y + V[2].Y + V[3].Y) * f,
			(V[0].Z + V[1].Z + V[2].Z + V[3].Z) * f
			);
	}
};

typedef TTetrahedron3<float> FTetrahedron3f;
typedef TTetrahedron3<double> FTetrahedron3d;


} // end namespace UE::Geometry
} // end namespace UE