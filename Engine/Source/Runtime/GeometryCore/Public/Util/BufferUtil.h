// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

/**
 * Utility functions for processing arrays of objects
 * @todo possibly these functions should go in Algo:: or somewhere else in Core?
 */
namespace BufferUtil
{


	/**
	 * Count number of elements in array (or within requested range) that pass Predicate test
	 * @param Data Array to process
	 * @param Predicate filtering function, return true indicates valid
	 * @param MaxIndex optional maximum index in array (exclusive, default is to use array size)
	 * @param StartIndex optional start index in array (default 0)
	 * @return number of values in array that returned true for Predicate
	 */
	template<typename T>
	int CountValid(const TArray<T>& Data, const TFunction<bool(T)>& Predicate, int MaxIndex = -1, int StartIndex = 0)
	{
		int StopIndex = (MaxIndex == -1) ? Data.Num() : MaxIndex;
		int NumValid = 0;
		for (int i = StartIndex; i < StopIndex; ++i)
		{
			if (Predicate(Data[i]) == true)
			{
				NumValid++;
			}
		}
		return NumValid;
	}

	
	/**
	 * Removes elements of array (or within requested range) that do not pass Predicate, by shifting forward. 
	 * Does not remove remaining elements
	 * @param Data Array to process
	 * @param Predicate filtering function, return true indicates valid
	 * @param MaxIndex optional maximum index in array (exclusive, default is to use array size)
	 * @param StartIndex optional start index in array (default 0)
	 * @return Number of valid elements in Data after filtering
	 */
	template<typename T>
	int FilterInPlace(TArray<T>& Data, const TFunction<bool(T)>& Predicate, int MaxIndex = -1, int StartIndex = 0) 
	{
		int StopIndex = (MaxIndex == -1) ? Data.Num() : MaxIndex;
		int StoreIndex = StartIndex;
		for (int i = StartIndex; i < StopIndex; ++i)
		{
			if (Predicate(Data[i]) == true)
			{
				Data[StoreIndex++] = Data[i];
			}
		}
		return StoreIndex;
	}



	/**
	 * Append enumerable elements to an array
	 * @param Data array to append to
	 * @param Enumerable object that can be iterated over with a range-based for loop
	 */
	template<typename T, typename EnumerableType>
	void AppendElements(TArray<T>& AppendTo, EnumerableType Enumerable)
	{
		for ( T value : Enumerable )
		{
			AppendTo.Add(value);
		}
	}


	/**
	 * Sort an array view's values by the values in a second array view
	 */
	template<typename SortType, typename ValueType>
	void SortByValuesArray(TArrayView<SortType> ToSort, TConstArrayView<ValueType> Values, bool bStableSort, bool bAscendingOrder)
	{
		const int32 NumValues = Values.Num();
		if (!ensureMsgf(ToSort.Num() == NumValues, TEXT("Values array must have same length as ToSort array, but did not (%d vs %d)"), NumValues, ToSort.Num()))
		{
			return;
		}

		// make a reference array of indices, that we will sort based on values
		TArray<int32> Indices;
		Indices.SetNumUninitialized(NumValues);
		for (int32 Idx = 0; Idx < NumValues; ++Idx)
		{
			Indices[Idx] = Idx;
		}

		// sort it
		if (!bStableSort)
		{
			if (bAscendingOrder)
			{
				Indices.Sort([&Values](int32 A, int32 B) { return Values[A] < Values[B]; });
			}
			else
			{
				Indices.Sort([&Values](int32 A, int32 B) { return Values[A] > Values[B]; });
			}
		}
		else
		{
			if (bAscendingOrder)
			{
				Indices.StableSort([&Values](int32 A, int32 B) { return Values[A] < Values[B]; });
			}
			else
			{
				Indices.StableSort([&Values](int32 A, int32 B) { return Values[A] > Values[B]; });
			}
		}

		// swap the ToSort array to follow the ordering of the reference array
		for (int32 Idx = 0; Idx < NumValues; ++Idx)
		{
			int32 SwapFromIndex = Indices[Idx];
			while (SwapFromIndex < Idx)
			{
				SwapFromIndex = Indices[SwapFromIndex];
			}

			if (SwapFromIndex != Idx)
			{
				Swap(ToSort[Idx], ToSort[SwapFromIndex]);
			}
		}
	}

}