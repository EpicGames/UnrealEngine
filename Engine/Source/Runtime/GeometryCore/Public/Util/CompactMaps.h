// Copyright Epic Games, Inc. All Rights Reserved.

// Index remapping struct extracted from Dynamic Mesh for more general use

#pragma once

#include "Containers/Array.h"

#include "IndexTypes.h"

namespace UE
{
namespace Geometry
{
/**
 * Stores index remapping for vertices and triangles.
 * Should only be used for compacting, and should maintain invariant that *Map[Idx] <= Idx for all maps
 */
class FCompactMaps
{
	TArray<int32> VertMap;
	TArray<int32> TriMap;

public:
	constexpr static int32 InvalidID = IndexConstants::InvalidID;

	/**
	 * Set up maps as identity maps.
	 *
	 * @param NumVertMappings Vertex map will be created from 0 to NumVertMappings - 1
	 * @param NumTriMappings Triangle map will be created from 0 to NumTriMappings - 1
	 */
	void SetIdentity(int32 NumVertMappings, int32 NumTriMappings)
	{
		SetIdentityVertexMap(NumVertMappings);
		SetIdentityTriangleMap(NumTriMappings);
	}

	/**
	* Set up vertex map as identity map.
	*
	* @param NumVertMappings Vertex map will be created from 0 to NumVertMappings - 1
	*/
	void SetIdentityVertexMap(int32 NumVertMappings)
	{
		VertMap.SetNumUninitialized(NumVertMappings);
		for (int32 i = 0; i < NumVertMappings; ++i)
		{
			VertMap[i] = i;
		}
	}

	/**
	* Set up triangle map as identity map.
	*
	* @param NumTriMappings Vertex map will be created from 0 to NumTriMappings - 1
	*/
	void SetIdentityTriangleMap(int32 NumTriMappings)
	{
		TriMap.SetNumUninitialized(NumTriMappings);
		for (int32 i = 0; i < NumTriMappings; ++i)
		{
			TriMap[i] = i;
		}
	}

	/**
	 * Resize vertex and triangle maps, and initialize with InvalidID.
	 * @param NumVertMappings Size of post-reset vertex map
	 * @param NumTriMappings Size of post-reset triangle map
	* @param bInitializeWithInvalidID If true, initializes maps with InvalidID 
	 */
	void Reset(int32 NumVertMappings, int32 NumTriMappings, bool bInitializeWithInvalidID)
	{
		ResetVertexMap(NumVertMappings, bInitializeWithInvalidID);
		ResetTriangleMap(NumTriMappings, bInitializeWithInvalidID);
	}

	/**
	* Resize vertex map, and optionally initialize with InvalidID.
	* @param NumVertMappings Size of post-reset vertex map
	* @param bInitializeWithInvalidID If true, initializes map with InvalidID 
	*/
	void ResetVertexMap(int32 NumVertMappings, bool bInitializeWithInvalidID)
	{
		VertMap.SetNumUninitialized(NumVertMappings);
		if (bInitializeWithInvalidID)
		{
			for (int32 i = 0; i < NumVertMappings; i++)
			{
				VertMap[i] = InvalidID;
			}
		}
	}

	/**
	* Resize triangle map, and optionally initialize with InvalidID.
	* @param NumTriMappings Size of post-reset triangle map
	* @param bInitializeWithInvalidID If true, initializes map with InvalidID 
	*/
	void ResetTriangleMap(int32 NumTriMappings, bool bInitializeWithInvalidID)
	{
		TriMap.SetNumUninitialized(NumTriMappings);
		if (bInitializeWithInvalidID)
		{
			for (int32 i = 0; i < NumTriMappings; ++i)
			{
				TriMap[i] = InvalidID;
			}
		}
	}

	/** Reset all maps, leaving them empty */
	void Reset()
	{
		VertMap.Reset();
		TriMap.Reset();
	}

	/** Returns true if there are vertex mappings. */
	bool VertexMapIsSet() const
	{
		return !VertMap.IsEmpty();
	}

	/** Returns true if there are triangle mappings. */
	bool TriangleMapIsSet() const
	{
		return !TriMap.IsEmpty();
	}

	/** Get number of vertex mappings */
	int32 NumVertexMappings() const
	{
		return VertMap.Num();
	}

	/** Get number of triangle mappings */
	int32 NumTriangleMappings() const
	{
		return TriMap.Num();
	}

	/** Set mapping for a vertex */
	void SetVertexMapping(int32 FromID, int32 ToID)
	{
		checkSlow(FromID >= ToID);
		VertMap[FromID] = ToID;
	}

	/** Set mapping for a triangle */
	void SetTriangleMapping(int32 FromID, int32 ToID)
	{
		checkSlow(FromID >= ToID);
		TriMap[FromID] = ToID;
	}

	/** Get mapping for a vertex */
	int32 GetVertexMapping(int32 FromID) const
	{
		return VertMap[FromID];
	}

	/** Get mapping for three vertices, e.g. a triangle */
	FIndex3i GetVertexMapping(FIndex3i FromIDs) const
	{
		return {VertMap[FromIDs[0]], VertMap[FromIDs[1]], VertMap[FromIDs[2]]};
	}

	/** Get mapping for a triangle */
	int32 GetTriangleMapping(int32 FromID) const
	{
		return TriMap[FromID];
	}

	/** Check data for validity; for testing */
	bool Validate() const
	{
		for (int32 Idx = 0; Idx < VertMap.Num(); Idx++)
		{
			if (VertMap[Idx] > Idx)
			{
				return false;
			}
		}
		for (int32 Idx = 0; Idx < TriMap.Num(); Idx++)
		{
			if (TriMap[Idx] > Idx)
			{
				return false;
			}
		}
		return true;
	}
};
} // end namespace UE::Geometry
} // end namespace UE
