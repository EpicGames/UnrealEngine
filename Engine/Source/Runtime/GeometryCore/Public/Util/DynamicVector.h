// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "Containers/StaticArray.h"
#include "Serialization/Archive.h"
#include <UObject/UE5MainStreamObjectVersion.h>
#include "VectorTypes.h"
#include "IndexTypes.h"
#include "Math/NumericLimits.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/*
 * Blocked array with fixed, power-of-two sized blocks.
 *
 * Iterator functions suitable for use with range-based for are provided
 */
template <typename Type, int32 BlockSize = 512>
class TDynamicVector
{
	static_assert(BlockSize > 0, "TDynamicVector: BlockSize must be larger than zero.");
	static_assert(((BlockSize & (BlockSize - 1)) == 0), "TDynamicVector: BlockSize must be a power of two.");

	static constexpr uint32 NumBitsNeeded(const uint32 N) {
		return N <= 1 ? 0 : 1 + NumBitsNeeded((N + 1) / 2);
	}

	static constexpr uint32 GetBlockIndex(const uint32 Index)
	{
		constexpr int BlockBitsShift = NumBitsNeeded(BlockSize);
		return Index >> BlockBitsShift;
	}

	static constexpr uint32 GetIndexInBlock(const uint32 Index)
	{
		constexpr int BlockBitMask = BlockSize - 1;
		return Index & BlockBitMask;
	}

public:
	using ElementType = Type;

	TDynamicVector()
	{
		AddAllocatedBlock();
	}

	TDynamicVector(const TDynamicVector& Copy)
		: CurBlock(Copy.CurBlock)
		, CurBlockUsed(Copy.CurBlockUsed)
	{
		const int32 N = Copy.Blocks.Num();
		Blocks.Reserve(N);
		for (int32 k = 0; k < N; ++k)
		{
			Blocks.Add(new TBlock(*Copy.Blocks[k]));
		}
	}

	TDynamicVector(TDynamicVector&& Moved)
		: CurBlock(Moved.CurBlock)
		, CurBlockUsed(Moved.CurBlockUsed)
		, Blocks(MoveTemp(Moved.Blocks))
	{
		Moved.CurBlock = 0;
		Moved.CurBlockUsed = 0;
		Moved.AddAllocatedBlock();
	}

	TDynamicVector& operator=(const TDynamicVector& Copy)
	{
		if (this != &Copy)
		{
			const int32 N = Copy.Blocks.Num();
			Empty(N);
			CurBlock = Copy.CurBlock;
			CurBlockUsed = Copy.CurBlockUsed;
			for (int32 k = 0; k < N; ++k)
			{
				Blocks.Add(new TBlock(*Copy.Blocks[k]));
			}
		}
		return *this;
	}

	TDynamicVector& operator=(TDynamicVector&& Moved)
	{
		if (this != &Moved)
		{
			Empty();

			CurBlock = Moved.CurBlock;
			CurBlockUsed = Moved.CurBlockUsed;
			Blocks = MoveTemp(Moved.Blocks);

			Moved.CurBlock = 0;
			Moved.CurBlockUsed = 0;
			Moved.AddAllocatedBlock();
		}
		return *this;
	}

	TDynamicVector(const TArray<Type>& Array)
	{
		const uint32 N = static_cast<uint32>(Array.Num());
		SetNum(N);
		const Type* ArrayPtr = Array.GetData();
		for (uint32 Idx = 0; Idx < N; ++Idx)
		{
			(*this)[Idx] = ArrayPtr[Idx];
		}
	}

	TDynamicVector(TArrayView<const Type> Array)
	{
		const uint32 N = static_cast<uint32>(Array.Num());
		SetNum(N);
		const Type* ArrayPtr = Array.GetData();
		for (uint32 Idx = 0; Idx < N; ++Idx)
		{
			(*this)[Idx] = ArrayPtr[Idx];
		}
	}

	~TDynamicVector()
	{
		Empty();
	}

	inline void Clear();
	inline void Fill(const Type& Value);
	inline void Resize(unsigned int Count);
	inline void Resize(unsigned int Count, const Type& InitValue);
	/// Resize if Num() is less than Count; returns true if resize occurred
	inline bool SetMinimumSize(unsigned int Count, const Type& InitValue);
	inline void SetNum(unsigned int Count) { Resize(Count); }

	inline bool IsEmpty() const { return CurBlock == 0 && CurBlockUsed == 0; }
	inline size_t GetLength() const { return CurBlock * BlockSize + CurBlockUsed; }
	inline size_t Num() const { return GetLength(); }
	static constexpr int32 GetBlockSize() { return BlockSize; }
	inline size_t GetByteCount() const { return Blocks.Num() * BlockSize * sizeof(Type); }

	inline void Add(const Type& Data);
	template <int32 BlockSizeData> void Add(const TDynamicVector<Type, BlockSizeData>& Data);
	void Add(const TArray<Type>& Data);
	void Add(TArrayView<const Type> Data);
	inline void PopBack();

	inline void InsertAt(const Type& Data, unsigned int Index);
	inline void InsertAt(const Type& Data, unsigned int Index, const Type& InitValue);
	inline Type& ElementAt(unsigned int Index, Type InitialValue = Type{});

	inline const Type& Front() const
	{
		checkSlow(CurBlockUsed > 0);
		return GetElement(0, 0);
	}

	inline const Type& Back() const
	{
		checkSlow(CurBlockUsed > 0);
		return GetElement(CurBlock, CurBlockUsed - 1);
	}

#if USING_ADDRESS_SANITISER
	FORCENOINLINE
#endif
	const Type& operator[](uint32 Index) const
	{
		checkSlow(Index < Num());
		return GetElement(GetBlockIndex(Index), GetIndexInBlock(Index));
	}

#if USING_ADDRESS_SANITISER
	FORCENOINLINE
#endif
	Type& operator[](uint32 Index)
	{
		return const_cast<Type&>(const_cast<const TDynamicVector&>(*this)[Index]);
	}

	// apply ApplyFunc() to each member sequentially
	template <typename Func>
	void Apply(const Func& ApplyFunc);

	/**
	 * Serialization operator for TDynamicVector.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Vec Vector to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, TDynamicVector& Vec)
	{
		Vec.Serialize<false, false>(Ar);
		return Ar;
	}

	/**
	 * Serialize vector to and from an archive
	 * @tparam bForceBulkSerialization Forces serialization to consider data to be trivial and serialize it in bulk to potentially achieve better performance.
	 * @tparam bUseCompression Use compression to serialize data; the resulting size will likely be smaller but serialization will take significantly longer.
	 * @param Ar Archive to serialize with
	 */
	template <bool bForceBulkSerialization = false, bool bUseCompression = false>
	void Serialize(FArchive& Ar);

	/*
	 * FIterator class iterates over values of vector
	 */
	class FIterator
	{
	public:
		inline const Type& operator*() const
		{
			return (*DVector)[Idx];
		}
		inline Type& operator*()
		{
			return (*DVector)[Idx];
		}
		inline FIterator& operator++()   // prefix
		{
			Idx++;
			return *this;
		}
		inline FIterator operator++(int) // postfix
		{
			FIterator Copy(*this);
			Idx++;
			return Copy;
		}
		inline bool operator==(const FIterator& Itr2) const
		{
			return DVector == Itr2.DVector && Idx == Itr2.Idx;
		}
		inline bool operator!=(const FIterator& Itr2) const
		{
			return DVector != Itr2.DVector || Idx != Itr2.Idx;
		}

	private:
		friend class TDynamicVector;
		FIterator(TDynamicVector* DVectorIn, unsigned int IdxIn)
			: DVector(DVectorIn), Idx(IdxIn){}
		TDynamicVector* DVector{};
		unsigned int Idx{0};
	};

	/** @return iterator at beginning of vector */
	FIterator begin()
	{
		return FIterator{this, 0};
	}
	/** @return iterator at end of vector */
	FIterator end()
	{
		return FIterator{this,  (unsigned int)GetLength()};
	}

	/*
	 * FConstIterator class iterates over values of vector
	 */
	class FConstIterator
	{
	public:
		inline const Type& operator*() const
		{
			return (*DVector)[Idx];
		}
		inline FConstIterator& operator++()   // prefix
		{
			Idx++;
			return *this;
		}
		inline FConstIterator operator++(int) // postfix
		{
			FConstIterator Copy(*this);
			Idx++;
			return Copy;
		}
		inline bool operator==(const FConstIterator& Itr2) const
		{
			return DVector == Itr2.DVector && Idx == Itr2.Idx;
		}
		inline bool operator!=(const FConstIterator& Itr2) const
		{
			return DVector != Itr2.DVector || Idx != Itr2.Idx;
		}

	private:
		friend class TDynamicVector;
		FConstIterator(const TDynamicVector* DVectorIn, unsigned int IdxIn)
			: DVector(DVectorIn), Idx(IdxIn){}
		const TDynamicVector* DVector{};
		unsigned int Idx{0};
	};

	/** @return iterator at beginning of vector */
	FConstIterator begin() const
	{
		return FConstIterator{this, 0};
	}
	/** @return iterator at end of vector */
	FConstIterator end() const
	{
		return FConstIterator{this,  (unsigned int)GetLength()};
	}

private:
	struct TBlock
	{
		Type Elements[BlockSize];
	};

	unsigned int CurBlock{0};  //< Current block index; always points to the block with the last item in the vector, or is set to zero if the vector is empty. 
	unsigned int CurBlockUsed{0};  //< Number of used items in the current block.

	TArray<TBlock*> Blocks;

	void AddAllocatedBlock()
	{
		Blocks.Add(new TBlock);
	}

	void Empty(int32 NewReservedBlockCount = 0)
	{
		for (int32 k = 0, N = Blocks.Num(); k < N; ++k)
		{
			delete Blocks[k];
		}
		Blocks.Empty(NewReservedBlockCount);
	}

	const Type& GetElement(int32 BlockIndex, int32 IndexInBlock) const
	{
		checkSlow(0 <= BlockIndex && BlockIndex < Blocks.Num() && 0 <= IndexInBlock && IndexInBlock < BlockSize);
		return Blocks.GetData()[BlockIndex]->Elements[IndexInBlock];
	}

	Type& GetElement(int32 BlockIndex, int32 IndexInBlock)
	{
		return const_cast<Type&>(const_cast<const TDynamicVector&>(*this).GetElement(BlockIndex, IndexInBlock));
	}

	void TruncateBlocks(int32 NewBlockCount, EAllowShrinking AllowShrinking)
	{
		if (Blocks.Num() - NewBlockCount <= 0)
		{
			return;
		}
		
		for (int32 k = NewBlockCount; k < Blocks.Num(); ++k)
		{
			delete Blocks[k];
			Blocks[k] = nullptr;
		}
		Blocks.RemoveAt(NewBlockCount, Blocks.Num() - NewBlockCount, AllowShrinking);
	}

	template <int32 BlockSizeRhs>
	friend bool operator==(const TDynamicVector& Lhs, const TDynamicVector<Type, BlockSizeRhs>& Rhs)
	{
		if (Lhs.Num() != Rhs.Num())
		{
			return false;
		}

		if (Lhs.IsEmpty())
		{
			return true;
		}

		if constexpr (BlockSize == BlockSizeRhs)
		{
			const uint32 LhsCurBlock = Lhs.CurBlock;
			for (uint32 BlockIndex = 0; BlockIndex < LhsCurBlock; ++BlockIndex)
			{
				if (!CompareItems(&Lhs.Blocks[BlockIndex]->Elements[0], &Rhs.Blocks[BlockIndex]->Elements[0], BlockSize))
				{
					return false;
				}
			}
			return CompareItems(&Lhs.Blocks[LhsCurBlock]->Elements[0], &Rhs.Blocks[LhsCurBlock]->Elements[0], Lhs.CurBlockUsed);
		}
		else
		{
			for (int32 Index = 0, Num = Lhs.Num(); Index < Num; ++Index)
			{
				if (!(Lhs[Index] == Rhs[Index]))
				{
					return false;
				}
			}

			return true;
		}
	}

	template <int32 BlockSizeRhs>
	friend bool operator!=(const TDynamicVector& Lhs, const TDynamicVector<Type, BlockSizeRhs>& Rhs)
	{
		return !(Lhs == Rhs);
	}

	void SetCurBlock(SIZE_T Count)
	{
		// Reset block index for the last item and used item count within the last block.
		// This is similar to what happens when computing the indices in operator[], but we additionally account for (1) the vector being empty and (2) that the
		// used item count within the last block needs to be one more than the index of the last item. 
		const int32 LastItemIndex = int32(Count - 1);
		CurBlock = Count != 0 ? GetBlockIndex(LastItemIndex) : 0;
		CurBlockUsed = Count != 0 ? GetIndexInBlock(LastItemIndex) + 1 : 0;
	}
};

template <class Type, int N>
class TDynamicVectorN
{
public:
	TDynamicVectorN() = default;
	TDynamicVectorN(const TDynamicVectorN& Copy) = default;
	TDynamicVectorN(TDynamicVectorN&& Moved) = default;
	TDynamicVectorN& operator=(const TDynamicVectorN& Copy) = default;
	TDynamicVectorN& operator=(TDynamicVectorN&& Moved) = default;

	inline void Clear()
	{
		Data.Clear();
	}
	inline void Fill(const Type& Value)
	{
		Data.Fill(Value);
	}
	inline void Resize(unsigned int Count)
	{
		Data.Resize(Count * N);
	}
	inline void Resize(unsigned int Count, const Type& InitValue)
	{
		Data.Resize(Count * N, InitValue);
	}
	inline bool IsEmpty() const
	{
		return Data.IsEmpty();
	}
	inline size_t GetLength() const
	{
		return Data.GetLength() / N;
	}
	inline int GetBlockSize() const
	{
		return Data.GetBlockSize();
	}
	inline size_t GetByteCount() const
	{
		return Data.GetByteCount();
	}

	// simple struct to help pass N-dimensional data without presuming a vector type (e.g. just via initializer list)
	struct ElementVectorN
	{
		Type Data[N];
	};

	inline void Add(const ElementVectorN& AddData)
	{
		// todo specialize for N=2,3,4
		for (int i = 0; i < N; i++)
		{
			Data.Add(AddData.Data[i]);
		}
	}

	inline void PopBack()
	{
		for (int i = 0; i < N; i++)
		{
			PopBack();
		}
	} // TODO specialize

	inline void InsertAt(const ElementVectorN& AddData, unsigned int Index)
	{
		for (int i = 1; i <= N; i++)
		{
			Data.InsertAt(AddData.Data[N - i], N * (Index + 1) - i);
		}
	}

	inline Type& operator()(unsigned int TopIndex, unsigned int SubIndex)
	{
		return Data[TopIndex * N + SubIndex];
	}
	inline const Type& operator()(unsigned int TopIndex, unsigned int SubIndex) const
	{
		return Data[TopIndex * N + SubIndex];
	}
	inline void SetVector2(unsigned int TopIndex, const TVector2<Type>& V)
	{
		check(N >= 2);
		unsigned int i = TopIndex * N;
		Data[i] = V.X;
		Data[i + 1] = V.Y;
	}
	inline void SetVector3(unsigned int TopIndex, const TVector<Type>& V)
	{
		check(N >= 3);
		unsigned int i = TopIndex * N;
		Data[i] = V.X;
		Data[i + 1] = V.Y;
		Data[i + 2] = V.Z;
	}
	inline TVector2<Type> AsVector2(unsigned int TopIndex) const
	{
		check(N >= 2);
		return TVector2<Type>(
			Data[TopIndex * N + 0],
			Data[TopIndex * N + 1]);
	}
	inline TVector<Type> AsVector3(unsigned int TopIndex) const
	{
		check(N >= 3);
		return TVector<Type>(
			Data[TopIndex * N + 0],
			Data[TopIndex * N + 1],
			Data[TopIndex * N + 2]);
	}
	inline FIndex2i AsIndex2(unsigned int TopIndex) const
	{
		check(N >= 2);
		return FIndex2i(
			(int)Data[TopIndex * N + 0],
			(int)Data[TopIndex * N + 1]);
	}
	inline FIndex3i AsIndex3(unsigned int TopIndex) const
	{
		check(N >= 3);
		return FIndex3i(
			(int)Data[TopIndex * N + 0],
			(int)Data[TopIndex * N + 1],
			(int)Data[TopIndex * N + 2]);
	}
	inline FIndex4i AsIndex4(unsigned int TopIndex) const
	{
		check(N >= 4);
		return FIndex4i(
			(int)Data[TopIndex * N + 0],
			(int)Data[TopIndex * N + 1],
			(int)Data[TopIndex * N + 2],
			(int)Data[TopIndex * N + 3]);
	}

private:
	TDynamicVector<Type> Data;

	friend class FIterator;
};

template class TDynamicVectorN<double, 2>;

using  TDynamicVector3f = TDynamicVectorN<float, 3>;
using  TDynamicVector2f = TDynamicVectorN<float, 2>;
using  TDynamicVector3d = TDynamicVectorN<double, 3>;
using  TDynamicVector2d = TDynamicVectorN<double, 2>;
using  TDynamicVector3i = TDynamicVectorN<int, 3>;
using  TDynamicVector2i = TDynamicVectorN<int, 2>;

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Clear()
{
	TruncateBlocks(1, EAllowShrinking::No);
	CurBlock = 0;
	CurBlockUsed = 0;
	if (Blocks.Num() == 0)
	{
		AddAllocatedBlock();
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Fill(const Type& Value)
{
	for (uint32 BlockIndex = 0, NumBlocks = Blocks.Num(); BlockIndex < NumBlocks; ++BlockIndex)
	{
		const uint32 NumElementsInBlock = BlockIndex < NumBlocks - 1 ? BlockSize : GetLength() - BlockSize * (NumBlocks - 1);
		for (uint32 IndexInBlock = 0; IndexInBlock < NumElementsInBlock; ++IndexInBlock)
		{
			GetElement(BlockIndex, IndexInBlock) = Value;
		}
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Resize(unsigned int Count)
{
	if (GetLength() == Count)
	{
		return;
	}

	// Determine how many blocks we need, but make sure we have at least one block available.
	const bool bCountIsNotMultipleOfBlockSize = Count % BlockSize != 0;
	const int32 NumBlocksNeeded = FMath::Max(1, static_cast<int32>(Count) / BlockSize + bCountIsNotMultipleOfBlockSize);

	// Determine how many blocks are currently allocated.
	int32 NumBlocksCurrent = Blocks.Num();

	// Allocate needed additional blocks.
	while (NumBlocksCurrent < NumBlocksNeeded)
	{
		AddAllocatedBlock();
		++NumBlocksCurrent;
	}

	// Remove unneeded blocks.
	if (NumBlocksCurrent > NumBlocksNeeded)
	{
		TruncateBlocks(NumBlocksNeeded, EAllowShrinking::No);
	}

	// Set current block.
	SetCurBlock(Count);
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Resize(unsigned int Count, const Type& InitValue)
{
	size_t nCurSize = GetLength();
	Resize(Count);
	for (unsigned int Index = (unsigned int)nCurSize; Index < Count; ++Index)
	{
		(*this)[Index] = InitValue;
	}
}

template <typename Type, int32 BlockSize>
bool TDynamicVector<Type, BlockSize>::SetMinimumSize(unsigned int Count, const Type& InitValue)
{
	size_t nCurSize = GetLength();
	if (Count <= nCurSize)
	{
		return false;
	}
	Resize(Count);
	for (unsigned int Index = (unsigned int)nCurSize; Index < Count; ++Index)
	{
		(*this)[Index] = InitValue;
	}
	return true;
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Add(const Type& Data)
{
	checkSlow(size_t(MAX_uint32) >= GetLength() + 1)
	if (CurBlockUsed == BlockSize)
	{
		if (CurBlock == Blocks.Num() - 1)
		{
			AddAllocatedBlock();
		}
		++CurBlock;
		CurBlockUsed = 0;
	}
	GetElement(CurBlock, CurBlockUsed) = Data;
	++CurBlockUsed;
}

template <typename Type, int32 BlockSize>
template <int32 BlockSizeData>
void TDynamicVector<Type, BlockSize>::Add(const TDynamicVector<Type, BlockSizeData>& Data)
{
	// @todo it could be more efficient to use memcopies here...
	const uint32 Offset = Num();
	const uint32 DataNum = static_cast<uint32>(Data.Num());
	SetNum(Offset + DataNum);
	for (uint32 DataIndex = 0; DataIndex < DataNum; ++DataIndex)
	{
		(*this)[Offset + DataIndex] = Data[DataIndex];
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Add(const TArray<Type>& Data)
{
	const uint32 Offset = Num();
	const uint32 DataNum = static_cast<uint32>(Data.Num());
	SetNum(Offset + DataNum);
	for (uint32 DataIndex = 0; DataIndex < DataNum; ++DataIndex)
	{
		(*this)[Offset + DataIndex] = Data[DataIndex];
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::Add(TArrayView<const Type> Data)
{
	const uint32 Offset = Num();
	const uint32 DataNum = static_cast<uint32>(Data.Num());
	SetNum(Offset + DataNum);
	for (uint32 DataIndex = 0; DataIndex < DataNum; ++DataIndex)
	{
		(*this)[Offset + DataIndex] = Data[DataIndex];
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::PopBack()
{
	if (CurBlockUsed > 0)
	{
		CurBlockUsed--;
	}
	if (CurBlockUsed == 0 && CurBlock > 0)
	{
		CurBlock--;
		CurBlockUsed = BlockSize;
	}
}

template <typename Type, int32 BlockSize>
Type& TDynamicVector<Type, BlockSize>::ElementAt(unsigned int Index, Type InitialValue)
{
	size_t s = GetLength();
	if (Index == s)
	{
		Add(InitialValue);
	}
	else if (Index > s)
	{
		Resize(Index);
		Add(InitialValue);
	}
	return (*this)[Index];
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::InsertAt(const Type& Data, unsigned int Index)
{
	size_t s = GetLength();
	if (Index == s)
	{
		Add(Data);
	}
	else if (Index > s)
	{
		Resize(Index);
		Add(Data);
	}
	else
	{
		(*this)[Index] = Data;
	}
}

template <typename Type, int32 BlockSize>
void TDynamicVector<Type, BlockSize>::InsertAt(const Type& AddData, unsigned int Index, const Type& InitValue)
{
	size_t nCurSize = GetLength();
	InsertAt(AddData, Index);
	// initialize all new values up to (but not including) the inserted index
	for (unsigned int i = (unsigned int)nCurSize; i < Index; ++i)
	{
		(*this)[i] = InitValue;
	}
}

template <typename Type, int32 BlockSize>
template <typename Func>
void TDynamicVector<Type, BlockSize>::Apply(const Func& ApplyFunc)
{
	for (int32 BlockIndex = 0; BlockIndex <= CurBlock; ++BlockIndex)
	{
		TBlock* Block = Blocks[BlockIndex];
		const int32 NumElements = BlockIndex < CurBlock ? BlockSize : CurBlockUsed;
		for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
		{
			ApplyFunc(Block[ElementIndex]);
		}
	}
}

template <typename Type, int32 BlockSize>
template <bool bForceBulkSerialization, bool bUseCompression>
void TDynamicVector<Type, BlockSize>::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
	{
		// In this version the serialization was done with a fixed block size of 512, and blocks were serialized in their entirety even if they were not
		// fully occupied, i.e. the last block might have had garbage in it.
		// To load this data, we first serialize all legacy blocks into a temporary buffer, and then copy out all valid elements one by one. While this
		// solution is making an additional copies, it is simple and robust.

		constexpr int32 LegacyBlockSize = 512;

		uint32 LegacyCurBlock = 0;
		uint32 LegacyCurBlockUsed = 0;
		int32 BlockNum = 0;
		Ar << LegacyCurBlock;
		Ar << LegacyCurBlockUsed;
		Ar << BlockNum;

		// Bulk serialization for a number of double types was enabled as part of the transition to Large World Coordinates.
		// If the currently stored type is one of these types, and the archive is from before bulk serialization for these types was enabled,
		// we need to still use per element serialization for legacy data.
		constexpr bool bIsLWCBulkSerializedDoubleType =
			std::is_same_v<Type, FVector2d> ||
			std::is_same_v<Type, FVector3d> ||
			std::is_same_v<Type, FVector4d> ||
			std::is_same_v<Type, FQuat4d> ||
			std::is_same_v<Type, FTransform3d>;
		const bool bUseBulkSerialization = TCanBulkSerialize<Type>::Value && !(bIsLWCBulkSerializedDoubleType && Ar.UEVer() <
			EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES);

		// Lambda for serializing a block either via bulk serializing the contained data or via serializing the block container itself.
		// Note that the static_cast<> was necessary to resolve compiler errors when using MSVC.
		const auto SerializeElements = bUseBulkSerialization
			                               ? static_cast<void(*)(FArchive&, Type*)>([](FArchive& Archive, Type* BlockElements)
			                               {
				                               Archive.Serialize(BlockElements, static_cast<int64>(LegacyBlockSize) * sizeof(Type));
			                               })
			                               : static_cast<void(*)(FArchive&, Type*)>([](FArchive& Archive, Type* BlockElements)
			                               {
				                               for (int32 Index = 0; Index < LegacyBlockSize; ++Index)
				                               {
					                               Archive << BlockElements[Index];
				                               }
			                               });

		// Serialize all blocks into a temporary buffer.
		TArray<Type> TempElementBuffer;
		TempElementBuffer.SetNum(BlockNum * LegacyBlockSize);
		Type* TempElementBufferPtr = TempElementBuffer.GetData();
		for (int32 BlockIndex = 0; BlockIndex < BlockNum; ++BlockIndex)
		{
			SerializeElements(Ar, TempElementBufferPtr);
			TempElementBufferPtr += LegacyBlockSize;
		}

		// Add all valid elements from the temporary buffer into the vector.
		const uint32 ElementsNum = LegacyCurBlock * LegacyBlockSize + LegacyCurBlockUsed;
		Empty(ElementsNum / LegacyBlockSize + (ElementsNum % LegacyBlockSize != 0));
		CurBlock = 0;
		CurBlockUsed = 0;
		AddAllocatedBlock();
		for (uint32 ElementIndex = 0; ElementIndex < ElementsNum; ElementIndex++)
		{
			Add(TempElementBuffer[ElementIndex]);
		}
	}
	else
	{
		uint32 SerializeNum = Num();
		const SIZE_T CountBytes =  sizeof(uint32) + SerializeNum * sizeof(Type);
		Ar.CountBytes(CountBytes, CountBytes);
		Ar << SerializeNum;
		if (SerializeNum == 0 && Ar.IsLoading())
		{
			Clear();
		}
		else if (SerializeNum > 0)
		{
			SetCurBlock(SerializeNum);

			constexpr bool bUseBulkSerialization = bForceBulkSerialization || TCanBulkSerialize<Type>::Value || sizeof(Type) == 1;
			static_assert(!bUseCompression || bUseBulkSerialization, "Compression only available when using bulk serialization");

			// Serialize compression flag, which adds flexibility when de-serializing existing data even if some implementation details change.  
			bool bUseCompressionForBulkSerialization = bUseBulkSerialization && bUseCompression;
			Ar << bUseCompressionForBulkSerialization;

			// Determine number of blocks.
			const bool bNumIsNotMultipleOfBlockSize = SerializeNum % BlockSize != 0;
			const uint32 NumBlocks = SerializeNum / BlockSize + bNumIsNotMultipleOfBlockSize;

			if (bUseCompressionForBulkSerialization)
			{
				// When using compression, copy everything to/from a big single buffer and serialize the big buffer.
				// This results in better compression ratios while at the same time accelerating compression. 

				TArray<Type> Buffer;
				Buffer.SetNumUninitialized(SerializeNum);
				Type* BufferPtr = Buffer.GetData();
				SIZE_T NumCopyRemaining = SerializeNum;

				if (!Ar.IsLoading())
				{
					for (uint32 Index = 0; Index < NumBlocks; ++Index)
					{
						const SIZE_T NumCopy = FMath::Min<SIZE_T>(NumCopyRemaining, static_cast<SIZE_T>(BlockSize));
						FMemory::Memcpy(BufferPtr, &Blocks[Index]->Elements[0], NumCopy * sizeof(Type));
						BufferPtr += BlockSize;
						NumCopyRemaining -= BlockSize;
					}
				}

				Ar.SerializeCompressedNew(Buffer.GetData(), SerializeNum * sizeof(Type), NAME_Oodle, NAME_Oodle, COMPRESS_NoFlags, false, nullptr);

				if (Ar.IsLoading())
				{
					Empty(NumBlocks);
					for (uint32 Index = 0; Index < NumBlocks; ++Index)
					{
						TBlock *const NewBlock = new TBlock;
						const SIZE_T NumCopy = FMath::Min<SIZE_T>(NumCopyRemaining, static_cast<SIZE_T>(BlockSize));
						FMemory::Memcpy(&NewBlock->Elements[0], BufferPtr, NumCopy * sizeof(Type));
						Blocks.Add(NewBlock);
						BufferPtr += BlockSize;
						NumCopyRemaining -= BlockSize;
					}
				}
			}
			else
			{
				const auto SerializeBlock = [&Ar, bUseBulkSerialization](TBlock* Block, uint32 NumElements)
				{
					if (bUseBulkSerialization)
					{
						Ar.Serialize(&Block->Elements[0], NumElements * sizeof(Type));
					}
					else
					{
						for (uint32 Index = 0; Index < NumElements; ++Index)
						{
							Ar << Block->Elements[Index];
						}
					}
				};

				if (Ar.IsLoading())
				{
					Empty(NumBlocks);
					for (uint32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
					{
						TBlock *const NewBlock = new TBlock;
						SerializeBlock(NewBlock, FMath::Min<uint32>(SerializeNum, BlockSize));
						Blocks.Add(NewBlock);
						SerializeNum -= BlockSize;
					}
				}
				else
				{
					for (uint32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
					{
						SerializeBlock(Blocks[BlockIndex], FMath::Min<uint32>(SerializeNum, BlockSize));
						SerializeNum -= BlockSize;
					}
				}
			}
		}
	}

	if (Ar.IsLoading() && Blocks.Num() == 0)
	{
		AddAllocatedBlock();
	}
}

} // end namespace UE::Geometry
} // end namespace UE
