// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "IndexTypes.h"
#include "VectorTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
* FElementLinearization maps a potentially-sparse index list into a linear array.
* Used linearize things like VtxIds of a mesh as a single array and allow bidirectional mapping between array offset and mesh VtxId.
* Linearized array offset can then be used for things like matrix row indexing when building a Laplacian matrix.
*/
class FElementLinearization
{
public:
	FElementLinearization() = default;

	// Lookup   ToVtxId(Index) = VtxId;
	const TArray<int32>& ToId() const { return ToIdMap; }

	// Lookup   ToIndex(VtxId) = Index;  may return FDynamicMesh3::InvalidID 
	const TArray<int32>& ToIndex() const { return ToIndexMap; }

	/** @return maximum number of IDs in linearization (However not all IDs necessarily have a linear index) */
	int32 NumIds() const { return ToIdMap.Num(); }

	// Following the FDynamicMesh3 convention this is really MaxId + 1
	int32 MaxId() const { return ToIndexMap.Num(); }

	/** @return number of Indices in linearization */
	int32 NumIndices() const { return ToIdMap.Num(); }

	/** @return true if Id is in valid range and maps to a valid index */
	bool IsValidId(int32 Id) const 
	{ 
		return Id >= 0 && Id < ToIndexMap.Num() && ToIndexMap[Id] >= 0; 
	}

	/** @return true if Index is in valid range */
	bool IsValidIndex(int32 Index) const 
	{ 
		return Index >= 0 && Index < ToIdMap.Num(); 
	}

	/** @return Index for given Id */
	int32 GetIndex(int32 Id) const 
	{ 
		checkSlow(IsValidId(Id));
		return ToIndexMap[Id];
	}

	/** @return Id for given Index  */
	int32 GetId(int32 Index) const
	{
		checkSlow(IsValidIndex(Index));
		return ToIdMap[Index];
	}

	void Empty() { ToIdMap.Empty();  ToIndexMap.Empty(); }

	template<typename IterableType>
	void Populate(const int32 MaxId, const int32 Count, IterableType Iterable)
	{
		ToIndexMap.SetNumUninitialized(MaxId);
		ToIdMap.SetNumUninitialized(Count);

		for (int32 i = 0; i < MaxId; ++i)
		{
			ToIndexMap[i] = IndexConstants::InvalidID;
		}

		// create the mapping
		{
			int32 N = 0;
			for (int32 Id : Iterable)
			{
				ToIdMap[N] = Id;
				ToIndexMap[Id] = N;
				N++;
			}
		}
	}

protected:
	TArray<int32>  ToIdMap;
	TArray<int32>  ToIndexMap;

private:
	FElementLinearization(const FElementLinearization&);
};



/**
 * Structure-of-Array (SoA) storage for a list of 3-vectors
 */
template<typename RealType>
class TVector3Arrays
{
protected:
	TArray<RealType> XVector;
	TArray<RealType> YVector;
	TArray<RealType> ZVector;

public:

	TVector3Arrays(int32 Size)
	{
		XVector.SetNum(Size);
		YVector.SetNum(Size);
		ZVector.SetNum(Size);
	}

	TVector3Arrays()
	{}

	void SetZero(int32 NumElements)
	{
		XVector.Reset(0);
		XVector.SetNumZeroed(NumElements, EAllowShrinking::No);
		YVector.Reset(0);
		YVector.SetNumZeroed(NumElements, EAllowShrinking::No);
		ZVector.Reset(0);
		ZVector.SetNumZeroed(NumElements, EAllowShrinking::No);
	}

	// Test that all the arrays have the same given size.
	bool bHasSize(int32 Size) const
	{
		return (XVector.Num() == Size && YVector.Num() == Size && ZVector.Num() == Size);
	}

	int32 Num() const
	{
		int32 Size = XVector.Num();
		if (!bHasSize(Size))
		{
			Size = -1;
		}
		return Size;
	}

	RealType X(int32 i) const
	{
		return XVector[i];
	}

	RealType Y(int32 i) const
	{
		return YVector[i];
	}

	RealType Z(int32 i) const
	{
		return ZVector[i];
	}

	void SetX(int32 i, const RealType& Value)
	{
		XVector[i] = Value;
	}

	void SetY(int32 i, const RealType& Value)
	{
		YVector[i] = Value;
	}

	void SetZ(int32 i, const RealType& Value)
	{
		ZVector[i] = Value;
	}

	void SetXYZ(int32 i, const TVector<RealType>& Value)
	{
		XVector[i] = Value.X;
		YVector[i] = Value.Y;
		ZVector[i] = Value.Z;
	}

	TVector<RealType> Get(int32 i)
	{
		return TVector<RealType>(XVector[i], YVector[i], ZVector[i]);
	}

	void Set(int32 i, const TVector<RealType>& Value)
	{
		XVector[i] = Value.X;
		YVector[i] = Value.Y;
		ZVector[i] = Value.Z;
	}
};






/**
 * Structure-of-Array (SoA) storage for a list of 2-vectors
 */
template<typename RealType>
class TVector2Arrays
{
protected:
	TArray<RealType> XVector;
	TArray<RealType> YVector;

public:

	TVector2Arrays(int32 Size)
	{
		XVector.SetNum(Size);
		YVector.SetNum(Size);
	}

	TVector2Arrays()
	{}

	void SetZero(int32 NumElements)
	{
		XVector.Reset(0);
		XVector.SetNumZeroed(NumElements, EAllowShrinking::No);
		YVector.Reset(0);
		YVector.SetNumZeroed(NumElements, EAllowShrinking::No);
	}

	// Test that all the arrays have the same given size.
	bool bHasSize(int32 Size) const
	{
		return (XVector.Num() == Size && YVector.Num() == Size);
	}

	int32 Num() const
	{
		int32 Size = XVector.Num();
		if (!bHasSize(Size))
		{
			Size = -1;
		}
		return Size;
	}

	RealType X(int32 i) const
	{
		return XVector[i];
	}

	RealType Y(int32 i) const
	{
		return YVector[i];
	}

	void SetX(int32 i, const RealType& Value)
	{
		XVector[i] = Value;
	}

	void SetY(int32 i, const RealType& Value)
	{
		YVector[i] = Value;
	}

	void SetXY(int32 i, const UE::Math::TVector2<RealType>& Value)
	{
		XVector[i] = Value.X;
		YVector[i] = Value.Y;
	}
};


} // end namespace UE::Geometry
} // end namespace UE