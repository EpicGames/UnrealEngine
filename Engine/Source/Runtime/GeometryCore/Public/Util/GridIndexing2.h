// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp GridIndexing2

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "Math/IntRect.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Convert between integer grid coordinates and scaled real-valued coordinates (ie assumes integer grid origin == real origin)
 */
template<typename RealType>
struct TScaleGridIndexer2
{
	/** Real-valued size of an integer grid cell */
	RealType CellSize;

	TScaleGridIndexer2() : CellSize((RealType)1) 
	{
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	TScaleGridIndexer2(RealType CellSize) : CellSize(CellSize)
	{
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	/** Convert real-valued point to integer grid coordinates */
	inline FVector2i ToGrid(const TVector2<RealType>& P) const
	{
		return FVector2i(
			(int)TMathUtil<RealType>::Floor(P.X / CellSize),
			(int)TMathUtil<RealType>::Floor(P.Y / CellSize));
	}

	/** Convert integer grid coordinates to real-valued point */
	inline TVector2<RealType> FromGrid(const FVector2i& GridPoint) const
	{
		return TVector2<RealType>(GridPoint.X*CellSize, GridPoint.Y*CellSize);
	}
};
typedef TScaleGridIndexer2<float> FScaleGridIndexer2f;
typedef TScaleGridIndexer2<double> FScaleGridIndexer2d;


/**
 * Convert between integer grid coordinates and scaled+translated real-valued coordinates
 */
template<typename RealType>
struct TShiftGridIndexer2
{
	/** Real-valued size of an integer grid cell */
	RealType CellSize;
	/** Real-valued origin of grid, position of integer grid origin */
	TVector2<RealType> Origin;

	TShiftGridIndexer2(const TVector2<RealType>& origin, RealType cellSize)
	{
		Origin = origin;
		CellSize = cellSize;
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	/** Convert real-valued point to integer grid coordinates */
	inline FVector2i ToGrid(const TVector2<RealType>& Point) const
	{
		return FVector2i(
			(int)TMathUtil<RealType>::Floor((Point.X - Origin.X) / CellSize),
			(int)TMathUtil<RealType>::Floor((Point.Y - Origin.Y) / CellSize));
	}

	/** Convert real-valued 2D box to a rectangle using integer grid coordinates */
	inline FIntRect ToGrid(const TBox2<RealType>& Box) const
	{
		const FVector2i Min = ToGrid(Box.Min);
		const FVector2i Max = ToGrid(Box.Max);
		return FIntRect(Min.X, Min.Y, Max.X, Max.Y);
	}

	/** Convert real-valued 3D box to a rectangle using integer grid coordinates */
	inline FIntRect ToGrid(const TBox<RealType>& Box) const
	{
		const FVector2i Min = ToGrid(TVector2<RealType>(Box.Min));
		const FVector2i Max = ToGrid(TVector2<RealType>(Box.Max));
		return FIntRect(Min.X, Min.Y, Max.X, Max.Y);
	}

	/** Convert real-valued point to real-valued grid coordinates */
	inline TVector2<RealType> ToRealGrid(const TVector2<RealType>& Point) const
	{
		return TVector2<RealType>(
			TMathUtil<RealType>::Floor((Point.X - Origin.X) / CellSize),
			TMathUtil<RealType>::Floor((Point.Y - Origin.Y) / CellSize));
	}

	/** Convert integer grid coordinates to real-valued point */
	inline TVector2<RealType> FromGrid(const FVector2i& GridPoint) const
	{
		return TVector2<RealType>(
			((RealType)GridPoint.X * CellSize) + Origin.X,
			((RealType)GridPoint.Y * CellSize) + Origin.Y);
	}

	/** Convert real-valued grid coordinates to real-valued point */
	inline TVector2<RealType> FromGrid(const TVector2<RealType>& RealGridPoint) const
	{
		return TVector2<RealType>(
			((RealType)RealGridPoint.X * CellSize) + Origin.X,
			((RealType)RealGridPoint.Y * CellSize) + Origin.Y);
	}

	/** Compute the real-valued 2D box of an integer grid coordinates */
	inline TBox2<RealType> BoxFromGrid(const FVector2i& GridPoint) const
	{
		const TVector2<RealType> Min = FromGrid(GridPoint);
		const TVector2<RealType> Max = Min + TVector2<RealType>(CellSize);
		return TBox2<RealType>(Min, Max);
	}

	/** Indicate if the real-valued point is within the valid range of the indexer. If it returns false, it means that the conversion to grid coordinates would cause an integer overflow. */
	bool IsInValidRange(const TVector2<RealType>& Point) const
	{
		const TVector2<RealType> RealGridPoint = ToRealGrid(Point);
		return (RealGridPoint.X >= static_cast<RealType>(std::numeric_limits<int32>::min())) &&
			(RealGridPoint.X <= static_cast<RealType>(std::numeric_limits<int32>::max())) &&
			(RealGridPoint.Y >= static_cast<RealType>(std::numeric_limits<int32>::min())) &&
			(RealGridPoint.Y <= static_cast<RealType>(std::numeric_limits<int32>::max()));
	}
};
typedef TShiftGridIndexer2<float> FShiftGridIndexer2f;
typedef TShiftGridIndexer2<double> FShiftGridIndexer2d;


} // end namespace UE::Geometry
} // end namespace UE
