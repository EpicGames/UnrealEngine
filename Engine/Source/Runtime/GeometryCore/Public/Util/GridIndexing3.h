// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp GridIndexing3

#pragma once

#include "VectorTypes.h"
#include "HAL/Platform.h" // int32
#include "IntVectorTypes.h"
#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

/**
 * Convert between integer grid coordinates and scaled real-valued coordinates (ie assumes integer grid origin == real origin)
 */
template<typename RealType>
struct TScaleGridIndexer3
{
	/** Real-valued size of an integer grid cell */
	RealType CellSize;

	TScaleGridIndexer3() : CellSize((RealType)1) 
	{
	}
	
	TScaleGridIndexer3(RealType CellSize) : CellSize(CellSize)
	{
		// Note a very small cell size is likely to overflow the integer grid coordinates
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	/** Convert real-valued point to integer grid coordinates */
	inline FVector3i ToGrid(const TVector<RealType>& P) const
	{
		return FVector3i(
			(int)TMathUtil<RealType>::Floor(P.X / CellSize),
			(int)TMathUtil<RealType>::Floor(P.Y / CellSize),
			(int)TMathUtil<RealType>::Floor(P.Z / CellSize));
	}

	/** Convert integer grid coordinates to real-valued point */
	inline TVector<RealType> FromGrid(const FVector3i& GridPoint) const
	{
		return TVector<RealType>(GridPoint.X*CellSize, GridPoint.Y*CellSize, GridPoint.Z*CellSize);
	}
};
typedef TScaleGridIndexer3<float> FScaleGridIndexer3f;
typedef TScaleGridIndexer3<double> FScaleGridIndexer3d;

/**
 * Converts real-valued coordinates to integer grid coordinates, wrapping around outside 
 *  the representable integer range.
 * 
 * Lacks a conversion back from the integer vector since it maps to multiple ranges, but
 *  this is frequently not needed. Also requires use of a function to iterate over a
 *  range, to ensure proper overflow.
 */
template<typename RealType>
struct TWrapAroundGridIndexer3
{
	/** Real-valued size of an integer grid cell */
	RealType CellSize;

	TWrapAroundGridIndexer3() : CellSize((RealType)1)
	{
	}

	TWrapAroundGridIndexer3(RealType CellSize) : CellSize(CellSize)
	{
		// Note a very small cell size could overflow division by cell size
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	/** 
	 * Convert real-valued point to integer grid coordinates, wrapping around when
	 *  P/CellSize falls outside the representable integer range.
	 */
	inline FVector3i ToGrid(const TVector<RealType>& P) const
	{
		// The conversion below always goes through double. It could be done in RealType, but that is actually messy
		//  and error-prone because for floats, we would not want to do the WraparoundDistance addition below, as it
		//  loses precision unnecessarily for small negative numbers. So we would have to do more work to get the 
		//  same kind of single-direction rounding and wraparound behavior, and it is not worth the bother (note if
		//  you do choose to switch: WraparoundDistance initialization would need to change too, using std::nextafter
		//  if we happen to round up)

		static const double WraparoundDistance = static_cast<double>(TNumericLimits<uint32>::Max()) + 1;
		auto GetInt = [](RealType Real) -> int32
		{
			double ModResult = FMath::Fmod(static_cast<double>(Real), WraparoundDistance);
			ModResult = ModResult < 0 ? ModResult + WraparoundDistance : ModResult;
			uint32 Truncated = static_cast<uint32>(ModResult);

			// Seems that even with 2's complement being required in C++20, static_cast might theoretically still be
			//  undefined outside representable range (?), hence the reinterpret here
			return reinterpret_cast<int32&>(Truncated);
		};

		return FVector3i(
			GetInt(P.X / CellSize),
			GetInt(P.Y / CellSize),
			GetInt(P.Z / CellSize));
	}

	/**
	 * Iterate across a range of indices going from the one that corresponds to RealMean to one that corresponds
	 *  to RealMax. Iteration stops early if the passed function returns false.
	 */
	void IterateAcrossBounds(const TVector<RealType>& RealMin, const TVector<RealType>& RealMax, 
		TFunctionRef<bool(const FVector3i& Index)> ShouldContinue) const
	{
		FVector3i IntMin = ToGrid(RealMin);
		FVector3i ExclusiveIntMax = ToGrid(RealMax) + FVector3i::One();

		// Technically signed int overflow is undefined, so we use unsigned int to overflow safely.
		//  Overflow is also the reason we have to use != for the comparison. 
		for (uint32 zi = static_cast<uint32>(IntMin.Z); zi != static_cast<uint32>(ExclusiveIntMax.Z); ++zi)
		{
			for (uint32 yi = static_cast<uint32>(IntMin.Y); yi != static_cast<uint32>(ExclusiveIntMax.Y); ++yi)
			{
				for (uint32 xi = static_cast<uint32>(IntMin.X); xi != static_cast<uint32>(ExclusiveIntMax.X); ++xi)
				{
					// Seems that even with 2's complement being required in C++20, static_cast might theoretically still be
					//  undefined outside representable range (?), hence the reinterpret here
					if (!ShouldContinue(FVector3i(reinterpret_cast<int32&>(xi), reinterpret_cast<int32&>(yi), reinterpret_cast<int32&>(zi))))
					{
						return;
					}
				}
			}
		}
	}
};

typedef TWrapAroundGridIndexer3<float> FWrapAroundGridIndexer3f;
typedef TWrapAroundGridIndexer3<double> FWrapAroundGridIndexer3d;

/**
 * Convert between integer grid coordinates and scaled+translated real-valued coordinates
 */
template<typename RealType>
struct TShiftGridIndexer3
{
	/** Real-valued size of an integer grid cell */
	RealType CellSize;
	/** Real-valued origin of grid, position of integer grid origin */
	TVector<RealType> Origin;

	TShiftGridIndexer3() 
		: CellSize((RealType)1), Origin(TVector<RealType>::Zero())
	{
	}

	TShiftGridIndexer3(const TVector<RealType>& origin, RealType cellSize)
		: CellSize(cellSize), Origin(origin)
	{
		ensure(CellSize >= TMathUtil<RealType>::ZeroTolerance);
	}

	/** Convert real-valued point to integer grid coordinates */
	inline FVector3i ToGrid(const TVector<RealType>& point) const
	{
		return FVector3i(
			(int)TMathUtil<RealType>::Floor((point.X - Origin.X) / CellSize),
			(int)TMathUtil<RealType>::Floor((point.Y - Origin.Y) / CellSize),
			(int)TMathUtil<RealType>::Floor((point.Z - Origin.Z) / CellSize));
	}

	/** Convert integer grid coordinates to real-valued point */
	inline TVector<RealType> FromGrid(const FVector3i& gridpoint) const
	{
		return TVector<RealType>(
			((RealType)gridpoint.X * CellSize) + Origin.X,
			((RealType)gridpoint.Y * CellSize) + Origin.Y,
			((RealType)gridpoint.Z * CellSize) + Origin.Z);
	}

	/** Convert real-valued grid coordinates to real-valued point */
	inline TVector<RealType> FromGrid(const TVector<RealType>& RealGridPoint)  const
	{
		return TVector<RealType>(
			((RealType)RealGridPoint.X * CellSize) + Origin.X,
			((RealType)RealGridPoint.Y * CellSize) + Origin.Y,
			((RealType)RealGridPoint.Z * CellSize) + Origin.Z);
	}
};
typedef TShiftGridIndexer3<float> FShiftGridIndexer3f;
typedef TShiftGridIndexer3<double> FShiftGridIndexer3d;


template<typename RealType>
struct TBoundsGridIndexer3 : public TShiftGridIndexer3<RealType>
{
	using TShiftGridIndexer3<RealType>::CellSize;
	using TShiftGridIndexer3<RealType>::Origin;

	TVector<RealType> BoundsMax;

	TBoundsGridIndexer3(const TAxisAlignedBox3<RealType>& Bounds, RealType CellSize)
		: TShiftGridIndexer3<RealType>(Bounds.Min, CellSize),
		BoundsMax(Bounds.Max)
	{}

	FVector3i GridResolution() const
	{
		const RealType InvCellSize = 1.0 / CellSize;
		const TVector<RealType> Extents = BoundsMax - Origin;
		return CeilInt(Extents * InvCellSize);
	}

	static FVector3i CeilInt(const TVector<RealType>& V)
	{
		return FVector3i{ (int)TMathUtil<RealType>::Ceil(V[0]),
			(int)TMathUtil<RealType>::Ceil(V[1]),
			(int)TMathUtil<RealType>::Ceil(V[2]) };
	}

};

typedef TBoundsGridIndexer3<float> FBoundsGridIndexer3f;
typedef TBoundsGridIndexer3<double> FBoundsGridIndexer3d;

} // end namespace UE::Geometry
} // end namespace UE