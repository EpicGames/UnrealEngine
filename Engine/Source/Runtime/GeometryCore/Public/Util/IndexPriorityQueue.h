// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "Util/DynamicVector.h"

namespace UE
{
namespace Geometry
{

/**
 * This is a min-heap priority queue class that does not use an object for each queue node.
 * Integer IDs must be provided by the user to identify unique nodes.
 * Internally an array is used to keep track of the mapping from ids to internal indices, so the max ID must also be provided.
 *
 * @todo based on C# code where TDynamicVector could not return a reference to internal element. In C++ we can, and code could be updated to be more efficient in many places.
 * @todo id_to_index could be sparse in many situations...
 */
class FIndexPriorityQueue
{
public:
	/** set this to true during development to catch issues */
	bool EnableDebugChecks = false;

	struct FQueueNode
	{
		int id = -1;         // external id

		float priority;      // the priority of this id
		int index;           // index in tree data structure (tree is stored as flat array)
	};

	/** tree of allocated nodes, stored linearly. active up to num_nodes (allocated may be larger) */
	TDynamicVector<FQueueNode> nodes;
	/** count of active nodes */
	int num_nodes;
	/** mapping from external ids to internal node indices */
	TArray<int> id_to_index;


	/**
	 * This constructor is provided for convenience, you must call Initialize()
	 */
	FIndexPriorityQueue()
	{
		Initialize(0);
	}

	/**
	 * Calls Initialize()
	 * @param maxID maximum external ID that will be passed to any public functions
	 */
	FIndexPriorityQueue(int maxID)
	{
		Initialize(maxID);
	}


	/**
	 * Initialize internal data structures. Internally a fixed-size array is used to track mapping
	 * from IDs to internal node indices, so maxID must be provided up-front.
	 * If this seems problematic or inefficient, this is not the Priority Queue for you.
	 * @param MaxNodeID maximum external ID that will be passed to any public functions
	 */
	void Initialize(int MaxNodeID)
	{
		nodes.Clear();
		id_to_index.Init(-1, MaxNodeID);
		num_nodes = 0;
	}


	/**
	 * Reset the queue to empty state. 
	 * if bFreeMemory is false, we don't discard internal data structures, so there will be less allocation next time
	 */
	void Clear(bool bFreeMemory = true)
	{
		check(bFreeMemory);  // not sure exactly how to implement bFreeMemory=false w/ classes below...
		nodes.Clear();
		id_to_index.Init(-1, id_to_index.Num());
		num_nodes = 0;
	}

	/** @return current size of queue */
	int GetCount() const
	{
		return num_nodes;
	}

	/** @return id of node at head of queue */
	int GetFirstNodeID() const
	{
		return nodes[1].id;
	}

	/** @return  id of node at head of queue */
	float GetFirstNodePriority() const
	{
		return nodes[1].priority;
	}

	/** @return true if id is already in queue */
	bool Contains(int NodeID) const
	{
		if (NodeID > id_to_index.Num() + 1)
			return false;

		int NodeIndex = id_to_index[NodeID];
		if (NodeIndex <= 0 || NodeIndex > num_nodes)
			return false;
		return nodes[NodeIndex].index > 0;
	}


	/**
	 * Add id to list w/ given priority. Do not call with same id twice!
	 */
	void Insert(int NodeID, float priority)
	{
		if (EnableDebugChecks)
		{
			check(Contains(NodeID) == false);
		}

		FQueueNode node;
		node.id = NodeID;
		node.priority = priority;
		num_nodes++;
		node.index = num_nodes;
		id_to_index[NodeID] = node.index;
		nodes.InsertAt(node, num_nodes);
		move_up(nodes[num_nodes].index);
	}

	/**
	 * Remove node at head of queue, update queue, and return id for that node
	 * @return ID of node at head of queue
	 */
	int Dequeue()
	{
		if (EnableDebugChecks)
		{
			check(GetCount() > 0);
		}

		int NodeID = nodes[1].id;
		remove_at_index(1);
		return NodeID;
	}

	/**
	 * Remove node associated with given ID from queue. Behavior is undefined if you call w/ id that is not in queue
	 */
	void Remove(int NodeID)
	{
		if (EnableDebugChecks)
		{
			check(Contains(NodeID) == true);
		}

		int NodeIndex = id_to_index[NodeID];
		remove_at_index(NodeIndex);
	}


	/**
	 * Update priority at node id, and then move it to correct position in queue. Behavior is undefined if you call w/ id that is not in queue
	 */
	void Update(int NodeID, float Priority)
	{
		if (EnableDebugChecks)
		{
			check(Contains(NodeID) == true);
		}

		int NodeIndex = id_to_index[NodeID];

		FQueueNode n = nodes[NodeIndex];
		n.priority = Priority;
		nodes[NodeIndex] = n;

		on_node_updated(NodeIndex);
	}


	/**
	 * Query the priority at node id, assuming it exists in queue. Behavior is undefined if you call w/ id that is not in queue
	 * @return priority for node
	 */
	float GetPriority(int id)
	{
		if (EnableDebugChecks)
		{
			check(Contains(id) == true);
		}

		int iNode = id_to_index[id];
		return nodes[iNode].priority;
	}


	/*
	 * Internals
	 */
private:

	/** remove node at index and update tree */
	void remove_at_index(int NodeIndex)
	{
		// node-is-last-node case
		if (NodeIndex == num_nodes)
		{

			// null id_to_index entry for the node we remove.
			int id = nodes[num_nodes].id;
			id_to_index[id] = -1;

			nodes[num_nodes] = FQueueNode();
			num_nodes--;
			return;
		}

		// TODO: is there a better way to do this? seems random to move the last node to
		// top of tree? But I guess otherwise we might have to shift entire branches??

		//Swap the node with the last node
		swap_nodes_at_indices(NodeIndex, num_nodes);
		// after swap, NodeIndex is the one we want to keep, and numNodes is the one we discard
		{
			int id = nodes[num_nodes].id;
			id_to_index[id] = -1;

			nodes[num_nodes] = FQueueNode();
			num_nodes--;
		}

		//Now shift iNode (ie the former last node) up or down as appropriate
		on_node_updated(NodeIndex);
	}


	/** swap two nodes in the true */
	void swap_nodes_at_indices(int i1, int i2)
	{
		FQueueNode n1 = nodes[i1];
		n1.index = i2;
		FQueueNode n2 = nodes[i2];
		n2.index = i1;
		nodes[i1] = n2;
		nodes[i2] = n1;

		id_to_index[n2.id] = i1;
		id_to_index[n1.id] = i2;
	}

	/** move node at iFrom to iTo */
	void move(int iFrom, int iTo)
	{
		FQueueNode n = nodes[iFrom];
		n.index = iTo;
		nodes[iTo] = n;
		id_to_index[n.id] = iTo;
	}

	/** set node at iTo */
	void set(int iTo, FQueueNode& n)
	{
		n.index = iTo;
		nodes[iTo] = n;
		id_to_index[n.id] = iTo;
	}


	/** move iNode up tree to correct position by iteratively swapping w/ parent */
	void move_up(int iNode)
	{
		// save start node, we will move this one to correct position in tree
		int iStart = iNode;
		FQueueNode iStartNode = nodes[iStart];

		// while our priority is lower than parent, we swap upwards, ie move parent down
		int iParent = iNode / 2;
		while (iParent >= 1) 
		{
			if (nodes[iParent].priority < iStartNode.priority)
			{
				break;
			}
			move(iParent, iNode);  
			iNode = iParent;
			iParent = nodes[iNode].index / 2;
		}

		// write input node into final position, if we moved it
		if (iNode != iStart) 
		{
			set(iNode, iStartNode);
		}
	}


	/** move iNode down tree branches to correct position, by iteratively swapping w/ children */
	void move_down(int iNode)
	{
		// save start node, we will move this one to correct position in tree
		int iStart = iNode;
		FQueueNode iStartNode = nodes[iStart];

		// keep moving down until lower nodes have higher priority
		while (true) 
		{
			int iMoveTo = iNode;
			int iLeftChild = 2 * iNode;

			// past end of tree, must be in the right spot
			if (iLeftChild > num_nodes) 
			{
				break;
			}

			// check if priority is larger than either child - if so we want to swap
			float min_priority = iStartNode.priority;
			float left_child_priority = nodes[iLeftChild].priority;
			if (left_child_priority < min_priority) 
			{
				iMoveTo = iLeftChild;
				min_priority = left_child_priority;
			}
			int iRightChild = iLeftChild + 1;
			if (iRightChild <= num_nodes) 
			{
				if (nodes[iRightChild].priority < min_priority) 
				{
					iMoveTo = iRightChild;
				}
			}

			// if we found node with higher priority, swap with it (ie move it up) and take its place
			// (but we only write start node to final position, not intermediary slots)
			if (iMoveTo != iNode) 
			{
				move(iMoveTo, iNode);
				iNode = iMoveTo;
			}
			else 
			{
				break;
			}
		}

		// if we moved node, write it to its new position
		if (iNode != iStart) 
		{
			set(iNode, iStartNode);
		}
	}


	/** call this after node is modified, to move it to correct position in queue */
	void on_node_updated(int iNode) 
	{
		int iParent = iNode / 2;
		if (iParent > 0 && has_higher_priority(iNode, iParent))
			move_up(iNode);
		else
			move_down(iNode);
	}


	/** @return true if priority at iHigher is less than at iLower */
	bool has_higher_priority(int iHigher, int iLower) const
	{
		return (nodes[iHigher].priority < nodes[iLower].priority);
	}




public:
	/**
	 * Check if node ordering is correct (for debugging/testing)
	 */
	bool IsValidQueue() const
	{
		for (int i = 1; i < num_nodes; i++) {
			int childLeftIndex = 2 * i;
			if (childLeftIndex < num_nodes && has_higher_priority(childLeftIndex, i))
				return false;

			int childRightIndex = childLeftIndex + 1;
			if (childRightIndex < num_nodes && has_higher_priority(childRightIndex, i))
				return false;
		}
		return true;
	}

	/**
	 * Check if node ordering is correct (for debugging/testing)
	 */
	bool CheckIds()
	{
		// check that every valid node is correctly found by id_to_index
		bool bValidNodesInSycn = true;
		for (int i = 1; i < num_nodes + 1; ++i)
		{
			const FQueueNode& n = nodes[i];
			int id = n.id;
			int index = n.index;

			bValidNodesInSycn = bValidNodesInSycn && (index == id_to_index[id]);
			bValidNodesInSycn = bValidNodesInSycn && (index == i);		
		}

		// check that id_to_index doesn't believe that none valid nodes exist
		bool bNoOutOfSyncNodes = true;
		for (int id = 0; id < id_to_index.Num(); ++id)
		{
			
			if (Contains(id))
			{
				int index = id_to_index[id];
				const FQueueNode& n = nodes[index];
				bNoOutOfSyncNodes = bNoOutOfSyncNodes && (n.id == id);
			}
		}

		return bValidNodesInSycn && bNoOutOfSyncNodes;
	}


	//void DebugPrint() {
	//	for (int i = 1; i <= num_nodes; ++i)
	//		WriteLine("{0} : p {1}  index {2}  id {3}", i, nodes[i].priority, nodes[i].index, nodes[i].id);
	//}


};


} // end namespace UE::Geometry
} // end namespace UE