// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/IndirectArray.h"
#include "VectorTypes.h"


namespace UE
{
namespace Geometry
{


/**
 * FArrayIndexSetsDecomposition represents a decomposition of an integer index set
 * into subsets, which are stored as arrays (ie this is just an array of arrays,
 * where each sub-array has a unique integer identifier)
 */
class FArrayIndexSetsDecomposition
{
protected:

	struct FIndexSet
	{
		int32 Identifier;
		TArray<int32> IndexArray;
		// todo alternate storage: set, arbitrary function
	};

	TIndirectArray<FIndexSet> Sets;
	TMap<int32, FIndexSet*> SetIdentifierMap;
	int32 IDCounter = 1;

	TArray<int32> AllSetIDs;

public:

	/*
	 * @return list of ID integers which identify the IndexSets of this Decomposition
	 */
	const TArray<int32>& GetIndexSetIDs() const
	{
		return AllSetIDs;
	}

	/**
	 * Create a new IndexSet for this Decomposition
	 * @return ID of the new IndexSet
	 */
	int32 CreateNewIndexSet()
	{
		FIndexSet* Set = new FIndexSet();
		Set->Identifier = IDCounter++;
		Sets.Add(Set);
		SetIdentifierMap.Add(Set->Identifier, Set);
		AllSetIDs.Add(Set->Identifier);
		return Set->Identifier;
	}

	/**
	 * @return IndexSet for the given SetID
	 */
	TArray<int32>& GetIndexSetArray(int32 SetID)
	{
		FIndexSet** Found = SetIdentifierMap.Find(SetID);
		check(Found != nullptr);
		return (*Found)->IndexArray;
	}

	/**
	 * @return IndexSet for the given SetID
	 */
	const TArray<int32>& GetIndexSetArray(int32 SetID) const
	{
 		FIndexSet*const* Found = SetIdentifierMap.Find(SetID);
		check(Found != nullptr);
		return (*Found)->IndexArray;
	}

};

} // end namespace UE::Geometry
} // end namespace UE
