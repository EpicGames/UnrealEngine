// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp index_util.h

#pragma once

#include "Containers/Array.h"
#include "GeometryBase.h"
#include "IndexTypes.h"
#include "IntVectorTypes.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Util/DynamicVector.h"
#include "VectorTypes.h"


namespace IndexUtil
{
	using namespace UE::Geometry;
	using UE::Geometry::FIndex2i;

	/**
	 * @return true if  [a0,a1] and [b0,b1] are the same pair, ignoring order
	 */
	template<typename T>
	inline bool SamePairUnordered(T a0, T a1, T b0, T b1)
	{
		return (a0 == b0) ?
			(a1 == b1) :
			(a0 == b1 && a1 == b0);
	}


	/**
	 * @return the vertex that is the same in both EdgeVerts1 and EdgeVerts2, or InvalidID if not found
	 */
	inline int FindSharedEdgeVertex(const FIndex2i & EdgeVerts1, const FIndex2i & EdgeVerts2)
	{
		if (EdgeVerts1.A == EdgeVerts2.A)             return EdgeVerts1.A;
		else if (EdgeVerts1.A == EdgeVerts2.B)        return EdgeVerts1.A;
		else if (EdgeVerts1.B == EdgeVerts2.A)        return EdgeVerts1.B;
		else if (EdgeVerts1.B == EdgeVerts2.B)        return EdgeVerts1.B;
		else return IndexConstants::InvalidID;
	}

	/**
	 * @return the vertex in the pair ev that is not v, or InvalidID if not found
	 */
	inline int FindEdgeOtherVertex(const FIndex2i & EdgeVerts, int VertexID)
	{
		if (EdgeVerts.A == VertexID)		return EdgeVerts.B;
		else if (EdgeVerts.B == VertexID)	return EdgeVerts.A;
		else								return IndexConstants::InvalidID;
	}


	/**
	 * @return index 0/1/2 of VertexID in TriangleVerts, or InvalidID if not found
	 */
	template<typename T, typename Vec>
	inline int FindTriIndex(T VertexID, const Vec & TriangleVerts)
	{
		if (TriangleVerts[0] == VertexID) return 0;
		if (TriangleVerts[1] == VertexID) return 1;
		if (TriangleVerts[2] == VertexID) return 2;
		return IndexConstants::InvalidID;
	}

	/**
	 * Find unordered edge [VertexID1,VertexID2] in TriangleVerts
	 * @return index in range 0-2, or InvalidID if not found
	 */
	template<typename T, typename Vec>
	inline int FindEdgeIndexInTri(T VertexID1, T VertexID2, const Vec & TriangleVerts)
	{
		if (SamePairUnordered(VertexID1, VertexID2, TriangleVerts[0], TriangleVerts[1])) return 0;
		if (SamePairUnordered(VertexID1, VertexID2, TriangleVerts[1], TriangleVerts[2])) return 1;
		if (SamePairUnordered(VertexID1, VertexID2, TriangleVerts[2], TriangleVerts[0])) return 2;
		return IndexConstants::InvalidID;
	}


	/**
	 * Find ordered edge [VertexID1,VertexID2] in TriangleVerts
	 * @return index in range 0-2, or InvalidID if not found
	 */
	template<typename T, typename Vec>
	inline int FindTriOrderedEdge(T VertexID1, T VertexID2, const Vec & TriangleVerts)
	{
		if (TriangleVerts[0] == VertexID1 && TriangleVerts[1] == VertexID2)	return 0;
		if (TriangleVerts[1] == VertexID1 && TriangleVerts[2] == VertexID2)	return 1;
		if (TriangleVerts[2] == VertexID1 && TriangleVerts[0] == VertexID2)	return 2;
		return IndexConstants::InvalidID;
	}

	/**
	 * Find third vertex of triangle that is not VertexID1 or VertexID2. Triangle must contain VertexID1 and VertexID2 or result will be incorrect.
	 * @return VertexID of third vertex, or incorrect result if precondition is not met
	 */
	template<typename T, typename Vec>
	inline int FindTriOtherVtxUnsafe(T VertexID1, T VertexID2, const Vec& TriangleVerts)
	{
		if (TriangleVerts[0] == VertexID1)
		{
			return (TriangleVerts[1] == VertexID2) ? TriangleVerts[2] : TriangleVerts[1];
		}
		else if (TriangleVerts[1] == VertexID1)
		{
			return (TriangleVerts[0] == VertexID2) ? TriangleVerts[2] : TriangleVerts[0];
		}
		else
		{
			return (TriangleVerts[0] == VertexID2) ? TriangleVerts[1] : TriangleVerts[0];
		}
	}


	/**
	 * Find ordered edge [VertexID1,VertexID2] in TriangleVerts and then return the remaining third vertex
	 * @return vertex id of other vertex, or InvalidID if not found
	 */
	template<typename T, typename Vec>
	inline int FindTriOtherVtx(T VertexID1, T VertexID2, const Vec & TriangleVerts)
	{
		for (int j = 0; j < 3; ++j) 
		{
			if (SamePairUnordered(VertexID1, VertexID2, TriangleVerts[j], TriangleVerts[(j + 1) % 3]))
			{
				return TriangleVerts[(j + 2) % 3];
			}
		}
		return IndexConstants::InvalidID;
	}

	/**
	 * Find ordered edge [VertexID1,VertexID2] in a triangle that is in an array of triangles, and return remaining third vertex
	 * @param VertexID1 first vertex of edge
	 * @param VertexID2 second vertex of edge
	 * @param TriIndexArray array of triangle tuples
	 * @param TriangleIndex which triangle in array to search
	 * @return vertex id of other vertex, or InvalidID if not found
	 */
	inline int FindTriOtherVtx(int VertexID1, int VertexID2, const TDynamicVector<FIndex3i> & TriIndexArray, int TriangleIndex)
	{
		const FIndex3i& Triangle = TriIndexArray[TriangleIndex];
		for (int j = 0; j < 3; ++j)
		{
			if (SamePairUnordered(VertexID1, VertexID2, Triangle[j], Triangle[(j+1)%3]))
			{
				return Triangle[((j + 2) % 3)];
			}
		}
		return IndexConstants::InvalidID;
	}



	/**
	 * Find ordered edge [VertexID1,VertexID2] in TriangleVerts and then return the index of the remaining third vertex
	 * @return index of third vertex, or InvalidID if not found
	 */
	template<typename T, typename Vec>
	inline int FindTriOtherIndex(T VertexID1, T VertexID2, const Vec & TriangleVerts)
	{
		for (int j = 0; j < 3; ++j) 
		{
			if (SamePairUnordered(VertexID1, VertexID2, TriangleVerts[j], TriangleVerts[(j + 1) % 3]))
			{
				return (j + 2) % 3;
			}
		}
		return IndexConstants::InvalidID;
	}


	/**
	 * If i0 and i1 are unordered indices into a triangle, each in range 0-2, return the third index
	 */
	inline int GetOtherTriIndex(int i0, int i1)
	{
		// @todo can we do this with a formula? I don't see it right now.
		static const int values[4] = { 0, 2, 1, 0 };
		return values[i0+i1];
	}


	/**
	 * Assuming [Vertex1,Vertex2] is an unordered edge in TriangleVerts, return Vertex1 and Vertex2 in the correct order (ie the same as TriangleVerts)
	 * @warning result is garbage if either vertex is not an edge in TriangleVerts
	 * @return true if order was swapped
	 */
	template<typename T, typename Vec>
	inline bool OrientTriEdge(T & Vertex1, T & Vertex2, const Vec & TriangleVerts)
	{
		if (Vertex1 == TriangleVerts[0]) 
		{
			if (TriangleVerts[2] == Vertex2) 
			{
				T Temp = Vertex1; Vertex1 = Vertex2; Vertex2 = Temp;
				return true;
			}
		}
		else if (Vertex1 == TriangleVerts[1]) 
		{
			if (TriangleVerts[0] == Vertex2) 
			{
				T Temp = Vertex1; Vertex1 = Vertex2; Vertex2 = Temp;
				return true;
			}
		}
		else if (Vertex1 == TriangleVerts[2]) 
		{
			if (TriangleVerts[1] == Vertex2) 
			{
				T Temp = Vertex1; Vertex1 = Vertex2; Vertex2 = Temp;
				return true;
			}
		}
		return false;
	}


	/**
	 * Assuming [Vertex1,Vertex2] is an unordered edge in TriangleVerts, return Vertex1 and Vertex2 in the correct order (ie the same as TriangleVerts),
	 * and returns the vertex ID of the other vertex
	 * @return ID of third vertex, or InvalidID if the edge was not found
	 */
	template<typename T, typename Vec>
	inline int OrientTriEdgeAndFindOtherVtx(T & Vertex1, T & Vertex2, const Vec & TriangleVerts)
	{
		for (int j = 0; j < 3; ++j) 
		{
			if (SamePairUnordered(Vertex1, Vertex2, TriangleVerts[j], TriangleVerts[(j + 1) % 3]))
			{
				Vertex1 = TriangleVerts[j];
				Vertex2 = TriangleVerts[(j + 1) % 3];
				return TriangleVerts[(j + 2) % 3];
			}
		}
		return IndexConstants::InvalidID;
	}


	/**
	 * Replace Val with MapFunc[Val] using index operator
	 */
	template<typename Func>
	void ApplyMap(FIndex3i & Val, Func MapFunc)
	{
		Val[0] = MapFunc[Val[0]];
		Val[1] = MapFunc[Val[1]];
		Val[2] = MapFunc[Val[2]];
	}

	/**
	 * Replace Val with MapFunc[Val] using index operator
	 */
	template<typename T, typename Func>
	void ApplyMap(UE::Math::TVector<T> & Val, Func MapFunc)
	{
		Val[0] = MapFunc[Val[0]];
		Val[1] = MapFunc[Val[1]];
		Val[2] = MapFunc[Val[2]];
	}


	/**
	 * @return MapFunc[Val] using index operator
	 */
	template<typename Func>
	FIndex3i ApplyMap(const FIndex3i & Val, Func MapFunc)
	{
		return FIndex3i(MapFunc[Val[0]], MapFunc[Val[1]], MapFunc[Val[2]]);
	}


	/**
	 * @return MapFunc[Val] using index operator
	 */
	template<typename T, typename Func>
	UE::Math::TVector<T> ApplyMap(const UE::Math::TVector<T> & Val, Func MapFunc)
	{
		return UE::Math::TVector<T>(MapFunc[Val[0]], MapFunc[Val[1]], MapFunc[Val[2]]);
	}
	
	/**
	 * @return false if CheckFn returns false on any element of ToCheck, true otherwise
	 */
	template<typename T, typename Func>
	bool ArrayCheck(const TArray<T>& ToCheck, Func CheckFn)
	{
		for (const T& Value : ToCheck)
		{
			if (!CheckFn(Value))
			{
				return false;
			}
		}
		return true;
	}

	/**
	 * integer indices offsets in x/y directions
	 */
	extern GEOMETRYCORE_API const FVector2i GridOffsets4[4];

	/**
	 * integer indices offsets in x/y directions and diagonals
	 */
	extern GEOMETRYCORE_API const FVector2i GridOffsets8[8];

	/**
	 * integer indices offsets in x/y/z directions, corresponds w/ BoxFaces directions
	 */
	extern GEOMETRYCORE_API const FVector3i GridOffsets6[6];

	/**
	 * all permutations of (+-1, +-1, +-1), can be used to iterate over connected face/edge/corner neighbours of a grid cell
	 */
	extern GEOMETRYCORE_API const FVector3i GridOffsets26[26];

	/**
	 * Corner vertices of box faces  -  see FOrientedBox3.GetCorner for points associated w/ indexing
	 */
	extern GEOMETRYCORE_API const int BoxFaces[6][4];

	/**
	 * Corner unit-UV ordering of box faces - {0,0}, {1,0}, {1,1}, {0,1}
	 */
	extern GEOMETRYCORE_API const FVector2i BoxFacesUV[4];

	/**
	 * Box Face Normal Axes associated with BoxFaces. Use Sign(BoxFaceNormals[i]) * Box.Axis( Abs(BoxFaceNormals[i])-1 ) to get vector
	 */
	extern GEOMETRYCORE_API const int BoxFaceNormals[6];


	// @todo other index array constants
}
