// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp iterator_util.h

#pragma once
#include "IndexTypes.h"
#include "Templates/Function.h"

namespace UE
{
namespace Geometry
{


/**
 * Wrapper around an object of type IteratorT that provides STL
 * iterator-like semantics, that converts from the iteration type
 * (FromType) to a new type (ToType).
 *
 * Conversion is done via a provided mapping function
 */
template<typename FromType, typename ToType, typename IteratorT>
class MappedIterator
{
	using MapFunctionT = TFunction<ToType(FromType)>;

public:
	inline MappedIterator() { }

	inline bool operator==(const MappedIterator& Other) const
	{
		return Cur == Other.Cur;
	}
	inline bool operator!=(const MappedIterator& Other) const 
	{
		return Cur != Other.Cur;
	}

	inline ToType operator*() const 
	{
		return MapFunction(*Cur);
	}

	inline const MappedIterator& operator++() 		// prefix
	{
		Cur++;
		return *this;
	}

	inline MappedIterator(const IteratorT& CurItr, const MapFunctionT& MapFunctionIn)
	{
		Cur = CurItr;
		MapFunction = MapFunctionIn;
	}

	IteratorT Cur;
	MapFunctionT MapFunction;
};







/**
 * Wrapper around an existing iterator that skips over
 * values for which the filter_func returns false.
 */
template<typename ValueType, typename IteratorT>
class FilteredIterator
{
	using FilterFunctionT = TFunction<bool(ValueType)>;

public:
	inline FilteredIterator() { }

	inline bool operator==(const FilteredIterator& Other) const 
	{
		return Cur == Other.Cur;
	}
	inline bool operator!=(const FilteredIterator& Other) const
	{
		return Cur != Other.Cur;
	}

	inline ValueType operator*() const 
	{
		return *Cur;
	}

	inline const FilteredIterator& operator++() 		// prefix
	{
		GotoNextElement();
		return *this;
	}

	inline void GotoNextElement() 
	{
		do {
			Cur++;
		} while (Cur != End && FilterFunc(*Cur) == false);
	}

	inline FilteredIterator(const IteratorT& CurItr, const IteratorT& EndItr, const FilterFunctionT& FilterFuncIn)
	{
		Cur = CurItr;
		End = EndItr;
		this->FilterFunc = FilterFuncIn;
		if (Cur != End && FilterFunc(*Cur) == false)
		{
			GotoNextElement();
		}
	}

	IteratorT Cur;
	IteratorT End;
	FilterFunctionT FilterFunc;
};










/**
 * Wrapper around existing iterator that returns multiple values, of potentially
 * different type, for each value that input iterator returns.
 *
 * This is done via an "expansion" function that takes an int reference which
 * indicates "where" we are in the expansion (eg like a state machine).
 * How you use this value is up to you.
 *
 * When the input is -1, you should interpret this as the "beginning" of
 * handling the input value (ie we have not returned any values yet for
 * this input value)
 *
 * When you are "done" with an input value, set the outgoing int reference to -1
 * and the base iterator will be incremented.
 *
 * If you have more values to return for this input value, set it to some positive
 * number of your choosing.
 *
 * See FDynamicMesh3::VtxTrianglesItr for an example
 */
template<typename OutputType, typename InputType, typename InputIteratorT>
class ExpandIterator
{
	using ExpandFunctionT = TFunction<OutputType(InputType, int&)>;

public:
	inline ExpandIterator() { }

	inline bool operator==(const ExpandIterator& Other) const 
	{
		return Cur == Other.Cur;
	}
	inline bool operator!=(const ExpandIterator & Other) const 
	{
		return Cur != Other.Cur;
	}

	inline OutputType operator*() const 
	{
		return CurValue;
	}

	inline const ExpandIterator& operator++() 		// prefix
	{
		goto_next();
		return *this;
	}

	inline void goto_next() 
	{
		while (Cur != End) 
		{
			CurValue = ExpandFunc(*Cur, CurExpandI);
			if (CurExpandI == -1)
			{
				++Cur;  // done with this base value
			}
			else
			{
				break; // want caller to see current output value
			}
		}
	}

	inline ExpandIterator(const InputIteratorT& CurItr, const InputIteratorT& EndItr, const ExpandFunctionT& ExpandFuncIn)
	{
		Cur = CurItr;
		End = EndItr;
		ExpandFunc = ExpandFuncIn;
		CurExpandI = -1;
		goto_next();
	}

	InputIteratorT Cur;
	InputIteratorT End;
	OutputType CurValue;
	int CurExpandI;
	ExpandFunctionT ExpandFunc;
};



/**
 * Generic "enumerable" object that provides begin/end semantics for an ExpandIterator suitable for use with range-based for.
 * You can either provide begin/end iterators, or another "enumerable" object that has begin()/end() functions.
 */
template<typename OutputType, typename InputType, typename InputIteratorT>
class ExpandEnumerable
{
	using ExpandFunctionT = TFunction<OutputType(InputType, int&)>;
	using ExpandIteratorT = ExpandIterator<OutputType, InputType, InputIteratorT>;

public:
	ExpandFunctionT ExpandFunc;
	InputIteratorT BeginItr, EndItr;

	ExpandEnumerable(const InputIteratorT& BeginIn, const InputIteratorT& EndIn, ExpandFunctionT ExpandFuncIn) 
	{
		this->BeginItr = BeginIn;
		this->EndItr = EndIn;
		this->ExpandFunc = ExpandFuncIn;
	}

	template<typename IteratorSource>
	ExpandEnumerable(const IteratorSource& Source, ExpandFunctionT ExpandFuncIn)
	{
		this->BeginItr = Source.begin();
		this->EndItr = Source.end();
		this->ExpandFunc = ExpandFuncIn;
	}

	ExpandIteratorT begin() 
	{
		return ExpandIteratorT(BeginItr, EndItr, ExpandFunc);
	}

	ExpandIteratorT end()
	{
		return ExpandIteratorT(EndItr, EndItr, ExpandFunc);
	}
};












/**
 * Wrapper around existing integer iterator that returns either 0, 1, or 2 integers
 * for each value that the original iterator returns.
 * 
 * This is specifically used by FDynamicMesh3::VtxTrianglesItr, where for each edge
 * around a vertex, between 0 and 2 triangles need to be returned. 
 * 
 * This is done via the PairExpandFunctionT TFunction, which returns a FIndex2i for
 * a given integer. This pair must be either (a,invalid), (a, b), or (invalid, invalid),
 * where invalid is integer < 0
 */
template<typename InputIteratorT>
class TPairExpandIterator
{
	using PairExpandFunctionT = TFunction<FIndex2i(int)>;

public:
	inline TPairExpandIterator() { }

	inline bool operator==(const TPairExpandIterator& Other) const
	{
		return Cur == Other.Cur;
	}
	inline bool operator!=(const TPairExpandIterator & Other) const
	{
		return Cur != Other.Cur;
	}

	inline int operator*() const
	{
		return CurValue;
	}

	inline const TPairExpandIterator& operator++() 		// prefix
	{
		goto_next();
		return *this;
	}

	inline void goto_next()
	{
		while (Cur != End)
		{
			if (CurPairI == 0)
			{
				CurPair = PairFunc(*Cur);
				if (CurPair.A >= 0)
				{
					CurValue = CurPair.A;
					CurPairI = 1;	// want to take second branch
					return;			// let caller see value
				}
				else
				{
					CurPairI = 0;
					++Cur;  // done with this base value
				}
			}
			else if (CurPairI == 1)
			{
				if (CurPair.B >= 0)
				{
					CurValue = CurPair.B;
					CurPairI = 2;	// want to take third branch
					return;		// let caller see value
				}
				else
				{
					CurPairI = 0;
					++Cur;  // done with this base value
				}
			}
			else
			{
				CurPairI = 0;
				++Cur;  // done with this base value
			}
		}
	}

	inline TPairExpandIterator(const InputIteratorT& CurItr, const InputIteratorT& EndItr, const PairExpandFunctionT& PairFuncIn)
	{
		Cur = CurItr;
		End = EndItr;
		PairFunc = PairFuncIn;
		CurPairI = 0;
		goto_next();
	}

	InputIteratorT Cur;
	InputIteratorT End;
	FIndex2i CurPair;
	int CurValue;
	int CurPairI;
	PairExpandFunctionT PairFunc;
};




/**
 * Generic "enumerable" object that provides begin/end semantics for an TPairExpandIterator suitable for use with range-based for.
 * You can either provide begin/end iterators, or another "enumerable" object that has begin()/end() functions.
 */
template<typename InputIteratorT>
class TPairExpandEnumerable
{
	using ExpandFunctionT = TFunction<FIndex2i(int)>;
	using ExpandIteratorT = TPairExpandIterator<InputIteratorT>;

public:
	ExpandFunctionT ExpandFunc;
	InputIteratorT BeginItr, EndItr;

	TPairExpandEnumerable(const InputIteratorT& BeginIn, const InputIteratorT& EndIn, ExpandFunctionT ExpandFuncIn)
	{
		this->BeginItr = BeginIn;
		this->EndItr = EndIn;
		this->ExpandFunc = ExpandFuncIn;
	}

	template<typename IteratorSource>
	TPairExpandEnumerable(const IteratorSource& Source, ExpandFunctionT ExpandFuncIn)
	{
		this->BeginItr = Source.begin();
		this->EndItr = Source.end();
		this->ExpandFunc = ExpandFuncIn;
	}

	ExpandIteratorT begin()
	{
		return ExpandIteratorT(BeginItr, EndItr, ExpandFunc);
	}

	ExpandIteratorT end()
	{
		return ExpandIteratorT(EndItr, EndItr, ExpandFunc);
	}
};



/**
 * FModuloIteration is used to iterate over a range of indices [0,N) using modulo-arithmetic.
 * The iteration proceeds as NextValue = (CurValue + ModuloValue) % N.
 * As long as the ModuloValue is a prime number > N/2, then every integer in the sequence 0...N-1
 * will appear exactly once before 0 re-appears (and in fact any index in the range can be used
 * as the starting value). 
 * 
 * FModuloIteration computes in 64-bit with (by default) a large enough prime that will work
 * for any 32-bit unsigned integer. If 64-bit iterations are needed, some larger primes can 
 * be found here: https://en.wikipedia.org/wiki/P%C3%A9pin%27s_test
 * 
 * (The prime does not strictly need to be > N/2, any prime will work as long as it is not a divisor of N. 
 *  And it doesn't even need to be a prime number, just a co-prime of N, ie GCD(N, ModuloValue) = 1. 
 *  It is possible to check GCD relatively quickly to search for valid constants, for example if
 *  many values were needed to use as seeds/etc)
 * 
 * Usage:
 * 
	FModuloIteration Iter(N);
	uint32 Index;
	while (Iter.GetNextIndex(Index)) { ... }
 */
struct FModuloIteration
{
	uint64 MaxIndex = 0;
	uint64 ModuloPrime = 4294967311ull;		// prime > max_unsigned_int
	uint64 CurIndex = 0;
	uint64 StartIndex = 0;
	uint64 Count = 0;
	uint64 ModuloNum = 1;

	FModuloIteration(uint32 MaxIndexIn, uint32 StartIndexIn = 0, uint64 ModuloPrimeIn = 3208642561)
	{
		MaxIndex = (uint64)FMath::Max((uint32)0, MaxIndexIn);
		StartIndex = (uint64)FMath::Max((uint32)0, StartIndexIn);
		CurIndex = StartIndex;
		Count = 0;
		ModuloNum = FMath::Max((uint64)1, MaxIndex);  // can't be zero or we hit integer-divide. If MaxIndex is 0 we will terminate on first iteration anyway
		ModuloPrime = ModuloPrimeIn;
		check(ModuloPrime > MaxIndex);
	}

	bool GetNextIndex(uint32& NextIndexOut)
	{
		NextIndexOut = (uint32)CurIndex;
		CurIndex = (CurIndex + ModuloPrime) % ModuloNum;
		return (Count++ != MaxIndex);
	}

	bool GetNextIndex(int32& NextIndexOut)
	{
		NextIndexOut = (int32)CurIndex;
		CurIndex = (CurIndex + ModuloPrime) % ModuloNum;
		return (Count++ != MaxIndex);
	}
};



} // end namespace UE::Geometry
} // end namespace UE