// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicVector.h"

#include "Async/ParallelFor.h"
#include "MeshQueries.h"

namespace UE
{
namespace Geometry
{

/*
 * Basic cache of per-triangle information for a mesh
 */
struct FMeshTriInfoCache
{
	TDynamicVector<FVector3d> Centroids;
	TDynamicVector<FVector3d> Normals;
	TDynamicVector<double> Areas;

	void GetTriInfo(int TriangleID, FVector3d& NormalOut, double& AreaOut, FVector3d& CentroidOut) const
	{
		NormalOut = Normals[TriangleID];
		AreaOut = Areas[TriangleID];
		CentroidOut = Centroids[TriangleID];
	}

	template<class TriangleMeshType>
	static FMeshTriInfoCache BuildTriInfoCache(const TriangleMeshType& Mesh)
	{
		FMeshTriInfoCache Cache;
		int NT = Mesh.MaxTriangleID();
		Cache.Centroids.Resize(NT);
		Cache.Normals.Resize(NT);
		Cache.Areas.Resize(NT);

		ParallelFor(NT, [&](int TID)
		{
			if (Mesh.IsTriangle(TID))
			{
				TMeshQueries<TriangleMeshType>::GetTriNormalAreaCentroid(Mesh, TID, Cache.Normals[TID], Cache.Areas[TID], Cache.Centroids[TID]);
			}
		});

		return Cache;
	}
};


} // end namespace UE::Geometry
} // end namespace UE