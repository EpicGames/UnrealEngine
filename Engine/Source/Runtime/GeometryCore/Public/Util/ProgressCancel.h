// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp ProgressCancel

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "MathUtil.h"
#include "Misc/DateTime.h"
#include "Misc/ScopeLock.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>

class FProgressCancel;


namespace UE
{
namespace Geometry
{


/**
 * FGeometryError represents an error code/message emitted by a geometry operation.
 * The intention of Errors is that they are fatal, IE if an operation emits Errors then
 * it did not complete successfully. If that is not the case, use a FGeometryWarning instead.
 */
struct FGeometryError
{
	int32 ErrorCode = 0;
	FText Message;
	FDateTime Timestamp;
	TArray<unsigned char> CustomData;

	FGeometryError() 
	{ 
		Timestamp = FDateTime::Now();
	}

	FGeometryError(int32 Code, const FText& MessageIn) : ErrorCode(Code), Message(MessageIn) 
	{
		Timestamp = FDateTime::Now();
	}
};

/**
 * FGeometryWarning represents a warning code/message emitted by a geometry operation.
 * The intention of Warnings is that they are non-fatal, IE an operation might successfully
 * complete but still emit Warnings
 */
struct FGeometryWarning
{
	int32 WarningCode = 0;
	FText Message;
	FDateTime Timestamp;
	TArray<unsigned char> CustomData;


	FGeometryWarning()
	{
		Timestamp = FDateTime::Now();
	}

	FGeometryWarning(int32 Code, const FText& MessageIn) : WarningCode(Code), Message(MessageIn)
	{
		Timestamp = FDateTime::Now();
	}
};


/**
 * EGeometryResultType is a generic result-code for use by geometry operations.
 */
enum class EGeometryResultType
{
	/** The Geometry Operation successfully completed */
	Success = 0,
	/** The Geometry Operation is in progress (this is intended for use in cases where there is a background computation that can be queried for incremental status/etc) */
	InProgress = 1,
	/** The Geometry Operation was Cancelled and did not complete */
	Cancelled = 2,
	/** The Geometry Operation completed but was not fully successful (eg some sub-computation failed and was gracefully handled, etc) */
	PartialResult = 3,
	/** The Geometry Operation failed and no valid result was produced */
	Failure = 4
};


/**
 * FGeometryResult represents a combined "success/failure/etc" state for a geometry operation
 * along with a set of Error and Warning codes/messages. 
 */
struct FGeometryResult
{
	EGeometryResultType Result;
	TArray<FGeometryError> Errors;
	TArray<FGeometryWarning> Warnings;


	FGeometryResult()
	{
		Result = EGeometryResultType::Success;
	}

	FGeometryResult(EGeometryResultType ResultType) 
	{
		Result = ResultType;
	}

	void UpdateResultType(EGeometryResultType NewType)
	{
		Result = NewType;
	}

	void SetFailed() { Result = EGeometryResultType::Failure; }
	void SetCancelled() { Result = EGeometryResultType::Cancelled; }

	void SetSuccess() { Result = EGeometryResultType::Success; }

	/**
	 * Set to Success/Failure based on bSuccess, or Cancelled if the (optional) FProgressCancel indicates that it was Cancelled
	 */
	inline void SetSuccess(bool bSuccess, FProgressCancel* Progress = nullptr );

	/**
	 * Set state of the Result to Failure, and append a FGeometryError with the given ErrorMessage and ResultCode
	 */
	inline void SetFailed(FText ErrorMessage, int ResultCode = 0);

	/**
	 * Test if the given Progress has been cancelled, if so, set the Result to Cancelled 
	 * @return true if cancelled, false if not cancelled
	 */
	inline bool CheckAndSetCancelled( FProgressCancel* Progress );

	/**
	 * Append an Error to the result
	 */
	void AddError(FGeometryError Error) 
	{ 
		Errors.Add(Error); 
	}
	
	/**
	 * Append a Warning to the result
	 */
	void AddWarning(FGeometryWarning Warning)
	{ 
		Warnings.Add(Warning); 
	}

	/**
	 * @return true if the geometry operation failed
	 */
	bool HasFailed() const { return Result == EGeometryResultType::Failure; }

	/**
	 * @return true if the geometry operation has some result (ie was a Success or returned a PartialResult)
	 */
	bool HasResult() const { return Result == EGeometryResultType::Success || Result == EGeometryResultType::PartialResult; }

	static FGeometryResult Failed() { return FGeometryResult(EGeometryResultType::Failure); }
	static FGeometryResult Cancelled() { return FGeometryResult(EGeometryResultType::Cancelled); }
};



} // end namespace UE::Geometry
} // end namespace UE





/**
 * FProgressCancel is intended to be passed to long-running
 * computes to do two things:
 * 1) provide progress info back to caller
 * 2) allow caller to cancel the computation
 */
class FProgressCancel
{
private:
	bool WasCancelled = false;  // will be set to true if CancelF() ever returns true

	// Progress tracking data
	struct FProgressData
	{
		// Active range of progress tracking -- progress should not go out of this range
		float CurrentMin = 0;
		float CurrentMax = 1;
		// Number of nested scopes currently affecting the active range
		int ScopeDepth = 0;
		// Message describing the current work being done
		FText Message;

		float Range() const
		{
			return CurrentMax - CurrentMin;
		}

		// Advance progress without going beyond the current range
		float SafeAdvance(float CurrentFraction, float AdvanceAmount) const
		{
			return FMathf::Min(CurrentMax, CurrentFraction + Range() * AdvanceAmount);
		}
	};
	FProgressData Progress;
	FText ProgressMessage;
	int MaxMessageDepth = -1; // If set to a positive value, message will not be updated by scopes below this depth

	// Current total progress, in range [0, 1]
	std::atomic<float> ProgressFraction = 0;

	// critical section for accesses to the Progress Message
	mutable FCriticalSection MessageCS;

	GEOMETRYCORE_API void StartWorkScope(FProgressData& SaveProgressFrameOut, float StepSize, const FText& Message);

	GEOMETRYCORE_API void EndWorkScope(const FProgressData& SavedProgressFrame);

public:
	TFunction<bool()> CancelF = []() { return false; };

	void SetMaxMessageDepth(int32 ScopeDepth)
	{
		MaxMessageDepth = ScopeDepth;
	}

	void ClearMaxMessageDepth()
	{
		MaxMessageDepth = -1;
	}

	/**
	 * @return true if client would like to cancel operation
	 */
	bool Cancelled()
	{
		if (WasCancelled)
		{
			return true;
		}
		WasCancelled = CancelF();
		return WasCancelled;
	}

	friend class FProgressScope;

	// Simple helper to track progress in a local scope on an optional FProgressCancel
	// Will still work if the ProgressCancel is null (just does nothing in that case)
	class FProgressScope
	{
		FProgressCancel* ProgressCancel;
		FProgressData SavedProgressData;
		bool bEnded = false;

	public:

		/**
		 * @param ProgressCancel		Progress will be tracked on this. If null, the FProgressScope will do nothing.
		 * @param ProgressAmount		Amount to increase progress w/in this scope (as a fraction of the current outer-scope active progress range)
		 */
		GEOMETRYCORE_API FProgressScope(FProgressCancel* ProgressCancel, float ProgressAmount, const FText& Message = FText());

		/**
		 * Create a dummy/inactive FProgressScope
		 */
		FProgressScope() : ProgressCancel(nullptr)
		{}

		FProgressScope(const FProgressScope& Other) = delete;
		FProgressScope& operator=(const FProgressScope& Other) = delete;
		FProgressScope(FProgressScope&& Other) noexcept : ProgressCancel(Other.ProgressCancel), SavedProgressData(MoveTemp(Other.SavedProgressData)), bEnded(Other.bEnded)
		{
			Other.ProgressCancel = nullptr;
			Other.bEnded = true;
		}
		FProgressScope& operator=(FProgressScope&& Other) noexcept
		{
			if (this != &Other)
			{
				ProgressCancel = Other.ProgressCancel;
				SavedProgressData = MoveTemp(Other.SavedProgressData);
				bEnded = Other.bEnded;
				Other.ProgressCancel = nullptr;
				Other.bEnded = true;
			}
			return *this;
		}

		~FProgressScope()
		{
			if (!bEnded)
			{
				Done();
			}
		}

		/**
		 * Advance to the end of the scope's progress range and close the scope
		 */
		GEOMETRYCORE_API void Done();

		/**
		 * @param Amount	Amount to increase the progress fraction, as a fraction of the current active progress range
		 */
		inline void AdvanceProgressBy(float Amount)
		{
			if (ProgressCancel)
			{
				ProgressCancel->AdvanceCurrentScopeProgressBy(Amount);
			}
		}

		/**
		 * Advance current progress a fraction of the way toward a target value
		 * For example: if progress is .5, AdvanceProgressToward(1, .5) will take a half step to 1 and set progress to .75
		 * As with all public progress function, progress is expressed relative to the current active progress range and cannot go backward.
		 */
		inline void AdvanceProgressToward(float TargetProgressFrac, float FractionToward)
		{
			if (ProgressCancel)
			{
				ProgressCancel->AdvanceCurrentScopeProgressToward(TargetProgressFrac, FractionToward);
			}
		}

		/**
		 * @return Amount to progress in the current scope to reach the target (scope-relative) progress value
		 */
		inline float GetDistanceTo(float TargetProgressFrac)
		{
			if (ProgressCancel)
			{
				return ProgressCancel->GetCurrentScopeDistanceTo(TargetProgressFrac);
			}

			return 0;
		}

		/**
		 * @return Progress in the current scope
		 */
		inline float GetProgress()
		{
			if (ProgressCancel)
			{
				return ProgressCancel->GetCurrentScopeProgress();
			}

			return 1;
		}

		/**
		 * Note: This function will leave the current progress unchanged if the target value is less than the current progress --
		 *  it will not allow the progress value to go backward.
		 *
		 * @param ProgressFrac	Value to set current progress to, as a fraction of the current active progress range
		 */
		inline void SetProgressTo(float NewProgressFrac)
		{
			if (ProgressCancel)
			{
				ProgressCancel->SetCurrentScopeProgressTo(NewProgressFrac);
			}
		}
	};

	/**
	 * @param ProgressCancel		Progress will be tracked on this. If null, the FProgressScope will do nothing.
	 * @param ProgressTo			Target value to increase progress to w/in this scope (as a fraction of the current outer-scope active progress range)
	 * @param Message				Optional message describing the work to be done
	 * @return						A new FProgressScope that covers work from the current progress to the target progress value (relative to the current scope)
	 */
	static GEOMETRYCORE_API FProgressScope CreateScopeTo(FProgressCancel* ProgressCancel, float ProgressTo, const FText& Message = FText());

	float GetProgress() const
	{
		return FMathf::Clamp(ProgressFraction, 0, 1);
	}

	FText GetProgressMessage() const
	{
		FScopeLock MessageLock(&MessageCS);
		return ProgressMessage;
	}

	void SetProgressMessage(const FText& Message)
	{
		if (MaxMessageDepth == -1 || Progress.ScopeDepth < MaxMessageDepth)
		{
			Progress.Message = Message;
			FScopeLock MessageLock(&MessageCS);
			ProgressMessage = Message;
		}
	}

	/**
	 * @param Amount	Amount to increase the progress fraction, as a fraction of the current active progress range
	 */
	void AdvanceCurrentScopeProgressBy(float Amount)
	{
		ProgressFraction = Progress.SafeAdvance(ProgressFraction, Amount);
	}

	/**
	 * Advance current progress a fraction of the way toward a target value
	 * For example: if progress is .5, AdvanceProgressToward(1, .5) will take a half step to 1 and set progress to .75
	 * As with all public progress function, progress is expressed relative to the current active progress range and cannot go backward.
	 */
	void AdvanceCurrentScopeProgressToward(float TargetProgressFrac, float FractionToward)
	{
		float TargetProgressFraction = FMathf::Max(ProgressFraction, Progress.CurrentMin + Progress.Range() * TargetProgressFrac);
		ProgressFraction = Progress.SafeAdvance(ProgressFraction, FMathf::Lerp(ProgressFraction, TargetProgressFraction, FractionToward));
	}

	/**
	 * @return Progress as a fraction of the current scope's active progress range
	 */
	float GetCurrentScopeProgress()
	{
		float Range = Progress.Range();
		return Range > 0 ? (ProgressFraction - Progress.CurrentMin) / Range : 1;
	}

	/**
	 * @param TargetProgressFrac	Target progress value as a fraction of the current scope's active progress range
	 * @return Amount to progress in the current scope to reach the target progress value
	 */
	float GetCurrentScopeDistanceTo(float TargetProgressFrac)
	{
		return TargetProgressFrac - GetCurrentScopeProgress();
	}

	/**
	 * Note: This function will leave the current progress unchanged if the target value is less than the current progress --
	 *  it will not allow the progress value to go backward.
	 * 
	 * @param ProgressFrac	Value to set current progress to, as a fraction of the current active progress range
	 */
	void SetCurrentScopeProgressTo(float NewProgressFrac)
	{
		ProgressFraction = FMathf::Clamp(Progress.CurrentMin + Progress.Range() * NewProgressFrac, ProgressFraction, Progress.CurrentMax);
	}

public:

	enum class EMessageLevel
	{
		// Note: Corresponds to EToolMessageLevel in InteractiveToolsFramework/ToolContextInterfaces.h

		/** Development message goes into development log */
		Internal = 0,
		/** User message should appear in user-facing log */
		UserMessage = 1,
		/** Notification message should be shown in a non-modal notification window */
		UserNotification = 2,
		/** Warning message should be shown in a non-modal notification window with panache */
		UserWarning = 3,
		/** Error message should be shown in a modal notification window */
		UserError = 4
	};

	struct FMessageInfo
	{
		FText MessageText;
		EMessageLevel MessageLevel;
		FDateTime Timestamp;
	};

	void AddWarning(const FText& MessageText, EMessageLevel MessageLevel)
	{
		Warnings.Add(FMessageInfo{ MessageText , MessageLevel, FDateTime::Now() });
	}

	TArray<FMessageInfo> Warnings;
};




void UE::Geometry::FGeometryResult::SetSuccess(bool bSuccess, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		Result = EGeometryResultType::Cancelled;
	}
	else
	{
		Result = (bSuccess) ? EGeometryResultType::Success : EGeometryResultType::Failure;
	}
}


void UE::Geometry::FGeometryResult::SetFailed(FText ErrorMessage, int ResultCode)
{
	Result = EGeometryResultType::Failure;
	Errors.Add(FGeometryError(ResultCode, ErrorMessage));
}


bool UE::Geometry::FGeometryResult::CheckAndSetCancelled(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		Result = EGeometryResultType::Cancelled;
		return true;
	}
	return false;
}
