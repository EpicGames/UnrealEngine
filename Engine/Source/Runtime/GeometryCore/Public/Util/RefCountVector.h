// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp FRefCountVector

#pragma once

#include "CoreMinimal.h"
#include "Util/DynamicVector.h"
#include "Util/IteratorUtil.h"

namespace UE
{
namespace Geometry
{

/**
 * FRefCountVector is used to keep track of which indices in a linear Index list are in use/referenced.
 * A free list is tracked so that unreferenced indices can be re-used.
 * 
 * The enumerator iterates over valid indices (ie where refcount > 0)
 * @warning refcounts are 16-bit ints (shorts) so the maximum count is 65536. behavior is undefined if this overflows.
 * @warning No overflow checking is done in release builds.
 */
class FRefCountVector
{
public:
	static constexpr unsigned short INVALID_REF_COUNT = MAX_uint16;

	FRefCountVector() = default;
	FRefCountVector(const FRefCountVector&) = default;
	FRefCountVector(FRefCountVector&& From) : 
		RefCounts(MoveTemp(From.RefCounts)), FreeIndices(MoveTemp(From.FreeIndices)), UsedCount(From.UsedCount)
	{
		From.UsedCount = 0;
	}
	FRefCountVector& operator=(const FRefCountVector&) = default;
	FRefCountVector& operator=(FRefCountVector&& From)
	{
		RefCounts = MoveTemp(From.RefCounts);
		FreeIndices = MoveTemp(From.FreeIndices);
		UsedCount = From.UsedCount;
		From.UsedCount = 0;
		return *this;
	}

	bool IsEmpty() const
	{
		return UsedCount == 0;
	}

	size_t GetCount() const
	{
		return UsedCount;
	}

	size_t GetMaxIndex() const
	{
		return RefCounts.GetLength();
	}

	bool IsDense() const
	{
		return FreeIndices.GetLength() == 0;
	}

	bool IsValid(int Index) const
	{
		return (Index >= 0 && Index < (int)RefCounts.GetLength() && IsValidUnsafe(Index));
	}

	bool IsValidUnsafe(int Index) const
	{
		return RefCounts[Index] > 0 && RefCounts[Index] < INVALID_REF_COUNT;
	}

	int GetRefCount(int Index) const
	{
		int n = RefCounts[Index];
		return (n == INVALID_REF_COUNT) ? 0 : n;
	}

	int GetRawRefCount(int Index) const
	{
		return RefCounts[Index];
	}

	// Append all ref counts from another RefCountVector, offsetting FreeIndices to refer to their corresponded new array positions
	// Note this does not try to 'fill in' original free indices, by design -- all existing IDs remain untouched, and all new IDs are simply offsets of the Other's IDs
	void Append(const FRefCountVector& Other)
	{
		size_t OrigNum = RefCounts.Num();
		size_t OrigFree = FreeIndices.Num();
		RefCounts.Add(Other.RefCounts);
		FreeIndices.Add(Other.FreeIndices);
		for (size_t Idx = OrigFree, N = FreeIndices.Num(); Idx < N; ++Idx)
		{
			FreeIndices[Idx] += OrigNum;
		}
		UsedCount += Other.UsedCount;
	}

	int Allocate()
	{
		UsedCount++;
		if (FreeIndices.IsEmpty())
		{
			// TODO: do we need this branch anymore?
			RefCounts.Add(1);
			return (int)RefCounts.GetLength() - 1;
		}
		else
		{
			int iFree = INDEX_NONE;
			while (iFree == INDEX_NONE && FreeIndices.IsEmpty() == false)
			{
				iFree = FreeIndices.Back();
				FreeIndices.PopBack();
			}
			if (iFree != INDEX_NONE)
			{
				RefCounts[iFree] = 1;
				return iFree;
			}
			else
			{
				RefCounts.Add(1);
				return (int)RefCounts.GetLength() - 1;
			}
		}
	}

	int Increment(int Index, unsigned short IncrementCount = 1)
	{
		checkSlow(RefCounts[Index] != INVALID_REF_COUNT);
		RefCounts[Index] += IncrementCount;
		return RefCounts[Index];
	}

	void Decrement(int Index, unsigned short DecrementCount = 1)
	{
		checkSlow(RefCounts[Index] != INVALID_REF_COUNT && RefCounts[Index] >= DecrementCount);
		RefCounts[Index] -= DecrementCount;
		if (RefCounts[Index] == 0)
		{
			FreeIndices.Add(Index);
			RefCounts[Index] = INVALID_REF_COUNT;
			UsedCount--;
		}
	}

	/**
	 * allocate at specific Index, which must either be larger than current max Index,
	 * or on the free list. If larger, all elements up to this one will be pushed onto
	 * free list. otherwise we have to do a linear search through free list.
	 * If you are doing many of these, it is likely faster to use
	 * AllocateAtUnsafe(), and then RebuildFreeList() after you are done.
	 */
	bool AllocateAt(int Index)
	{
		if (Index >= (int)RefCounts.GetLength())
		{
			int j = (int)RefCounts.GetLength();
			while (j < Index)
			{
				unsigned short InvalidCount = INVALID_REF_COUNT;	// required on older clang because a constexpr can't be passed by ref
				RefCounts.Add(InvalidCount);
				FreeIndices.Add(j);
				++j;
			}
			RefCounts.Add(1);
			UsedCount++;
			return true;
		}
		else
		{
			if (IsValidUnsafe(Index))
			{
				return false;
			}
			int N = (int)FreeIndices.GetLength();
			for (int i = 0; i < N; ++i)
			{
				if (FreeIndices[i] == Index)
				{
					FreeIndices[i] = FreeIndices.Back();
					FreeIndices.PopBack();
					RefCounts[Index] = 1;
					UsedCount++;
					return true;
				}
			}
			return false;
		}
	}

	/**
	 * allocate at specific Index, which must be free or larger than current max Index.
	 * However, we do not update free list. So, you probably need to do RebuildFreeList() after calling this.
	 */
	bool AllocateAtUnsafe(int Index)
	{
		if (Index >= (int)RefCounts.GetLength())
		{
			int j = (int)RefCounts.GetLength();
			while (j < Index)
			{
				unsigned short InvalidCount = INVALID_REF_COUNT;	// required on older clang because a constexpr can't be passed by ref
				RefCounts.Add(InvalidCount);
				++j;
			}
			RefCounts.Add(1);
			UsedCount++;
			return true;

		}
		else
		{
			if (IsValidUnsafe(Index))
			{
				return false;
			}
			RefCounts[Index] = 1;
			UsedCount++;
			return true;
		}
	}

	const TDynamicVector<unsigned short>& GetRawRefCounts() const
	{
		return RefCounts;
	}

	/**
	 * @warning you should not use this!
	 */
	TDynamicVector<unsigned short>& GetRawRefCountsUnsafe()
	{
		return RefCounts;
	}

	/**
	 * @warning you should not use this!
	 */
	void SetRefCountUnsafe(int Index, unsigned short ToCount)
	{
		RefCounts[Index] = ToCount;
	}

	// todo:
	//   remove
	//   clear

	/**
	 * Rebuilds all reference counts from external source data via callables.
	 *
	 * For example, we could rebuild ref counts for vertices by iterating over all triangles via the Iterate callable, and in the process any vertex we
	 * encounter for the first time gets a ref count of 1 via the AllocateRefCount callable, and any vertex we encounter repeatedly has its ref count
	 * incremented by one via the IncrementRefCount callable. 
	 *
	 * @param Num Number of items that get referenced.
	 * @param Iterate Callable that iterates over all referenced items in the external source data.
	 * @param AllocateRefCount Callable for allocating an item, i.e. an item that is referenced for the first time.
	 * @param IncrementRefCount Callable for incrementing an item, i.e. an item that has been referenced before.
	 */
	template <typename IterateFunc, typename AllocateRefCountFunc, typename IncrementRefCountFunc>
	void Rebuild(unsigned int Num, IterateFunc&& Iterate, AllocateRefCountFunc&& AllocateRefCount, IncrementRefCountFunc&& IncrementRefCount)
	{
		// Initialize ref counts to the given number of elements.
		RefCounts.Resize(Num);
		RefCounts.Fill(INVALID_REF_COUNT);
		UsedCount = 0;

		// Lambda for updating ref count for a given index. This is passed to the external iterate function.
		const auto UpdateRefCount = [this, &AllocateRefCount, &IncrementRefCount](int32 Index)
		{
			unsigned short& RefCount = RefCounts[Index];
			if (RefCount == INVALID_REF_COUNT)
			{
				// Increase used counter and call external function to initialize value.
				++UsedCount;
				Forward<AllocateRefCountFunc>(AllocateRefCount)(RefCount);
			}
			else
			{
				// Call external function to initialize value.
				Forward<IncrementRefCountFunc>(IncrementRefCount)(RefCount);
			}
		};

		// Call external function that iterates over all external pieces of data, which will in turn call the lambda to update ref counts. 
		Forward<IterateFunc>(Iterate)(UpdateRefCount);

		// Add unused elements to free list.
		const unsigned int FreeIndicesNum = Num - UsedCount;
		FreeIndices.SetNum(FreeIndicesNum);
		unsigned int FreeIndicesIndex = 0;
		for (unsigned int Index = 0; (Index < Num) & (FreeIndicesIndex < FreeIndicesNum); ++Index)
		{
			if (RefCounts[Index] == INVALID_REF_COUNT)
			{
				FreeIndices[FreeIndicesIndex++] = Index;
			}
		}
	}

	void RebuildFreeList()
	{
		FreeIndices.Clear();
		UsedCount = 0;

		int N = (int)RefCounts.GetLength();
		for (int i = 0; i < N; ++i) 
		{
			if (IsValidUnsafe(i))
			{
				UsedCount++;
			}
			else
			{
				FreeIndices.Add(i);
			}
		}
	}

	void Trim(int maxIndex)
	{
		FreeIndices.Clear();
		RefCounts.Resize(maxIndex);
		UsedCount = maxIndex;
	}

	void Clear()
	{
		FreeIndices.Clear();
		RefCounts.Clear();
		UsedCount = 0;
	}

	// initialize and set all refcounts to given value
	void InitDense(int Size, uint16 RefCountValue = 1)
	{
		FreeIndices.Clear();
		RefCounts.Clear();
		RefCounts.Resize(Size, RefCountValue);
		UsedCount = Size;
	}
	//
	// Iterators
	//

	/**
	 * base iterator for indices with valid refcount (skips zero-refcount indices)
	 */
	class BaseIterator
	{
	public:
		inline BaseIterator()
		{
			Vector = nullptr;
			Index = 0;
			LastIndex = 0;
		}

		inline bool operator==(const BaseIterator& Other) const
		{
			return Index == Other.Index;
		}
		inline bool operator!=(const BaseIterator& Other) const
		{
			return Index != Other.Index;
		}

	protected:
		inline void goto_next()
		{
			Index++;
			while (Index < LastIndex && Vector->IsValidUnsafe(Index) == false)
			{
				Index++;
			}
		}

		inline BaseIterator(const FRefCountVector* VectorIn, int IndexIn, int LastIn)
		{
			Vector = VectorIn;
			Index = IndexIn;
			LastIndex = LastIn;
			if (Index != LastIndex && Vector->IsValidUnsafe(Index) == false)
			{
				goto_next();		// initialize
			}
		}
		const FRefCountVector * Vector;
		int Index;
		int LastIndex;
		friend class FRefCountVector;
	};

	/*
	 *  iterator over valid indices (ie non-zero refcount)
	 */
	class IndexIterator : public BaseIterator
	{
	public:
		inline IndexIterator() : BaseIterator() {}

		inline int operator*() const
		{
			return this->Index;
		}

		inline IndexIterator & operator++() 		// prefix
		{
			this->goto_next();
			return *this;
		}
		inline IndexIterator operator++(int) 		// postfix
		{
			IndexIterator copy(*this);
			this->goto_next();
			return copy;
		}

	protected:
		inline IndexIterator(const FRefCountVector* VectorIn, int Index, int Last) : BaseIterator(VectorIn, Index, Last)
		{}
		friend class FRefCountVector;
	};

	inline IndexIterator BeginIndices() const
	{
		return IndexIterator(this, (int)0, (int)RefCounts.GetLength());
	}

	inline IndexIterator EndIndices() const
	{
		return IndexIterator(this, (int)RefCounts.GetLength(), (int)RefCounts.GetLength());
	}

	/**
	 * enumerable object that provides begin()/end() semantics, so
	 * you can iterate over valid indices using range-based for loop
	 */
	class IndexEnumerable
	{
	public:
		const FRefCountVector* Vector;
		IndexEnumerable() { Vector = nullptr; }
		IndexEnumerable(const FRefCountVector* VectorIn) { Vector = VectorIn; }
		typename FRefCountVector::IndexIterator begin() const { return Vector->BeginIndices(); }
		typename FRefCountVector::IndexIterator end() const { return Vector->EndIndices(); }
	};

	/**
	 * returns iteration object over valid indices
	 * usage: for (int idx : indices()) { ... }
	 */
	inline IndexEnumerable Indices() const
	{
		return IndexEnumerable(this);
	}


	/*
	 * enumerable object that maps indices output by Index_iteration to a second type
	 */
	template<typename ToType>
	class MappedEnumerable
	{
	public:
		TFunction<ToType(int)> MapFunc;
		IndexEnumerable enumerable;

		MappedEnumerable(const IndexEnumerable& enumerable, TFunction<ToType(int)> MapFunc)
		{
			this->enumerable = enumerable;
			this->MapFunc = MapFunc;
		}

		MappedIterator<int, ToType, IndexIterator> begin()
		{
			return MappedIterator<int, ToType, IndexIterator>(enumerable.begin(), MapFunc);
		}

		MappedIterator<int, ToType, IndexIterator> end()
		{
			return MappedIterator<int, ToType, IndexIterator>(enumerable.end(), MapFunc);
		}
	};


	/**
	* returns iteration object over mapping applied to valid indices
	* eg usage: for (FVector3d v : mapped_indices(fn_that_looks_up_mesh_vtx_from_id)) { ... }
	*/
	template<typename ToType>
	inline MappedEnumerable<ToType> MappedIndices(TFunction<ToType(int)> MapFunc) const
	{
		return MappedEnumerable<ToType>(Indices(), MapFunc);
	}

	/*
	* iteration object that maps indices output by Index_iteration to a second type
	*/
	class FilteredEnumerable
	{
	public:
		TFunction<bool(int)> FilterFunc;
		IndexEnumerable enumerable;
		FilteredEnumerable(const IndexEnumerable& enumerable, TFunction<bool(int)> FilterFuncIn)
		{
			this->enumerable = enumerable;
			this->FilterFunc = FilterFuncIn;
		}

		FilteredIterator<int, IndexIterator> begin()
		{
			return FilteredIterator<int, IndexIterator>(enumerable.begin(), enumerable.end(), FilterFunc);
		}

		FilteredIterator<int, IndexIterator> end()
		{
			return FilteredIterator<int, IndexIterator>(enumerable.end(), enumerable.end(), FilterFunc);
		}
	};

	inline FilteredEnumerable FilteredIndices(TFunction<bool(int)> FilterFunc) const
	{
		return FilteredEnumerable(Indices(), FilterFunc);
	}

	FString UsageStats() const
	{
		return FString::Printf(TEXT("RefCountSize %zu  FreeSize %zu  FreeMem %zukb"),
			RefCounts.GetLength(), FreeIndices.GetLength(), (FreeIndices.GetByteCount() / 1024));
	}

	SIZE_T GetByteCount() const
	{
		return RefCounts.GetByteCount() + FreeIndices.GetByteCount();
	}

	/**
	 * Serialization operator for FRefCountVector.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Vec Vector to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FRefCountVector& Vec)
	{
		Vec.Serialize(Ar, false, false);
		return Ar;
	}

	/**
	 * Serialize FRefCountVector to an archive.
	 * @param Ar Archive to serialize with
	 * @param bCompactData Only serialize unique data and/or recompute redundant data when loading.
	 * @param bUseCompression Use compression to serialize data; the resulting size will likely be smaller but serialization will take significantly longer.
	 */
	void Serialize(FArchive& Ar, bool bCompactData, bool bUseCompression)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::DynamicMeshCompactedSerialization)
		{
			Ar << RefCounts;
			Ar << FreeIndices;
			Ar << UsedCount;
		}
		else
		{
			// Serialize options.
			Ar << bCompactData;
			Ar << bUseCompression;

			// Serialize number of used ref counts. While this is redundant to the contents of the ref count vector, it allows us to accelerate restoring the
			// free indices and therefore it is worth the very minor storage overhead.
			Ar << UsedCount;

			// Serialize ref counts.
			if (bUseCompression)
			{
				RefCounts.Serialize<true, true>(Ar);
			}
			else
			{
				RefCounts.Serialize<true, false>(Ar);
			}

			if (UsedCount == RefCounts.Num())
			{
				// If all ref counts are used then we can ignore the free indices entirely during save, and just clear them during load.

				if (Ar.IsLoading() && !FreeIndices.IsEmpty())
				{
					FreeIndices.Clear();
				}
			}
			else
			{
				// Compact the data by omitting the free indices entirely during save and recompute them during load.
				// Considering the significant time overhead for compression, we compact if either bCompactData or bUseCompression is enabled.
				if (bCompactData || bUseCompression)
				{
					if (Ar.IsLoading())
					{
						// Rebuild the free indices from the invalid ref count values.
						const size_t NumFree = RefCounts.Num() - UsedCount;
						FreeIndices.Resize(NumFree);
						size_t Index = 0;
						for (size_t i = 0, Num = RefCounts.Num(); (i < Num) & (Index < NumFree); ++i)
						{
							if (RefCounts[i] == INVALID_REF_COUNT)
							{
								FreeIndices[Index++] = i;
							}
						}
						ensure(Index == NumFree);
					}
				}
				else
				{
					FreeIndices.Serialize<true, false>(Ar);
				}
			}
		}
	}

	friend bool operator==(const FRefCountVector& Lhs, const FRefCountVector& Rhs)
	{
		if (Lhs.GetCount() != Rhs.GetCount())
		{
			return false;
		}

		const size_t Num = FMath::Max(Lhs.GetMaxIndex(), Rhs.GetMaxIndex());
		for (size_t Idx = 0; Idx < Num; ++Idx)
		{
			const bool LhsIsValid = Lhs.IsValid(Idx);
			if (LhsIsValid != Rhs.IsValid(Idx))
			{
				return false;
			}
			if (LhsIsValid && Lhs.GetRefCount(Idx) != Rhs.GetRefCount(Idx))
			{
				return false;
			}
		}

		return true;
	}

	friend bool operator!=(const FRefCountVector& Lhs, const FRefCountVector& Rhs)
	{
		return !(Lhs == Rhs);
	}

private:
	TDynamicVector<unsigned short> RefCounts{};
	TDynamicVector<int> FreeIndices{};
	int UsedCount{0};


};


} // end namespace UE::Geometry
} // end namespace UE