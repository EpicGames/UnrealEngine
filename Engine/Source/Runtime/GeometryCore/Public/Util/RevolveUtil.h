// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "FrameTypes.h"
#include "GeometryBase.h"
#include "Math/UnrealMathSSE.h"
#include "VectorTypes.h"

namespace RevolveUtil
{
	using namespace UE::Geometry;
	using UE::Geometry::FFrame3d;

	/** 
	 * Generates a sweep curve with the initial frame at the origin and the rest rotated around the given axis.
	 *
	 * @param RevolutionAxisOrigin
	 * @param RevolutionAxisDirection Should be normalized.
	 * @param DegreesOffset Number of degrees to rotate the first frame.
	 * @param DegreesPerStep Number of degrees each frame is rotated from the previous
	 * @param DownAxisOffset Distance to move each frame up the axis from the previous (for spirals)
	 * @param TotalNumFrames Number of frames to create, including the first frame
	 * @param SweepCurveOut Output
	 */
	GEOMETRYCORE_API
	void GenerateSweepCurve(const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection, double DegreesOffset,
		double DegreesPerStep, double DownAxisOffset, int TotalNumFrames, TArray<FFrame3d>& SweepCurveOut);

	/**
	 * Checks the profile curve for points that lie within a particular tolerance of the revolution axis. If they
	 * are found, the points are projected directly onto the revolution axis and their indices are added to a set
	 * of welded points.
	 *
	 * Assumes that RevolutionAxisDirection is normalized.
	 */
	GEOMETRYCORE_API
	void WeldPointsOnAxis(TArray<FVector3d>& ProfileCurve, const FVector3d& RevolutionAxisOrigin,
		const FVector3d& RevolutionAxisDirection, double Tolerance, TSet<int32>& ProfileVerticesToWeldOut);

	/**
	 * Returns true if the profile curve is counterclockwise relative to the rotation direction. I.e., 
	 * if you are looking from the side with the revolution axis pointing up and the curve to the left of
	 * the axis, the curve should be counterclockwise. 
	 * If the curve is not closed, the function looks at the bottom of the profile curve relative to the
	 * revolution axis and checks whether the outer edge of the bottom goes in the direction that it would on
	 * a counterclockwise curve (up, in our example).
	 *
	 * Results are undefined if a fully revolved mesh would be self intersecting.
	 * Assumes that RevolutionAxisDirection is normalized.
	 */
	GEOMETRYCORE_API
	bool ProfileIsCCWRelativeRevolve(TArray<FVector3d>& ProfileCurve, const FVector3d& RevolutionAxisOrigin,
		const FVector3d& RevolutionAxisDirection, bool bProfileCurveIsClosed);

	/**
	 * Shifts the profile curve in such a way that it becomes the midpoint of the first rotation step (i.e.,
	 * rotates it back half a step while projecting it outward onto the plane passing through the profile).
	 */
	GEOMETRYCORE_API
	void MakeProfileCurveMidpointOfFirstStep(TArray<FVector3d>& ProfileCurve, double DegreesPerStep,
		const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection);
}
