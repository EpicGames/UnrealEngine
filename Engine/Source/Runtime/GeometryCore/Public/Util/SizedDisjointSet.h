// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

namespace UE::Geometry
{

/// Disjoint set with additional storage to track the size of each set
struct FSizedDisjointSet
{
	TArray<int32> Parents, Sizes;

	FSizedDisjointSet() = default;
	FSizedDisjointSet(int32 NumIDs)
	{
		Init(NumIDs);
	}

	void Init(int32 NumIDs)
	{
		Parents.SetNumUninitialized(NumIDs);
		Sizes.SetNumUninitialized(NumIDs);
		for (int32 Idx = 0; Idx < NumIDs; Idx++)
		{
			Parents[Idx] = Idx;
			Sizes[Idx] = 1;
		}
	}

	// IsElement(Index) returns false if Index is not valid.  Find, Union and GetSize should never be called on an invalid index.
	template<typename IsElementFunction>
	void Init(int32 NumIDs, IsElementFunction IsElement)
	{
		Parents.SetNumUninitialized(NumIDs);
		Sizes.SetNumUninitialized(NumIDs);
		for (int32 Idx = 0; Idx < NumIDs; Idx++)
		{
			if (IsElement(Idx))
			{
				Parents[Idx] = Idx;
				Sizes[Idx] = 1;
			}
			else
			{
				Parents[Idx] = -1;
				Sizes[Idx] = 0;
			}
		}
	}

	// @return the Parent of the Union
	int32 Union(int32 A, int32 B)
	{
		int32 ParentA = Find(A);
		int32 ParentB = Find(B);
		if (ParentA == ParentB)
		{
			return ParentB;
		}
		if (Sizes[ParentA] > Sizes[ParentB])
		{
			Swap(ParentA, ParentB);
		}
		Parents[ParentA] = ParentB;
		Sizes[ParentB] += Sizes[ParentA];
		return ParentB;
	}

	// Find the representative cluster element for this Idx
	// Note: Collapses parent references as it walks them, to make future Find() calls faster --
	//  Use FindWithoutCollapse for a const version without this feature.
	int32 Find(int32 Idx)
	{
		int32 Parent = Idx;

		while (Parents[Parent] != Parent)
		{
			Parents[Parent] = Parents[Parents[Parent]];
			Parent = Parents[Parent];
		}

		return Parent;
	}

	// Find w/out collapsing the upward links
	// Note this can be less efficient over many calls, if the trees have not been collapsed by previous accesses,
	// but it is a const access, and easier to use in parallel
	int32 FindWithoutCollapse(int32 Idx) const
	{
		int32 Parent = Idx;

		while (Parents[Parent] != Parent)
		{
			Parent = Parents[Parent];
		}

		return Parent;
	}

	int32 GetSize(int32 Idx) const
	{
		int32 Parent = FindWithoutCollapse(Idx);
		return Sizes[Parent];
	}

	/**
	 * Create mappings between compacted Group Index and group ID, where the compacted indices numbers the groups from 0 to NumGroups
	 * 
	 * @param CompactIdxToGroupID  Array to fill with the Compact Index -> Original Group ID mapping.  If null, array will not be set.
	 * @param GroupIDToCompactIdx  Array to fill with the Original Group ID -> Compact Index mapping.  If null, array will not be set.
	 * @param MinGroupSize  The minimum size of group to consider. Groups smaller than this will not be counted.
	 * @return The number of groups found
	 */
	int32 CompactedGroupIndexToGroupID(TArray<int32>* CompactIdxToGroupID, TArray<int32>* GroupIDToCompactIdx, int32 MinGroupSize = 1) const
	{
		MinGroupSize = FMath::Max(1, MinGroupSize);

		int32 NumIDs = Sizes.Num();
		if (CompactIdxToGroupID)
		{
			CompactIdxToGroupID->Reset();
		}
		if (GroupIDToCompactIdx)
		{
			GroupIDToCompactIdx->Init(-1, NumIDs);
		}
		int32 NumGroups = 0;
		for (int32 ID = 0; ID < NumIDs; ++ID)
		{
			if (Parents[ID] == INDEX_NONE) // ignore invalid IDs
			{
				continue;
			}

			int32 Parent = FindWithoutCollapse(ID);
			if (Parent != ID) // only count groups on the Parent==ID node
			{
				continue;
			}

			if (Sizes[Parent] < MinGroupSize) // ignore too-small groups
			{
				continue;
			}

			// Record the unique GroupID
			if (CompactIdxToGroupID)
			{
				CompactIdxToGroupID->Add(Parent);
			}
			if (GroupIDToCompactIdx)
			{
				(*GroupIDToCompactIdx)[ID] = NumGroups;
			}
			NumGroups++;
		}
		return NumGroups;
	}
};


} // end namespace UE::Geometry
