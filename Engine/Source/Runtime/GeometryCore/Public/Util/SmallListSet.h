// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp small_list_set

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Util/DynamicVector.h"

class FArchive;

namespace UE
{
namespace Geometry
{

/**
 * FSmallListSet stores a set of short integer-valued variable-size lists.
 * The lists are encoded into a few large TDynamicVector buffers, with internal pooling,
 * so adding/removing lists usually does not involve any new or delete ops.
 * 
 * The lists are stored in two parts. The first N elements are stored in a linear
 * subset of a TDynamicVector. If the list spills past these N elements, the extra elements
 * are stored in a linked list (which is also stored in a flat array).
 * 
 * Each list stores its count, so list-size operations are constant time.
 * All the internal "pointers" are 32-bit.
 * 
 * @todo look at usage of TFunction, are we making unnecessary copies?
 */
class FSmallListSet
{
protected:
	/** This value is used to indicate Null in internal pointers */
	static constexpr int32 NullValue = -1;

	/** size of initial linear-memory portion of lists */
	static constexpr int32 BLOCKSIZE = 8;
	/** offset from start of linear-memory portion of list that contains pointer to head of variable-length linked list */
	static constexpr int32 BLOCK_LIST_OFFSET = BLOCKSIZE + 1;

	/** mapping from list index to offset into ListBlocks that contains list data */
	TDynamicVector<int32> ListHeads{};

	/** 
	 * flat buffer used to store per-list linear-memory blocks. 
	 * blocks are BLOCKSIZE+2 long, elements are [CurrentCount, item0...itemN, LinkedListPtr]
	 */
	TDynamicVector<int32> ListBlocks{};

	/** list of free blocks as indices/offsets into ListBlocks */
	TDynamicVector<int32> FreeBlocks{};

	/** number of allocated lists */
	int32 AllocatedCount{0};

	/**
	 * flat buffer used to store linked-list "spill" elements
	 * each element is [value, next_ptr]
	 */
	TDynamicVector<int32> LinkedListElements{};

	/** index of first free element in LinkedListElements */
	int32 FreeHeadIndex{NullValue};


public:
	/**
	 * @return largest current list index
	 */
	size_t Size() const 
	{
		return ListHeads.GetLength();
	}

	/**
	 * set new number of lists
	 */
	GEOMETRYCORE_API void Resize(int32 NewSize);


	/**
	 * resize to a new number of lists and pre-allocate an initial empty block for each list.
	 */
	GEOMETRYCORE_API void ResizeAndAllocateBlocks(int32 NewSize);

	/**
	 * Reset to initial state
	 */
	void Reset()
	{
		ListHeads.Clear();
		ListBlocks.Clear();
		FreeBlocks.Clear();
		AllocatedCount = 0;
		LinkedListElements.Clear();
		FreeHeadIndex = NullValue;
	}

	/**
	 * Clearing any lists at or after MaxListIndex and compact the ListBlocks and LinkedListElements so there are no free blocks or free linked list elements
	 */
	void Compact(int32 MaxListIndex);


	/**
	 * @return true if a list has been allocated at the given ListIndex
	 */
	bool IsAllocated(int32 ListIndex) const
	{
		return (ListIndex >= 0 && ListIndex < (int32)ListHeads.GetLength() && ListHeads[ListIndex] != NullValue);
	}

	/**
	 * Create a list at the given ListIndex
	 */
	GEOMETRYCORE_API void AllocateAt(int32 ListIndex);


	/**
	 * Insert Value into list at ListIndex
	 */
	GEOMETRYCORE_API void Insert(int32 ListIndex, int32 Value);



	/**
	 * remove Value from the list at ListIndex
	 * @return false if Value was not in this list
	 */
	GEOMETRYCORE_API bool Remove(int32 ListIndex, int32 Value);



	/**
	 * Move list at FromIndex to ToIndex
	 */
	GEOMETRYCORE_API void Move(int32 FromIndex, int32 ToIndex);



	/**
	 * Remove all elements from the list at ListIndex
	 */
	GEOMETRYCORE_API void Clear(int32 ListIndex);


	/**
	 * @return the size of the list at ListIndex
	 */
	inline int32 GetCount(int32 ListIndex) const
	{
		checkSlow(ListIndex >= 0);
		int32 block_ptr = ListHeads[ListIndex];
		return (block_ptr == NullValue) ? 0 : ListBlocks[block_ptr];
	}


	/**
	 * @return the first item in the list at ListIndex
	 * @warning does not check for zero-size-list!
	 */
	inline int32 First(int32 ListIndex) const
	{
		checkSlow(ListIndex >= 0);
		int32 block_ptr = ListHeads[ListIndex];
		return ListBlocks[block_ptr + 1];
	}


	/**
	 * Search for the given Value in list at ListIndex
	 * @return true if found
	 */
	GEOMETRYCORE_API bool Contains(int32 ListIndex, int32 Value) const;


	/**
	 * Search the list at ListIndex for a value where PredicateFunc(value) returns true
	 * @return the found value, or the InvalidValue argument if not found
	 */
	template<typename IntToBoolFunc>
	inline int32 Find(int32 ListIndex, const IntToBoolFunc& PredicateFunc, int32 InvalidValue = -1) const
	{
		int32 block_ptr = ListHeads[ListIndex];
		if (block_ptr != NullValue)
		{
			int32 N = ListBlocks[block_ptr];
			if (N < BLOCKSIZE)
			{
				int32 iEnd = block_ptr + N;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					int32 Value = ListBlocks[i];
					if (PredicateFunc(Value))
					{
						return Value;
					}
				}
			}
			else
			{
				// we spilled to linked list, have to iterate through it as well
				int32 iEnd = block_ptr + BLOCKSIZE;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					int32 Value = ListBlocks[i];
					if (PredicateFunc(Value))
					{
						return Value;
					}
				}
				int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
				while (cur_ptr != NullValue)
				{
					int32 Value = LinkedListElements[cur_ptr];
					if (PredicateFunc(Value))
					{
						return Value;
					}
					cur_ptr = LinkedListElements[cur_ptr + 1];
				}
			}
		}
		return InvalidValue;
	}



	/**
	 * Search the list at ListIndex for a value where PredicateFunc(value) returns true, and replace it with NewValue
	 * @return true if the value was found and replaced
	 */
	template<typename IntToBoolFunc>
	inline bool Replace(int32 ListIndex, const IntToBoolFunc& PredicateFunc, int32 NewValue)
	{
		int32 block_ptr = ListHeads[ListIndex];
		if (block_ptr != NullValue)
		{
			int32 N = ListBlocks[block_ptr];
			if (N < BLOCKSIZE)
			{
				int32 iEnd = block_ptr + N;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					int32 Value = ListBlocks[i];
					if (PredicateFunc(Value))
					{
						ListBlocks[i] = NewValue;
						return true;
					}
				}
			}
			else
			{
				// we spilled to linked list, have to iterate through it as well
				int32 iEnd = block_ptr + BLOCKSIZE;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					int32 Value = ListBlocks[i];
					if (PredicateFunc(Value))
					{
						ListBlocks[i] = NewValue;
						return true;
					}
				}
				int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
				while (cur_ptr != NullValue)
				{
					int32 Value = LinkedListElements[cur_ptr];
					if (PredicateFunc(Value))
					{
						LinkedListElements[cur_ptr] = NewValue;
						return true;
					}
					cur_ptr = LinkedListElements[cur_ptr + 1];
				}
			}
		}
		return false;
	}



	/**
	 * Call ApplyFunc on each element of the list at ListIndex
	 */
	template<typename IntToVoidFunc>
	inline void Enumerate(int32 ListIndex, const IntToVoidFunc& ApplyFunc) const
	{
		int32 block_ptr = ListHeads[ListIndex];
		if (block_ptr != NullValue)
		{
			int32 N = ListBlocks[block_ptr];
			if (N < BLOCKSIZE)
			{
				int32 iEnd = block_ptr + N;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					ApplyFunc(ListBlocks[i]);
				}
			}
			else
			{
				// we spilled to linked list, have to iterate through it as well
				int32 iEnd = block_ptr + BLOCKSIZE;
				for (int32 i = block_ptr + 1; i <= iEnd; ++i)
				{
					ApplyFunc(ListBlocks[i]);
				}
				int32 cur_ptr = ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
				while (cur_ptr != NullValue)
				{
					ApplyFunc(LinkedListElements[cur_ptr]);
					cur_ptr = LinkedListElements[cur_ptr + 1];
				}
			}
		}
	}

	/**
	 * Append all elements from another small list set, with ElementOffset added to all such appended elements
	 * Note: Also directly appends free blocks / free lists; does no compacting
	 * @param Other the list to append
	 * @param ElementOffset amount to shift all appended elements
	 */
	void AppendWithElementOffset(const FSmallListSet& Other, int32 ElementOffset);

	/**
	 * Call ApplyFunc on each element of the list at ListIndex, until ApplyFunc returns false
	 * @return true if all elements were processed and ApplyFunc never returned false
	 */
	bool EnumerateEarlyOut(int32 ListIndex, TFunctionRef<bool(int32)> ApplyFunc) const;

	/**
	 * Serialization operator for FSmallListSet.
	 *
	 * @param Ar Archive to serialize with.
	 * @param Set Set to serialize.
	 * @returns Passing down serializing archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FSmallListSet& Set)
	{
		Set.Serialize(Ar, false, false);
		return Ar;
	}

	/**
	* Serialize FSmallListSet to an archive.
	* @param Ar Archive to serialize with
	* @param bCompactData Only serialize unique data and/or recompute redundant data when loading.
	* @param bUseCompression Use compression to serialize data; the resulting size will likely be smaller but serialization will take significantly longer.
	*/
	GEOMETRYCORE_API void Serialize(FArchive& Ar, bool bCompactData, bool bUseCompression);

	friend bool operator==(const FSmallListSet& Lhs, const FSmallListSet& Rhs)
	{
		if (Lhs.Size() != Rhs.Size())
		{
			return false;
		}
	
		for (int32 ListIndex = 0, ListNum = Lhs.Size(); ListIndex < ListNum; ++ListIndex)
		{
			if (Lhs.GetCount(ListIndex) != Rhs.GetCount(ListIndex))
			{
				return false;
			}

			ValueIterator ItLhs = Lhs.BeginValues(ListIndex);
			ValueIterator ItRhs = Rhs.BeginValues(ListIndex);
			const ValueIterator ItLhsEnd = Lhs.EndValues(ListIndex);
			const ValueIterator ItRhsEnd = Rhs.EndValues(ListIndex);
			while (ItLhs != ItLhsEnd && ItRhs != ItRhsEnd)
			{
				if (*ItLhs != *ItRhs)
				{
					return false;
				}
				++ItLhs;
				++ItRhs;
			}
			if (ItLhs != ItLhsEnd || ItRhs != ItRhsEnd)
			{
				return false;
			}
		}
		
		return true;
	}

	friend bool operator!=(const FSmallListSet& Lhs, const FSmallListSet& Rhs)
	{
		return !(Lhs == Rhs);
	}

	//
	// iterator support
	// 


	friend class ValueIterator;
	friend class BaseValueIterator;

	/**
	 * BaseValueIterator is a base class for ValueIterator and MappedValueIterator below.
	 */
	class BaseValueIterator
	{
	public:
		BaseValueIterator()
		{ 
			ListSet = nullptr;
			ListIndex = 0; 
		}

		inline bool operator==(const BaseValueIterator& Other) const
		{
			return ListSet == Other.ListSet && ListIndex == Other.ListIndex;
		}
		inline bool operator!=(const BaseValueIterator& Other) const
		{
			return ListSet != Other.ListSet || ListIndex != Other.ListIndex || iCur != Other.iCur || cur_ptr != Other.cur_ptr;
		}

	protected:
		inline void GotoNext() 
		{
			if (N == 0)
			{
				SetToEnd();
				return;
			}
			GotoNextOverflow();
		}

		inline void GotoNextOverflow() 
		{
			if (iCur <= iEnd) 
			{
				cur_value = ListSet->ListBlocks[iCur];
				iCur++;
			}
			else if (cur_ptr != NullValue) 
			{
				cur_value = ListSet->LinkedListElements[cur_ptr];
				cur_ptr = ListSet->LinkedListElements[cur_ptr + 1];
			}
			else
			{
				SetToEnd();
			}
		}

		BaseValueIterator(
			const FSmallListSet* ListSetIn,
			int32 ListIndex, 
			bool is_end)
		{
			this->ListSet = ListSetIn;
			this->ListIndex = ListIndex;
			if (is_end) 
			{
				SetToEnd();
			}
			else 
			{
				block_ptr = ListSet->ListHeads[ListIndex];
				if (block_ptr != ListSet->NullValue)
				{
					N = ListSet->ListBlocks[block_ptr];
					iEnd = (N < BLOCKSIZE) ? (block_ptr + N) : (block_ptr + BLOCKSIZE);
					iCur = block_ptr + 1;
					cur_ptr = (N < BLOCKSIZE) ? NullValue : ListSet->ListBlocks[block_ptr + BLOCK_LIST_OFFSET];
					GotoNext();
				}
				else
				{
					SetToEnd();
				}
			}
		}

		inline void SetToEnd() 
		{
			block_ptr = ListSet->NullValue;
			N = 0;
			iCur = -1;
			cur_ptr = -1;
		}

		const FSmallListSet * ListSet;
		int32 ListIndex;
		int32 block_ptr;
		int32 N;
		int32 iEnd;
		int32 iCur;
		int32 cur_ptr;
		int32 cur_value;
		friend class FSmallListSet;
	};


	/**
	 * ValueIterator iterates over the values of a small list
	 */
	class ValueIterator : public BaseValueIterator
	{
	public:
		ValueIterator() : BaseValueIterator() {}

		inline int32 operator*() const
		{
			return cur_value;
		}

		inline const ValueIterator& operator++() 		// prefix
		{
			this->GotoNext();
			return *this;
		}

	protected:
		ValueIterator(const FSmallListSet* ListSetIn, int32 ListIndex, bool is_end) 
			: BaseValueIterator(ListSetIn, ListIndex, is_end) {}

		friend class FSmallListSet;
	};

	/**
	 * @return iterator for start of list at ListIndex
	 */
	inline ValueIterator BeginValues(int32 ListIndex) const 
	{
		return ValueIterator(this, ListIndex, false);
	}

	/**
	 * @return iterator for end of list at ListIndex
	 */
	inline ValueIterator EndValues(int32 ListIndex) const 
	{
		return ValueIterator(this, ListIndex, true);
	}

	/**
	 * ValueEnumerable is an object that provides begin/end semantics for a small list, suitable for use with a range-based for loop
	 */
	class ValueEnumerable
	{
	public:
		const FSmallListSet* ListSet;
		int32 ListIndex;
		ValueEnumerable() {}
		ValueEnumerable(const FSmallListSet* ListSetIn, int32 ListIndex)
		{
			this->ListSet = ListSetIn;
			this->ListIndex = ListIndex;
		}
		typename FSmallListSet::ValueIterator begin() const { return ListSet->BeginValues(ListIndex); }
		typename FSmallListSet::ValueIterator end() const { return ListSet->EndValues(ListIndex); }
	};

	/**
	 * @return a value enumerable for the given ListIndex
	 */
	inline ValueEnumerable Values(int32 ListIndex) const
	{
		return ValueEnumerable(this, ListIndex);
	}






	//
	// mapped iterator support - mapped iterator applies an arbitrary function to the iterator value
	// 

	friend class MappedValueIterator;

	/**
	 * MappedValueIterator iterates over the values of a small list
	 * An optional mapping function can be provided which will then be applied to the values returned by the * operator
	 */
	class MappedValueIterator : public BaseValueIterator
	{
	public:
		MappedValueIterator() : BaseValueIterator() 
		{
			MapFunc = [](int32 value) { return value; };
		}

		inline int32 operator*() const
		{
			return MapFunc(cur_value);
		}

		inline const MappedValueIterator& operator++() 		// prefix
		{
			this->GotoNext();
			return *this;
		}

	protected:
		MappedValueIterator(const FSmallListSet* ListSetIn, int32 ListIndex, bool is_end, TFunction<int32(int32)> MapFuncIn)
			: BaseValueIterator(ListSetIn, ListIndex, is_end) 
		{
			MapFunc = MapFuncIn;
		}

		TFunction<int32(int32)> MapFunc;
		friend class FSmallListSet;
	};

	/**
	 * @return iterator for start of list at ListIndex, with given value mapping function
	 */
	inline MappedValueIterator BeginMappedValues(int32 ListIndex, const TFunction<int32(int32)>& MapFunc) const
	{
		return MappedValueIterator(this, ListIndex, false, MapFunc);
	}

	/**
	 * @return iterator for end of list at ListIndex, with given value mapping function
	 */
	inline MappedValueIterator EndMappedValues(int32 ListIndex, const TFunction<int32(int32)>& MapFunc) const
	{
		return MappedValueIterator(this, ListIndex, true, MapFunc);
	}

	/**
	 * MappedValueEnumerable is an object that provides begin/end semantics for a small list, suitable for use with a range-based for loop
	 */
	class MappedValueEnumerable
	{
	public:
		const FSmallListSet* ListSet;
		int32 ListIndex;
		TFunction<int32(int32)> MapFunc;
		MappedValueEnumerable() {}
		MappedValueEnumerable(const FSmallListSet* ListSetIn, int32 ListIndex, TFunction<int32(int32)> MapFunc)
		{
			this->ListSet = ListSetIn;
			this->ListIndex = ListIndex;
			this->MapFunc = MoveTemp(MapFunc);
		}
		typename FSmallListSet::MappedValueIterator begin() const { return ListSet->BeginMappedValues(ListIndex, MapFunc); }
		typename FSmallListSet::MappedValueIterator end() const { return ListSet->EndMappedValues(ListIndex, MapFunc); }
	};

	/**
	 * @return a value enumerable for the given ListIndex, with the given value mapping function
	 */
	inline MappedValueEnumerable MappedValues(int32 ListIndex, TFunction<int32(int32)> MapFunc) const
	{
		return MappedValueEnumerable(this, ListIndex, MapFunc);
	}







protected:


	// grab a block from the free list, or allocate a new one
	GEOMETRYCORE_API int32 AllocateBlock();

	// push a link-node onto the free list
	inline void AddFreeLink(int32 ptr)
	{
		LinkedListElements[ptr + 1] = FreeHeadIndex;
		FreeHeadIndex = ptr;
	}


	// remove val from the linked-list attached to block_ptr
	GEOMETRYCORE_API bool RemoveFromLinkedList(int32 block_ptr, int32 val);


public:
	inline FString MemoryUsage() const
	{
		return FString::Printf(TEXT("ListSize %zu  Blocks Count %d  Free %zu  Mem %zukb   Linked Mem %zukb"),
			ListHeads.GetLength(), AllocatedCount, (FreeBlocks.GetLength() * sizeof(int32) / 1024),
			ListBlocks.GetLength(), (LinkedListElements.GetLength() * sizeof(int32) / 1024));
	}

	SIZE_T GetByteCount() const
	{
		return ListHeads.GetByteCount() + FreeBlocks.GetByteCount() + ListBlocks.GetByteCount() + LinkedListElements.GetByteCount();
	}

};


} // end namespace UE::Geometry
} // end namespace UE
