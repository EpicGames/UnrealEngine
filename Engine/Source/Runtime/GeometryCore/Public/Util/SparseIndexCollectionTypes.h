// Copyright Epic Games, Inc. All Rights Reserved.

// partial port of geometry3sharp Indexing.cs

#pragma once

#include "IndexTypes.h"

#include <CoreMinimal.h>


namespace UE
{
namespace Geometry
{

/**
 *  This class provides optionally sparse or dense boolean flags for a set of integer indices
 */
class FIndexFlagSet
{
private:
	TOptional<TArray<bool>> Dense; // TODO: consider switching to bit array; but test first as performance of bit array may by significantly worse
	int DenseCount;      // only tracked for Dense
	TOptional<TSet<int>> Sparse;

public:
	FIndexFlagSet(bool bSetSparse = true, int MaxIndex = -1)
	{
		InitManual(bSetSparse, MaxIndex);
	}

	FIndexFlagSet(int MaxIndex, int SubsetCountEst)
	{
		InitAuto(MaxIndex, SubsetCountEst);
	}

	/**
	 * Initialize to either sparse or dense automatically, based on max index and estimated number of set indices
	 */
	void InitAuto(int MaxIndex, int SubsetCountEst)
	{
		bool bSmall = MaxIndex < 32000;
		constexpr float PercentThresh = 0.05f;           

		InitManual(!bSmall && ((float)SubsetCountEst / (float)MaxIndex < PercentThresh), MaxIndex);
	}

	/**
	 * Initialize to sparse or dense based on the explicit caller choice
	 * @param bSetSparse if true, will use sparse storage; otherwise will use dense
	 * @param MaxIndex maximum valid index in set; must be non-negative for dense case
	 */
	void InitManual(bool bSetSparse, int MaxIndex = -1)
	{
		if (bSetSparse)
		{
			Sparse = TSet<int>();
		}
		else
		{
			Dense = TArray<bool>();
			check(MaxIndex >= 0);
			Dense->SetNumZeroed(FMath::Max(0, MaxIndex));
		}
		DenseCount = 0;
	}

	/**
	 * @param Index index to check for set membership
	 * @return true if Index is in set
	 */
	inline bool Contains(int Index) const
	{
		if (Dense.IsSet())
		{
			return Dense.GetValue()[Index];
		}
		else
		{
			check(Sparse.IsSet());
			return Sparse->Contains(Index);
		}
	}

	/**
	 * @param Index index to add to set
	 */
	inline void Add(int Index)
	{
		if (Dense.IsSet())
		{
			bool &Value = Dense.GetValue()[Index];
			if (!Value)
			{
				Value = true;
				DenseCount++;
			}
		}
		else
		{
			check(Sparse.IsSet());
			Sparse->Add(Index);
		}
	}

	/**
	 * @param Index index to remove from set
	 */
	inline void Remove(int Index)
	{
		if (Dense.IsSet())
		{
			bool &Value = Dense.GetValue()[Index];
			if (Value)
			{
				Value = false;
				DenseCount--;
			}
		}
		else
		{
			check(Sparse.IsSet());
			Sparse->Remove(Index);
		}
	}

	/**
	 *  Returns number of values in set
	 */
	inline int Count() const
	{
		if (Dense.IsSet())
		{
			return DenseCount;
		}
		else
		{
			return Sparse->Num();
		}
	}

	/**
	 * Array-style accessor that returns true if Index is in set, false otherwise
	 */
	inline const bool operator[](unsigned int Index) const
	{
		return Contains(Index);
	}
	

	// TODO: iterator support

};

/**
 * Index map that supports dense or sparse storage, or a simple formula-based map (e.g. constant, identity, shift)
 * For dense and sparse, the formula can be used to set default values.
 */
struct FOptionallySparseIndexMap
{
	// storage used when map is dense
	TArray<int> Dense;
	// storage used when map is sparse
	TMap<int, int> Sparse;

	// mapping to assign defaults for indices that are not explicitly assigned; computed as Index*DefaultScale + DefaultOffset
	int DefaultOffset = -1, DefaultScale = 0;

	// max index in map; -1 will leave the map unbounded (invalid for Dense maps)
	int MaxIndex = -1;

	// different kinds of storage that could be used for the map
	enum class EMapType : uint8
	{
		Dense, Sparse, ScaleAndOffset
	};

	// choice of storage backing the map
	EMapType MapType = EMapType::Sparse;

	/** default constructor sets map to sparse, w/ constant -1 default value */
	FOptionallySparseIndexMap()
	{}

	/** construct map with explicit choice of storage type */
	FOptionallySparseIndexMap(EMapType MapType, int MaxIndex = -1) : MaxIndex(MaxIndex), MapType(MapType)
	{
		if (MapType == EMapType::Dense)
		{
			check(MaxIndex > -1);
			Dense.SetNum(MaxIndex);
			InitDefaults();
		}
	}

	/** construct map with dense storage copied from an existing array */
	FOptionallySparseIndexMap(TArray<int> Dense, int MaxIndex = -1) : Dense(Dense), MaxIndex(MaxIndex), MapType(EMapType::Dense)
	{
	}

	/** construct map with automatically-chosen dense or sparse storage, based on max index and estimated element count */
	FOptionallySparseIndexMap(int MaxIndex, int SubsetCountEst)
	{
		Initialize(MaxIndex, SubsetCountEst);
	}

	/**
	 * Automatically choose sparse or dense storage based on use estimate
	 */
	void Initialize(int MaxIndexIn, int SubsetCountEst)
	{
		Dense.Reset();
		Sparse.Reset();
		MaxIndex = MaxIndexIn;

		// if buffer is less than threshold, just use dense map
		bool bSmall = MaxIndex < 32000;
		float fPercent = (float)SubsetCountEst / (float)MaxIndex;
		float fPercentThresh = 0.1f;

		if (bSmall || fPercent > fPercentThresh)
		{
			Dense.SetNum(MaxIndex);
			MapType = EMapType::Dense;
		}
		else
		{
			MapType = EMapType::Sparse;
		}
		
		InitDefaults();
	}

	/**
	 * @return an identity map
	 */
	static FOptionallySparseIndexMap IdentityMap(int MaxIndex = -1)
	{
		FOptionallySparseIndexMap ToRet(EMapType::ScaleAndOffset, MaxIndex);
		ToRet.DefaultScale = 1;
		ToRet.DefaultOffset = 0;
		return ToRet;
	}
	
	/**
	 * @return a constant map
	 */
	static FOptionallySparseIndexMap ConstantMap(int ConstantValue, int MaxIndex = -1)
	{
		FOptionallySparseIndexMap ToRet(EMapType::ScaleAndOffset, MaxIndex);
		ToRet.DefaultScale = 0;
		ToRet.DefaultOffset = ConstantValue;
		return ToRet;
	}

	/**
	 * @return a sparsely-overriden an identity map; elements not explicitly set are identity
	 */
	static FOptionallySparseIndexMap SparseIdentityMap(int MaxIndex = -1)
	{
		FOptionallySparseIndexMap ToRet(EMapType::Sparse, MaxIndex);
		ToRet.DefaultScale = 1;
		ToRet.DefaultOffset = 0;
		return ToRet;
	}


	/* Initialize default values for dense map (sparse defaults are computed on the fly) */
	void InitDefaults()
	{
		if (MapType == EMapType::Dense)
		{
			for (int i = 0; i < Dense.Num(); ++i)
			{
				Dense[i] = i*DefaultScale + DefaultOffset;
			}
		}
	}

	/** @return true if a given Index is invalid (below zero or above MaxIndex if MaxIndex is valid) */
	inline bool BadIndex(int Index) const
	{
		return (Index < 0) || (MaxIndex >= 0 && Index >= MaxIndex);
	}

	/**
	 * dense variant: returns true unless you have set index to InvalidIndex (eg via SetToInvalid)
	 * sparse variant: returns true if index is in map or default index is >= 0
	 * scaleandoffset variant: returns true if default index is >= 0
	 * all return false if index is out-of-bounds
	 */
	bool Contains(int Index)
	{
		if (BadIndex(Index))
		{
			return false;
		}
		switch (MapType)
		{
		case EMapType::Dense:
			return Dense[Index] >= 0;
		case EMapType::Sparse:
			return Sparse.Contains(Index) || (Index * DefaultScale + DefaultOffset >= 0);
		case EMapType::ScaleAndOffset:
			return Index * DefaultScale + DefaultOffset >= 0;
		}
		check(false);
		return false;
	}

	/**
	 * Array-style accessor to the map
	 */
	inline const int operator[](int Index) const
	{
		if (BadIndex(Index))
		{
			return IndexConstants::InvalidID;
		}
		switch (MapType)
		{
		case EMapType::Dense:
			return Dense[Index];
		case EMapType::Sparse:
		{
			const int *Value = Sparse.Find(Index);
			return Value ? (*Value) : (Index * DefaultScale + DefaultOffset);
		}
		case EMapType::ScaleAndOffset:
			return Index * DefaultScale + DefaultOffset;
		}
		check(false);
		return IndexConstants::InvalidID;
	}

	/**
	 * @param Index this index will be unset in map, restoring the default mapping (Index -> Index*DefaultScale + DefaultOffset)
	 */
	void Unset(int Index)
	{
		check(!BadIndex(Index));
		if (MapType == EMapType::Dense)
		{
			Dense[Index] = Index*DefaultScale + DefaultOffset;
		}
		else
		{
			Sparse.Remove(Index);
		}
	}

	/** Sets the map at a given index.  Do not call if MapType is ScaleAndOffset. */
	void Set(int Index, int Value)
	{
		check(!BadIndex(Index));
		check(MapType != EMapType::ScaleAndOffset);
		if (MapType == EMapType::Dense)
		{
			Dense[Index] = Value;
		}
		else
		{
			Sparse.FindOrAdd(Index) = Value;
		}
	}

	/** @param Index Explicitly sets the given index to Invalid (*not* default).  Do not call if MapType is ScaleAndOffset. */
	inline void SetInvalid(int Index)
	{
		Set(Index, IndexConstants::InvalidID);
	}
};

} // end namespace UE::Geometry
} // end namespace UE
