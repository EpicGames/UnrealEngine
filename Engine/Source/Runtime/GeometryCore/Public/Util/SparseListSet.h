// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp small_list_set

#pragma once

#include "HAL/PlatformMath.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Util/DynamicVector.h"

class FArchive;

namespace UE
{
namespace Geometry
{

/**
 * TSparseListSet stores a sparse set of indexed variable-size lists. This is useful in
 * cases where many relatively small lists that have a 1-1 correspondence with a bounded
 * index space are needed, eg for per-vertex lists for a mesh, per-cell lists for an octree, etc.
 *
 * The interface is compatible with FSmallListSet, which may be more effective for very small
 * lists (optimized for ~8 elements), but does not scale very well to larger lists (10s-to-100s). 
 * 
 * Internally, TSparseListSet uses a strategy where the small lists are stored in sequences of fixed-size blocks,
 * similar to FDynamicVector, but the block size is configurable. The blocks are allocated from
 * larger "chunks" of blocks (also of configurable size). These parameters can be set on construction
 * to values appropriate for particular use cases.
 * 
 * TSparseListSet can support parallel insertion into the lists. Note however that lists cannot
 * be added (via AllocateAt) while parallel insertion occurs via Insert(Index). To allow for
 * this type of usage, AllocateAt() returns a FListHandle which can then be used with Insert()
 * or SetValues() even while other AllocateAt() calls occur (within a lock). 
 */ 
template<typename ElementType>
struct TSparseListSet
{
public:

	TSparseListSet()
	{
		ChunkList.Add(MakeUnique<FChunk>());
		ChunkList[0]->Initialize(BlocksPerChunk * BlockSize);
	}

	/**
	 * @param BlockSizeIn defines size of small blocks that each list is made up of
	 * @param BlocksPerChunkIn defines size of large chunks that small blocks are allocated from
	 */
	TSparseListSet(int BlockSizeIn, int BlocksPerChunkIn)
	{
		BlockSize = FMath::Clamp(BlockSizeIn, 4, 65536);
		BlocksPerChunk = FMath::Clamp(BlocksPerChunkIn, 4, 65536);
		ChunkList.Add(MakeUnique<FChunk>());
		ChunkList[0]->Initialize(BlocksPerChunk * BlockSize);
	}


	/**
	 * @return true if a list has been allocated at the given ListIndex
	 */
	bool IsAllocated(int32 ListIndex) const
	{
		return (ListIndex >= 0 && ListIndex < (int32)Lists.Num() && Lists[ListIndex].CurBlock != nullptr );
	}

	// handle to an allocated list returned by AllocateAt()
	struct FListHandle
	{
		void* ListRef = nullptr;
	};

	/**
	 * Create a list at the given ListIndex
	 * @return Handle to allocted list that can be used with Insert and SetValues in multi-threaded contexts
	 */
	FListHandle AllocateAt(int32 ListIndex)
	{
		FList NewList;
		NewList.Blocks.Add( AllocateNewBlockMemory() );
		NewList.CurBlock = NewList.Blocks[0];
		NewList.CurIndex = 0;
		Lists.InsertAt(NewList, ListIndex);
		return FListHandle{(void*)&Lists[ListIndex]};  // this pointer is stable because of DynamicVector
	}

	/**
	 * Insert Value into List at ListIndex
	 */
	void Insert(int32 ListIndex, ElementType Value)
	{
		checkSlow(IsAllocated(ListIndex));
		Insert_Internal(Lists[ListIndex], Value);
	}

	/**
	 * Insert Value into List identified by ListHandle
	 */
	void Insert(FListHandle ListHandle, ElementType Value)
	{
		FList& List = *(FList*)ListHandle.ListRef;
		Insert_Internal(List, Value);
	}


	/**
	 * Set the values of the List at ListIndex
	 */
	void SetValues(int32 ListIndex, const TArray<ElementType>& Values)
	{
		checkSlow(IsAllocated(ListIndex));
		SetValues_Internal(Lists[ListIndex], Values);
	}


	/**
	 * Set the values of the List identified by ListHandle
	 */
	void SetValues(FListHandle ListHandle, const TArray<ElementType>& Values)
	{
		FList& List = *(FList*)ListHandle.ListRef;
		SetValues_Internal(List, Values);
	}


	/**
	 * remove first occurrence of Value from the list at ListIndex
	 * @return false if Value was not in this list
	 */
	inline bool Remove(int32 ListIndex, ElementType Value);


	/**
	 * Remove all elements from the list at ListIndex
	 */
	void Clear(int32 ListIndex)
	{
		checkSlow(IsAllocated(ListIndex));
		FList& List = Lists[ListIndex];
		Clear_Internal(List);
	}


	/**
	 * @return the size of the list at ListIndex
	 */
	int32 GetCount(int32 ListIndex) const
	{
		checkSlow(IsAllocated(ListIndex));
		const FList& List = Lists[ListIndex];
		return FMath::Max(0, List.Blocks.Num()-1)*BlockSize + List.CurIndex;
	}


	/**
	 * Search for the given Value in list at ListIndex
	 * @return true if found
	 */
	bool Contains(int32 ListIndex, ElementType Value) const;


	/**
	 * Call ApplyFunc on each element of the list at ListIndex
	 */
	template<typename FuncType>
	void Enumerate(int32 ListIndex, FuncType Func) const
	{
		checkSlow(IsAllocated(ListIndex));
		const FList& List = Lists[ListIndex];
		if ( List.CurBlock != nullptr )
		{
			int32 N = List.Blocks.Num();
			for (int32 BlockIndex = 0; BlockIndex < N; ++BlockIndex)
			{
				const ElementType* Elements = List.Blocks[BlockIndex];
				const ElementType* EndElement = (BlockIndex == N-1) ? (Elements + List.CurIndex) : (Elements + BlockSize);
				while (Elements != EndElement)
				{
					Func(*Elements++);
				}
			}
		}
	}

	/**
	 * Call ApplyFunc on each element of the list at ListIndex, until ApplyFunc returns false
	 * @return true if all elements were processed and ApplyFunc never returned false
	 */
	template<typename FuncType>
	bool EnumerateEarlyOut(int32 ListIndex, FuncType ApplyFunc) const
	{
		checkSlow(IsAllocated(ListIndex));
		const FList& List = Lists[ListIndex];
		if (List.CurBlock != nullptr)
		{
			int32 N = List.Blocks.Num();
			for (int32 BlockIndex = 0; BlockIndex < N; ++BlockIndex)
			{
				const ElementType* Elements = List.Blocks[BlockIndex];
				const ElementType* EndElement = (BlockIndex == N - 1) ? (Elements + List.CurIndex) : (Elements + BlockSize);
				while (Elements != EndElement)
				{
					if (!ApplyFunc(*Elements++))
					{
						return false;
					}
				}
			}
		}
		return true;
	}



private:

	// size of a list memory block, ie lists are allocated in blocks of this size, so also the minimum size (in memory) of a list even if it has 0 elements
	int BlockSize = 32;
	// number of memory blocks in a "chunk", chunks are allocated as needed, so minimum size of a TSparseListSet is (BlocksPerChunk * BlockSize * sizeof(ElementType))
	int BlocksPerChunk = 256;

	// large block of memory from which smaller blocks are allocated
	struct FChunk
	{
		int BlocksAllocated = 0;
		void Initialize(int Size) { ElementBuffer.SetNumUninitialized(Size); }
		ElementType* GetBlockBufferPtr(int32 Offset) { return &ElementBuffer[Offset]; }
	private:
		TArray<ElementType> ElementBuffer;		// note: using TArray here to auto-delete, but this array cannot be resized or modified after creation
	};
	TArray<TUniquePtr<FChunk>> ChunkList;		// could use dynamic vector here to avoid realloc's...?
	TArray<ElementType*> FreeList;
	FCriticalSection AllocationLock;			// lock for ChunkList and FreeList

	struct FList
	{
		TArray<ElementType*, TInlineAllocator<8>> Blocks;		// todo: could replace this with a linked list packed in with the element buffers? would avoid overflow issue...
		ElementType* CurBlock = nullptr;
		int32 CurIndex = 0;
	};

	TDynamicVector<FList> Lists;		// this is the list of small lists


	void Insert_Internal(FList& List, ElementType Value)
	{
		if (List.CurIndex == BlockSize)
		{
			List.Blocks.Add( AllocateNewBlockMemory() );
			List.CurBlock = List.Blocks.Last();
			List.CurIndex = 0;
		}
		List.CurBlock[List.CurIndex++] = Value;
	}

	void SetValues_Internal(FList& List, const TArray<ElementType>& Values)
	{
		int32 N = Values.Num();
		if (List.CurIndex != 0)
		{
			Clear_Internal(List);
		}
		for (int32 k = 0; k < N; ++k)
		{
			// could avoid this branch by doing sub-iterations over BlockSize?
			if (List.CurIndex == BlockSize)
			{
				List.Blocks.Add( AllocateNewBlockMemory() );
				List.CurBlock = List.Blocks.Last();
				List.CurIndex = 0;
			}
			List.CurBlock[List.CurIndex++] = Values[k];
		}
	}


	void Clear_Internal(FList& List)
	{
		for ( int32 k = 1; k < List.Blocks.Num(); ++k)
		{
			FreeBlockMemory(List.Blocks[k]);
		}
		List.Blocks.SetNum(1);
		List.CurBlock = List.Blocks[0];
		List.CurIndex = 0;
	}


	ElementType* AllocateNewBlockMemory()
	{
		FScopeLock Lock(&AllocationLock);

		if (FreeList.Num() > 0)		// return from free list if available
		{
			return FreeList.Pop();
		}

		FChunk* LastChunk = ChunkList.Last().Get();
		if (LastChunk->BlocksAllocated == BlocksPerChunk)
		{
			ChunkList.Add(MakeUnique<FChunk>());
			LastChunk = ChunkList.Last().Get();
			LastChunk->Initialize( BlocksPerChunk * BlockSize );
		}

		ElementType* BlockBuffer = LastChunk->GetBlockBufferPtr(LastChunk->BlocksAllocated * BlockSize);
		LastChunk->BlocksAllocated++;

		return BlockBuffer;
	}

	void FreeBlockMemory(ElementType* Block)
	{
		FScopeLock Lock(&AllocationLock);
		FreeList.Add(Block);
	}

};



template<typename ElementType>
bool TSparseListSet<ElementType>::Remove(int32 ListIndex, ElementType Value)
{
	checkSlow(IsAllocated(ListIndex));
	FList& List = Lists[ListIndex];
	if ( List.CurBlock != nullptr )
	{
		int32 N = List.Blocks.Num();
		for (int32 BlockIndex = 0; BlockIndex < N; ++BlockIndex)
		{
			ElementType* Elements = List.Blocks[BlockIndex];
			int Stop = (BlockIndex == N-1) ? List.CurIndex : BlockSize;
			for (int32 k = 0; k < Stop; ++k)
			{
				if (Elements[k] == Value)
				{
					bool bIsLastBlock = (Elements == List.CurBlock);
					if (bIsLastBlock)
					{
						if (k == List.CurIndex-1)
						{
							// removing last element, just decrement counter
							List.CurIndex--;		
						}
						else
						{
							// swap with last element, then decrement
							Swap(Elements[k], Elements[List.CurIndex-1]);
							List.CurIndex--;
						}
					}
					else
					{
						if (List.CurIndex == 0)
						{
							check(false);		// this should only happen if this is the first block and the list is empty, in which case, how could we have found Value?
						}
						Swap(Elements[k], List.CurBlock[List.CurIndex-1]);
						List.CurIndex--;
					}

					// if we got back to index 0, we need to release this block
					if (List.CurIndex == 0 && List.Blocks.Num() > 1)
					{
						FreeBlockMemory(List.CurBlock);
						List.Blocks.Pop();
						List.CurBlock = List.Blocks.Last();
						List.CurIndex = BlockSize;
					}

					return true;
				}
			}
		}
	}	

	return false;
}



template<typename ElementType>
bool TSparseListSet<ElementType>::Contains(int32 ListIndex, ElementType Value) const
{
	checkSlow(IsAllocated(ListIndex));
	const FList& List = Lists[ListIndex];
	int32 N = List.Blocks.Num();
	for (int32 BlockIndex = 0; BlockIndex < N; ++BlockIndex)
	{
		const ElementType* Elements = List.Blocks[BlockIndex];
		const ElementType* EndElement = (BlockIndex == N-1) ? (Elements + List.CurIndex) : (Elements + BlockSize);
		while (Elements != EndElement)
		{
			if ((*Elements++) == Value)
			{
				return true;
			}
		}
	}
	return false;
}


} // end namespace UE::Geometry
} // end namespace UE