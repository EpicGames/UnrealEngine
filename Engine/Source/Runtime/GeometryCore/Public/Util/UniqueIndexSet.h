// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace Geometry
{

/**
 * FUniqueIndexSet is used to construct a list of unique elements of integers in range [0, MaxValue].
 * General usage is for things like converting a vertex set to the set of connected one-ring triangles.
 * The approach is to have a bit-array for set membership, and then accumulate unique entries in a TArray.
 * 
 * This is faster for construction than a TSet, but does require allocating and/or clearing the bit-array.
 * Still generally quite a bit more efficient, particularly for large sets.
 */
class FUniqueIndexSet
{
	// TODO: second bit set that tracks which blocks of ints have been set.
	// This would allow for clears to be much quicker for subsets.
	// (would want to profile to see that it helps, but the bit set is still 0.125mb per million indices,
	// which isn't a trivial amount to set to zero...)

	// TODO: alternately, use a TSet instead of Bits array if we know that the number 
	// of values we are going to add is very small? This would avoid clears.

	// TODO: AtomicAdd()

	// TODO: support growing bit set

public:
	GEOMETRYCORE_API ~FUniqueIndexSet();

	/**
	 * Initialize the set with maximum index. 
	 * @param MaxValuesHint if non-zero, we pre-allocate this much memory for the index array
	 */
	inline void Initialize(int32 MaxIndexIn, int32 MaxValuesHint = 0)
	{
		this->MaxIndex = MaxIndexIn;
		int32 NeedWords = (MaxIndex / 64) + 1;
		if (NumWords < NeedWords)
		{
			if (Bits != nullptr)
			{
				delete[] Bits;
			}
			NumWords = NeedWords;
			Bits = new int64[NumWords];
		}
		FMemory::Memset(&Bits[0], (uint8)0, NeedWords * 8);
		Values.Reset();
		Values.Reserve(MaxValuesHint);
	}

	/** @return number of indices in set */
	inline const int32 Num() const { return Values.Num(); }

	/** @return array of unique indices */
	inline const TArray<int32>& Indices() const { return Values; }

	/**
	 * Add Index to the set
	 */
	inline bool Add(int32 Index)
	{
		int64& Word = Bits[(int64)(Index / 64)];
		int64 Offset = (int64)(Index % 64);
		int64 Mask = (int64)1 << Offset;
		if ( (Word & Mask) == 0 )
		{
			Word |= Mask;
			Values.Add(Index);
			return true;
		}
		return false;
	}

	/**
	 * @return true if Index is in the current set
	 */
	inline bool Contains(int32 Index) const
	{
		const int64& Word = Bits[(int64)(Index / 64)];
		int64 Offset = (int64)(Index % 64);
		int64 Mask = (int64)1 << Offset;
		return (Word & Mask) != 0;
	}

	/**
	 * Add the members of the current index set to an object that supports .Reserve(int) and .Add(int)
	 */
	template<typename ArrayType>
	void Collect(ArrayType& Storage) const
	{
		Storage.Reserve(Values.Num());
		for (int32 Value : Values)
		{
			Storage.Add(Value);
		}
	}

	/**
	 * @return the internal TArray. After calling this, the set is invalid and needs to be Initialize()'d again
	 */
	inline TArray<int32>&& TakeValues()
	{
		return MoveTemp(Values);
	}

	/**
	 * Swap the internal TArray with another TArray. After calling this, the set is invalid and needs to be Initialize()'d again
	 */
	inline void SwapValuesWith(TArray<int32>& OtherArray)
	{
		TArray<int32> Tmp = MoveTemp(Values);
		Values = MoveTemp(OtherArray);
		OtherArray = MoveTemp(Tmp);
	}


public:
	// range-for support. Do not directly call these functions.
	inline TArray<int32>::RangedForConstIteratorType begin() const { return Values.begin(); }
	inline TArray<int32>::RangedForConstIteratorType end() const { return Values.end(); }


protected:
	int32 MaxIndex;
	TArray<int32> Values;
	int64* Bits = nullptr;
	int32 NumWords = 0;
};


} // end namespace UE::Geometry
} // end namespace UE
