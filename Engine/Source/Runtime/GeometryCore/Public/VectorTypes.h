// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "MathUtil.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTypeTraits.h"
#include <sstream>

namespace UE {
namespace Geometry {



/** @return dot product of V1 with PerpCW(V2), ie V2 rotated 90 degrees clockwise */
template <typename T>
constexpr T DotPerp(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	return V1.X * V2.Y - V1.Y * V2.X;
}

/** @return right-Perpendicular vector to V, ie V rotated 90 degrees clockwise */
template <typename T>
constexpr UE::Math::TVector2<T> PerpCW(const UE::Math::TVector2<T>& V)
{
	return UE::Math::TVector2<T>(V.Y, -V.X);
}

/** @return > 0 if C is to the left of the line from A to B, < 0 if to the right, 0 if on the line */
template<typename T>
T Orient(const UE::Math::TVector2<T>& A, const UE::Math::TVector2<T>& B, const UE::Math::TVector2<T>& C)
{
	return DotPerp((B - A), (C - A));
}


template <typename T>
constexpr bool IsNormalized(const UE::Math::TVector2<T>& Vector, const T Tolerance = TMathUtil<T>::ZeroTolerance)
{
	return TMathUtil<T>::Abs((Vector.X*Vector.X + Vector.Y*Vector.Y) - 1) < Tolerance;
}

template <typename T>
T Normalize(UE::Math::TVector2<T>& Vector, const T Epsilon = 0)
{
	 T length = Vector.Length();
	 if (length > Epsilon)
	 {
		 T invLength = ((T)1) / length;
		 Vector.X *= invLength;
		 Vector.Y *= invLength;
		 return length;
	 }
	 Vector.X = Vector.Y = (T)0;
	 return (T)0;
}

template <typename T>
UE::Math::TVector2<T> Normalized(const UE::Math::TVector2<T>& Vector, const T Epsilon = 0)
{
	T length = Vector.Length();
	if (length > Epsilon)
	{
		T invLength = ((T)1) / length;
		return UE::Math::TVector2<T>(Vector.X*invLength, Vector.Y*invLength);
	}
	return UE::Math::TVector2<T>((T)0, (T)0);
}


template<typename T>
T Distance(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	T dx = V2.X - V1.X;
	T dy = V2.Y - V1.Y;
	return TMathUtil<T>::Sqrt(dx * dx + dy * dy);
}

template<typename T>
T DistanceSquared(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	T dx = V2.X - V1.X;
	T dy = V2.Y - V1.Y;
	return dx * dx + dy * dy;
}


// Angle in Degrees
template <typename T>
T AngleD(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	T DotVal = V1.Dot(V2);
	T ClampedDot = (DotVal < (T)-1) ? (T)-1 : ((DotVal > (T)1) ? (T)1 : DotVal);
	return TMathUtil<T>::ACos(ClampedDot) * TMathUtil<T>::RadToDeg;
}

// Angle in Radians
template <typename T>
T AngleR(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	T DotVal = V1.Dot(V2);
	T ClampedDot = (DotVal < (T)-1) ? (T)-1 : ((DotVal > (T)1) ? (T)1 : DotVal);
	return TMathUtil<T>::ACos(ClampedDot);
}

// Angle in Radians
template <typename T>
T SignedAngleR(const UE::Math::TVector2<T>& V1, const UE::Math::TVector2<T>& V2)
{
	T DotVal = V1.Dot(V2);
	T ClampedDot = (DotVal < (T)-1) ? (T)-1 : ((DotVal > (T)1) ? (T)1 : DotVal);
	T Direction = DotPerp(V1, V2);
	if (Direction * Direction < TMathUtil<T>::ZeroTolerance)
	{
		return (DotVal < 0) ? TMathUtil<T>::Pi : (T)0;
	}
	else
	{
		T Sign = Direction < 0 ? (T)-1 : (T)1;
		return Sign * TMathUtil<T>::ACos(ClampedDot);
	}
}

template <typename T>
UE::Math::TVector2<T> Lerp(const UE::Math::TVector2<T>& A, const UE::Math::TVector2<T>& B, T Alpha)
{
	T OneMinusAlpha = (T)1 - Alpha;
	return UE::Math::TVector2<T>(OneMinusAlpha * A.X + Alpha * B.X,
		OneMinusAlpha * A.Y + Alpha * B.Y);
}




/** @return unit vector along axis X=0, Y=1, Z=2 */
template <typename T>
constexpr UE::Math::TVector<T> MakeUnitVector3(int32 Axis)
{
	UE::Math::TVector<T> UnitVec((T)0, (T)0, (T)0);
	UnitVec[FMath::Clamp(Axis, 0, 2)] = (T)1;
	return UnitVec;
}


template<typename T>
T Length(const UE::Math::TVector<T>& V)
{
	return TMathUtil<T>::Sqrt(V.X*V.X + V.Y*V.Y + V.Z*V.Z);
}

template<typename T>
T SquaredLength(const UE::Math::TVector<T>& V)
{
	return V.X*V.X + V.Y*V.Y + V.Z*V.Z;
}


template <typename T>
constexpr bool IsNormalized(const UE::Math::TVector<T>& Vector, const T Tolerance = TMathUtil<T>::ZeroTolerance)
{
	return TMathUtil<T>::Abs((Vector.X*Vector.X + Vector.Y*Vector.Y + Vector.Z*Vector.Z) - 1) < Tolerance;
}


template<typename T>
T Normalize(UE::Math::TVector<T>& Vector, const T Epsilon = 0)
{
	T length = Vector.Length();
	if (length > Epsilon)
	{
		T invLength = ((T)1) / length;
		Vector.X *= invLength;
		Vector.Y *= invLength;
		Vector.Z *= invLength;
		return length;
	}
	Vector.X = Vector.Y = Vector.Z = (T)0;
	return (T)0;
}

template<typename T>
constexpr UE::Math::TVector<T> Normalized(const UE::Math::TVector<T>& Vector, const T Epsilon = 0)
{
	T length = Vector.Length();
	if (length > Epsilon)
	{
		T invLength = ((T)1) / length;
		return UE::Math::TVector<T>(Vector.X*invLength, Vector.Y*invLength, Vector.Z*invLength);
	}
	return UE::Math::TVector<T>((T)0, (T)0, (T)0);
}

template<typename T>
T Distance(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	T dx = V2.X - V1.X;
	T dy = V2.Y - V1.Y;
	T dz = V2.Z - V1.Z;
	return TMathUtil<T>::Sqrt(dx * dx + dy * dy + dz * dz);
}

template<typename T>
T DistanceSquared(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	T dx = V2.X - V1.X;
	T dy = V2.Y - V1.Y;
	T dz = V2.Z - V1.Z;
	return dx * dx + dy * dy + dz * dz;
}



template<typename T>
T Dot(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	return V1.X * V2.X + V1.Y * V2.Y + V1.Z * V2.Z;
}

template<typename T>
UE::Math::TVector<T> Cross(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	return UE::Math::TVector<T>(
		V1.Y * V2.Z - V1.Z * V2.Y,
		V1.Z * V2.X - V1.X * V2.Z,
		V1.X * V2.Y - V1.Y * V2.X);
}

template<typename T>
UE::Math::TVector<T> UnitCross(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	UE::Math::TVector<T> N = V1.Cross(V2);
	return Normalized(N);
}

/**
 * Computes the Angle between V1 and V2, assuming they are already normalized
 * @return the (positive) angle between V1 and V2 in degrees
 */
template <typename T>
T AngleD(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	T DotVal = V1.Dot(V2);
	T ClampedDot = (DotVal < (T)-1) ? (T)-1 : ((DotVal > (T)1) ? (T)1 : DotVal);
	return TMathUtil<T>::ACos(ClampedDot) * TMathUtil<T>::RadToDeg;
}

/**
 * Computes the Angle between V1 and V2, assuming they are already normalized
 * @return the (positive) angle between V1 and V2 in radians
 */
template <typename T>
T AngleR(const UE::Math::TVector<T>& V1, const UE::Math::TVector<T>& V2)
{
	T DotVal = V1.Dot(V2);
	T ClampedDot = (DotVal < (T)-1) ? (T)-1 : ((DotVal > (T)1) ? (T)1 : DotVal);
	return TMathUtil<T>::ACos(ClampedDot);
}

template <typename T>
constexpr UE::Math::TVector2<T> GetXY(const UE::Math::TVector<T>& V)
{
	return UE::Math::TVector2<T>(V.X, V.Y);
}

template <typename T>
constexpr UE::Math::TVector2<T> GetXZ(const UE::Math::TVector<T>& V)
{
	return UE::Math::TVector2<T>(V.X, V.Z);
}

template <typename T>
constexpr UE::Math::TVector2<T> GetYZ(const UE::Math::TVector<T>& V)
{
	return UE::Math::TVector2<T>(V.Y, V.Z);
}


template<typename T>
constexpr UE::Math::TVector<T> Min(const UE::Math::TVector<T>& V0, const UE::Math::TVector<T>& V1)
{
	return UE::Math::TVector<T>(TMathUtil<T>::Min(V0.X, V1.X),
		TMathUtil<T>::Min(V0.Y, V1.Y),
		TMathUtil<T>::Min(V0.Z, V1.Z));
}

template<typename T>
constexpr UE::Math::TVector<T> Max(const UE::Math::TVector<T>& V0, const UE::Math::TVector<T>& V1)
{
	return UE::Math::TVector<T>(TMathUtil<T>::Max(V0.X, V1.X),
		TMathUtil<T>::Max(V0.Y, V1.Y),
		TMathUtil<T>::Max(V0.Z, V1.Z));
}


template<typename T>
constexpr T MaxElement(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Max3(Vector.X, Vector.Y, Vector.Z);
}

/** @return 0/1/2 index of maximum element */
template<typename T>
constexpr int32 MaxElementIndex(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Max3Index(Vector.X, Vector.Y, Vector.Z);
}

template<typename T>
constexpr T MinElement(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Min3(Vector.X, Vector.Y, Vector.Z);
}

/** @return 0/1/2 index of minimum element */
template<typename T>
constexpr int32 MinElementIndex(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Min3Index(Vector.X, Vector.Y, Vector.Z);
}

template<typename T>
constexpr T MaxAbsElement(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Max3(TMathUtil<T>::Abs(Vector.X), TMathUtil<T>::Abs(Vector.Y), TMathUtil<T>::Abs(Vector.Z));
}

/** @return 0/1/2 index of maximum absolute-value element */
template<typename T>
constexpr int32 MaxAbsElementIndex(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Max3Index(TMathUtil<T>::Abs(Vector.X), TMathUtil<T>::Abs(Vector.Y), TMathUtil<T>::Abs(Vector.Z));
}

template<typename T>
constexpr T MinAbsElement(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Min3(TMathUtil<T>::Abs(Vector.X), TMathUtil<T>::Abs(Vector.Y), TMathUtil<T>::Abs(Vector.Z));
}

/** @return 0/1/2 index of minimum absolute-value element */
template<typename T>
constexpr int32 MinAbsElementIndex(const UE::Math::TVector<T>& Vector)
{
	return TMathUtil<T>::Min3Index(TMathUtil<T>::Abs(Vector.X), TMathUtil<T>::Abs(Vector.Y), TMathUtil<T>::Abs(Vector.Z));
}

template<typename T>
constexpr FLinearColor ToLinearColor(const UE::Math::TVector<T>& Vector)
{
	return FLinearColor((float)Vector.X, (float)Vector.Y, (float)Vector.Z);
}

template<typename T>
UE::Math::TVector<T> Lerp(const UE::Math::TVector<T>& A, const UE::Math::TVector<T>& B, T Alpha)
{
	T OneMinusAlpha = (T)1 - Alpha;
	return UE::Math::TVector<T>(OneMinusAlpha * A.X + Alpha * B.X,
		OneMinusAlpha * A.Y + Alpha * B.Y,
		OneMinusAlpha * A.Z + Alpha * B.Z);
}

template<typename T>
UE::Math::TVector<T> Blend3(const UE::Math::TVector<T>& A, const UE::Math::TVector<T>& B, const UE::Math::TVector<T>& C, const T& WeightA, const T& WeightB, const T& WeightC)
{
	return UE::Math::TVector<T>(
		WeightA * A.X + WeightB * B.X + WeightC * C.X,
		WeightA * A.Y + WeightB * B.Y + WeightC * C.Y,
		WeightA * A.Z + WeightB * B.Z + WeightC * C.Z);
}


template <typename RealType>
std::ostream& operator<<(std::ostream& os, const UE::Math::TVector<RealType>& Vec)
{
	os << Vec.X << " " << Vec.Y << " " << Vec.Z;
	return os;
}





template<typename T>
FLinearColor ToLinearColor(const UE::Math::TVector4<T>& V)
{
	return FLinearColor((float)V.X, (float)V.Y, (float)V.Z, (float)V.W);
}

template<typename T>
UE::Math::TVector4<T> ToVector4(const FLinearColor& Color)
{
	return UE::Math::TVector4<T>( (T)Color.R, (T)Color.G, (T)Color.B, (T)Color.A );
}

template<typename T>
constexpr FColor ToFColor(const UE::Math::TVector4<T>& Vector)
{
	return FColor(
		FMathf::Clamp((int)((float)Vector.X * 255.0f), 0, 255),
		FMathf::Clamp((int)((float)Vector.Y * 255.0f), 0, 255),
		FMathf::Clamp((int)((float)Vector.Z * 255.0f), 0, 255),
		FMathf::Clamp((int)((float)Vector.W * 255.0f), 0, 255));
}


template<typename T>
UE::Math::TVector4<T> Lerp(const UE::Math::TVector4<T>& A, const UE::Math::TVector4<T>& B, T Alpha)
{
	T OneMinusAlpha = (T)1 - Alpha;
	return UE::Math::TVector4<T>(
		OneMinusAlpha * A.X + Alpha * B.X,
		OneMinusAlpha * A.Y + Alpha * B.Y,
		OneMinusAlpha * A.Z + Alpha * B.Z,
		OneMinusAlpha * A.W + Alpha * B.W);
}

} // end namespace UE::Geometry

namespace Math
{

template <typename RealType>
std::ostream& operator<<(std::ostream& os, const TVector2<RealType>& Vec)
{
	os << Vec.X << " " << Vec.Y;
	return os;
}

template <typename RealType>
std::ostream& operator<<(std::ostream& os, const TVector4<RealType>& Vec)
{
	os << Vec.X << " " << Vec.Y << " " << Vec.Z << " " << Vec.W;
	return os;
}

} // end namespace UE::Math

} // end namespace UE

