// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "Math/Transform.h"

enum class EIntersectionResult
{
	NotComputed,
	Intersects,
	NoIntersection,
	InvalidQuery
};

enum class EIntersectionType
{
	Empty,
	Point,
	Segment,
	Line,
	Polygon,
	Plane,
	MultiSegment,
	Unknown
};

namespace UE
{
namespace Geometry
{
	using namespace UE::Math;

namespace VectorUtil
{

	/**
	 * @return true if all components of V are finite
	 */
	template <typename RealType>
	inline bool IsFinite(const TVector2<RealType>& V)
	{
		return TMathUtil<RealType>::IsFinite(V.X) && TMathUtil<RealType>::IsFinite(V.Y);
	}

	/**
	 * @return true if all components of V are finite
	 */
	template <typename RealType>
	inline bool IsFinite(const TVector<RealType>& V)
	{
		return TMathUtil<RealType>::IsFinite(V.X) && TMathUtil<RealType>::IsFinite(V.Y) && TMathUtil<RealType>::IsFinite(V.Z);
	}


	/**
	 * @return true if all components of V are finite
	 */
	template <typename RealType>
	inline bool IsFinite(const TVector4<RealType>& V)
	{
		return TMathUtil<RealType>::IsFinite(V.X) && TMathUtil<RealType>::IsFinite(V.Y) && 
			TMathUtil<RealType>::IsFinite(V.Z) && TMathUtil<RealType>::IsFinite(V.W);
	}

	/**
	 * @return input Value clamped to range [MinValue, MaxValue]
	 */
	template <typename RealType>
	inline RealType Clamp(RealType Value, RealType MinValue, RealType MaxValue)
	{
		return (Value < MinValue) ? MinValue : ((Value > MaxValue) ? MaxValue : Value);
	}

	/**
	 * @return normalized vector that is perpendicular to triangle V0,V1,V2  (triangle normal)
	 */
	template <typename RealType>
	inline TVector<RealType> Normal(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		TVector<RealType> edge1(V1 - V0);
		TVector<RealType> edge2(V2 - V0);
		// Unreal has Left-Hand Coordinate System so we need to reverse this cross-product to get proper triangle normal
		TVector<RealType> vCross(edge2.Cross(edge1));
		return Normalized(vCross);
	}

	/**
	 * @return un-normalized direction that is parallel to normal of triangle V0,V1,V2
	 */
	template <typename RealType>
	inline TVector<RealType> NormalDirection(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		// Unreal has Left-Hand Coordinate System so we need to reverse this cross-product to get proper triangle normal
		return (V2 - V0).Cross(V1 - V0);
	}

	/**
	 * @return area of 3D triangle V0,V1,V2
	 */
	template <typename RealType>
	inline RealType Area(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		TVector<RealType> Edge1(V1 - V0);
		TVector<RealType> Edge2(V2 - V0);
		TVector<RealType> Cross = Edge2.Cross(Edge1);
		return (RealType)0.5 * Cross.Length();
	}

	/**
	 * @return area of 2D triangle V0,V1,V2
	 */
	template <typename RealType>
	inline RealType Area(const TVector2<RealType>& V0, const TVector2<RealType>& V1, const TVector2<RealType>& V2)
	{
		TVector2<RealType> Edge1(V1 - V0);
		TVector2<RealType> Edge2(V2 - V0);
		RealType CrossZ = DotPerp(Edge1, Edge2);
		return (RealType)0.5 * TMathUtil<RealType>::Abs(CrossZ);
	}

	/**
	 * @return signed area of 2D triangle V0,V1,V2
	 */
	template <typename RealType>
	inline RealType SignedArea(const TVector2<RealType>& V0, const TVector2<RealType>& V1, const TVector2<RealType>& V2)
	{
		return ((RealType)0.5) * ((V0.X * V1.Y - V0.Y * V1.X) + (V1.X * V2.Y - V1.Y * V2.X) + (V2.X * V0.Y - V2.Y * V0.X));
	}

	/**
	 * @return true if triangle V1,V2,V3 is obtuse
	 */
	template <typename RealType>
	inline bool IsObtuse(const TVector<RealType>& V1, const TVector<RealType>& V2, const TVector<RealType>& V3)
	{
		RealType a2 = DistanceSquared(V1, V2);
		RealType b2 = DistanceSquared(V1, V3);
		RealType c2 = DistanceSquared(V2, V3);
		return (a2 + b2 < c2) || (b2 + c2 < a2) || (c2 + a2 < b2);
	}

	/**
	 * Calculate Normal and Area of triangle V0,V1,V2
	 * @return triangle normal
	 */
	template <typename RealType>
	inline TVector<RealType> NormalArea(const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2, RealType& AreaOut)
	{
		TVector<RealType> edge1(V1 - V0);
		TVector<RealType> edge2(V2 - V0);
		// Unreal has Left-Hand Coordinate System so we need to reverse this cross-product to get proper triangle normal
		FVector3d vCross = edge2.Cross(edge1);
		AreaOut = RealType(0.5) * Normalize(vCross);
		return vCross;
	}

	/** @return true if Abs(A-B) is less or equal than Epsilon */
	template <typename RealType>
	inline bool EpsilonEqual(RealType A, RealType B, RealType Epsilon)
	{
		return TMathUtil<RealType>::Abs(A - B) <= Epsilon;
	}

	/** @return true if all coordinates of V0 and V1 are within Epsilon of each other */
	template <typename RealType>
	inline bool EpsilonEqual(const TVector2<RealType>& V0, const TVector2<RealType>& V1, RealType Epsilon)
	{
		return EpsilonEqual(V0.X, V1.X, Epsilon) && EpsilonEqual(V0.Y, V1.Y, Epsilon);
	}

	/** @return true if all coordinates of V0 and V1 are within Epsilon of each other */
	template <typename RealType>
	inline bool EpsilonEqual(const TVector<RealType>& V0, const TVector<RealType>& V1, RealType Epsilon)
	{
		return EpsilonEqual(V0.X, V1.X, Epsilon) && EpsilonEqual(V0.Y, V1.Y, Epsilon) && EpsilonEqual(V0.Z, V1.Z, Epsilon);
	}

	/** @return true if all coordinates of V0 and V1 are within Epsilon of each other */
	template <typename RealType>
	inline bool EpsilonEqual(const TVector4<RealType>& V0, const TVector4<RealType>& V1, RealType Epsilon)
	{
		return EpsilonEqual(V0.X, V1.X, Epsilon) && EpsilonEqual(V0.Y, V1.Y, Epsilon) && EpsilonEqual(V0.Z, V1.Z, Epsilon) && EpsilonEqual(V0.W, V1.W, Epsilon);
	}


	/** @return 0/1/2 index of smallest value in Vector3 */
	template <typename ValueVecType>
	inline int Min3Index(const ValueVecType& Vector3)
	{
		if (Vector3[0] <= Vector3[1])
		{
			return Vector3[0] <= Vector3[2] ? 0 : 2;
		}
		else
		{
			return (Vector3[1] <= Vector3[2]) ? 1 : 2;
		}
	}

	/** @return 0/1/2 index of largest value in Vector3 */
	template <typename ValueVecType>
	inline int Max3Index(const ValueVecType& Vector3)
	{
		if (Vector3[0] >= Vector3[1])
		{
			return Vector3[0] >= Vector3[2] ? 0 : 2;
		}
		else
		{
			return (Vector3[1] >= Vector3[2]) ? 1 : 2;
		}
	}


	/**
	 * Calculates two vectors perpendicular to input Normal, as efficiently as possible.
	 */
	template <typename RealType>
	inline void MakePerpVectors(const TVector<RealType>& Normal, TVector<RealType>& OutPerp1, TVector<RealType>& OutPerp2)
	{
		// Duff et al method, from https://graphics.pixar.com/library/OrthonormalB/paper.pdf
		if (Normal.Z < (RealType)0)
		{
			RealType A = (RealType)1 / ((RealType)1 - Normal.Z);
			RealType B = Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = -B;
			OutPerp1.Z = Normal.X;
			OutPerp2.X = B;
			OutPerp2.Y = Normal.Y * Normal.Y * A - (RealType)1;
			OutPerp2.Z = -Normal.Y;
		}
		else
		{
			RealType A = (RealType)1 / ((RealType)1 + Normal.Z);
			RealType B = -Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = B;
			OutPerp1.Z = -Normal.X;
			OutPerp2.X = B;
			OutPerp2.Y = (RealType)1 - Normal.Y * Normal.Y * A;
			OutPerp2.Z = -Normal.Y;
		}
	}

	/**
	 * Calculates one vector perpendicular to input Normal, as efficiently as possible.
	 */
	template <typename RealType>
	inline TVector<RealType> MakePerpVector(const TVector<RealType>& Normal)
	{
		// Duff et al method, from https://graphics.pixar.com/library/OrthonormalB/paper.pdf
		TVector<RealType> OutPerp1;
		if (Normal.Z < (RealType)0)
		{
			RealType A = (RealType)1 / ((RealType)1 - Normal.Z);
			RealType B = Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = -B;
			OutPerp1.Z = Normal.X;
		}
		else
		{
			RealType A = (RealType)1 / ((RealType)1 + Normal.Z);
			RealType B = -Normal.X * Normal.Y * A;
			OutPerp1.X = (RealType)1 - Normal.X * Normal.X * A;
			OutPerp1.Y = B;
			OutPerp1.Z = -Normal.X;
		}
		return OutPerp1;
	}

	/**
	 * Calculates one vector perpendicular to input Normal, as efficiently as possible.
	 */
	template <typename RealType>
	inline void MakePerpVector(const TVector<RealType>& Normal, TVector<RealType>& OutPerp1)
	{
		OutPerp1 = MakePerpVector(Normal);
	}


	/**
	 * Calculates angle between VFrom and VTo after projection onto plane with normal defined by PlaneN
	 * @return angle in degrees
	 */
	template <typename RealType>
	inline RealType PlaneAngleSignedD(const TVector<RealType>& VFrom, const TVector<RealType>& VTo, const TVector<RealType>& PlaneN)
	{
		TVector<RealType> vFrom = VFrom - VFrom.Dot(PlaneN) * PlaneN;
		TVector<RealType> vTo = VTo - VTo.Dot(PlaneN) * PlaneN;
		Normalize(vFrom);
		Normalize(vTo);
		TVector<RealType> C = vFrom.Cross(vTo);
		if (C.SquaredLength() < TMathUtil<RealType>::ZeroTolerance)
		{ // vectors are parallel
			return vFrom.Dot(vTo) < 0 ? (RealType)180 : (RealType)0;
		}
		RealType fSign = C.Dot(PlaneN) < 0 ? (RealType)-1 : (RealType)1;
		return (RealType)(fSign * AngleD(vFrom, vTo));
	}

	/**
	 * Calculates angle between VFrom and VTo after projection onto plane with normal defined by PlaneN
	 * @return angle in radians
	 */
	template <typename RealType>
	inline RealType PlaneAngleSignedR(const TVector<RealType>& VFrom, const TVector<RealType>& VTo, const TVector<RealType>& PlaneN)
	{
		TVector<RealType> vFrom = VFrom - VFrom.Dot(PlaneN) * PlaneN;
		TVector<RealType> vTo = VTo - VTo.Dot(PlaneN) * PlaneN;
		Normalize(vFrom);
		Normalize(vTo);
		TVector<RealType> C = vFrom.Cross(vTo);
		if (C.SquaredLength() < TMathUtil<RealType>::ZeroTolerance)
		{ // vectors are parallel
			return vFrom.Dot(vTo) < 0 ? TMathUtil<RealType>::Pi : (RealType)0;
		}
		RealType fSign = C.Dot(PlaneN) < 0 ? (RealType)-1 : (RealType)1;
		return (RealType)(fSign * AngleR(vFrom, vTo));
	}

	/**
	 * tan(theta/2) = +/- sqrt( (1-cos(theta)) / (1+cos(theta)) )
	 * @return positive value of tan(theta/2) where theta is angle between normalized vectors A and B
	 */
	template <typename RealType>
	RealType VectorTanHalfAngle(const TVector<RealType>& A, const TVector<RealType>& B)
	{
		RealType cosAngle = A.Dot(B);
		RealType sqr = ((RealType)1 - cosAngle) / ((RealType)1 + cosAngle);
		sqr = Clamp(sqr, (RealType)0, TMathUtil<RealType>::MaxReal);
		return TMathUtil<RealType>::Sqrt(sqr);
	}

	/**
	 * tan(theta/2) = +/- sqrt( (1-cos(theta)) / (1+cos(theta)) )
	 * @return positive value of tan(theta/2) where theta is angle between normalized vectors A and B
	 */
	template <typename RealType>
	RealType VectorTanHalfAngle(const TVector2<RealType>& A, const TVector2<RealType>& B)
	{
		RealType cosAngle = A.Dot(B);
		RealType sqr = ((RealType)1 - cosAngle) / ((RealType)1 + cosAngle);
		sqr = Clamp(sqr, (RealType)0, TMathUtil<RealType>::MaxReal);
		return TMathUtil<RealType>::Sqrt(sqr);
	}

	/**
	 * Fast cotangent of angle between two vectors (*do not have to be normalized unit vectors*).
	 * cot = cos/sin, both of which can be computed from vector identities
	 * @return cotangent of angle between V1 and V2, or zero if result would be unstable (eg infinity)
	 */ 
	template <typename RealType>
	RealType VectorCot(const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		// formula from http://www.geometry.caltech.edu/pubs/DMSB_III.pdf
		RealType fDot = V1.Dot(V2);
		RealType lensqr1 = V1.SquaredLength();
		RealType lensqr2 = V2.SquaredLength();
		RealType d = Clamp(lensqr1 * lensqr2 - fDot * fDot, (RealType)0.0, TMathUtil<RealType>::MaxReal);
		if (d < TMathUtil<RealType>::ZeroTolerance)
		{
			return (RealType)0;
		}
		else
		{
			return fDot / TMathUtil<RealType>::Sqrt(d);
		}
	}

	/**
	 * Compute barycentric coordinates/weights of vPoint inside 3D triangle (V0,V1,V2). 
	 * If point is in triangle plane and inside triangle, coords will be positive and sum to 1.
	 * ie if result is a, then vPoint = a.x*V0 + a.y*V1 + a.z*V2.
	 * TODO: make robust to degenerate triangles?
	 */
	template <typename RealType>
	TVector<RealType> BarycentricCoords(const TVector<RealType>& Point, const TVector<RealType>& V0, const TVector<RealType>& V1, const TVector<RealType>& V2)
	{
		TVector<RealType> kV02 = V0 - V2;
		TVector<RealType> kV12 = V1 - V2;
		TVector<RealType> kPV2 = Point - V2;
		RealType fM00 = kV02.Dot(kV02);
		RealType fM01 = kV02.Dot(kV12);
		RealType fM11 = kV12.Dot(kV12);
		RealType fR0 = kV02.Dot(kPV2);
		RealType fR1 = kV12.Dot(kPV2);
		RealType fDet = fM00 * fM11 - fM01 * fM01;
		RealType fInvDet = 1.0 / fDet;
		RealType fBary1 = (fM11 * fR0 - fM01 * fR1) * fInvDet;
		RealType fBary2 = (fM00 * fR1 - fM01 * fR0) * fInvDet;
		RealType fBary3 = 1.0 - fBary1 - fBary2;
		return TVector<RealType>(fBary1, fBary2, fBary3);
	}

	/**
	* Compute barycentric coordinates/weights of vPoint inside 2D triangle (V0,V1,V2). 
	* If point is inside triangle, coords will be positive and sum to 1.
	* ie if result is a, then vPoint = a.x*V0 + a.y*V1 + a.z*V2.
	* TODO: make robust to degenerate triangles?
	*/
	template <typename RealType>
	TVector<RealType> BarycentricCoords(const TVector2<RealType>& Point, const TVector2<RealType>& V0, const TVector2<RealType>& V1, const TVector2<RealType>& V2)
	{
		TVector2<RealType> kV02 = V0 - V2;
		TVector2<RealType> kV12 = V1 - V2;
		TVector2<RealType> kPV2 = Point - V2;
		RealType fM00 = kV02.Dot(kV02);
		RealType fM01 = kV02.Dot(kV12);
		RealType fM11 = kV12.Dot(kV12);
		RealType fR0 = kV02.Dot(kPV2);
		RealType fR1 = kV12.Dot(kPV2);
		RealType fDet = fM00 * fM11 - fM01 * fM01;
		RealType fInvDet = (RealType)1.0 / fDet;
		RealType fBary1 = (fM11 * fR0 - fM01 * fR1) * fInvDet;
		RealType fBary2 = (fM00 * fR1 - fM01 * fR0) * fInvDet;
		RealType fBary3 = (RealType)1.0 - fBary1 - fBary2;
		return TVector<RealType>(fBary1, fBary2, fBary3);
	}

	/**
	 * Sample uniform random barycentric coordinates on a triangle
	 * The inputs should be two random floating point numbers in the [0,1] range
	 * (See eq. 1 of 'Shape Distributions' by Osada, Funkhouser, Chazelle and Dobkin)
	 * Note the sample positions will not match those of UniformSampleTrianglePoint for the same R1, R2 values
	 */
	template<typename RealType>
	TVector<RealType> UniformSampleTriangleBarycentricCoords(RealType R1, RealType R2)
	{
		checkSlow(R1 >= 0);
		RealType SqrtR1 = FMath::Sqrt(R1);
		return TVector<RealType>(1 - SqrtR1, SqrtR1 * (1 - R2), SqrtR1 * R2);
	}

	/**
	 * Sample a uniform random point on a triangle
	 * The inputs should be two random floating point numbers in the [0,1] range, and the triangle coordinates
	 * ref: http://mathworld.wolfram.com/TrianglePointPicking.html
	 * Note the sample positions will not match those of UniformSampleTriangleBarycentricCoords for the same R1, R2 values
	 */
	template<typename RealType>
	TVector<RealType> UniformSampleTrianglePoint(RealType R1, RealType R2, const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C)
	{
		// reflect samples that would go outside the triangle
		if (R1 + R2 > (RealType)1)
		{
			R1 = (RealType)1 - R1;
			R2 = (RealType)1 - R2;
		}
		return A + R1 * (B - A) + R2 * (C - A);
	}

	/**
	 * Sample a uniform random point on a 2D triangle
	 * The inputs should be two random floating point numbers in the [0,1] range, and the triangle coordinates
	 * ref: http://mathworld.wolfram.com/TrianglePointPicking.html
	 * Note the sample positions will not match those of UniformSampleTriangleBarycentricCoords for the same R1, R2 values
	 */
	template<typename RealType>
	TVector2<RealType> UniformSampleTrianglePoint(RealType R1, RealType R2, const TVector2<RealType>& A, const TVector2<RealType>& B, const TVector2<RealType>& C)
	{
		// reflect samples that would go outside the triangle
		if (R1 + R2 > (RealType)1)
		{
			R1 = (RealType)1 - R1;
			R2 = (RealType)1 - R2;
		}
		return A + R1 * (B - A) + R2 * (C - A);
	}

	/**
	 * @return solid angle at point P for triangle A,B,C
	 */
	template <typename RealType>
	inline RealType TriSolidAngle(TVector<RealType> A, TVector<RealType> B, TVector<RealType> C, const TVector<RealType>& P)
	{
		// Formula from https://igl.ethz.ch/projects/winding-number/
		A -= P;
		B -= P;
		C -= P;
		RealType la = A.Length(), lb = B.Length(), lc = C.Length();
		RealType top = (la * lb * lc) + A.Dot(B) * lc + B.Dot(C) * la + C.Dot(A) * lb;
		RealType bottom = A.X * (B.Y * C.Z - C.Y * B.Z) - A.Y * (B.X * C.Z - C.X * B.Z) + A.Z * (B.X * C.Y - C.X * B.Y);
		// -2 instead of 2 to account for UE winding
		return RealType(-2.0) * atan2(bottom, top);
	}


	/**
	 * Calculate gradient of scalar field values fi,fj,fk defined at corners of triangle Vi,Vj,Vk and interpolated across triangle using linear basis functions.
	 * This gradient is a 3D vector lying in the plane of the triangle (or zero if field is constant).
	 * @return gradient (3D vector) lying in plane of triangle.
	 */
	template <typename RealType>
	inline TVector<RealType> TriGradient(TVector<RealType> Vi, TVector<RealType> Vj, TVector<RealType> Vk, RealType fi, RealType fj, RealType fk)
	{
		// recenter (better for precision)
		TVector<RealType> Centroid = (Vi + Vj + Vk) / (RealType)3;
		Vi -= Centroid; Vj -= Centroid; Vk -= Centroid;
		// calculate tangent-normal frame
		TVector<RealType> Normal = VectorUtil::Normal<RealType>(Vi, Vj, Vk);
		TVector<RealType> Perp0, Perp1;
		VectorUtil::MakePerpVectors<RealType>(Normal, Perp0, Perp1);
		// project points to triangle plane coordinates
		TVector2<RealType> vi(Vi.Dot(Perp0), Vi.Dot(Perp1));
		TVector2<RealType> vj(Vj.Dot(Perp0), Vj.Dot(Perp1));
		TVector2<RealType> vk(Vk.Dot(Perp0), Vk.Dot(Perp1));
		// calculate gradient
		TVector2<RealType> GradX = (fj-fi)*PerpCW(vi-vk) + (fk-fi)*PerpCW(vj-vi);
		// map back to 3D vector in triangle plane
		RealType AreaScale = (RealType)1 / ((RealType)2 * VectorUtil::Area<RealType>(Vi, Vj, Vk));
		return AreaScale * (GradX.X * Perp0 + GradX.Y * Perp1);
	}



	/**
	 * @return angle in degrees between vectors (A-CornerPt) and (B-CornerPt)
	 */
	template<typename RealType>
	inline RealType OpeningAngleD(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& P)
	{
		return AngleD(Normalized(A - P), Normalized(B - P));
	}

	/**
	 * @return angle in radians between vectors (A-CornerPt) and (B-CornerPt)
	 */
	template<typename RealType>
	inline RealType OpeningAngleR(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& P)
	{
		return AngleR(Normalized(A - P), Normalized(B - P));
	}

	/**
	 * @return the internal angles of triangle ABC
	 */
	template<typename RealType>
	inline TVector<RealType> TriangleInternalAngles(const TVector<RealType> A, const TVector<RealType> B, const TVector<RealType>& C)
	{
		const TVector<RealType> ABhat = Normalized(B - A);
		const TVector<RealType> BChat = Normalized(C - B);
		const TVector<RealType> AChat = Normalized(C - A);
		return FVector3d(
			AngleR(ABhat, AChat),
			AngleR(-ABhat, BChat),
			AngleR(AChat, BChat)
		);
	}

	/** 
	 * @return oriented dihedral angle of two triangles sharing an oriented edge
	 * 
	 * @note Provided normals and Edge should be unit length, and Edge be perpendicular to both normals 
	 * (see Keenan Crane, Discrete Differential Geometry: An Applied Introduction, Appendix 1.5) 
	 */
	template <typename RealType>
	inline RealType OrientedDihedralAngle(const TVector<RealType>& Normal0, const TVector<RealType>& Normal1, const TVector<RealType>& Edge)
	{
		return FMath::Atan2( Edge | (Normal0 ^ Normal1), Normal0 | Normal1 );
	}

	/**
	 * @return circumcenter of triangle ABC
	 */
	template<typename RealType>
	inline TVector2<RealType> Circumcenter(TVector2<RealType> A, TVector2<RealType> B, const TVector2<RealType>& C, RealType Epsilon = TMathUtilConstants<RealType>::Epsilon)
	{
		// Compute in offset space w/ A translated to origin
		TVector2<RealType> AB = B - A;
		TVector2<RealType> AC = C - A;
		
		RealType Denom = 2 * DotPerp(AB,AC);
		if (FMath::Abs(Denom) <= Epsilon)
		{
			// degenerate (flat) triangle does not have a (finite) circumcenter ...
			// we'll just fall back to the centroid in this case
			return A + (AB + AC) * RealType(1.0/3.0);
		}
		RealType ABLenSq = AB.SquaredLength();
		RealType ACLenSq = AC.SquaredLength();
		TVector2<RealType> Center(
			A.X + (AC.Y * ABLenSq - AB.Y * ACLenSq) / Denom,
			A.Y + (AB.X * ACLenSq - AC.X * ABLenSq) / Denom
			);
		return Center;
	}


	/*
	 * @return incircle / circumcircle radii ratio of triangle ABC
	 */
	template <typename RealType>
	RealType TriangleRegularity(const TVector<RealType>& A, const TVector<RealType>& B, const TVector<RealType>& C)
	{
		const RealType EAB = (A - B).Length();
		const RealType EBC = (B - C).Length();
		const RealType ECA = (C - A).Length();

		const RealType SP = RealType(0.5) * (EAB + EBC + ECA); // Semiperimeter
		const RealType Area = FMath::Sqrt(FMath::Max(RealType(0.), SP * (SP - EAB) * (SP - EBC) * (SP - ECA))); // Heron
		const RealType IncircleRadius = Area / SP;
		const RealType CircumcircleRadius = EAB * EBC * ECA / (RealType(4.) * Area);

		return IncircleRadius / CircumcircleRadius;
	}

	/**
	 * @return sign of Bitangent relative to Normal and Tangent
	 */
	template<typename RealType>
	inline RealType BitangentSign(const TVector<RealType>& NormalIn, const TVector<RealType>& TangentIn, const TVector<RealType>& BitangentIn)
	{
		// following math from RenderUtils.h::GetBasisDeterminantSign()
		RealType Cross00 = BitangentIn.Y*NormalIn.Z - BitangentIn.Z*NormalIn.Y;
		RealType Cross10 = BitangentIn.Z*NormalIn.X - BitangentIn.X*NormalIn.Z;
		RealType Cross20 = BitangentIn.X*NormalIn.Y - BitangentIn.Y*NormalIn.X;
		RealType Determinant = TangentIn.X*Cross00 + TangentIn.Y*Cross10 + TangentIn.Z*Cross20;
		return (Determinant < 0) ? (RealType)-1 : (RealType)1;
	}

	/**
	 * @return Bitangent vector based on given Normal, Tangent, and Sign value (+1/-1)
	 */
	template<typename RealType>
	inline TVector<RealType> Bitangent(const TVector<RealType>& NormalIn, const TVector<RealType>& TangentIn, RealType BitangentSign)
	{
		return BitangentSign * TVector<RealType>(
			NormalIn.Y*TangentIn.Z - NormalIn.Z*TangentIn.Y,
			NormalIn.Z*TangentIn.X - NormalIn.X*TangentIn.Z,
			NormalIn.X*TangentIn.Y - NormalIn.Y*TangentIn.X);
	}

	/**
	 * @return Tangent-Space vector based on given Normal and Bitangent
	 */
	template<typename RealType>
	inline TVector<RealType> TangentFromBitangent(const TVector<RealType>& NormalIn, const TVector<RealType>& BitangentIn)
	{
		return BitangentIn.Cross(NormalIn);
	}

	/**
	 * @return Bitangent vector based on given Normal and Tangent
	 */
	template<typename RealType>
	inline TVector<RealType> BitangentFromTangent(const TVector<RealType>& NormalIn, const TVector<RealType>& TangentIn)
	{
		return NormalIn.Cross(TangentIn);
	}

	/// @return Aspect ratio of triangle 
	inline double AspectRatio(const FVector3d& v1, const FVector3d& v2, const FVector3d& v3)
	{
		double a = Distance(v1, v2), b = Distance(v2, v3), c = Distance(v3, v1);
		double s = (a + b + c) / 2.0;
		return (a * b * c) / (8.0 * (s - a) * (s - b) * (s - c));
	}


	template<typename RealType>
	inline TVector<RealType> TransformNormal(const TTransform<RealType>& Transform, const TVector<RealType>& Normal)
	{
		const TVector<RealType>& S = Transform.GetScale3D();
		RealType DetSign = FMathd::SignNonZero(S.X * S.Y * S.Z); // we only need to multiply by the sign of the determinant, rather than divide by it, since we normalize later anyway
		TVector<RealType> SafeInvS(S.Y * S.Z * DetSign, S.X * S.Z * DetSign, S.X * S.Y * DetSign);
		return Transform.TransformVectorNoScale(Normalized(SafeInvS * Normal));
	}


	template<typename RealType>
	inline TVector<RealType> InverseTransformNormal(const TTransform<RealType>& Transform, const TVector<RealType>& Normal)
	{
		return Normalized(Transform.GetScale3D() * Transform.InverseTransformVectorNoScale(Normal));
	}

}; // namespace VectorUtil



/**
 * Snap Value to steps of Increment (centered at 0, ie steps are -Increment, 0, Increment, 2*Increment, ...).
 * Optional Offset can be used to snap relative value.
 */
template<typename RealType>
RealType SnapToIncrement(RealType Value, RealType Increment, RealType Offset = 0)
{
	if (!FMath::IsFinite(Value))
	{
		return (RealType)0;
	}
	Value -= Offset;
	RealType ValueSign = FMath::Sign(Value);
	Value = FMath::Abs(Value);
	int64 IntegerIncrement = (int64)(Value / Increment);
	RealType Remainder = (RealType)fmod(Value, Increment);
	if (Remainder > Increment / 2.0)
	{
		++IntegerIncrement;
	}
	return ValueSign * (RealType)IntegerIncrement * Increment + Offset;
}



}  // end namespace Geometry
}  // end namespace UE
