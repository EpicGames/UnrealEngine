// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "IndexTypes.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/PointHashGrid3.h"
#include "Util/SizedDisjointSet.h"

namespace UE
{
namespace Geometry
{

/// Vertex-based connected components class -- can work with any mesh that has vertex IDs
/// Also supports linking spatially-close vertices in the same component
/// 
/// Functions templated on TemplateMeshType are designed to work with any mesh that 
/// implements the standard MeshAdapter functions (see MeshAdapter.h)
/// Functions templated on TriangleType are designed to work with triangles 
/// with vertex IDs that can be array-accessed (i.e.: Tri[0], Tri[1], Tri[2])
class FVertexConnectedComponents
{
public:
	FVertexConnectedComponents()
	{
		Init(0);
	}

	FVertexConnectedComponents(int32 MaxVertexID)
	{
		Init(MaxVertexID);
	}
	
	void Init(int32 MaxVertexID)
	{
		DisjointSet.Init(MaxVertexID);
	}

	template<typename TriangleMeshType>
	void Init(const TriangleMeshType& Mesh)
	{
		DisjointSet.Init(Mesh.MaxVertexID(), [&Mesh](int32 VID) -> bool { return Mesh.IsVertex(VID); });
	}

	template<typename TriangleMeshType>
	void ConnectTriangles(const TriangleMeshType& Mesh)
	{
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); TID++)
		{
			if (Mesh.IsTriangle(TID))
			{
				FIndex3i Triangle = Mesh.GetTriangle(TID);
				DisjointSet.Union(Triangle[0], Triangle[1]);
				DisjointSet.Union(Triangle[1], Triangle[2]);
			}
		}
	}

	template<typename TriangleType>
	void ConnectTriangles(TArrayView<const TriangleType> Triangles)
	{
		for (const TriangleType& Triangle : Triangles)
		{
			DisjointSet.Union(Triangle[0], Triangle[1]);
			DisjointSet.Union(Triangle[1], Triangle[2]);
		}
	}
	
	/**
	 * Union (filtered) vertices to nearest different component w/in threshold distance --
	 * Intended to be called after ConnectTriangles and other local Connect methods, to merge nearby-but-disconnected components.
	 * 
	 * @param MeshSpatial AABB tree for the mesh, used for spatial triangle queries
	 * @param DistanceThreshold Maximum point-to-triangle distance for bridging components
	 * @param VertexIDFilter Return true if the vertex should be considered for connection
	 * @param KeepSizeThreshold Skip considering vertices that are in a smaller cluster size than this. Note: A value > 1 after calling ConnectTriangles will ensure the unreferenced vertices are skipped.
	 */
	template<typename TriangleMeshType>
	void ConnectVerticesToNearestDifferentComponent(const TMeshAABBTree3<TriangleMeshType>& MeshSpatial, double DistanceThreshold, TFunctionRef<bool(int32)> VertexIDFilter = [](int32) {return true;}, int32 KeepSizeThreshold = 0)
	{
		const TriangleMeshType* Mesh = MeshSpatial.GetMesh();

		for (int32 VID = 0; VID < Mesh->MaxVertexID(); VID++)
		{
			if (!Mesh->IsVertex(VID) || !VertexIDFilter(VID))
			{
				continue;
			}
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}

			FVector3d Pt = Mesh->GetVertex(VID);
			double NearestDistSqr;
			IMeshSpatial::FQueryOptions Options;
			Options.MaxDistance = DistanceThreshold;
			Options.TriangleFilterF = [this, Mesh, SetID](int32 TriID) -> bool
			{
				FIndex3i Triangle = Mesh->GetTriangle(TriID);
				// Accept only triangles where at least one vertex is in a different component
				return DisjointSet.Find(Triangle[0]) != SetID
					|| DisjointSet.Find(Triangle[1]) != SetID
					|| DisjointSet.Find(Triangle[2]) != SetID;
			};
			int32 NearestTriID = MeshSpatial.FindNearestTriangle(Pt, NearestDistSqr, Options);
			if (NearestTriID >= 0)
			{
				FIndex3i Triangle = Mesh->GetTriangle(NearestTriID);
				DisjointSet.Union(VID, Triangle[0]);
				DisjointSet.Union(VID, Triangle[1]);
				DisjointSet.Union(VID, Triangle[2]);
			}
		}
	}

	/**
	 * Connect a filtered subset of vertices that are closer than a threshold distance in space
	 * @param CloseVertexThreshold Distance below which vertices should be connected
	 * @param VertexIDFilter Return true if the vertex should be considered for connection to other vertices
	 * @param KeepSizeThreshold Skip considering vertices that are in a smaller cluster size than this. Note: A value > 1 after calling ConnectTriangles will ensure the unreferenced vertices are skipped.
	 */
	template<typename TriangleMeshType>
	void ConnectCloseFilteredVertices(const TriangleMeshType& Mesh, double CloseVertexThreshold, TFunctionRef<bool(int32)> VertexIDFilter, int32 KeepSizeThreshold = 0)
	{
		TPointHashGrid3d<int32> VertexHash(CloseVertexThreshold * 3, -1);
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
		{
			if (!Mesh.IsVertex(VID) || !VertexIDFilter(VID))
			{
				continue;
			}
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}

			int32 MaxSetSize = Mesh.VertexCount();
			FVector3d Pt = Mesh.GetVertex(VID);
			VertexHash.EnumeratePointsInBall(Pt, CloseVertexThreshold, [&Mesh, Pt](int32 OtherVID)
				{
					return DistanceSquared(Pt, Mesh.GetVertex(OtherVID));
				}, [this, SetID, MaxSetSize](const int32& NbrVID, double DistSq) 
				{
					int32 UnionSetID = DisjointSet.Union(SetID, NbrVID);
					return DisjointSet.Sizes[UnionSetID] < MaxSetSize; // stop iterating if all vertices are in the same component
				});
			VertexHash.InsertPointUnsafe(VID, Pt);
		}
	}
	
	/**
	 * Connect vertices that are closer than a threshold distance in space
	 * @param CloseVertexThreshold Distance below which vertices should be connected
	 * @param KeepSizeThreshold Skip considering vertices that are in a smaller cluster size than this. Note: A value > 1 after calling ConnectTriangles will ensure the unreferenced vertices are skipped.
	 */
	template<typename TriangleMeshType>
	void ConnectCloseVertices(const TriangleMeshType& Mesh, double CloseVertexThreshold, int32 KeepSizeThreshold = 0)
	{
		ConnectCloseFilteredVertices(Mesh, CloseVertexThreshold, [&Mesh](int32 VID)->bool {return true;}, KeepSizeThreshold);
	}

	// TODO: support more overlap strategies, currently just uses AABB
	// Note this merges components based on overlap of their bounding boxes as computed *before* any merges; multiple passes may merge additional components
	template<typename TriangleMeshType>
	void ConnectOverlappingComponents(const TriangleMeshType& Mesh, int32 KeepSizeThreshold = 0)
	{
		TMap<int32, FAxisAlignedBox3d> SetIDToBounds;
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
		{
			if (!Mesh.IsVertex(VID))
			{
				continue;
			}
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}
			FVector3d Pt = Mesh.GetVertex(VID);
			FAxisAlignedBox3d& Bounds = SetIDToBounds.FindOrAdd(SetID, FAxisAlignedBox3d::Empty());
			Bounds.Contain(Pt);
		}
		// For each component's bounding box
		for (auto It = SetIDToBounds.CreateConstIterator(); It; ++It)
		{
			auto NextIt = It;
			// Iterate over the subsequent component bounding boxes
			++NextIt;
			for (; NextIt; ++NextIt)
			{
				// Union the two components if their bounds overlap
				if (It.Value().Intersects(NextIt.Value()))
				{
					DisjointSet.Union(It.Key(), NextIt.Key());
				}
			}
		}
	}

	template<typename TriangleMeshType>
	bool HasMultipleComponents(const TriangleMeshType& Mesh, int32 KeepSizeThreshold = 0)
	{
		int32 FoundComponent = -1;
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
		{
			if (!Mesh.IsVertex(VID))
			{
				continue;
			}
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}
			if (FoundComponent == -1)
			{
				FoundComponent = SetID;
			}
			else if (FoundComponent != SetID)
			{
				return true;
			}
		}
		return false;
	}

	bool HasMultipleComponents(int32 MaxVID, int32 KeepSizeThreshold = 0)
	{
		int32 FoundComponent = -1;
		for (int32 VID = 0; VID < MaxVID; VID++)
		{
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}
			if (FoundComponent == -1)
			{
				FoundComponent = SetID;
			}
			else if (FoundComponent != SetID)
			{
				return true;
			}
		}
		return false;
	}

	// Map arbitrary set IDs to indices from 0 to k-1 (if there are k components)
	template<typename TriangleMeshType>
	TMap<int32, int32> MakeComponentMap(const TriangleMeshType& Mesh, int32 KeepSizeThreshold = 0)
	{
		TMap<int32, int32> ComponentMap;
		int32 CurrentIdx = 0;
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
		{
			if (!Mesh.IsVertex(VID))
			{
				continue;
			}
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}
			if (!ComponentMap.Contains(SetID))
			{
				ComponentMap.Add(SetID, CurrentIdx++);
			}
		}
		return ComponentMap;
	}

	TMap<int32, int32> MakeComponentMap(int32 MaxVID, int32 KeepSizeThreshold = 0)
	{
		TMap<int32, int32> ComponentMap;
		int32 CurrentIdx = 0;
		for (int32 VID = 0; VID < MaxVID; VID++)
		{
			int32 SetID = DisjointSet.Find(VID);
			if (KeepSizeThreshold > 0 && DisjointSet.Sizes[SetID] < KeepSizeThreshold)
			{
				continue;
			}
			if (!ComponentMap.Contains(SetID))
			{
				ComponentMap.Add(SetID, CurrentIdx++);
			}
		}
		return ComponentMap;
	}

	// Return an ordering of the vertex indices so that each connected component is in a contiguous block
	TArray<int32> MakeContiguousComponentsArray(int32 MaxVID)
	{
		TMap<int32, int32> ComponentLoc;
		TArray<int32> Contiguous;
		Contiguous.SetNum(MaxVID);
		int32 LastSingleEntryIdx = MaxVID;
		int32 FirstUnusedIdx = 0;
		for (int32 VID = 0; VID < MaxVID; VID++)
		{
			int32 SetID = DisjointSet.Find(VID);
			int32 SetSize = DisjointSet.Sizes[SetID];
			if (SetSize == 1) // just place the single-element groups at the end, no need to track in map
			{
				LastSingleEntryIdx--;
				Contiguous[LastSingleEntryIdx] = VID;
				continue;
			}
			int32* Loc = ComponentLoc.Find(SetID);
			if (Loc)
			{
				Contiguous[(*Loc)++] = VID;
			}
			else
			{
				Contiguous[FirstUnusedIdx] = VID;
				ComponentLoc.Add(SetID, FirstUnusedIdx + 1);
				FirstUnusedIdx += SetSize;
			}
		}
		return Contiguous;
	}

	/// Apply ProcessComponentFn() to each connected component, or until the function returns false
	/// @param ContiguousComponentsArray	Must be the array returned by MakeContiguousComponentsArray()
	/// @param ProcessComponentFn			Function of (ComponentID, Component Members). If the function returns false, enumeration will stop.
	/// @return True if every component was processed, false if ProcessComponentFn returned false and the enumeration returned early.
	bool EnumerateContiguousComponentsFromArray(const TArray<int32>& ContiguousComponentsArray, TFunctionRef<bool(int32, TArrayView<const int32>)> ProcessComponentFn)
	{
		for (int32 ContigStart = 0, NextStart = -1; ContigStart < ContiguousComponentsArray.Num(); ContigStart = NextStart)
		{
			int32 ComponentID = GetComponent(ContiguousComponentsArray[ContigStart]);
			int32 ComponentSize = GetComponentSize(ComponentID);
			NextStart = ContigStart + ComponentSize;
			if (!ensure(NextStart <= ContiguousComponentsArray.Num()))
			{
				return false;
			}
			TArrayView<const int32> ComponentView(ContiguousComponentsArray.GetData() + ContigStart, ComponentSize);
			bool bContinue = ProcessComponentFn(ComponentID, ComponentView);
			if (!bContinue)
			{
				return false;
			}
		}
		return true;
	}

	// Note: May not be thread-safe to make multiple GetComponent() calls in parallel, as it will also collapse the underlying disjoint set paths.
	inline int32 GetComponent(int32 VertexID)
	{
		return DisjointSet.Find(VertexID);
	}

	// Note: Should be safe to make mulitple GetComponentThreadsafe() calls in parallel, as it does not modify the underlying data.
	// Will not collapse disjoint set paths, so repeated accesses may be slower in an un-threaded context.
	inline int32 GetComponentThreadsafe(int32 VertexID) const
	{
		return DisjointSet.FindWithoutCollapse(VertexID);
	}

	inline int32 GetComponentSize(int32 VertexID) const
	{
		return DisjointSet.GetSize(VertexID);
	}

	// Note: May not be thread-safe to make multiple GetComponent() calls in parallel, as it will also collapse the underlying disjoint set paths.
	template<typename TriangleType>
	inline int32 GetComponent(const TriangleType& Triangle)
	{
		return GetComponent(Triangle[0]);
	}

	inline void ConnectVertices(int32 VertexID0, int32 VertexID1)
	{
		DisjointSet.Union(VertexID0, VertexID1);
	}


protected:

	FSizedDisjointSet DisjointSet;
};


} // end namespace UE::Geometry
} // end namespace UE