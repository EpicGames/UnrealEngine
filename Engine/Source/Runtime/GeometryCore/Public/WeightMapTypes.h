// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "IndexTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * FIndexedWeightMap stores an array of values, the intention is that these are "weights" on indices,
 * for example per-vertex weights.
 */
template<typename RealType>
class TIndexedWeightMap
{
public:
	RealType DefaultValue;
	TArray<RealType> Values;

	int32 Num() const { return Values.Num(); }

	void SetNum(int32 NewNum)
	{
		Values.SetNum(NewNum);
	}

	RealType GetValueUnsafe(int32 Index) const
	{
		return Values[Index];
	}

	RealType GetValue(int32 Index) const
	{
		return (Index >= 0 && Index < Values.Num()) ? Values[Index] : DefaultValue;
	}

	RealType GetInterpValueUnsafe(const FIndex3i& Indices, const FVector3d& BaryCoords) const
	{
		return (RealType)((double)Values[Indices.A] * BaryCoords.X
			+ (double)Values[Indices.B] * BaryCoords.Y
			+ (double)Values[Indices.C] * BaryCoords.Z);
	}

	RealType GetInterpValue(const FIndex3i& Indices, const FVector3d& BaryCoords) const
	{
		RealType A = GetValue(Indices.A);
		RealType B = GetValue(Indices.B);
		RealType C = GetValue(Indices.C);

		return (RealType)((double)A*BaryCoords.X + (double)B*BaryCoords.Y + (double)C*BaryCoords.Z);
	}

	void InvertWeightMap(TInterval1<RealType> Range = TInterval1<RealType>((RealType)0, (RealType)1.0))
	{
		int32 Num = Values.Num();
		for (int32 k = 0; k < Num; ++k)
		{
			Values[k] = Range.Clamp((Range.Max - (Values[k] - Range.Min)));
		}
	}
};

typedef TIndexedWeightMap<float> FIndexedWeightMap;
typedef TIndexedWeightMap<float> FIndexedWeightMap1f;
typedef TIndexedWeightMap<double> FIndexedWeightMap1d;


} // end namespace UE::Geometry
} // end namespace UE